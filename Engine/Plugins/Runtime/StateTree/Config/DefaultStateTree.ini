[CoreRedirects]

+PropertyRedirects=(OldName="StateTreeBindableStructDesc.ScriptStruct",NewName="StateTreeBindableStructDesc.Struct")
+PropertyRedirects=(OldName="StateTreeEditorData.Routines",NewName="StateTreeEditorData.SubTrees")

+StructRedirects=(OldName="StateTreeTransition2",NewName="/script/StateTreeEditorModule.StateTreeTransition")
+StructRedirects=(OldName="StateTreeTask2Base",NewName="/script/StateTreeModule.StateTreeTaskBase")
+StructRedirects=(OldName="StateTreeEvaluator2Base",NewName="/script/StateTreeModule.StateTreeEvaluatorBase")

+StructRedirects=(OldName="StateTreeItem",NewName="/script/StateTreeEditorModule.StateTreeEditorNode")
+StructRedirects=(OldName="StateTreeConditionItem",NewName="/script/StateTreeEditorModule.StateTreeEditorNode")
+StructRedirects=(OldName="StateTreeEvaluatorItem",NewName="/script/StateTreeEditorModule.StateTreeEditorNode")
+StructRedirects=(OldName="StateTreeTaskItem",NewName="/script/StateTreeEditorModule.StateTreeEditorNode")

+StructRedirects=(OldName="BakedStateTransition",NewName="/script/StateTreeModule.CompactStateTransition")
+StructRedirects=(OldName="BakedStateTreeState",NewName="/script/StateTreeModule.CompactStateTreeState")
+StructRedirects=(OldName="StateTreeHandle",NewName="/script/StateTreeModule.StateTreeStateHandle")

+PropertyRedirects=(OldName="StateTreeState.Transitions2",NewName="StateTreeState.Transitions")
+PropertyRedirects=(OldName="StateTreeState.EnterConditions2",NewName="StateTreeState.EnterConditions")
+PropertyRedirects=(OldName="StateTreeState.Evaluators2",NewName="StateTreeState.Evaluators")
+PropertyRedirects=(OldName="StateTreeState.Tasks2",NewName="StateTreeState.Tasks")
+PropertyRedirects=(OldName="StateTreeState.LinkedState",NewName="StateTreeState.LinkedSubtree")

+PropertyRedirects=(OldName="StateTree.Conditions2",NewName="StateTree.Conditions")
+PropertyRedirects=(OldName="StateTree.Items",NewName="StateTree.Nodes")
+PropertyRedirects=(OldName="StateTreeEditorNode.Item",NewName="StateTreeEditorNode.Node")

+EnumRedirects=(OldName="EStateTreePropertyAccessType",NewName="/Script/PropertyBindingUtilsModule.EPropertyBindingAccessType")
+EnumRedirects=(OldName="EStateTreePropertyCopyType",NewName="/Script/PropertyBindingUtilsModule.EPropertyCopyType",ValueChanges=(("Plain","CopyPlain"),("Complex","CopyComplex"),("Bool","CopyBool"),("Struct","CopyStruct"),("Object","CopyObject"),("Name","CopyName"),("FixedArray","CopyFixedArray")))

+EnumRedirects=(OldName="/script/StateTreeModule.EPropertyBindingAccessType",ValueChanges=(("Array","IndexArray"),("EStateTreePropertyAccessType::Offset","EPropertyBindingAccessType::Offset")))

+StructRedirects=(OldName="StateTreeCondition_CompareIntInstanceData",NewName="/script/StateTreeModule.StateTreeCompareIntConditionInstanceData")
+StructRedirects=(OldName="StateTreeCondition_CompareInt",NewName="/script/StateTreeModule.StateTreeCompareIntCondition")

+StructRedirects=(OldName="StateTreeCondition_CompareFloatInstanceData",NewName="/script/StateTreeModule.StateTreeCompareFloatConditionInstanceData")
+StructRedirects=(OldName="StateTreeCondition_CompareFloat",NewName="/script/StateTreeModule.StateTreeCompareFloatCondition")

+StructRedirects=(OldName="StateTreeCondition_CompareBoolInstanceData",NewName="/script/StateTreeModule.StateTreeCompareBoolConditionInstanceData")
+StructRedirects=(OldName="StateTreeCondition_CompareBool",NewName="/script/StateTreeModule.StateTreeCompareBoolCondition")

+StructRedirects=(OldName="StateTreeCondition_CompareEnumInstanceData",NewName="/script/StateTreeModule.StateTreeCompareEnumConditionInstanceData")
+StructRedirects=(OldName="StateTreeCondition_CompareEnum",NewName="/script/StateTreeModule.StateTreeCompareEnumCondition")

+StructRedirects=(OldName="StateTreeCondition_CompareDistanceInstanceData",NewName="/script/StateTreeModule.StateTreeCompareDistanceConditionInstanceData")
+StructRedirects=(OldName="StateTreeCondition_CompareDistance",NewName="/script/StateTreeModule.StateTreeCompareDistanceCondition")

+StructRedirects=(OldName="StateTreeCondition_RandomInstanceData",NewName="/script/StateTreeModule.StateTreeRandomConditionInstanceData")
+StructRedirects=(OldName="StateTreeCondition_Random",NewName="/script/StateTreeModule.StateTreeRandomCondition")

+EnumRedirects=(OldName="EStateTreeTransitionEvent",NewName="/Script/StateTreeModule.EStateTreeTransitionTrigger")

+EnumRedirects=(OldName="EStateTreeTransitionTrigger",ValueChanges=(("OnCompleted", "OnStateCompleted"), ("OnSucceeded", "OnStateSucceeded"), ("OnFailed", "OnStateFailed"), ("OnCondition", "OnTick")))

+PropertyRedirects=(OldName="StateTreeTransition.Event",NewName="StateTreeTransition.Trigger")
+PropertyRedirects=(OldName="CompactStateTransition.Event",NewName="CompactStateTransition.Trigger")
+ClassRedirects=(OldName="/Script/StateTreeModule.StateTreeItemBlueprintBase",NewName="/Script/StateTreeModule.StateTreeNodeBlueprintBase")

+EnumRedirects=(OldName="/Script/StateTreeModule.EStateTreeBindableStructSource",ValueChanges=(("TreeData", "Context"), ("TreeParameter", "Parameter"), ("StartParameter", "State"), ("NotSet", "None")))

+PropertyRedirects=(OldName="/Script/StateTreeModule.StateTree.NamedExternalDataDescs",NewName="/Script/StateTreeModule.StateTree.ContextDataDescs")

+PropertyRedirects=(OldName="/Script/StateTreeModule.StateTreeTransition.GateDelay",NewName="/Script/StateTreeModule.StateTreeTransition.DelayDuration")

+StructRedirects=(OldName="StateTreeEditorPropertyBinding",NewName="/script/StateTreeModule.StateTreePropertyPathBinding")
+StructRedirects=(OldName="StateTreePropCopyBatch",NewName="/script/PropertyBindingUtilsModule.PropertyBindingCopyInfoBatch")
+StructRedirects=(OldName="/script/StateTreeModule.StateTreePropertyCopyBatch",NewName="/script/PropertyBindingUtilsModule.PropertyBindingCopyInfoBatch")

+EnumRedirects=(OldName="/Script/StateTreeModule.EStateTreeDataSourceType",ValueChanges=(("LinkedStateParameterData", "StateParameterData")))
+EnumRedirects=(OldName="/Script/StateTreeModule.EStateTreeBindableStructSource",ValueChanges=(("State", "StateParameter")))
+EnumRedirects=(OldName="/Script/StateTreeEditorModule.FStateTreeViewModelInsert",NewName="EStateTreeViewModelInsert")

+EnumRedirects=(OldName="EStateTreeConditionOperand",NewName="/Script/StateTreeModule.EStateTreeExpressionOperand")
+PropertyRedirects=(OldName="StateTreeEditorNode.ConditionIndent",NewName="StateTreeEditorNode.ExpressionIndent")
+PropertyRedirects=(OldName="StateTreeEditorNode.ConditionOperand",NewName="StateTreeEditorNode.ExpressionOperand")

+StructRedirects=(OldName="StateTreePropertyPathSegment",NewName="/script/PropertyBindingUtilsModule.PropertyBindingPathSegment")
+StructRedirects=(OldName="StateTreePropertyPath",NewName="/script/PropertyBindingUtilsModule.PropertyBindingPath")

+StructRedirects=(OldName="StateTreePropertyIndirection",NewName="/script/PropertyBindingUtilsModule.PropertyBindingPropertyIndirection")
+StructRedirects=(OldName="StateTreePropertyCopy",NewName="/script/PropertyBindingUtilsModule.PropertyBindingCopyInfo")
