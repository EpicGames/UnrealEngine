// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompactTreeView.h"

#if WITH_STATETREE_TRACE_DEBUGGER
#include "Debugger/StateTreeTraceTypes.h"
#include "Misc/Attribute.h"
#endif

#include "SCompactTreeDebuggerView.generated.h"

namespace UE::StateTree::Editor
{

namespace CompactTreeView
{
	USTRUCT()
	struct FStateItemDebuggerData : public FStateItemCustomData
	{
		GENERATED_BODY()

		FStateItemDebuggerData() = default;
		explicit FStateItemDebuggerData(const bool bIsActive)
			: bIsActive(bIsActive)
		{
		}

		bool bIsActive = false;
	};
} // CompactTreeView
} // UE::StateTree::Editor

#if WITH_STATETREE_TRACE_DEBUGGER

#define UE_API STATETREEDEVELOPER_API

namespace UE::StateTree::Editor
{

/**
 * Widget that displays a list of State Tree nodes which match base types and specified schema.
 * Can be used e.g. in popup menus to select node types.
 */
class SCompactTreeDebuggerView final : public SCompactTreeView
{
public:

	DECLARE_DELEGATE_OneParam(FOnSelectionChanged, TConstArrayView<FGuid> /*SelectedStateIDs*/);

	SLATE_BEGIN_ARGS(SCompactTreeDebuggerView)
	{}
		SLATE_ATTRIBUTE(FStateTreeTraceActiveStates, ActiveStates)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, TNotNull<const UStateTree*> StateTree);

private:
	UE_API virtual TSharedRef<FStateItem> CreateStateItemInternal() const override;
	UE_API virtual void CacheStatesInternal() override;
	UE_API virtual TSharedRef<SWidget> CreateNameWidgetInternal(TSharedPtr<FStateItem> Item) const override;

	struct FProcessedState
	{
		bool operator==(const FProcessedState& Other) const = default;

		FObjectKey StateTree;
		uint16 StateIdx;
	};

	void CacheStateRecursive(TNotNull<const FStateTreeTraceActiveStates::FAssetActiveStates*> InAssetActiveStates
		, TSharedPtr<FStateItem> InParentItem
		, const uint16 InStateIdx
		, TArray<FProcessedState>& OutProcessedStates);

	TAttribute<FStateTreeTraceActiveStates> AllActiveStates;
};

} // UE::StateTree::Editor

#undef UE_API

#endif // WITH_STATETREE_TRACE_DEBUGGER
