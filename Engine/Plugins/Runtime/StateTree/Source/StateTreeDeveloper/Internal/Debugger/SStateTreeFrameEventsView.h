// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_STATETREE_TRACE_DEBUGGER

#include "Debugger/StateTreeTraceTypes.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API STATETREEDEVELOPER_API

class UStateTree;

template <typename ItemType> class STreeView;

namespace UE::StateTreeDebugger
{
struct FFrameEventTreeElement;
struct FInstanceEventCollection;
struct FScrubState;

/**
 * TreeView representing all traced events on a StateTree instance at a given frame
 */
class SFrameEventsView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFrameEventsView) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, TNotNull<const UStateTree*> InStateTree);

	/** Selects an element in the list based on a predicate applied on the currently displayed events */
	UE_API void SelectByPredicate(TFunctionRef<bool(const FStateTreeTraceEventVariantType& Event)> InPredicate);

	/** Rebuilds the view from events for a given frame */
	UE_API void RequestRefresh(const FScrubState& InScrubState);

private:
	/** Recursively sets tree items as expanded. */
	void ExpandAll(const TArray<TSharedPtr<FFrameEventTreeElement>>& Items);

	static void GenerateElementsForProperties(const FStateTreeTraceEventVariantType& Event, const TSharedRef<FFrameEventTreeElement>& ParentElement);

	TWeakObjectPtr<const UStateTree> WeakStateTree;

	/** All trace events received for a given instance. */
	TArray<TSharedPtr<FFrameEventTreeElement>> EventsTreeElements;

	/** Tree view displaying the frame events of the instance associated to the selected track. */
	TSharedPtr<STreeView<TSharedPtr<FFrameEventTreeElement>>> EventsTreeView;
};

} // UE::StateTreeDebugger

#undef UE_API

#endif // WITH_STATETREE_TRACE_DEBUGGER