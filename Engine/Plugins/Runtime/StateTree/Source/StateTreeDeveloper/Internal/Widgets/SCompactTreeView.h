// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "Framework/Views/ITypedTableView.h"
#include "StructUtils/InstancedStruct.h"
#include "UObject/ObjectKey.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"
#include "SCompactTreeView.generated.h"

class SHorizontalBox;
class SSearchBox;
class UStateTree;
template <typename ItemType> class STreeView;

#define UE_API STATETREEDEVELOPER_API

namespace UE::StateTree::Editor
{
namespace CompactTreeView
{
/**
 * Base type that SCompactTreeView derived classes can inherit from to store additional data
 * through the instanced struct inside SCompactTreeView::FStateItem.
 * USTRUCT is required to be compatible with instanced struct but properties inside the struct
 * are not exposed to the garbage collector.
 */
USTRUCT()
struct FStateItemCustomData
{
	GENERATED_BODY()
};
} // namespace CompactTreeView


/**
 * Widget that displays a list of State Tree states.
 */
class SCompactTreeView : public SCompoundWidget
{
public:

	DECLARE_DELEGATE_OneParam(FOnSelectionChanged, TConstArrayView<FGuid> /*SelectedStateIDs*/);

	SLATE_BEGIN_ARGS(SCompactTreeView)
		: _TextStyle(nullptr)
		, _SelectionMode(ESelectionMode::Single)
	{}
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, TextStyle)
		SLATE_ARGUMENT(ESelectionMode::Type, SelectionMode)
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)
		SLATE_EVENT(FOnContextMenuOpening, OnContextMenuOpening)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, TNotNull<const UStateTree*> StateTree);

	/** @returns widget to focus (search box) when the picker is opened. */
	UE_API TSharedPtr<SWidget> GetWidgetToFocusOnOpen();

	UE_API void SetSelection(TConstArrayView<FGuid> Selection);
	UE_API TArray<FGuid> GetSelection() const;

	UE_API void Refresh();

protected:
	/** Stores info about state */
	struct FStateItem : TSharedFromThis<FStateItem>
	{
		FStateItem() = default;
		FStateItem(const FText& InDesc, const FText& InTooltipText, const FSlateBrush* InIcon)
			: Desc(InDesc)
			, TooltipText(InTooltipText)
			, Icon(InIcon)
		{
		}

		FText Desc;
		FText TooltipText;
		FGuid StateID = {};
		const FSlateBrush* Icon = nullptr;
		TInstancedStruct<CompactTreeView::FStateItemCustomData> CustomData;
		TArray<TSharedPtr<FStateItem>> Children;
		FSlateColor Color = FSlateColor(FLinearColor::White);
		bool bIsEnabled = true;
	};

	/**
	 * Creates the item to represent the State in the tree view.
	 * Can be overridden to create with some specific type of custom data.
	 * Default implementation returns an item without a custom data struct.
	 */
	UE_API virtual TSharedRef<FStateItem> CreateStateItemInternal() const;

	/**
	 * Called by GenerateStateItemRow to create the table row for a given item using the provided box.
	 * Derived classes can override to add extra slots to the box and to customize the row.
	 */
	UE_API virtual TSharedRef<STableRow<TSharedPtr<FStateItem>>> GenerateStateItemRowInternal(
		TSharedPtr<FStateItem> Item
		, const TSharedRef<STableViewBase>& OwnerTable
		, TSharedRef<SHorizontalBox> Box);

	/** Callback to allow derived classes to perform additional operations when selection changed. */
	UE_API virtual void OnSelectionChangedInternal(TConstArrayView<TSharedPtr<FStateItem>> SelectedStates);

	/** Callback to allow derived classes to perform additional operations when filtered root is about to get updated and the tree refreshed. */
	UE_API virtual void OnUpdatingFilteredRootInternal();

	/** Callback that derived classes need to implement to build their tree hierarchy. */
	virtual void CacheStatesInternal() = 0;

	/**
	 * Create widget to represent the state name.
	 * Base implementation creates a text block with search highlighting and enabled status.
	 * Derived classes can override to add extra layer (e.g., borders)
	 */
	UE_API virtual TSharedRef<SWidget> CreateNameWidgetInternal(TSharedPtr<FStateItem> Item) const;

	UE_API static void FindStatesByIDRecursive(const TSharedPtr<FStateItem>& Item, TConstArrayView<FGuid> StateIDs, TArray<TSharedPtr<FStateItem>>& OutStates);

	TWeakObjectPtr<const UStateTree> WeakStateTree = nullptr;
	TSharedPtr<FStateItem> RootItem;
	TSharedPtr<FStateItem> FilteredRootItem;

private:

	/** Stores per session node expansion state for a node type. */
	struct FStateExpansionState
	{
		TSet<FGuid> CollapsedStates;
	};

	void CacheStates();
	void GetStateItemChildren(TSharedPtr<FStateItem> Item, TArray<TSharedPtr<FStateItem>>& OutItems) const;
	TSharedRef<ITableRow> GenerateStateItemRow(TSharedPtr<FStateItem> Item, const TSharedRef<STableViewBase>& OwnerTable);

	void OnStateItemSelected(TSharedPtr<FStateItem> SelectedItem, ESelectInfo::Type);
	void OnStateItemExpansionChanged(TSharedPtr<FStateItem> ExpandedItem, bool bInExpanded) const;
	void OnSearchBoxTextChanged(const FText& NewText);

	void UpdateFilteredRoot(bool bRestoreSelection = true);

	static int32 FilterStateItemChildren(const TArray<FString>& FilterStrings, const bool bParentMatches, const TArray<TSharedPtr<FStateItem>>& SourceArray, TArray<TSharedPtr<FStateItem>>& OutDestArray);
	static bool FindStateByIDRecursive(const TSharedPtr<FStateItem>& Item, const FGuid StateID, TArray<TSharedPtr<FStateItem>>& OutPath);

	void ExpandAll(const TArray<TSharedPtr<FStateItem>>& Items);
	void RestoreExpansionState();

	FOnSelectionChanged OnSelectionChanged;
	FOnContextMenuOpening OnContextMenuOpening;

	TSharedPtr<SSearchBox> SearchBox;
	TSharedPtr<STreeView<TSharedPtr<FStateItem>>> StateItemTree;

	/** Style used for the State name */
	const FTextBlockStyle* TextStyle = nullptr;

	TArray<FString> FilterStrings;

	bool bIsSettingSelection = false;
	bool bIsRestoringExpansion = false;

	/** Save expansion state for each base node type. */
	static TMap<FObjectKey, FStateExpansionState> StateExpansionStates;
};

} // namespace UE::StateTree::Editor

#undef UE_API