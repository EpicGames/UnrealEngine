// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debugger/SCompactTreeDebuggerView.h"
#include "StateTree.h"
#include "StateTreeStyle.h"
#include "Widgets/Views/SListView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SCompactTreeDebuggerView)

#if WITH_STATETREE_TRACE_DEBUGGER

namespace UE::StateTree::Editor
{

void SCompactTreeDebuggerView::Construct(const FArguments& InArgs, const TNotNull<const UStateTree*> StateTree)
{
	AllActiveStates = InArgs._ActiveStates;

	SCompactTreeView::Construct(
		SCompactTreeView::FArguments()
		.SelectionMode(ESelectionMode::Single)
		.TextStyle(&FStateTreeStyle::Get().GetWidgetStyle<FTextBlockStyle>("StateTree.State.Title"))
		, StateTree);
}

void SCompactTreeDebuggerView::CacheStatesInternal()
{
	if (const UStateTree* StateTree = WeakStateTree.Get())
	{
		const FStateTreeTraceActiveStates::FAssetActiveStates* PerAssetActiveStates = AllActiveStates.Get().PerAssetStates.FindByPredicate(
			[StateTree](const FStateTreeTraceActiveStates::FAssetActiveStates& AssetActiveStates)
			{
				return AssetActiveStates.WeakStateTree == StateTree;
			});

		if (PerAssetActiveStates && PerAssetActiveStates->ActiveStates.Num() > 0)
		{
			FStateTreeTraceActiveStates::FAssetActiveStates AssetActiveStates = *PerAssetActiveStates;
			TArray<FProcessedState> ProcessedStates;
			CacheStateRecursive(&AssetActiveStates, RootItem, AssetActiveStates.ActiveStates[0].Index, ProcessedStates);
		}
	}
}

void SCompactTreeDebuggerView::CacheStateRecursive(
	TNotNull<const FStateTreeTraceActiveStates::FAssetActiveStates*> InAssetActiveStates
	, TSharedPtr<FStateItem> InParentItem
	, const uint16 InStateIdx
	, TArray<FProcessedState>& OutProcessedStates)
{
	TNotNull<const UStateTree*> StateTree = InAssetActiveStates->WeakStateTree.Get();
	const FProcessedState ProcessedState(StateTree, InStateIdx);
	if (OutProcessedStates.Contains(ProcessedState))
	{
		return;
	}

	OutProcessedStates.Add(ProcessedState);

	const FCompactStateTreeState& State = StateTree->GetStates()[InStateIdx];
	FString StringForDebug = State.Name.ToString();

	const bool bIsActiveState = InAssetActiveStates->ActiveStates.Contains(FStateTreeStateHandle(InStateIdx));
	auto AddStateItem = [&InParentItem, State, StateTree, InStateIdx](const TSharedRef<FStateItem>& StateItem, const bool bIsActiveState)
		{
			StateItem->Desc = FText::FromName(State.Name);
			StateItem->StateID = StateTree->GetStateIdFromHandle(FStateTreeStateHandle(InStateIdx));
			StateItem->TooltipText = FText::FromName(State.Name);
			StateItem->bIsEnabled = State.bEnabled;
			StateItem->CustomData.GetMutable<CompactTreeView::FStateItemDebuggerData>().bIsActive = bIsActiveState;
			StateItem->Color = FStateTreeStyle::Get().GetSlateColor("StateTree.CompactView.State");
			StateItem->Icon = FStateTreeStyle::GetBrushForSelectionBehaviorType(State.SelectionBehavior, State.HasChildren(), State.Type);

			InParentItem->Children.Add(StateItem);
			InParentItem = StateItem;
		};

	// Add subtree
	if (State.LinkedState.IsValid())
	{
		AddStateItem(CreateStateItemInternal(), bIsActiveState);

		// Recurse to linked tree only for active states
		if (bIsActiveState)
		{
			CacheStateRecursive(InAssetActiveStates, InParentItem, State.LinkedState.Index, OutProcessedStates);
		}
	}
	// Add external subtree
	else if (State.LinkedAsset)
	{
		AddStateItem(CreateStateItemInternal(), bIsActiveState);

		// Recurse to linked tree only for active states
		if (bIsActiveState)
		{
			const TConstArrayView<FCompactStateTreeState> States = State.LinkedAsset->GetStates();
			if (States.Num())
			{
				const FStateTreeTraceActiveStates::FAssetActiveStates* LinkedAssetActiveStates = AllActiveStates.Get().PerAssetStates.FindByPredicate(
					[StateTree = State.LinkedAsset](const FStateTreeTraceActiveStates::FAssetActiveStates& AssetActiveStates)
					{
						return AssetActiveStates.WeakStateTree == StateTree;
					});
				if (LinkedAssetActiveStates && LinkedAssetActiveStates->ActiveStates.Num() > 0)
				{
					FStateTreeTraceActiveStates::FAssetActiveStates AssetActiveStates = *LinkedAssetActiveStates;
					TArray<FProcessedState> ProcessedStates;
					CacheStateRecursive(&AssetActiveStates, InParentItem, AssetActiveStates.ActiveStates[0].Index, OutProcessedStates);
				}
			}
		}
	}
	// Skip empty groups
	else if (!(State.Type == EStateTreeStateType::Group && !State.HasChildren()))
	{
		AddStateItem(CreateStateItemInternal(), bIsActiveState);

		if (State.HasChildren())
		{
			for (uint16 ChildIdx = State.ChildrenBegin; ChildIdx < State.ChildrenEnd; ChildIdx++)
			{
				CacheStateRecursive(InAssetActiveStates, InParentItem, ChildIdx, OutProcessedStates);
			}
		}
	}
}

TSharedRef<SCompactTreeView::FStateItem> SCompactTreeDebuggerView::CreateStateItemInternal() const
{
	const TSharedRef<FStateItem> StateItem = MakeShared<FStateItem>();
	StateItem->CustomData = TInstancedStruct<CompactTreeView::FStateItemDebuggerData>::Make({});
	return StateItem;
}

TSharedRef<SWidget> SCompactTreeDebuggerView::CreateNameWidgetInternal(TSharedPtr<FStateItem> Item) const
{
	return SNew(SBox)
		.VAlign(VAlign_Fill)
		.Padding(0.f, 2.f)
		[
			SNew(SBorder)
			.BorderImage(FStateTreeStyle::Get().GetBrush("StateTree.State.Border"))
			.BorderBackgroundColor_Lambda(
				[Item]
				{
					if (const FStateItem* StateItem = Item.Get())
					{
						if (StateItem->CustomData.Get<CompactTreeView::FStateItemDebuggerData>().bIsActive)
						{
							return FStateTreeStyle::Get().GetSlateColor("StateTree.Debugger.State.Active");
						}
					}
					return FSlateColor(FLinearColor::Transparent);
				})
			[
				SNew(SBorder)
				.BorderImage(FStateTreeStyle::Get().GetBrush("StateTree.State"))
				.BorderBackgroundColor(FStateTreeStyle::Get().GetSlateColor("StateTree.CompactView.State"))
				.Padding(FMargin(0.f, 0.f, 12.f, 0.f))
				.ToolTipText(Item->TooltipText)
				.IsEnabled_Lambda([Item]
					{
						return Item && Item->bIsEnabled;
					})
				[
					SCompactTreeView::CreateNameWidgetInternal(Item)
				]
			]
		];
}

} // namespace UE::StateTree::Editor
#endif // WITH_STATETREE_TRACE_DEBUGGER
