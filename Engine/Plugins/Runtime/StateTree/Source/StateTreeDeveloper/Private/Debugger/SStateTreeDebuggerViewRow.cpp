// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_STATETREE_TRACE_DEBUGGER

#include "SStateTreeDebuggerViewRow.h"
#include "StateTreeStyle.h"
#include "StateTree.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"

namespace UE::StateTreeDebugger
{

void SFrameEventViewRow::Construct(const FArguments& InArgs,
		const TSharedPtr<STableViewBase>& InOwnerTableView,
		const TSharedPtr<FFrameEventTreeElement>& InElement)
{
	Item = InElement;
	STableRow::Construct(InArgs, InOwnerTableView.ToSharedRef());

	const TSharedPtr<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);
	HorizontalBox->SetToolTipText(GetEventTooltip());

	HorizontalBox->AddSlot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		[
			SNew(SExpanderArrow, SharedThis(this))
			.ShouldDrawWires(false)
			.IndentAmount(32)
			.BaseIndentLevel(0)
		];

	const TSharedPtr<SWidget> EventImage = CreateImageForEvent();
	if (EventImage.IsValid())
	{
		HorizontalBox->AddSlot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		[
			EventImage.ToSharedRef()
		];
	}

	HorizontalBox->AddSlot()
		.Padding(2)
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			SNew(STextBlock)
			.TextStyle(&GetEventTextStyle())
			.Text(GetEventDescription())
		];

	this->ChildSlot
		.HAlign(HAlign_Fill)
		[
			HorizontalBox.ToSharedRef()
		];
}

TSharedPtr<SWidget> SFrameEventViewRow::CreateImageForEvent() const
{
	const FStateTreeStyle& StyleSet = FStateTreeStyle::Get();

	// Phase events
	if (const FStateTreeTracePhaseEvent* PhaseEvent = Item->Event.TryGet<FStateTreeTracePhaseEvent>())
	{
		const FSlateBrush* Image = nullptr;
		switch (PhaseEvent->Phase)
		{
		case EStateTreeUpdatePhase::EnterStates:	Image = StyleSet.GetBrush("StateTreeEditor.Debugger.State.Enter");		break;
		case EStateTreeUpdatePhase::ExitStates:		Image = StyleSet.GetBrush("StateTreeEditor.Debugger.State.Exit");		break;
		case EStateTreeUpdatePhase::StateCompleted:	Image = StyleSet.GetBrush("StateTreeEditor.Debugger.State.Completed");	break;
		default:
			return nullptr;
		}

		return SNew(SImage).Image(Image);
	}

	// Log events
	if (const FStateTreeTraceLogEvent* LogEvent = Item->Event.TryGet<FStateTreeTraceLogEvent>())
	{
		const FSlateBrush* Image = nullptr;
		switch (LogEvent->Verbosity)
		{
		case ELogVerbosity::Fatal:
		case ELogVerbosity::Error:		Image = StyleSet.GetBrush("StateTreeEditor.Debugger.Log.Error");	break;
		case ELogVerbosity::Warning:	Image = StyleSet.GetBrush("StateTreeEditor.Debugger.Log.Warning");	break;
		default:
			return nullptr;
		}

		return SNew(SImage).Image(Image);
	}

	// State events
	if (const FStateTreeTraceStateEvent* StateEvent = Item->Event.TryGet<FStateTreeTraceStateEvent>())
	{
		const FSlateBrush* Image = nullptr;

		switch (StateEvent->EventType)
		{
		case EStateTreeTraceEventType::OnStateSelected:		Image = StyleSet.GetBrush("StateTreeEditor.Debugger.State.Selected");	break;
		default:
			return nullptr;
		}

		return SNew(SImage).Image(Image);
	}

	// Task events
	if (const FStateTreeTraceTaskEvent* TaskEvent = Item->Event.TryGet<FStateTreeTraceTaskEvent>())
	{
		const FSlateBrush* Image = nullptr;

		switch (TaskEvent->EventType)
		{
		case EStateTreeTraceEventType::OnEntered:
		case EStateTreeTraceEventType::OnTaskCompleted:
		case EStateTreeTraceEventType::OnTicked:
			switch (TaskEvent->Status)
			{
			case EStateTreeRunStatus::Failed:		Image = StyleSet.GetBrush("StateTreeEditor.Debugger.Task.Failed");		break;
			case EStateTreeRunStatus::Succeeded:	Image = StyleSet.GetBrush("StateTreeEditor.Debugger.Task.Succeeded");	break;
			case EStateTreeRunStatus::Stopped:		Image = StyleSet.GetBrush("StateTreeEditor.Debugger.Task.Stopped");		break;
			default:
				return nullptr;
			}
			break;
		default:
			return nullptr;
		}

		return SNew(SImage).Image(Image);
	}

	// Condition events
	if (Item->Event.IsType<FStateTreeTraceConditionEvent>())
	{
		const FStateTreeTraceConditionEvent& ConditionEvent = Item->Event.Get<FStateTreeTraceConditionEvent>();
		const FSlateBrush* Image = nullptr;

		switch (ConditionEvent.EventType)
		{
		case EStateTreeTraceEventType::Passed:					Image = StyleSet.GetBrush("StateTreeEditor.Debugger.Condition.Passed");			break;
		case EStateTreeTraceEventType::ForcedSuccess:			Image = StyleSet.GetBrush("StateTreeEditor.Debugger.Condition.Passed");			break;
		case EStateTreeTraceEventType::Failed:					Image = StyleSet.GetBrush("StateTreeEditor.Debugger.Condition.Failed");			break;
		case EStateTreeTraceEventType::ForcedFailure:			Image = StyleSet.GetBrush("StateTreeEditor.Debugger.Condition.Failed");			break;
		case EStateTreeTraceEventType::InternalForcedFailure:	Image = StyleSet.GetBrush("StateTreeEditor.Debugger.Condition.Failed");			break;
		case EStateTreeTraceEventType::OnEvaluating:			Image = StyleSet.GetBrush("StateTreeEditor.Debugger.Condition.OnEvaluating");	break;
		case EStateTreeTraceEventType::OnTransition:			Image = StyleSet.GetBrush("StateTreeEditor.Debugger.Condition.OnTransition");	break;
		default:
			Image = StyleSet.GetBrush("StateTreeEditor.Debugger.Unset");
		}

		return SNew(SImage).Image(Image);
	}

	return nullptr;
}

const FTextBlockStyle& SFrameEventViewRow::GetEventTextStyle() const
{
	const FStateTreeStyle& StyleSet = FStateTreeStyle::Get();

	if (Item->Event.IsType<FStateTreeTracePhaseEvent>())
	{
		return StyleSet.GetWidgetStyle<FTextBlockStyle>("StateTreeDebugger.Element.Bold");
	}

	if (Item->Event.IsType<FStateTreeTracePropertyEvent>())
	{
		return StyleSet.GetWidgetStyle<FTextBlockStyle>("StateTreeDebugger.Element.Subdued");
	}

	if (const FStateTreeTraceLogEvent* LogEvent = Item->Event.TryGet<FStateTreeTraceLogEvent>())
	{
		// Make verbose logs more subtle
		if (LogEvent->Verbosity >= ELogVerbosity::Verbose)
		{
			return StyleSet.GetWidgetStyle<FTextBlockStyle>("StateTreeDebugger.Element.Subdued");
		}
	}

	return StyleSet.GetWidgetStyle<FTextBlockStyle>("StateTreeDebugger.Element.Normal");
}

FText SFrameEventViewRow::GetEventDescription() const
{
	FString EventDescription;
	if (Item->Description.IsEmpty())
	{
		if (const UStateTree* StateTree = Item->WeakStateTree.Get())
		{
			// Some types have some custom representations so we want to use a more minimal description.
			if (Item->Event.IsType<FStateTreeTraceStateEvent>()
				|| Item->Event.IsType<FStateTreeTraceTaskEvent>()
				|| Item->Event.IsType<FStateTreeTraceEvaluatorEvent>()
				|| Item->Event.IsType<FStateTreeTracePropertyEvent>()
				|| Item->Event.IsType<FStateTreeTraceConditionEvent>()
				|| Item->Event.IsType<FStateTreeTraceLogEvent>())
			{
				Visit([&EventDescription, StateTree](auto& TypedEvent)
					{
						EventDescription = TypedEvent.GetValueString(*StateTree);
					}, Item->Event);
			}
			else
			{
				Visit([&EventDescription, StateTree](auto& TypedEvent)
					{
						EventDescription = TypedEvent.ToFullString(*StateTree);
					}, Item->Event);
			}
		}
	}
	else
	{
		EventDescription = Item->Description;
	}
	
	return FText::FromString(EventDescription);
}

FText SFrameEventViewRow::GetEventTooltip() const
{
	FString Tooltip;
	if (const UStateTree* StateTree = Item->WeakStateTree.Get())
	{
		Visit([&Tooltip, StateTree](auto& TypedEvent)
			{
				Tooltip = TypedEvent.ToFullString(*StateTree);
			}, Item->Event);
	}

	return FText::FromString(Tooltip);
}

} // UE::StateTreeDebugger

#endif // WITH_STATETREE_TRACE_DEBUGGER
