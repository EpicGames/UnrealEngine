// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_STATETREE_TRACE_DEBUGGER

#include "Debugger/StateTreeTraceTypes.h"
#include "StateTree.h"
#include "Templates/SharedPointer.h"
#include "TraceServices/Model/Frames.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class FStateTreeViewModel;

namespace UE::StateTreeDebugger
{

/** An item in the StateTreeDebugger trace event tree */
struct FFrameEventTreeElement : TSharedFromThis<FFrameEventTreeElement>
{
	explicit FFrameEventTreeElement(const TraceServices::FFrame& Frame, const FStateTreeTraceEventVariantType& Event, const UStateTree* StateTree)
		: Frame(Frame), Event(Event), WeakStateTree(StateTree)
	{
	}

	TraceServices::FFrame Frame;
	FStateTreeTraceEventVariantType Event;
	TArray<TSharedPtr<FFrameEventTreeElement>> Children;
	FString Description;
	TWeakObjectPtr<const UStateTree> WeakStateTree;
};


/**
 * Widget for row inside the StateTreeDebugger TreeView.
 */
class SFrameEventViewRow : public STableRow<TSharedPtr<FFrameEventTreeElement>>
{
public:
	void Construct(const FArguments& InArgs,
		const TSharedPtr<STableViewBase>& InOwnerTableView,
		const TSharedPtr<FFrameEventTreeElement>& InElement);

private:
	TSharedPtr<SWidget> CreateImageForEvent() const;
	const FTextBlockStyle& GetEventTextStyle() const;
	FText GetEventDescription() const;
	FText GetEventTooltip() const;

	TSharedPtr<FFrameEventTreeElement> Item;
};

} // UE::StateTreeDebugger

#endif // WITH_STATETREE_TRACE_DEBUGGER