// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_STATETREE_TRACE_DEBUGGER

#include "Debugger/SStateTreeFrameEventsView.h"
#include "Debugger/StateTreeDebuggerTypes.h"
#include "SStateTreeDebuggerViewRow.h"
#include "UObject/Package.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

namespace UE::StateTreeDebugger
{
/**
 * Iterates over all tree elements for the frame events
 * @param Elements Container of hierarchical tree element to visit
 * @param InFunc function called at each element, should return true if visiting is continued or false to stop.
 */
void VisitEventTreeElements(const TConstArrayView<TSharedPtr<FFrameEventTreeElement>> Elements
	, TFunctionRef<bool(TSharedPtr<FFrameEventTreeElement>& VisitedElement)> InFunc)
{
	TArray<TSharedPtr<FFrameEventTreeElement>> Stack;
	bool bContinue = true;

	for (const TSharedPtr<FFrameEventTreeElement>& RootElement : Elements)
	{
		if (RootElement == nullptr)
		{
			continue;
		}

		Stack.Add(RootElement);

		while (!Stack.IsEmpty() && bContinue)
		{
			TSharedPtr<FFrameEventTreeElement> StackedElement = Stack[0];
			check(StackedElement);

			Stack.RemoveAt(0);

			bContinue = InFunc(StackedElement);

			if (bContinue)
			{
				for (const TSharedPtr<FFrameEventTreeElement>& Child : StackedElement->Children)
				{
					if (Child.IsValid())
					{
						Stack.Add(Child);
					}
				}
			}
		}

		if (!bContinue)
		{
			break;
		}
	}
}


//----------------------------------------------------------------------//
// SFrameEventsView
//----------------------------------------------------------------------//

void SFrameEventsView::Construct(const FArguments& InArgs, TNotNull<const UStateTree*> InStateTree)
{
	WeakStateTree = InStateTree;

	// EventsTreeView scrollbars
	TSharedRef<SScrollBar> HorizontalScrollBar = SNew(SScrollBar)
		.Orientation(Orient_Horizontal)
		.Thickness(FVector2D(12.0f, 12.0f));

	TSharedRef<SScrollBar> VerticalScrollBar = SNew(SScrollBar)
		.Orientation(Orient_Vertical)
		.Thickness(FVector2D(12.0f, 12.0f));

	// EventsTreeView
	EventsTreeView = SNew(STreeView<TSharedPtr<FFrameEventTreeElement>>)
			.OnGenerateRow_Lambda([this](const TSharedPtr<FFrameEventTreeElement>& InElement, const TSharedRef<STableViewBase>& InOwnerTableView)
			{
				return SNew(SFrameEventViewRow, InOwnerTableView, InElement);
			})
			.OnGetChildren_Lambda([](const TSharedPtr<const FFrameEventTreeElement>& InParent, TArray<TSharedPtr<FFrameEventTreeElement>>& OutChildren)
			{
				if (const FFrameEventTreeElement* Parent = InParent.Get())
				{
					OutChildren.Append(Parent->Children);
				}
			})
		.TreeItemsSource(&EventsTreeElements)
		.AllowOverscroll(EAllowOverscroll::No)
		.ExternalScrollbar(VerticalScrollBar);

	ChildSlot
	[
		SNew(SBorder)
		.Padding(4.0f)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				SNew(SSplitter)
				.Orientation(Orient_Horizontal)
				+ SSplitter::Slot()
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.FillHeight(1.0f)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.Padding(0.0f)
						[
							SNew(SScrollBox)
							.Orientation(Orient_Horizontal)
							.ExternalScrollbar(HorizontalScrollBar)
							+ SScrollBox::Slot()
							.FillSize(1.0f)
							[
								EventsTreeView.ToSharedRef()
							]
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							VerticalScrollBar
						]
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						HorizontalScrollBar
					]
				]
			]
		]
	];
}

void SFrameEventsView::SelectByPredicate(TFunctionRef<bool(const FStateTreeTraceEventVariantType& Event)> InPredicate)
{
	TSharedPtr<FFrameEventTreeElement> MatchingElement = nullptr;
	VisitEventTreeElements(EventsTreeElements, [&MatchingElement, InPredicate](const TSharedPtr<FFrameEventTreeElement>& VisitedElement)
		{
			if (InPredicate(VisitedElement->Event))
			{
				MatchingElement = VisitedElement;
			}

			// Continue visit until we find a matching event
			return MatchingElement.IsValid() == false;
		});

	if (MatchingElement.IsValid())
	{
		EventsTreeView->SetSelection(MatchingElement);
	}
}

void SFrameEventsView::GenerateElementsForProperties(const FStateTreeTraceEventVariantType& Event, const TSharedRef<FFrameEventTreeElement>& ParentElement)
{
	FString TypePath;
	FString InstanceDataAsString;
	FString DebugText;

	Visit([&TypePath, &InstanceDataAsString, &DebugText](auto& TypedEvent)
		{
			TypePath = TypedEvent.GetDataTypePath();
			InstanceDataAsString = TypedEvent.GetDataAsText();
			DebugText = TypedEvent.GetDebugText();
		}, Event);

	auto CreateTreeElement = [ParentElement]<typename EventType>(const FString& Line)
		{
			// Create new event
			EventType Event(/*RecordingWorldTime*/0, ELogVerbosity::Verbose, *Line);

			// Create Tree element to hold the event
			const TSharedPtr<FFrameEventTreeElement> NewChildElement = MakeShareable(new FFrameEventTreeElement(
				ParentElement->Frame,
				FStateTreeTraceEventVariantType(TInPlaceType<EventType>(), Event),
				ParentElement->WeakStateTree.Get()));

			ParentElement->Children.Add(NewChildElement);
		};

	if (!InstanceDataAsString.IsEmpty())
	{
		auto CreatePropertyElement = [&CreateTreeElement](const FStringView Line, const int32 NestedCount = 0)
			{
				constexpr int32 Indent = 4;
				FString ConvertedString = FString::Printf(TEXT("%*s"), NestedCount * Indent, TEXT(""));
				ConvertedString.Append(Line);
				ConvertedString.ReplaceInline(TEXT("="), TEXT(" = "));
				ConvertedString.ReplaceInline(TEXT("\""), TEXT(""));
				CreateTreeElement.operator()<FStateTreeTracePropertyEvent>(ConvertedString);
			};

		// Try to parse Struct for which properties are exported between '(' and ')'
		if (InstanceDataAsString.StartsWith("(") && InstanceDataAsString.EndsWith(")"))
		{
			const FStringView View(GetData(InstanceDataAsString) + 1, InstanceDataAsString.Len() - 2);
			const TCHAR* ViewIt = View.GetData();
			const TCHAR* const ViewEnd = ViewIt + View.Len();
			const TCHAR* NextToken = ViewIt;
			int32 NestedCount = 0;

			for (; ViewIt != ViewEnd; ++ViewIt)
			{
				int32 LocalNestedCount = 0;
				if (*ViewIt == TCHAR('('))
				{
					LocalNestedCount++;
				}
				else if (*ViewIt == TCHAR(')'))
				{
					LocalNestedCount--;
				}
				else if (*ViewIt != TCHAR(','))
				{
					continue;
				}

				if (ViewIt != NextToken)
				{
					CreatePropertyElement(FStringView(NextToken, UE_PTRDIFF_TO_INT32(ViewIt - NextToken)), NestedCount);
				}
				NextToken = ViewIt + 1;
				NestedCount += LocalNestedCount;
			}

			if (ViewIt != NextToken)
			{
				CreatePropertyElement(FStringView(NextToken, UE_PTRDIFF_TO_INT32(ViewIt - NextToken)), NestedCount);
			}
		}
		else
		{
			const TCHAR* Buffer = *InstanceDataAsString;
			FParse::Next(&Buffer);
			FString StrLine;
			while (FParse::Line(&Buffer, StrLine))
			{
				const TCHAR* Str = *StrLine;
				if (!FParse::Command(&Str, TEXT("BEGIN OBJECT"))
					&& !FParse::Command(&Str, TEXT("END OBJECT")))
				{
					CreatePropertyElement(Str);
				}
			}
		}
	}

	if (!DebugText.IsEmpty())
	{
		CreateTreeElement.operator()<FStateTreeTraceLogEvent>(DebugText);
	}
}

void SFrameEventsView::ExpandAll(const TArray<TSharedPtr<FFrameEventTreeElement>>& Items)
{
	for (const TSharedPtr<FFrameEventTreeElement>& Item : Items)
	{
		bool bExpand = true;
		if (Item->Children.Num() > 0)
		{
			if (const FFrameEventTreeElement* FirstChild = Item->Children[0].Get())
			{
				if (const FStateTreeTraceLogEvent* LogEvent = FirstChild->Event.TryGet<FStateTreeTraceLogEvent>())
				{
					// Do not auto expand verbose logs
					bExpand = (LogEvent->Verbosity < ELogVerbosity::Verbose);
				}
				else if (FirstChild->Event.IsType<FStateTreeTracePropertyEvent>())
				{
					bExpand = false;
				}
			}
		}

		if (bExpand)
		{
			EventsTreeView->SetItemExpansion(Item, true);
			ExpandAll(Item->Children);
		}
	}
}

void SFrameEventsView::RequestRefresh(const FScrubState& InScrubState)
{
	const FInstanceEventCollection& EventCollection = InScrubState.GetEventCollection();
	if (EventCollection.IsInvalid())
	{
		return;
	}

	// Rebuild frame details from the events of that frame
	EventsTreeElements.Reset();
	ON_SCOPE_EXIT
	{
		EventsTreeView->ClearExpandedItems();
		ExpandAll(EventsTreeElements);
		EventsTreeView->RequestTreeRefresh();
	};

	const TConstArrayView<const FStateTreeTraceEventVariantType> Events = EventCollection.Events;

	if (Events.IsEmpty()
		|| !InScrubState.IsInBounds())
	{
		return;
	}

	const TConstArrayView<FFrameSpan> Spans = EventCollection.FrameSpans;
	check(Spans.Num());
	check(WeakStateTree.IsValid());

	TArray<TSharedPtr<FFrameEventTreeElement>, TInlineAllocator<8>> ScopeStack;

	const int32 SpanIdx = InScrubState.GetFrameSpanIndex();

	if (!Spans.IsValidIndex(SpanIdx))
	{
		UE_LOGF(LogStateTree, Error, "Invalid index in span: Idx: %i, Num Spans: %i", SpanIdx, Spans.Num());
		return;
	}

	const FFrameSpan& Span = Spans[SpanIdx];
	if (InScrubState.GetScrubTime() < Span.GetWorldTimeStart()
		|| InScrubState.GetScrubTime() > Span.GetWorldTimeEnd())
	{
		return;
	}

	const int32 FirstEventIdx = Span.EventIdx;
	const TraceServices::FFrame Frame = Span.Frame;
	const int32 MaxEventIdx = Spans.IsValidIndex(SpanIdx + 1) ? Spans[SpanIdx + 1].EventIdx : Events.Num();

	const UStateTree* const RootTree = WeakStateTree.Get();
	const UStateTree* ActiveTree = RootTree;

	for (int32 EventIdx = FirstEventIdx; EventIdx < MaxEventIdx; EventIdx++)
	{
		const FStateTreeTraceEventVariantType& Event = Events[EventIdx];
		FString CustomDescription;
		bool bShouldAddToScopeStack = false;
		bool bShouldPopScopeStack = false;

		if (const FStateTreeTraceStateEvent* StateEvent = Event.TryGet<FStateTreeTraceStateEvent>())
		{
			if (StateEvent->EventType == EStateTreeTraceEventType::OnEntering
				|| StateEvent->EventType == EStateTreeTraceEventType::OnExiting
				|| StateEvent->EventType == EStateTreeTraceEventType::Push)
			{
				bShouldAddToScopeStack = true;
			}
			else if (StateEvent->EventType == EStateTreeTraceEventType::OnEntered
				|| StateEvent->EventType == EStateTreeTraceEventType::OnExited
				|| StateEvent->EventType == EStateTreeTraceEventType::Pop)
			{
				bShouldPopScopeStack = true;
			}
		}
		else if (const FStateTreeTracePhaseEvent* PhaseEvent = Event.TryGet<FStateTreeTracePhaseEvent>())
		{
			if (PhaseEvent->EventType == EStateTreeTraceEventType::Push)
			{
				bShouldAddToScopeStack = true;
			}
			else if (PhaseEvent->EventType == EStateTreeTraceEventType::Pop)
			{
				bShouldPopScopeStack = true;
			}
		}
		else if (const FStateTreeTraceInstanceFrameEvent* FrameEvent = Event.TryGet<FStateTreeTraceInstanceFrameEvent>())
		{
			ActiveTree = FrameEvent->WeakStateTree.Get();
			check(ActiveTree);

			// We don't want to create an entry.
			continue;
		}

		if (bShouldPopScopeStack)
		{
			// Pop scope and remove associated element if empty
			if (ensureMsgf(ScopeStack.Num() > 0, TEXT("Expected to pop an entry in the scope stack but it is already empty.")))
			{
				TSharedPtr<FFrameEventTreeElement> Scope = ScopeStack.Pop();
				if (Scope->Children.IsEmpty())
				{
					TArray<TSharedPtr<FFrameEventTreeElement>>& TreeElements = ScopeStack.IsEmpty() ? EventsTreeElements : ScopeStack.Top()->Children;
					TreeElements.Remove(Scope);
				}
			}
			// We don't want to create a child when a scope is popped.

			continue;
		}

		const TSharedRef<FFrameEventTreeElement> NewElement = MakeShareable(new FFrameEventTreeElement(Frame, Event, ActiveTree));
		NewElement->Description = CustomDescription;

		TArray<TSharedPtr<FFrameEventTreeElement>>& TreeElements = ScopeStack.IsEmpty() ? EventsTreeElements : ScopeStack.Top()->Children;
		const TSharedPtr<FFrameEventTreeElement>& ElementPtr = TreeElements.Add_GetRef(NewElement);

		if (bShouldAddToScopeStack)
		{
			ScopeStack.Push(ElementPtr);
		}

		GenerateElementsForProperties(Event, NewElement);
	}
}

} // UE::StateTreeDebugger

#endif // WITH_STATETREE_TRACE_DEBUGGER