// Copyright Epic Games, Inc. All Rights Reserved.

#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"
#include "RewindDebuggerRuntimeInterface/IRewindDebuggerRuntimeExtension.h"
#include "StateTreeRewindDebuggerRecordingExtension.h"
#include "StateTreeStyle.h"

class FStateTreeDeveloperModule final : public IModuleInterface
{
	//~ Begin IModuleInterface
	virtual void StartupModule() override
	{
		FStateTreeStyle::Register();

#if WITH_STATETREE_TRACE
		RewindDebuggerRecordingExtension = MakeUnique<UE::StateTreeDebugger::FRewindDebuggerRecordingExtension>();
		IModularFeatures::Get().RegisterModularFeature(IRewindDebuggerRuntimeExtension::ModularFeatureName, RewindDebuggerRecordingExtension.Get());
#endif // WITH_STATETREE_TRACE
	}

	virtual void ShutdownModule() override
	{
		FStateTreeStyle::Unregister();

#if WITH_STATETREE_TRACE
		IModularFeatures::Get().UnregisterModularFeature(IRewindDebuggerRuntimeExtension::ModularFeatureName, RewindDebuggerRecordingExtension.Get());
#endif // WITH_STATETREE_TRAC
	}
	//~ End IModuleInterface

#if WITH_STATETREE_TRACE
	TUniquePtr<UE::StateTreeDebugger::FRewindDebuggerRecordingExtension> RewindDebuggerRecordingExtension;
#endif // WITH_STATETREE_TRACE
};

IMPLEMENT_MODULE(FStateTreeDeveloperModule, StateTreeDeveloper)
