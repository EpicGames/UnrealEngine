// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_STATETREE_TRACE
#include "StateTreeRewindDebuggerRecordingExtension.h"
#include "Debugger/StateTreeTraceTypes.h"
#include "StateTreeDelegates.h"
#include "Debugger/StateTreeTrace.h"

namespace UE::StateTreeDebugger
{

//----------------------------------------------------------------------//
// FRewindDebuggerRecordingExtension
//----------------------------------------------------------------------//
void FRewindDebuggerRecordingExtension::RecordingStarted()
{
	// Broadcast delegate before enabling the channel so event can't be traced until
	// the delegate listeners are notified.
	StateTree::Delegates::OnTracingStateChanged.Broadcast(EStateTreeTraceStatus::TracesStarted);

	if (!UE_TRACE_CHANNELEXPR_IS_ENABLED(StateTreeDebugChannel))
	{
		Trace::ToggleChannel(TEXT("StateTreeDebugChannel"), true);
	}
}

void FRewindDebuggerRecordingExtension::RecordingStopped()
{
	// Shouldn't fire as channel will already be disabled when rewind debugger stops, but just as safeguard
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(StateTreeDebugChannel))
	{
		StateTree::Delegates::OnTracingStateChanged.Broadcast(EStateTreeTraceStatus::StoppingTrace);
		Trace::ToggleChannel(TEXT("StateTreeDebugChannel"), false);
	}
}

void FRewindDebuggerRecordingExtension::Clear()
{
	StateTree::Delegates::OnTracingStateChanged.Broadcast(EStateTreeTraceStatus::Cleared);
}

} // UE::StateTreeDebugger

#endif // WITH_STATETREE_TRACE