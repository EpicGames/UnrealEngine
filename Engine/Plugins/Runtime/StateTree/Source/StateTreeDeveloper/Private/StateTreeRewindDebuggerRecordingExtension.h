// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_STATETREE_TRACE
#include "RewindDebuggerRuntimeInterface/IRewindDebuggerRuntimeExtension.h"

namespace UE::StateTreeDebugger
{

class FRewindDebuggerRecordingExtension final : public IRewindDebuggerRuntimeExtension
{
	virtual void RecordingStarted() override;
	virtual void RecordingStopped() override;
	virtual void Clear() override;
};

} // UE::StateTreeDebugger

#endif // WITH_STATETREE_TRACE