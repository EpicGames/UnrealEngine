// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeStyle.h"

#include "StateTreeTypes.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"

const FLazyName FStateTreeStyle::StateTitleTextStyleName("StateTree.State.Title");
const FString FStateTreeStyle::EngineSlateContentDir(FPaths::EngineContentDir() / TEXT("Slate"));
const FString FStateTreeStyle::StateTreePluginContentDir(FPaths::EnginePluginsDir() / TEXT("Runtime/StateTree/Resources"));

FStateTreeStyle::FStateTreeStyle() : FStateTreeStyle(TEXT("StateTreeStyle"))
{
}

FStateTreeStyle::FStateTreeStyle(const FName& InStyleSetName)
	: FSlateStyleSet(InStyleSetName)
{
	SetCoreContentRoot(EngineSlateContentDir);
	SetContentRoot(StateTreePluginContentDir);

	const FTextBlockStyle& NormalText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");

	// Debugger
	{
		Set("StateTreeDebugger.Element.Normal",
			FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 10)));

		Set("StateTreeDebugger.Element.Bold",
			FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Bold", 10)));

		Set("StateTreeDebugger.Element.Subdued",
			FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 10))
			.SetColorAndOpacity(FSlateColor::UseSubduedForeground()));
	}

	// State
	{
		const FTextBlockStyle StateTitle = FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Bold", 12))
			.SetColorAndOpacity(FLinearColor(230.0f / 255.0f, 230.0f / 255.0f, 230.0f / 255.0f, 0.9f));
		Set(StateTitleTextStyleName, StateTitle);

		Set("StateTree.State", new FSlateRoundedBoxBrush(FLinearColor::White, 2.0f));
		Set("StateTree.State.Border", new FSlateRoundedBoxBrush(FLinearColor::White, 2.0f));
		Set("StateTree.Debugger.State.Active", FSlateColor(FColor::Yellow));
		Set("StateTree.CompactView.State", FSlateColor(FLinearColor(FColor(10, 100, 120))));
	}

	// Normal rich text
	{
		Set("Normal.Normal", FTextBlockStyle(NormalText)
			.SetColorAndOpacity(FSlateColor::UseForeground())
			.SetFont(DEFAULT_FONT("Regular", 10)));

		Set("Normal.Bold", FTextBlockStyle(NormalText)
			.SetColorAndOpacity(FSlateColor::UseForeground())
			.SetFont(DEFAULT_FONT("Bold", 10)));

		Set("Normal.Italic", FTextBlockStyle(NormalText)
			.SetColorAndOpacity(FSlateColor::UseForeground())
			.SetFont(DEFAULT_FONT("Italic", 10)));

		Set("Normal.Subdued", FTextBlockStyle(NormalText)
			.SetColorAndOpacity(FSlateColor::UseSubduedForeground())
			.SetFont(DEFAULT_FONT("Regular", 10)));
	}

	{
		// From plugin
		Set("StateTreeEditor.SelectNone", new IMAGE_BRUSH_SVG("Icons/Select_None", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.TryEnterState", new IMAGE_BRUSH_SVG("Icons/Try_Enter_State", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.TrySelectChildrenInOrder", new IMAGE_BRUSH_SVG("Icons/Try_Select_Children_In_Order", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.TrySelectChildrenAtRandom", new IMAGE_BRUSH_SVG("Icons/Try_Select_Children_At_Random", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.TryFollowTransitions", new IMAGE_BRUSH_SVG("Icons/Try_Follow_Transitions", CoreStyleConstants::Icon16x16));

		Set("StateTreeEditor.Debugger.Condition.OnTransition", new IMAGE_BRUSH_SVG("Icons/State_Conditions", CoreStyleConstants::Icon16x16, FStyleColors::AccentGray));
	}

	{
		// From generic Engine
		FContentRootScope Scope(this, EngineSlateContentDir);
		Set("StateTreeEditor.Debugger.State.Enter", new CORE_IMAGE_BRUSH_SVG("Starship/Common/arrow-right", CoreStyleConstants::Icon16x16, FStyleColors::Foreground));
		Set("StateTreeEditor.Debugger.State.Exit", new CORE_IMAGE_BRUSH_SVG("Starship/Common/arrow-left", CoreStyleConstants::Icon16x16, FStyleColors::Foreground));
		Set("StateTreeEditor.Debugger.State.Completed", new CORE_IMAGE_BRUSH_SVG("Starship/Common/check", CoreStyleConstants::Icon16x16, FStyleColors::AccentGreen));
		Set("StateTreeEditor.Debugger.State.Selected", new CORE_IMAGE_BRUSH_SVG("Starship/Common/arrow-right", CoreStyleConstants::Icon16x16, FStyleColors::AccentYellow));

		Set("StateTreeEditor.Debugger.Log.Warning", new CORE_IMAGE_BRUSH_SVG("Starship/Common/alert-circle", CoreStyleConstants::Icon16x16, FStyleColors::AccentYellow));
		Set("StateTreeEditor.Debugger.Log.Error", new CORE_IMAGE_BRUSH_SVG("Starship/Common/x-circle", CoreStyleConstants::Icon16x16, FStyleColors::AccentRed));

		Set("StateTreeEditor.Debugger.Task.Failed", new CORE_IMAGE_BRUSH_SVG("Starship/Common/close-small", CoreStyleConstants::Icon16x16, FStyleColors::AccentRed));
		Set("StateTreeEditor.Debugger.Task.Succeeded", new CORE_IMAGE_BRUSH_SVG("Starship/Common/check", CoreStyleConstants::Icon16x16, FStyleColors::AccentGreen));
		Set("StateTreeEditor.Debugger.Task.Stopped", new CORE_IMAGE_BRUSH_SVG("Starship/Common/close-small", CoreStyleConstants::Icon16x16, FStyleColors::AccentRed));

		Set("StateTreeEditor.Debugger.Condition.Passed", new CORE_IMAGE_BRUSH_SVG("Starship/Common/check", CoreStyleConstants::Icon16x16, FStyleColors::AccentGreen));
		Set("StateTreeEditor.Debugger.Condition.Failed", new CORE_IMAGE_BRUSH_SVG("Starship/Common/close-small", CoreStyleConstants::Icon16x16, FStyleColors::AccentRed));
		Set("StateTreeEditor.Debugger.Condition.OnEvaluating", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Update", CoreStyleConstants::Icon16x16, FStyleColors::AccentYellow));
	}

}

void FStateTreeStyle::Register()
{
	FSlateStyleRegistry::RegisterSlateStyle(Get());
}

void FStateTreeStyle::Unregister()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(Get());
}

FStateTreeStyle& FStateTreeStyle::Get()
{
	static FStateTreeStyle Instance;
	return Instance;
}

const FSlateBrush* FStateTreeStyle::GetBrushForSelectionBehaviorType(const EStateTreeStateSelectionBehavior InSelectionBehavior, const bool bInHasChildren, const EStateTreeStateType InStateType)
{
	if (InSelectionBehavior == EStateTreeStateSelectionBehavior::None)
	{
		return Get().GetBrush("StateTreeEditor.SelectNone");
	}

	if (InSelectionBehavior == EStateTreeStateSelectionBehavior::TryEnterState)
	{
		return Get().GetBrush("StateTreeEditor.TryEnterState");
	}

	if (InSelectionBehavior == EStateTreeStateSelectionBehavior::TrySelectChildrenInOrder
		|| InSelectionBehavior == EStateTreeStateSelectionBehavior::TrySelectChildrenWithHighestUtility
		|| InSelectionBehavior == EStateTreeStateSelectionBehavior::TrySelectChildrenAtRandomWeightedByUtility)
	{
		if (!bInHasChildren
			|| InStateType == EStateTreeStateType::Linked
			|| InStateType == EStateTreeStateType::LinkedAsset)
		{
			return Get().GetBrush("StateTreeEditor.TryEnterState");
		}

		return Get().GetBrush("StateTreeEditor.TrySelectChildrenInOrder");

	}

	if (InSelectionBehavior == EStateTreeStateSelectionBehavior::TrySelectChildrenAtRandom)
	{
		return Get().GetBrush("StateTreeEditor.TrySelectChildrenAtRandom");
	}

	if (InSelectionBehavior == EStateTreeStateSelectionBehavior::TryFollowTransitions)
	{
		return Get().GetBrush("StateTreeEditor.TryFollowTransitions");
	}

	return nullptr;
}
