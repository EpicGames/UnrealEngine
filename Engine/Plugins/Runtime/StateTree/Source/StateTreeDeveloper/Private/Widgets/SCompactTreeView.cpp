// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SCompactTreeView.h"

#include "StateTree.h"
#include "StateTreeStyle.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STreeView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SCompactTreeView)

#define LOCTEXT_NAMESPACE "SCompactTreeView"

namespace UE::StateTree::Editor
{
TMap<FObjectKey, SCompactTreeView::FStateExpansionState> SCompactTreeView::StateExpansionStates;

void SCompactTreeView::Construct(const FArguments& InArgs, const TNotNull<const UStateTree*> StateTree)
{
	WeakStateTree = StateTree;
	TextStyle = InArgs._TextStyle ? InArgs._TextStyle : &FStateTreeStyle::Get().GetWidgetStyle<FTextBlockStyle>("Normal.Normal");
	OnSelectionChanged = InArgs._OnSelectionChanged;
	OnContextMenuOpening = InArgs._OnContextMenuOpening;

	CacheStates();

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		.Padding(4.f, 2.f, 4.f, 2.f)
		.AutoHeight()
		[
			SAssignNew(SearchBox, SSearchBox)
			.OnTextChanged(this, &SCompactTreeView::OnSearchBoxTextChanged)
		]
		+ SVerticalBox::Slot()
		[
			SAssignNew(StateItemTree, STreeView<TSharedPtr<FStateItem>>)
			.SelectionMode(InArgs._SelectionMode)
			.TreeItemsSource(&FilteredRootItem->Children)
			.OnGenerateRow(this, &SCompactTreeView::GenerateStateItemRow)
			.OnGetChildren(this, &SCompactTreeView::GetStateItemChildren)
			.OnSelectionChanged(this, &SCompactTreeView::OnStateItemSelected)
			.OnExpansionChanged(this, &SCompactTreeView::OnStateItemExpansionChanged)
			.OnContextMenuOpening_Lambda([this]()
			{
				if (OnContextMenuOpening.IsBound())
				{
					return OnContextMenuOpening.Execute();
				}
				return SNullWidget::NullWidget.ToSharedPtr();
			})
		]
	];

	// Restore category expansion state from previous use.
	RestoreExpansionState();
}

void SCompactTreeView::Refresh()
{
	if (!StateItemTree)
	{
		return;
	}

	TArray<FGuid> SelectedItemIDs = GetSelection();

	CacheStates();
	UpdateFilteredRoot(/*bRestoreSelection*/false);

	SetSelection(SelectedItemIDs);
}

void SCompactTreeView::SetSelection(const TConstArrayView<FGuid> Selection)
{
	if (!StateItemTree)
	{
		return;
	}

	if (bIsSettingSelection)
	{
		return;
	}

	TArray<TSharedPtr<FStateItem>> SelectedStates;
	FindStatesByIDRecursive(FilteredRootItem, Selection, SelectedStates);

	bIsSettingSelection = true;

	StateItemTree->ClearSelection();
	StateItemTree->SetItemSelection(SelectedStates, true);

	OnSelectionChangedInternal(SelectedStates);

	if (SelectedStates.Num() == 1)
	{
		StateItemTree->RequestScrollIntoView(SelectedStates[0]);
	}

	bIsSettingSelection = false;
}

TArray<FGuid> SCompactTreeView::GetSelection() const
{
	TArray<FGuid> SelectedItemIDs;

	TArray<TSharedPtr<FStateItem>> SelectedItems = StateItemTree->GetSelectedItems();
	for (const TSharedPtr<FStateItem>& Item : SelectedItems)
	{
		if (Item)
		{
			SelectedItemIDs.Add(Item->StateID);
		}
	}

	return SelectedItemIDs;
}

TSharedPtr<SWidget> SCompactTreeView::GetWidgetToFocusOnOpen()
{
	return SearchBox;
}

void SCompactTreeView::CacheStates()
{
	RootItem = CreateStateItemInternal();

	CacheStatesInternal();

	FilteredRootItem = RootItem;
}

TSharedRef<SWidget> SCompactTreeView::CreateNameWidgetInternal(TSharedPtr<FStateItem> Item) const
{
	return SNew(STextBlock)
		.Margin(FVector4f{4.f, 0.f, 0.f, 0.f})
		.Text(Item->Desc)
		.TextStyle(TextStyle)
		.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
		.IsEnabled(Item->bIsEnabled)
		.HighlightText_Lambda([this]() { return SearchBox.IsValid() ? SearchBox->GetText() : FText::GetEmpty(); });
}

TSharedRef<ITableRow> SCompactTreeView::GenerateStateItemRow(TSharedPtr<FStateItem> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<SHorizontalBox> Container = SNew(SHorizontalBox)
		.ToolTipText(Item->TooltipText)

		// Icon
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(0.0f, 2.0f, 4.0f, 2.0f)
		.AutoWidth()
		[
			SNew(SImage)
			.Visibility(Item->Icon ? EVisibility::Visible : EVisibility::Collapsed)
			.DesiredSizeOverride(FVector2D(16.0f, 16.0f))
			.Image(Item->Icon)
			.ColorAndOpacity(Item->Color)
			.IsEnabled(Item->bIsEnabled)
		]

		// Name
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			CreateNameWidgetInternal(Item)
		];

	return GenerateStateItemRowInternal(Item, OwnerTable, Container);
}

TSharedRef<SCompactTreeView::FStateItem> SCompactTreeView::CreateStateItemInternal() const
{
	return MakeShared<FStateItem>();
}

TSharedRef<STableRow<TSharedPtr<SCompactTreeView::FStateItem>>> SCompactTreeView::GenerateStateItemRowInternal(
	TSharedPtr<FStateItem> Item, const TSharedRef<STableViewBase>& OwnerTable, TSharedRef<SHorizontalBox> Box)
{
	return SNew(STableRow<TSharedPtr<FStateItem>>, OwnerTable)
		[
			Box
		];
}

void SCompactTreeView::OnSelectionChangedInternal(TConstArrayView<TSharedPtr<FStateItem>> SelectedStates)
{
	// Nothing to do in base implementation
}

void SCompactTreeView::OnUpdatingFilteredRootInternal()
{
	// Nothing to do in base implementation
}

void SCompactTreeView::GetStateItemChildren(TSharedPtr<FStateItem> Item, TArray<TSharedPtr<FStateItem>>& OutItems) const
{
	if (Item.IsValid())
	{
		OutItems = Item->Children;
	}
}

void SCompactTreeView::OnStateItemSelected(TSharedPtr<FStateItem> SelectedItem, const ESelectInfo::Type Type)
{
	// Skip selection from code
	if (Type == ESelectInfo::Direct)
	{
		return;
	}

	if (bIsSettingSelection)
	{
		return;
	}

	if (OnSelectionChanged.IsBound())
	{
		TArray<FGuid> SelectedStateIDs;

		TArray<TSharedPtr<FStateItem>> Selection = StateItemTree->GetSelectedItems();
		for (const TSharedPtr<FStateItem>& Item : Selection)
		{
			if (Item)
			{
				SelectedStateIDs.Add(Item->StateID);
			}
		}

		OnSelectionChanged.Execute(SelectedStateIDs);
	}
}

void SCompactTreeView::OnStateItemExpansionChanged(TSharedPtr<FStateItem> ExpandedItem, const bool bInExpanded) const
{
	// Do not save expansion state when restoring expansion state, or when showing filtered results.
	if (bIsRestoringExpansion || FilteredRootItem != RootItem)
	{
		return;
	}

	if (ExpandedItem.IsValid() && ExpandedItem->StateID.IsValid())
	{
		FStateExpansionState& ExpansionState = StateExpansionStates.FindOrAdd(FObjectKey(WeakStateTree.Get()));
		if (bInExpanded)
		{
			ExpansionState.CollapsedStates.Remove(ExpandedItem->StateID);
		}
		else
		{
			ExpansionState.CollapsedStates.Add(ExpandedItem->StateID);
		}
	}
}

void SCompactTreeView::OnSearchBoxTextChanged(const FText& NewText)
{
	if (!StateItemTree.IsValid())
	{
		return;
	}

	NewText.ToString().ParseIntoArrayWS(FilterStrings);
	FilterStrings.RemoveAll([](const FString& String) { return String.IsEmpty(); });

	UpdateFilteredRoot();
}

void SCompactTreeView::UpdateFilteredRoot(const bool bRestoreSelection)
{
	OnUpdatingFilteredRootInternal();

	FilteredRootItem.Reset();

	TArray<FGuid> Selection;
	if (bRestoreSelection)
	{
		Selection = GetSelection();
	}

	if (FilterStrings.IsEmpty())
	{
		// Show all when there's no filter string.
		FilteredRootItem = RootItem;
		StateItemTree->SetTreeItemsSource(&FilteredRootItem->Children);
		RestoreExpansionState();
	}
	else
	{
		FilteredRootItem = CreateStateItemInternal();
		FilterStateItemChildren(FilterStrings, /*bParentMatches*/false, RootItem->Children, FilteredRootItem->Children);

		StateItemTree->SetTreeItemsSource(&FilteredRootItem->Children);
		ExpandAll(FilteredRootItem->Children);
	}

	if (bRestoreSelection)
	{
		SetSelection(Selection);
	}

	StateItemTree->RequestTreeRefresh();
}

int32 SCompactTreeView::FilterStateItemChildren(const TArray<FString>& FilterStrings, const bool bParentMatches, const TArray<TSharedPtr<FStateItem>>& SourceArray, TArray<TSharedPtr<FStateItem>>& OutDestArray)
{
	int32 NumFound = 0;

	auto MatchFilter = [&FilterStrings](const TSharedPtr<FStateItem>& SourceItem)
	{
		const FString ItemName = SourceItem->Desc.ToString();
		for (const FString& Filter : FilterStrings)
		{
			if (ItemName.Contains(Filter))
			{
				return true;
			}
		}
		return false;
	};

	for (const TSharedPtr<FStateItem>& SourceItem : SourceArray)
	{
		// Check if our name matches the filters
		// If bParentMatches is true, the search matched a parent category.
		const bool bMatchesFilters = bParentMatches || MatchFilter(SourceItem);

		int32 NumChildren = 0;
		if (bMatchesFilters)
		{
			NumChildren++;
		}

		// if we don't match, then we still want to check all our children
		TArray<TSharedPtr<FStateItem>> FilteredChildren;
		NumChildren += FilterStateItemChildren(FilterStrings, bMatchesFilters, SourceItem->Children, FilteredChildren);

		// then add this item to the destination array
		if (NumChildren > 0)
		{
			TSharedPtr<FStateItem>& NewItem = OutDestArray.Add_GetRef(MakeShared<FStateItem>());
			*NewItem = *SourceItem;
			NewItem->Children = FilteredChildren;

			NumFound += NumChildren;
		}
	}

	return NumFound;
}

void SCompactTreeView::ExpandAll(const TArray<TSharedPtr<FStateItem>>& Items)
{
	for (const TSharedPtr<FStateItem>& Item : Items)
	{
		StateItemTree->SetItemExpansion(Item, true);
		ExpandAll(Item->Children);
	}
}

bool SCompactTreeView::FindStateByIDRecursive(const TSharedPtr<FStateItem>& Item, const FGuid StateID, TArray<TSharedPtr<FStateItem>>& OutPath)
{
	OutPath.Push(Item);

	if (Item->StateID == StateID)
	{
		return true;
	}

	for (const TSharedPtr<FStateItem>& ChildItem : Item->Children)
	{
		if (FindStateByIDRecursive(ChildItem, StateID, OutPath))
		{
			return true;
		}
	}

	OutPath.Pop();

	return false;
}

void SCompactTreeView::FindStatesByIDRecursive(const TSharedPtr<FStateItem>& Item, const TConstArrayView<FGuid> StateIDs, TArray<TSharedPtr<FStateItem>>& OutStates)
{
	if (StateIDs.Contains(Item->StateID))
	{
		OutStates.Add(Item);
	}

	for (const TSharedPtr<FStateItem>& ChildItem : Item->Children)
	{
		FindStatesByIDRecursive(ChildItem, StateIDs, OutStates);
	}
}

void SCompactTreeView::RestoreExpansionState()
{
	if (!StateItemTree.IsValid())
	{
		return;
	}

	const FStateExpansionState& ExpansionState = StateExpansionStates.FindOrAdd(FObjectKey(WeakStateTree.Get()));

	TGuardValue RestoringExpansionGuard(bIsRestoringExpansion, true);

	// Default state is expanded.
	ExpandAll(FilteredRootItem->Children);

	// Collapse the ones that are specifically collapsed.
	for (const FGuid& StateID : ExpansionState.CollapsedStates)
	{
		TArray<TSharedPtr<FStateItem>> Path;
		if (FindStateByIDRecursive(FilteredRootItem, StateID, Path))
		{
			StateItemTree->SetItemExpansion(Path.Last(), false);
		}
	}
}

} // UE::StateTree::Editor

#undef LOCTEXT_NAMESPACE
