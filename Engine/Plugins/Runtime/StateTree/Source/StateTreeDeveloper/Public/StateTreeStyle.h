// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

#define UE_API STATETREEDEVELOPER_API

enum class EStateTreeStateSelectionBehavior : uint8;
enum class EStateTreeStateType : uint8;

class ISlateStyle;

class FStateTreeStyle : public FSlateStyleSet
{
public:
	static UE_API FStateTreeStyle& Get();

	static UE_API const FSlateBrush* GetBrushForSelectionBehaviorType(EStateTreeStateSelectionBehavior InSelectionBehavior, bool bInHasChildren, EStateTreeStateType InStateType);

protected:
	struct FContentRootScope
	{
		FContentRootScope(FSlateStyleSet* InStyle, const FString& NewContentRoot)
			: Style(InStyle)
			, PreviousContentRoot(InStyle->GetContentRootDir())
		{
			Style->SetContentRoot(NewContentRoot);
		}

		~FContentRootScope()
		{
			Style->SetContentRoot(PreviousContentRoot);
		}
	private:
		FSlateStyleSet* Style;
		FString PreviousContentRoot;
	};

	friend class FStateTreeDeveloperModule;

	UE_API explicit FStateTreeStyle(const FName& InStyleSetName);

	UE_API static void Register();
	UE_API static void Unregister();

	UE_API static const FString EngineSlateContentDir;
	UE_API static const FString StateTreePluginContentDir;
	UE_API static const FLazyName StateTitleTextStyleName;
private:
	UE_API FStateTreeStyle();
};

#undef UE_API
