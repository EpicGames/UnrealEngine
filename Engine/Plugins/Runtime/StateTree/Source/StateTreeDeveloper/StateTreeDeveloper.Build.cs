// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

namespace UnrealBuildTool.Rules
{
	public class StateTreeDeveloper : StateTreeModuleBase
	{
		public StateTreeDeveloper(ReadOnlyTargetRules Target) : base(Target)
		{
			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

			bAllowUETypesInNamespaces = true;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"SlateCore",
				}
			);
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"CoreUObject",
					"InputCore",
					"Slate",
					"StateTreeModule",
				}
			);

			SetupStateTreeDebuggingSupport(Target);

			if (IsStateTreeTraceRecordingSupported(Target))
			{
				PublicDependencyModuleNames.AddRange(
					new []
					{
						"RewindDebuggerRuntimeInterface",
						"TraceLog"
					}
				);
			}
		}
	}
}