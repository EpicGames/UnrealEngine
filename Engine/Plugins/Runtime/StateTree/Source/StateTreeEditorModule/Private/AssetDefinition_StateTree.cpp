// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_StateTree.h"
#include "Modules/ModuleManager.h"
#include "SStateTreeDiff.h"
#include "StateTree.h"
#include "StateTreeEditorModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_StateTree)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

FText UAssetDefinition_StateTree::GetAssetDisplayName() const
{
	return LOCTEXT("FAssetTypeActions_StateTree", "StateTree");
}

FLinearColor UAssetDefinition_StateTree::GetAssetColor() const
{
	return FColor(201, 185, 29);
}

TSoftClassPtr<> UAssetDefinition_StateTree::GetAssetClass() const
{
	return UStateTree::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_StateTree::GetAssetCategories() const
{
	static FAssetCategoryPath Categories[] =
		{
			FAssetCategoryPath(EAssetCategoryPaths::AI, LOCTEXT("AssetDefinition_StateTreeSubMenu_AI", "NPC"), ECategoryMenuType::Section),
			FAssetCategoryPath(EAssetCategoryPaths::Gameplay, LOCTEXT("StateTree_SubMenu_Gameplay", "Other"), ECategoryMenuType::Section)
		};

	return Categories;
}

EAssetCommandResult UAssetDefinition_StateTree::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	FStateTreeEditorModule& EditorModule = FModuleManager::LoadModuleChecked<FStateTreeEditorModule>("StateTreeEditorModule");
	for (UStateTree* StateTree : OpenArgs.LoadObjects<UStateTree>())
	{
		EditorModule.CreateStateTreeEditor(EToolkitMode::Standalone, OpenArgs.ToolkitHost, StateTree);
	}
	return EAssetCommandResult::Handled;
}

EAssetCommandResult UAssetDefinition_StateTree::PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const
{
	const UStateTree* OldStateTree = Cast<UStateTree>(DiffArgs.OldAsset);
	const UStateTree* NewStateTree = Cast<UStateTree>(DiffArgs.NewAsset);

	if (OldStateTree == nullptr || NewStateTree == nullptr)
	{
		return EAssetCommandResult::Unhandled;
	}

	UE::StateTree::Diff::SDiffWidget::CreateDiffWindow(OldStateTree, NewStateTree, DiffArgs.OldRevision, DiffArgs.NewRevision, UStateTree::StaticClass());
	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
