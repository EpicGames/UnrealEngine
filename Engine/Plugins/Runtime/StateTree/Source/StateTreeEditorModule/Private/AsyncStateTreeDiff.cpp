// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncStateTreeDiff.h"
#include "DiffUtils.h"
#include "SStateTreeView.h"
#include "StateTreeDiffHelper.h"
#include "StateTreeEditorData.h"
#include "StateTreeState.h"
#include "StateTreeViewModel.h"

namespace UE::StateTree::Diff
{

static bool AreObjectsEqual(const UObject* ObjectA, const UObject* ObjectB)
{
	if (!ObjectA || !ObjectB)
	{
		return ObjectA == ObjectB;
	}

	if (ObjectA->GetClass() != ObjectB->GetClass())
	{
		return false;
	}

	if (ObjectA != ObjectB)
	{
		const FProperty* ClassProperty = ObjectA->GetClass()->PropertyLink;
		while (ClassProperty)
		{
			if (!ClassProperty->Identical_InContainer(ObjectA, ObjectB, /*ArrayIndex*/ 0, PPF_DeepComparison | PPF_ForDiff))
			{
				return false;
			}
			ClassProperty = ClassProperty->PropertyLinkNext;
		}
	}

	return true;
}

static bool AreNodesEqual(const FStateTreeEditorNode& NodeA, const FStateTreeEditorNode& NodeB)
{
	auto CompareDataView = [](FStateTreeDataView ViewA, FStateTreeDataView ViewB)
		{
			bool bInstanceEqual = false;
			if (ViewA.GetStruct() == ViewB.GetStruct())
			{
				if (Cast<UClass>(ViewA.GetStruct()))
				{
					bInstanceEqual = AreObjectsEqual(static_cast<const UObject*>(ViewA.GetMemory()), static_cast<const UObject*>(ViewB.GetMemory()));
				}
				else if (const UScriptStruct* ScriptStruct = Cast<const UScriptStruct>(ViewA.GetStruct()))
				{
					bInstanceEqual = ScriptStruct->CompareScriptStruct(ViewA.GetMemory(), ViewB.GetMemory(), PPF_DeepComparison | PPF_ForDiff);
				}
			}
			return bInstanceEqual;
		};
	auto CompareStructView = [](FStructView ViewA, FStructView ViewB)
		{
			bool bEqual = false;
			if (ViewA.GetScriptStruct() == ViewB.GetScriptStruct())
			{
				bEqual = ViewA.GetScriptStruct() == nullptr || ViewA.GetScriptStruct()->CompareScriptStruct(ViewA.GetMemory(), ViewB.GetMemory(), PPF_DeepComparison | PPF_ForDiff);
			}
			return bEqual;
		};

	return CompareDataView(NodeA.GetInstance(), NodeB.GetInstance())
		&& CompareDataView(NodeA.GetExecutionRuntimeData(), NodeB.GetExecutionRuntimeData())
		&& CompareStructView(NodeA.GetNode(), NodeB.GetNode())
		&& NodeA.ExpressionIndent == NodeB.ExpressionIndent
		&& NodeA.ExpressionOperand == NodeB.ExpressionOperand;
}

static bool AreNodeArraysEqual(const TArray<FStateTreeEditorNode>& ArrayA, const TArray<FStateTreeEditorNode>& ArrayB)
{
	const int32 Count = ArrayA.Num();
	if (Count != ArrayB.Num())
	{
		return false;
	}
	for (int32 Index = 0; Index < Count; Index++)
	{
		if (!AreNodesEqual(ArrayA[Index], ArrayB[Index]))
		{
			return false;
		}
	}
	return true;
}
static bool AreStateTreeStatePropertyBagsEqual(const FInstancedPropertyBag& ParametersA, const FInstancedPropertyBag& ParametersB)
{
	if (ParametersA.GetNumPropertiesInBag() != ParametersB.GetNumPropertiesInBag())
	{
		return false;
	}
	
	const UPropertyBag* BagA = ParametersA.GetPropertyBagStruct();
	const UPropertyBag* BagB = ParametersB.GetPropertyBagStruct();
	if (!BagA || !BagB)
	{
		return BagA == BagB;
	}

	const TConstArrayView<FPropertyBagPropertyDesc> DescsA = BagA->GetPropertyDescs();
	const TConstArrayView<FPropertyBagPropertyDesc> DescsB = BagB->GetPropertyDescs();
	const int32 Count = DescsA.Num();
	for (int32 Index = 0; Index < Count; Index++)
	{
		if (DescsA[Index].Name != DescsB[Index].Name
			|| !DescsA[Index].CompatibleType(DescsB[Index]))
		{
			return false;
		}

		const FName Name = DescsA[Index].Name;
		TValueOrError<FString, EPropertyBagResult> SerializedA = ParametersA.GetValueSerializedString(Name);
		TValueOrError<FString, EPropertyBagResult> SerializedB = ParametersB.GetValueSerializedString(Name);
		if (SerializedA.HasError() || SerializedB.HasError())
		{
			return false;
		}

		if (SerializedA.HasValue() != SerializedB.HasValue())
		{
			return false;
		}

		if (SerializedA.GetValue() != SerializedB.GetValue())
		{
			return false;
		}
	}
	return true;
}

static bool AreStateTreeStateParametersEqual(const FStateTreeStateParameters& ParametersA, const FStateTreeStateParameters& ParametersB)
{
	if (!AreStateTreeStatePropertyBagsEqual(ParametersA.Parameters, ParametersB.Parameters))
	{
		return false;
	}
	if (ParametersA.PropertyOverrides != ParametersB.PropertyOverrides)
	{
		return false;
	}

	return true;
}

static bool ArePropertiesEqual(const UStateTreeState* StateA, const UStateTreeState* StateB)
{
	if (!StateA || !StateB)
	{
		return StateA == StateB;
	}

	return StateA->Name == StateB->Name
		&& StateA->Tag == StateB->Tag
		&& StateA->ColorRef == StateB->ColorRef
		&& StateA->Type == StateB->Type
		&& StateA->SelectionBehavior == StateB->SelectionBehavior;
}

static bool AreParametersEqual(const UStateTreeState* StateA, const UStateTreeState* StateB)
{
	return AreStateTreeStateParametersEqual(StateA->Parameters, StateB->Parameters);
}

static bool AreConditionsEqual(const UStateTreeState* StateA, const UStateTreeState* StateB)
{
	return AreNodeArraysEqual(StateA->EnterConditions, StateB->EnterConditions);
}

static bool AreConsiderationsEqual(const UStateTreeState* StateA, const UStateTreeState* StateB)
{
	return AreNodeArraysEqual(StateA->Considerations, StateB->Considerations);
}

static bool AreTasksEqual(const UStateTreeState* StateA, const UStateTreeState* StateB)
{
	return AreNodeArraysEqual(StateA->Tasks, StateB->Tasks);
}

static bool AreTransitionsEqual(const UStateTreeState* StateA, const UStateTreeState* StateB)
{
	if (StateA->Transitions.Num() != StateB->Transitions.Num())
	{
		return false;
	}

	for (int32 Index = 0; Index < StateA->Transitions.Num(); Index++)
	{
		const FStateTreeTransition* TransitionA = &StateA->Transitions[Index];
		const FStateTreeTransition* TransitionB = &StateB->Transitions[Index];
		if (!TransitionA || !TransitionB)
		{
			return TransitionA == TransitionB;
		}
		// Not checking transitions on IDs
		const bool bEqual = FStateTreeTransition::StaticStruct()->CompareScriptStruct(TransitionA, TransitionB, 0);
		if (!bEqual)
		{
			return false;
		}
	}
	return true;
}

static bool AreStateTreePropertiesEqual(const UStateTreeEditorData* StateTreeDataA, const UStateTreeEditorData* StateTreeDataB)
{
	// Check the differences in Bindings
	bool bBindingsEqual = StateTreeDataA->EditorBindings.GetBindings().Num() == StateTreeDataB->EditorBindings.GetBindings().Num();
	if (bBindingsEqual)
	{
		for (const FPropertyBindingBinding& PropertyPathBinding : StateTreeDataA->EditorBindings.GetBindings())
		{
			const FPropertyBindingPath& PropertyPathTarget = PropertyPathBinding.GetTargetPath();
			if (StateTreeDataB->EditorBindings.HasBinding(PropertyPathBinding.GetTargetPath()))
			{
				if (*StateTreeDataA->EditorBindings.GetBindingSource(PropertyPathTarget) != *StateTreeDataB->EditorBindings.GetBindingSource(PropertyPathTarget))
				{
					bBindingsEqual = false;
					break;
				}
			}
			else
			{
				bBindingsEqual = false;
				break;
			}
		}
	}
	if (!bBindingsEqual)
	{
		return false;
	}

	// Check the differences in Evaluators and Tasks
	if (!AreNodeArraysEqual(StateTreeDataA->Evaluators, StateTreeDataB->Evaluators))
	{
		return false;
	}
	if (!AreNodeArraysEqual(StateTreeDataA->GlobalTasks, StateTreeDataB->GlobalTasks))
	{
		return false;
	}

	// Check the differences in StateTree root level parameters
	if (!AreStateTreeStatePropertyBagsEqual(StateTreeDataA->GetRootParametersPropertyBag(), StateTreeDataB->GetRootParametersPropertyBag()))
	{
		return false;
	}
	return true;
}

static FPropertySoftPath GetPropertyPath(const FPropertyBindingPath& StateTreePropertyPath, const UStateTreeState* StateTreeState)
{
	TArray<FName> Path;
	auto CheckNodes = [&StateTreePropertyPath, &Path](const TArray<FStateTreeEditorNode>& List, FName PathSegmentName)
	{
		for (int i = 0; i < List.Num(); i++)
		{
			if (List[i].ID == StateTreePropertyPath.GetStructID())
			{
				Path.Add(PathSegmentName);
				Path.Add(FName(FString::FromInt(i)));
				const FStateTreeDataView Instance = List[i].GetInstance();
				if (Cast<UClass>(Instance.GetStruct()))
				{
					Path.Add(FName("InstanceObject"));
				}
				else
				{
					Path.Add(FName("Instance"));
				}
				return true;
			}
		}
		return false;
	};
	auto CheckTransitions = [StateTreePropertyPath, &Path](const TArray<FStateTreeTransition>& List, FName PathSegmentName)
	{
		for (int i = 0; i < List.Num(); i++)
		{
			if (List[i].ID == StateTreePropertyPath.GetStructID())
			{
				Path.Add(PathSegmentName);
				Path.Add(FName(FString::FromInt(i)));
				return true;
			}
		}
		return false;
	};
	if (CheckNodes(StateTreeState->EnterConditions, "EnterConditions")
		|| CheckNodes(StateTreeState->Tasks, "Tasks")
		|| CheckTransitions(StateTreeState->Transitions, "Transitions"))
	{
		for (const FPropertyBindingPathSegment& PropertySegment : StateTreePropertyPath.GetSegments())
		{
			Path.Add(PropertySegment.GetName());
		}
	}

	return FPropertySoftPath(Path);
}

static void GetBindingsDifferences(UStateTreeEditorData* StateTreeDataA, UStateTreeEditorData* StateTreeDataB, TArray<FSingleDiffEntry>& OutDiffEntries)
{
	struct FBindingDiff
	{
		FPropertyBindingPath TargetPath;
		FPropertyBindingPath SourcePathA;
		FPropertyBindingPath SourcePathB;
	};

	TArray<FBindingDiff> BindingDiffs;
	// Check the differences in Bindings
	for (const FPropertyBindingBinding& PropertyPathBinding : StateTreeDataA->EditorBindings.GetBindings())
	{
		FPropertyBindingPath PropertyPathTarget = PropertyPathBinding.GetTargetPath();
		FPropertyBindingPath PropertyPathSource = PropertyPathBinding.GetSourcePath();

		FBindingDiff Entry;
		Entry.TargetPath = PropertyPathTarget;
		Entry.SourcePathA = PropertyPathSource;
		BindingDiffs.Add(Entry);
	}
	for (const FPropertyBindingBinding& PropertyPathBinding : StateTreeDataB->EditorBindings.GetBindings())
	{
		FPropertyBindingPath PropertyPathTarget = PropertyPathBinding.GetTargetPath();
		FPropertyBindingPath PropertyPathSource = PropertyPathBinding.GetSourcePath();
		bool bFound = false;
		for (FBindingDiff& Diff : BindingDiffs)
		{
			if (Diff.TargetPath == PropertyPathTarget)
			{
				Diff.SourcePathB = PropertyPathSource;
				bFound = true;
				break;
			}
		}
		if (!bFound)
		{
			FBindingDiff Entry;
			Entry.TargetPath = PropertyPathTarget;
			Entry.SourcePathB = PropertyPathSource;
			BindingDiffs.Add(Entry);
		}
	}

	for (const FBindingDiff& DiffEntry : BindingDiffs)
	{
		if (DiffEntry.SourcePathA != DiffEntry.SourcePathB)
		{
			const UStateTreeState* TargetStateA = StateTreeDataA->GetStateByStructID(DiffEntry.TargetPath.GetStructID());
			const UStateTreeState* TargetStateB = StateTreeDataB->GetStateByStructID(DiffEntry.TargetPath.GetStructID());

			if (TargetStateA && TargetStateB)
			{
				FStateSoftPath StatePathA(TargetStateA);
				FStateSoftPath StatePathB(TargetStateB);
				FPropertySoftPath PropertyPath = GetPropertyPath(DiffEntry.TargetPath, TargetStateA);

				EStateDiffType StateTreeDiffType = EStateDiffType::BindingChanged;
				if (DiffEntry.SourcePathA.IsPathEmpty())
				{
					StateTreeDiffType = EStateDiffType::BindingAddedToB;
				}
				else if (DiffEntry.SourcePathB.IsPathEmpty())
				{
					StateTreeDiffType = EStateDiffType::BindingAddedToA;
				}
				OutDiffEntries.Add(FSingleDiffEntry(StatePathA, StatePathB, StateTreeDiffType, PropertyPath));
			}
		}
	}
}

FAsyncDiff::FAsyncDiff(const TSharedRef<SStateTreeView>& LeftTree, const TSharedRef<SStateTreeView>& RightTree)
	: TAsyncTreeDifferences(RootNodesAttribute(LeftTree), RootNodesAttribute(RightTree))
	, LeftView(LeftTree)
	, RightView(RightTree)
{}

TAttribute<TArray<TWeakObjectPtr<UStateTreeState>>> FAsyncDiff::RootNodesAttribute(TWeakPtr<SStateTreeView> StateTreeView)
{
	return TAttribute<TArray<TWeakObjectPtr<UStateTreeState>>>::CreateLambda([StateTreeView]()
	{
		if (const TSharedPtr<SStateTreeView> TreeView = StaticCastSharedPtr<SStateTreeView>(StateTreeView.Pin()))
		{
			TArray<TWeakObjectPtr<UStateTreeState>> SubTrees;
			TreeView->GetViewModel()->GetSubTrees(SubTrees);
			return SubTrees;
		}
		return TArray<TWeakObjectPtr<UStateTreeState>>();
	});
}

void FAsyncDiff::GetStatesDifferences(TArray<FSingleDiffEntry>& OutDiffEntries) const
{
	TArray<FString> RemovedStates;
	TArray<FString> AddedStates;
	ForEach(ETreeTraverseOrder::PreOrder, [&](const TUniquePtr<DiffNodeType>& Node) -> ETreeTraverseControl
	{
		FStateSoftPath StatePath;
		UStateTreeState* LeftState = Node->ValueA.Get();
		UStateTreeState* RightState = Node->ValueB.Get();
		if (LeftState)
		{
			StatePath = FStateSoftPath(LeftState);
		}
		else if (RightState)
		{
			StatePath = FStateSoftPath(RightState);
		}

		EStateDiffType StateTreeDiffType = EStateDiffType::Invalid;
		bool bSkipChildren = false;
		switch (Node->DiffResult)
		{
		case ETreeDiffResult::MissingFromTree1:
			StateTreeDiffType = EStateDiffType::StateAddedToB;
			AddedStates.Add(StatePath.ToDisplayName(true));
			if (RemovedStates.Contains(StatePath.ToDisplayName(true)))
			{
				StateTreeDiffType = EStateDiffType::StateMoved;
			}
			bSkipChildren = true;
			break;
		case ETreeDiffResult::MissingFromTree2:
			StateTreeDiffType = EStateDiffType::StateAddedToA;
			RemovedStates.Add(StatePath.ToDisplayName(true));
			if (AddedStates.Contains(StatePath.ToDisplayName(true)))
			{
				StateTreeDiffType = EStateDiffType::StateMoved;
			}
			bSkipChildren = true;
			break;
		case ETreeDiffResult::DifferentValues:
			StateTreeDiffType = EStateDiffType::StateChanged;
			break;
		case ETreeDiffResult::Identical:
			StateTreeDiffType = EStateDiffType::Identical;
			if (LeftState && RightState)
			{
				if (LeftState->bEnabled != RightState->bEnabled)
				{
					StateTreeDiffType = RightState->bEnabled ? EStateDiffType::StateEnabled : EStateDiffType::StateDisabled;
				}
			}
			break;
		default:
			return ETreeTraverseControl::Continue;
		}

		if (StateTreeDiffType == EStateDiffType::Identical)
		{
			return ETreeTraverseControl::Continue;
		}

		if (StateTreeDiffType == EStateDiffType::StateMoved)
		{
			for (FSingleDiffEntry& DiffEntry : OutDiffEntries)
			{
				if (DiffEntry.Identifier.ToDisplayName(true) == StatePath.ToDisplayName(true))
				{
					if (DiffEntry.DiffType == EStateDiffType::StateAddedToA)
					{
						DiffEntry.SecondaryIdentifier = StatePath;
					}
					else
					{
						DiffEntry.SecondaryIdentifier = DiffEntry.Identifier;
						DiffEntry.Identifier = StatePath;
					}
					DiffEntry.DiffType = EStateDiffType::StateMoved;

					// For now, we are skipping children, we may need to revisit that
					return ETreeTraverseControl::SkipChildren;
				}
			}
		}

		OutDiffEntries.Add(FSingleDiffEntry(StatePath, StateTreeDiffType));
		
		return bSkipChildren ? ETreeTraverseControl::SkipChildren : ETreeTraverseControl::Continue;
	});
}

void FAsyncDiff::GetStateTreeDifferences(TArray<FSingleDiffEntry>& OutDiffEntries) const
{
	if (LeftView && RightView)
	{
		const FStateTreeViewModel* LeftViewModel = LeftView->GetViewModel().Get();
		const FStateTreeViewModel* RightViewModel = RightView->GetViewModel().Get();
		if (LeftViewModel && RightViewModel)
		{
			UStateTreeEditorData* LeftEditorData = Cast<UStateTreeEditorData>(LeftViewModel->GetStateTree()->EditorData);
			UStateTreeEditorData* RightEditorData = Cast<UStateTreeEditorData>(RightViewModel->GetStateTree()->EditorData);
			if (!AreStateTreePropertiesEqual(LeftEditorData, RightEditorData))
			{
				OutDiffEntries.Add(FSingleDiffEntry(
					/*Identifier*/FStateSoftPath(),
					EStateDiffType::StateTreePropertiesChanged));
			}

			GetStatesDifferences(OutDiffEntries);

			GetBindingsDifferences(LeftEditorData, RightEditorData, OutDiffEntries);
		}
	}
}

} // UE::AsyncStateTreeDiff


bool TTreeDiffSpecification<TWeakObjectPtr<UStateTreeState>>::AreValuesEqual(const TWeakObjectPtr<UStateTreeState>& StateTreeNodeA, const TWeakObjectPtr<UStateTreeState>& StateTreeNodeB, TArray<FPropertySoftPath>*) const
{
	const TStrongObjectPtr<UStateTreeState> StrongStateA = StateTreeNodeA.Pin();
	const TStrongObjectPtr<UStateTreeState> StrongStateB = StateTreeNodeB.Pin();
	const UStateTreeState* StateA = StrongStateA.Get();
	const UStateTreeState* StateB = StrongStateB.Get();

	if (!StateA || !StateB)
	{
		return StateA == StateB;
	}

	using namespace UE::StateTree::Diff;
	return ArePropertiesEqual(StateA, StateB)
		&& AreParametersEqual(StateA, StateB)
		&& AreConditionsEqual(StateA, StateB)
		&& AreTasksEqual(StateA, StateB)
		&& AreTransitionsEqual(StateA, StateB)
		&& AreConsiderationsEqual(StateA, StateB);
}

bool TTreeDiffSpecification<TWeakObjectPtr<UStateTreeState>>::AreMatching(const TWeakObjectPtr<UStateTreeState>& StateTreeNodeA, const TWeakObjectPtr<UStateTreeState>& StateTreeNodeB, TArray<FPropertySoftPath>*) const
{
	const TStrongObjectPtr<UStateTreeState> StrongStateA = StateTreeNodeA.Pin();
	const TStrongObjectPtr<UStateTreeState> StrongStateB = StateTreeNodeB.Pin();
	const UStateTreeState* StateA = StrongStateA.Get();
	const UStateTreeState* StateB = StrongStateB.Get();
	if (!StateA || !StateB)
	{
		return StateA == StateB;
	}

	return StateA->ID == StateB->ID;
}

void TTreeDiffSpecification<TWeakObjectPtr<UStateTreeState>>::GetChildren(const TWeakObjectPtr<UStateTreeState>& InParent, TArray<TWeakObjectPtr<UStateTreeState>>& OutChildren) const
{
	const TStrongObjectPtr<UStateTreeState> StrongParent = InParent.Pin();
	if (UStateTreeState* InParentPtr = StrongParent.Get())
	{
		OutChildren.Reserve(InParentPtr->Children.Num());
		for (const TObjectPtr<UStateTreeState>& Child : InParentPtr->Children)
		{
			OutChildren.Add(Child);
		}
	}
}