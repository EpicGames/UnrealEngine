// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AsyncTreeDifferences.h"

#define UE_API STATETREEEDITORMODULE_API

class SStateTreeView;
class UStateTreeState;

namespace UE::StateTree::Diff
{
struct FSingleDiffEntry;

class FAsyncDiff : public TAsyncTreeDifferences<TWeakObjectPtr<UStateTreeState>>
{
public:
	UE_API FAsyncDiff(const TSharedRef<SStateTreeView>& LeftTree, const TSharedRef<SStateTreeView>& RightTree);

	UE_API void GetStateTreeDifferences(TArray<FSingleDiffEntry>& OutDiffEntries) const;

private:
	UE_API void GetStatesDifferences(TArray<FSingleDiffEntry>& OutDiffEntries) const;

	static UE_API TAttribute<TArray<TWeakObjectPtr<UStateTreeState>>> RootNodesAttribute(TWeakPtr<SStateTreeView> StateTreeView);

	TSharedPtr<SStateTreeView> LeftView;
	TSharedPtr<SStateTreeView> RightView;
};

} // UE::StateTree::Diff

template <>
class TTreeDiffSpecification<TWeakObjectPtr<UStateTreeState>>
{
public:
	UE_API bool AreValuesEqual(const TWeakObjectPtr<UStateTreeState>& StateTreeNodeA, const TWeakObjectPtr<UStateTreeState>& StateTreeNodeB, TArray<FPropertySoftPath>* OutDifferingProperties = nullptr) const;

	UE_API bool AreMatching(const TWeakObjectPtr<UStateTreeState>& StateTreeNodeA, const TWeakObjectPtr<UStateTreeState>& StateTreeNodeB, TArray<FPropertySoftPath>* OutDifferingProperties = nullptr) const;

	UE_API void GetChildren(const TWeakObjectPtr<UStateTreeState>& InParent, TArray<TWeakObjectPtr<UStateTreeState>>& OutChildren) const;

	bool ShouldMatchByValue(const TWeakObjectPtr<UStateTreeState>&) const
	{
		return false;
	}

	bool ShouldInheritEqualFromChildren(const TWeakObjectPtr<UStateTreeState>&, const TWeakObjectPtr<UStateTreeState>&) const
	{
		return false;
	}
};

#undef UE_API
