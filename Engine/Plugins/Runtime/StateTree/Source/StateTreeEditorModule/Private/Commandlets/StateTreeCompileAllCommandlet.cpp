// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/StateTreeCompileAllCommandlet.h"
#include "Modules/ModuleManager.h"
#include "StateTree.h"
#include "PackageHelperFunctions.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "SourceControlOperations.h"
#include "SourceControlHelpers.h"
#include "StateTreeCompilerLog.h"
#include "StateTreeEditor.h"
#include "StateTreeCompiler.h"
#include "StateTreeDelegates.h"
#include "StateTreeEditingSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeCompileAllCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogStateTreeCompile, Log, Log);

UStateTreeCompileAllCommandlet::UStateTreeCompileAllCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UStateTreeCompileAllCommandlet::Main(const FString& Params)
{
	// Parse command line.
	TArray<FString> Tokens;
	TArray<FString> Switches;

	// want everything in upper case, it's a mess otherwise
	const FString ParamsUpperCase = Params.ToUpper();
	const TCHAR* Parms = *ParamsUpperCase;
	ParseCommandLine(Parms, Tokens, Switches);

	// Source control
	bool bNoSourceControl = Switches.Contains(TEXT("nosourcecontrol"));
	FScopedSourceControl SourceControl;
	SourceControlProvider = bNoSourceControl ? nullptr : &SourceControl.GetProvider();

	// Load assets
	UE_LOGF(LogStateTreeCompile, Display, "Loading Asset Registry...");
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName);
	AssetRegistryModule.Get().SearchAllAssets(/*bSynchronousSearch =*/true);
	UE_LOGF(LogStateTreeCompile, Display, "Finished Loading Asset Registry.");
	
	UE_LOGF(LogStateTreeCompile, Display, "Gathering All StateTrees From Asset Registry...");
	TArray<FAssetData> StateTreeAssetList;
	AssetRegistryModule.Get().GetAssetsByClass(UStateTree::StaticClass()->GetClassPathName(), StateTreeAssetList, /*bSearchSubClasses*/false);

	int32 Counter = 0;
	for (const FAssetData& Asset : StateTreeAssetList)
	{
		const FString ObjectPath = Asset.GetObjectPathString();
		UE_LOGF(LogStateTreeCompile, Display, "Loading and Compiling: '%ls' [%d/%d]...", *ObjectPath, Counter+1, StateTreeAssetList.Num());

		UStateTree* StateTree = Cast<UStateTree>(StaticLoadObject(Asset.GetClass(), /*Outer*/nullptr, *ObjectPath, /*FileName*/nullptr, LOAD_NoWarn));
		if (StateTree == nullptr)
		{
			UE_LOGF(LogStateTreeCompile, Error, "Failed to Load: '%ls'.", *ObjectPath);
		}
		else
		{
			CompileAndSaveStateTree(StateTree);
		}
		Counter++;
	}
		
	return 0;
}

bool UStateTreeCompileAllCommandlet::CompileAndSaveStateTree(TNonNullPtr<UStateTree> StateTree) const
{
	UPackage* Package = StateTree->GetPackage();
	const FString PackageFileName = SourceControlHelpers::PackageFilename(Package);

	// Compile the StateTree asset.
	FStateTreeCompilerLog Log;
	const bool bSuccess = UStateTreeEditingSubsystem::CompileStateTree(StateTree, Log);

	if (!bSuccess)
	{
		return false;
	}

	// Check out the StateTree asset
	if (SourceControlProvider != nullptr)
	{
		const FSourceControlStatePtr SourceControlState = SourceControlProvider->GetState(PackageFileName, EStateCacheUsage::ForceUpdate);

		if (SourceControlState.IsValid())
		{
			FString OtherCheckedOutUser;
			if (SourceControlState->IsCheckedOutOther(&OtherCheckedOutUser))
			{
				UE_LOGF(LogStateTreeCompile, Error, "Overwriting package %ls already checked out by %ls, will not submit", *PackageFileName, *OtherCheckedOutUser);
				return false;
			}
			else if (!SourceControlState->IsCurrent())
			{
				UE_LOGF(LogStateTreeCompile, Error, "Overwriting package %ls (not at head revision), will not submit", *PackageFileName);
				return false;
			}
			else if (SourceControlState->IsCheckedOut() || SourceControlState->IsAdded())
			{
				UE_LOGF(LogStateTreeCompile, Log, "Package %ls already checked out", *PackageFileName);
			}
			else if (SourceControlState->IsSourceControlled())
			{
				UE_LOGF(LogStateTreeCompile, Log, "Checking out package %ls from revision control", *PackageFileName);
				if (SourceControlProvider->Execute(ISourceControlOperation::Create<FCheckOut>(), PackageFileName) != ECommandResult::Succeeded)
				{
					UE_LOGF(LogStateTreeCompile, Log, "Failed to check out package %ls from revision control", *PackageFileName);
					return false;
				}
			}
		}
	}

	// Save StateTree asset.
	if (!SavePackageHelper(Package, PackageFileName))
	{
		UE_LOGF(LogStateTreeCompile, Error, "Failed to save %ls.", *PackageFileName);
		return false;
	}

	UE_LOGF(LogStateTreeCompile, Log, "Compile and save %ls succeeded.", *PackageFileName);

	return true;
}

