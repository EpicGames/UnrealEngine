// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/Commandlet.h"
#include "StateTreeCompileAllCommandlet.generated.h"

class ISourceControlProvider;

class UStateTree;

/**
 * Commandlet to recompile all StateTree assets in the project
 */
UCLASS()
class UStateTreeCompileAllCommandlet : public UCommandlet
{
	GENERATED_BODY()
public:
	UStateTreeCompileAllCommandlet(const FObjectInitializer& ObjectInitializer);

	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface

protected:
	bool CompileAndSaveStateTree(TNonNullPtr<UStateTree> StateTree) const;

	ISourceControlProvider* SourceControlProvider = nullptr;
};


