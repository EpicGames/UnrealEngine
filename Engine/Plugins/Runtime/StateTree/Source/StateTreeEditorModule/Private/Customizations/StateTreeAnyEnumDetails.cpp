// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeAnyEnumDetails.h"
#include "Widgets/Input/SComboButton.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "StateTreePropertyHelpers.h"
#include "StateTreeAnyEnum.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"

TSharedRef<IPropertyTypeCustomization> FStateTreeAnyEnumDetails::MakeInstance()
{
	return MakeShareable(new FStateTreeAnyEnumDetails);
}

void FStateTreeAnyEnumDetails::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructProperty = StructPropertyHandle;
	PropUtils = StructCustomizationUtils.GetPropertyUtilities().Get();

	ValueProperty = StructProperty->GetChildHandle(TEXT("Value"));
	EnumProperty = StructProperty->GetChildHandle(TEXT("Enum"));

	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(250.f)
	.VAlign(VAlign_Center)
	[
		SNew(SComboButton)
		.OnGetMenuContent(this, &FStateTreeAnyEnumDetails::OnGetComboContent)
		.ContentPadding(FMargin(2.0f, 0.0f))
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text(this, &FStateTreeAnyEnumDetails::GetDescription)
			.Font(IDetailLayoutBuilder::GetDetailFontBold())
		]
	];
}

void FStateTreeAnyEnumDetails::CustomizeChildren( TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
}

FText FStateTreeAnyEnumDetails::GetDescription() const
{
	check(StructProperty);

	FStateTreeAnyEnum StateTreeEnum;
	FPropertyAccess::Result Result = UE::StateTree::PropertyHelpers::GetStructValue<FStateTreeAnyEnum>(StructProperty, StateTreeEnum);
	if (Result == FPropertyAccess::Success)
	{
		if (StateTreeEnum.Enum)
		{
			return StateTreeEnum.Enum->GetDisplayNameTextByValue(int64(StateTreeEnum.Value));
		}
	}
	else if (Result == FPropertyAccess::MultipleValues)
	{
		return LOCTEXT("MultipleSelected", "Multiple Selected");
	}

	return LOCTEXT("None", "None");
}

TSharedRef<SWidget> FStateTreeAnyEnumDetails::OnGetComboContent() const
{
	check(StructProperty);

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection*/true, /*InCommandList*/nullptr);

	bool bSuccess = false;
	FStateTreeAnyEnum StateTreeEnum;
	if (UE::StateTree::PropertyHelpers::GetStructValue<FStateTreeAnyEnum>(StructProperty, StateTreeEnum) == FPropertyAccess::Success)
	{
		if (StateTreeEnum.Enum)
		{
			// This is the number of entry in the enum, - 1, because the last item in an enum is the _MAX item
			for (int32 i = 0; i < StateTreeEnum.Enum->NumEnums() - 1; i++)
			{
				const int64 Value = StateTreeEnum.Enum->GetValueByIndex(i);
				MenuBuilder.AddMenuEntry(StateTreeEnum.Enum->GetDisplayNameTextByIndex(i), TAttribute<FText>(), FSlateIcon(),
					FUIAction(FExecuteAction::CreateLambda([this, Value]() {
						if (ValueProperty)
						{
							ValueProperty->SetValue(uint32(Value));
						}
					}))
				);
			}
			bSuccess = true;
		}
	}

	if (!bSuccess)
	{
		MenuBuilder.AddMenuEntry(LOCTEXT("None", "None"), TAttribute<FText>(), FSlateIcon(), FUIAction());
	}

	return MenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
