// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeBindingExtension.h"

#include "ScopedTransaction.h"
#include "DetailLayoutBuilder.h"
#include "Features/IModularFeatures.h"
#include "IPropertyAccessEditor.h"
#include "IPropertyUtilities.h"
#include "StateTreeAnyEnum.h"
#include "StateTreeCompiler.h"
#include "StateTreeEditorPropertyBindings.h"
#include "StateTreeGraphSchema.h"
#include "StateTreeNodeBase.h"
#include "StateTreeTaskBase.h"
#include "UObject/EnumProperty.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "StateTreePropertyRef.h"
#include "StateTreePropertyRefHelpers.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "IPropertyUtilities.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "IDetailChildrenBuilder.h"
#include "IStructureDataProvider.h"
#include "PropertyBindingExtension.h"
#include "StateTreeEditorNodeUtils.h"
#include "StateTreeEditorData.h"
#include "StateTreeDelegate.h"
#include "StateTreeEditorDataClipboardHelpers.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"

namespace UE::StateTree::PropertyBinding
{

/**
 * Information for the types gathered from a FStateTreePropertyRef property meta-data 
 * Kept this type to facilitate introduction of base type FPropertyInfoOverride.
 */
struct FRefTypeInfo : UE::PropertyBinding::FPropertyInfoOverride
{
};

const FName AllowAnyBindingName(TEXT("AllowAnyBinding"));

void MakeBindingChainFromBindingPathIndirections(int32 InAccessibleStructIndex, TConstArrayView<FPropertyBindingPathIndirection> InResolvedIndirections, TArray<FBindingChainElement>& OutBindingChain)
{
	OutBindingChain.Reset();

	OutBindingChain.Reserve(InResolvedIndirections.Num() + 1);
	OutBindingChain.Emplace(nullptr, InAccessibleStructIndex);
	for (int32 Idx = 0; Idx < InResolvedIndirections.Num(); ++Idx)
	{
		const FPropertyBindingPathIndirection& Indirection = InResolvedIndirections[Idx];
		const int32 ArrayIndex = Indirection.GetArrayIndex();

		OutBindingChain.Emplace(const_cast<FProperty*>(Indirection.GetProperty()), ArrayIndex);

		// Indirection format: [ArrayProperty, InnerIndexInArray].[InnerProperty, -1] while BindingChain format skips the InnerProperty item.
		if (ArrayIndex != INDEX_NONE && Indirection.GetAccessType() == EPropertyBindingPropertyAccessType::IndexArray)
		{
			Idx += 1;
		}
	}
}

UObject* FindEditorBindingsOwner(UObject* InObject)
{
	UObject* Result = nullptr;

	for (UObject* Outer = InObject; Outer; Outer = Outer->GetOuter())
	{
		IStateTreeEditorPropertyBindingsOwner* BindingOwner = Cast<IStateTreeEditorPropertyBindingsOwner>(Outer);
		if (BindingOwner)
		{
			Result = Outer;
			break;
		}
	}
	return Result;
}

UObject* FindStateOrEditorDataOwner(UObject* InObject)
{
	UObject* Result = nullptr;

	for (UObject* Outer = InObject; Outer; Outer = Outer->GetOuter())
	{
		if (Outer->IsA<UStateTreeEditorData>() || Outer->IsA<UStateTreeState>())
		{
			Result = Outer;
			break;
		}
	}

	return Result;
}

static bool IsDelegateDispatcherProperty(const FProperty* InProperty)
{
	if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
	{
		return StructProperty->Struct == FStateTreeDelegateDispatcher::StaticStruct();
	}

	return false;
}

static bool IsDelegateListenerProperty(const FProperty* InProperty)
{
	if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
	{
		return StructProperty->Struct == FStateTreeDelegateListener::StaticStruct() || StructProperty->Struct == FStateTreeTransitionDelegateListener::StaticStruct();
	}

	return false;
}

static bool CanTargetBindToTaskCompletion(const FProperty* InProperty)
{
	return IsDelegateListenerProperty(InProperty);
}

static bool IsSourceTaskCompletionNode(const UStruct* SourceStruct)
{
	// Can only bind to task (or global task) template node.
	//The InstanceData struct is also in the GetAccessibleStructs array.
	return SourceStruct != nullptr && SourceStruct->IsChildOf(FStateTreeTaskBase::StaticStruct());
}

static TSharedPtr<const IPropertyHandle> GetParentPropertyWithNodeID(TSharedPtr<const IPropertyHandle> InPropertyHandle, FGuid& OutStructID)
{
	TSharedPtr<const IPropertyHandle> CurrentPropertyHandle = InPropertyHandle;
	while (CurrentPropertyHandle.IsValid())
	{
		if (const FString* IDString = CurrentPropertyHandle->GetInstanceMetaData(UE::PropertyBinding::MetaDataStructIDName))
		{
			LexFromString(OutStructID, **IDString);
			return CurrentPropertyHandle;
		}

		CurrentPropertyHandle = CurrentPropertyHandle->GetParentHandle();
	}

	return nullptr;
}

static bool IsNodeStructProperty(const TSharedPtr<const IPropertyHandle>& InPropertyHandle)
{
	void* Address = nullptr;
	if (InPropertyHandle && InPropertyHandle->GetValueData(Address) == FPropertyAccess::Success)
	{
		const FStructProperty* StructProperty = CastField<FStructProperty>(InPropertyHandle->GetProperty());
		if (Address && StructProperty)
		{
			const UScriptStruct* Struct = nullptr;

			// @todo: refactor this into UE::Object::GetInnerStruct(const UScriptStruct*)
			if (StructProperty->Struct == TBaseStructure<FInstancedStruct>::Get())
			{
				Struct = (static_cast<FInstancedStruct*>(Address))->GetScriptStruct();
			}
			else if (StructProperty->Struct == TBaseStructure<FSharedStruct>::Get())
			{
				Struct = (static_cast<FSharedStruct*>(Address))->GetScriptStruct();
			}
			else
			{
				Struct = StructProperty->Struct;
			}

			if (Struct)
			{
				return Struct->IsChildOf<FStateTreeNodeBase>();
			}
		}
	}

	return false;
}

FOnStateTreePropertyBindingChanged STATETREEEDITORMODULE_API OnStateTreePropertyBindingChanged;

struct FStateTreeCachedBindingData : UE::PropertyBinding::FCachedBindingData, TSharedFromThis<FStateTreeCachedBindingData>
{
	friend FStateTreeBindingExtension;

	using Super = FCachedBindingData;

	FStateTreeCachedBindingData(IPropertyBindingBindingCollectionOwner* InPropertyBindingsOwner, const FPropertyBindingPath& InTargetPath,
		const TSharedPtr<const IPropertyHandle>& InPropertyHandle,
		const TConstArrayView<TInstancedStruct<FPropertyBindingBindableStructDescriptor>>& InAccessibleStructs)
		: FCachedBindingData(InPropertyBindingsOwner, InTargetPath, InPropertyHandle, InAccessibleStructs)
	{
		FGuid StructID;
		TSharedPtr<const IPropertyHandle> RootPropertyHandle = GetParentPropertyWithNodeID(InPropertyHandle, StructID);

		// Record the Category of root property(that has StructID) on the struct as it affects binding eligibility
		if (RootPropertyHandle.IsValid() && StructID == GetTargetPath().GetStructID())
		{
			const FProperty* RootProperty = RootPropertyHandle->GetProperty();
			HierarchicalPropertyUsage = GetUsageFromMetaData(RootProperty);
		}
	}

	virtual TNotNull<const UEdGraphSchema_K2*> GetEdGraphSchema() const override
	{
		return GetDefault<UE::StateTree::Editor::UEdGraphSchema_StateTree>();
	}

	virtual bool AddBindingInternal(TConstStructView<FPropertyBindingBindableStructDescriptor> InDescriptor,
		FPropertyBindingPath& InOutSourcePath,
		const FPropertyBindingPath& InTargetPath) override
	{
		const FStateTreeBindableStructDesc& SourceDesc = InDescriptor.Get<FStateTreeBindableStructDesc>();
		IStateTreeEditorPropertyBindingsOwner* BindingsOwner = Cast<IStateTreeEditorPropertyBindingsOwner>(GetOwner());
		FStateTreeEditorPropertyBindings* EditorBindings = BindingsOwner->GetPropertyEditorBindings();

		if (SourceDesc.DataSource == EStateTreeBindableStructSource::PropertyFunction)
		{
			const UScriptStruct* PropertyFunctionNodeStruct = nullptr;

			BindingsOwner->EnumerateBindablePropertyFunctionNodes([ID = SourceDesc.ID, &PropertyFunctionNodeStruct](const UScriptStruct* NodeStruct, const FStateTreeBindableStructDesc& Desc, const FPropertyBindingDataView Value)
				{
					if (Desc.ID == ID)
					{
						PropertyFunctionNodeStruct = NodeStruct;
						return EStateTreeVisitor::Break;
					}

					return EStateTreeVisitor::Continue;
				});

			if (ensure(PropertyFunctionNodeStruct))
			{
				// If there are no segments, bindings leads directly into source struct's single output property. It's path has to be recovered.
				if (InOutSourcePath.NumSegments() == 0)
				{
					const FProperty* SingleOutputProperty = UE::StateTree::GetStructSingleOutputProperty(*SourceDesc.Struct);
					check(SingleOutputProperty);

					FPropertyBindingPathSegment SingleOutputPropertySegment = FPropertyBindingPathSegment(SingleOutputProperty->GetFName());
					InOutSourcePath = EditorBindings->AddFunctionBinding(PropertyFunctionNodeStruct, { SingleOutputPropertySegment }, InTargetPath);
				}
				else
				{
					InOutSourcePath = EditorBindings->AddFunctionBinding(PropertyFunctionNodeStruct, InOutSourcePath.GetSegments(), InTargetPath);
				}

				return true;
			}
		}
		else if (IsOutputBinding())
		{
			if (ensureMsgf(SourceDesc.DataSource == EStateTreeBindableStructSource::Parameter
				|| SourceDesc.DataSource == EStateTreeBindableStructSource::StateParameter,
				TEXT("Output binding can only has Parameter as Source")))
			{
				EditorBindings->AddOutputBinding(InOutSourcePath, InTargetPath);
			}

			return true;
		}
		else if (GetPropertyHandle()
			&& CanTargetBindToTaskCompletion(GetPropertyHandle()->GetProperty())
			&& IsSourceTaskCompletionNode(SourceDesc.Struct)
			&& InOutSourcePath.GetSegments().Num() == 0
			&& NextCompletionCondition.IsSet())
		{
			// To create a task completion binding, SetNextTaskCompletionBinding must be set first
			EditorBindings->AddTaskCompletionBinding(SourceDesc.ID, InTargetPath, NextCompletionCondition.GetValue());
			NextCompletionCondition.Reset();
			return true;
		}

		return false;
	}

	void SetNextTaskCompletionBinding(UE::StateTree::ETaskCompletionCondition Condition)
	{
		NextCompletionCondition = Condition;
	}

	void ResetTaskCompletionBinding()
	{
		NextCompletionCondition.Reset();
	}

	virtual void UpdatePropertyReferenceTooltip(const FProperty& InProperty, FTextBuilder& InOutTextBuilder) const override
	{
		if (InProperty.HasMetaData(PropertyRefHelpers::IsRefToArrayName))
		{
			InOutTextBuilder.AppendLineFormat(LOCTEXT("PropertyRefBindingTooltipArray", "Supported types are Array of {0}"),
				FText::FromString(InProperty.GetMetaData(PropertyRefHelpers::RefTypeName)));
		}
		else
		{
			InOutTextBuilder.AppendLineFormat(LOCTEXT("PropertyRefBindingTooltip", "Supported types are {0}"),
				FText::FromString(InProperty.GetMetaData(PropertyRefHelpers::RefTypeName)));
			if (InProperty.HasMetaData(PropertyRefHelpers::CanRefToArrayName))
			{
				InOutTextBuilder.AppendLine(LOCTEXT("PropertyRefBindingTooltipCanSupportArray", "Supports Arrays"));
			}
		}
	}

	virtual void UpdateSourcePropertyPath(const TConstStructView<FPropertyBindingBindableStructDescriptor> InDescriptor, const FPropertyBindingPath& InSourcePath, FString& OutString) override
	{
		const FStateTreeBindableStructDesc& SourceDesc = InDescriptor.Get<FStateTreeBindableStructDesc>();
		// Making first segment of the path invisible for the user if it's property function's single output property.
		if (SourceDesc.DataSource == EStateTreeBindableStructSource::PropertyFunction && GetStructSingleOutputProperty(*SourceDesc.Struct))
		{
			OutString = InSourcePath.ToString(/*HighlightedSegment*/ INDEX_NONE, /*HighlightPrefix*/ nullptr, /*HighlightPostfix*/ nullptr, /*bOutputInstances*/ false, 1);
		}
		else if (GetPropertyHandle()
			&& CanTargetBindToTaskCompletion(GetPropertyHandle()->GetProperty())
			&& InSourcePath.GetSegments().Num() == 0
			&& IsSourceTaskCompletionNode(SourceDesc.Struct))
		{
			if (const FPropertyBindingBinding* FoundBinding = FindBinding(FPropertyBindingBindingCollection::ESearchMode::Exact))
			{
				const FStateTreePropertyPathBinding* Binding = static_cast<const FStateTreePropertyPathBinding*>(FoundBinding);
				if (Binding->GetCompletionBinding().IsSet())
				{
					OutString = FString::Printf(TEXT("(%s)"), *UEnum::GetDisplayValueAsText(Binding->GetCompletionBinding().GetValue()).ToString());
				}
			}
		}
	}

	virtual void GetSourceDataViewForNewBinding(TNotNull<IPropertyBindingBindingCollectionOwner*> InBindingsOwner, TConstStructView<FPropertyBindingBindableStructDescriptor> InDescriptor, FPropertyBindingDataView& OutSourceDataView) override
	{
		const FStateTreeBindableStructDesc& SourceDesc = InDescriptor.Get<FStateTreeBindableStructDesc>();
		if (SourceDesc.DataSource == EStateTreeBindableStructSource::PropertyFunction)
		{
			OutSourceDataView = FPropertyBindingDataView(SourceDesc.Struct, nullptr);
		}
		else
		{
			Super::GetSourceDataViewForNewBinding(InBindingsOwner, InDescriptor, OutSourceDataView);
		}
	}

	virtual FPinInfo GetPinInfoForProperty(TNotNull<const FProperty*> InProperty, FPropertyBindingDataView InTargetDataView) const override
	{
		const bool bIsPropertyRef = PropertyRefHelpers::IsPropertyRef(*InProperty);
		if (bIsPropertyRef && InTargetDataView.IsValid())
		{
			FPinInfo Result = Super::GetPinInfoForProperty(InProperty, InTargetDataView);
			// Use internal type to construct PinType if it's property of PropertyRef type.
			TArray<FPropertyBindingPathIndirection> TargetIndirections;
			if (ensure(GetTargetPath().ResolveIndirectionsWithValue(InTargetDataView, TargetIndirections)))
			{
				const void* PropertyRef = TargetIndirections.Last().GetPropertyAddress();
				Result.Type = PropertyRefHelpers::GetPropertyRefInternalTypeAsPin(*InProperty, PropertyRef);
			}
			return Result;
		}

		if (IsDelegateListenerProperty(InProperty))
		{
			FPinInfo Result;
			Result.Type.PinCategory = UEdGraphSchema_K2::PC_Struct;
			Result.Type.PinSubCategoryObject = FStateTreeDelegateListener::StaticStruct();
			Result.IconBrush = FAppStyle::GetBrush("Icons.Event");
			Result.IconColor = FLinearColor::White;
			return Result;
		}

		return Super::GetPinInfoForProperty(InProperty, InTargetDataView);
	}

	virtual bool IsPropertyReference(const FProperty& InProperty) override
	{
		return PropertyRefHelpers::IsPropertyRef(InProperty);
	}

	virtual void AddPropertyInfoOverride(const FProperty& Property, TArray<TSharedPtr<const UE::PropertyBinding::FPropertyInfoOverride>>& OutPropertyInfoOverrides) const override
	{
		// Add the PropertyRef property type with its RefTypes
		const FStructProperty* StructProperty = CastField<const FStructProperty>(&Property);
		if (StructProperty && StructProperty->Struct && StructProperty->Struct->IsChildOf(FStateTreePropertyRef::StaticStruct()))
		{
			TArray<FEdGraphPinType, TInlineAllocator<1>> PinTypes;
			if (StructProperty->Struct->IsChildOf(FStateTreeBlueprintPropertyRef::StaticStruct()))
			{
				void* PropertyRefAddress = nullptr;
				if (GetPropertyHandle()->GetValueData(PropertyRefAddress) == FPropertyAccess::Result::Success)
				{
					check(PropertyRefAddress);
					PinTypes.Add(PropertyRefHelpers::GetBlueprintPropertyRefInternalTypeAsPin(*static_cast<const FStateTreeBlueprintPropertyRef*>(PropertyRefAddress)));
				}
			}
			else
			{
				PinTypes = PropertyRefHelpers::GetPropertyRefInternalTypesAsPins(Property);
			}

			// If Property supports Arrays, add the Array version of these pin types
			if (GetPropertyHandle()->HasMetaData(PropertyRefHelpers::CanRefToArrayName))
			{
				const int32 PinTypeNum = PinTypes.Num();
				for (int32 Index = 0; Index < PinTypeNum; ++Index)
				{
					const FEdGraphPinType& SourcePinType = PinTypes[Index];
					if (!SourcePinType.IsArray())
					{
						FEdGraphPinType& PinType = PinTypes.Emplace_GetRef(SourcePinType);
						PinType.ContainerType = EPinContainerType::Array;
					}
				}
			}

			for (const FEdGraphPinType& PinType : PinTypes)
			{
				TSharedRef<FRefTypeInfo> RefTypeInfo = MakeShared<FRefTypeInfo>();
				RefTypeInfo->PinType = PinType;

				FString TypeName;
				if (UObject* SubCategoryObject = PinType.PinSubCategoryObject.Get())
				{
					TypeName = SubCategoryObject->GetName();
				}
				else
				{
					TypeName = PinType.PinCategory.ToString() + TEXT(" ") + PinType.PinSubCategory.ToString();
				}

				RefTypeInfo->TypeNameText = FText::FromString(TypeName);
				OutPropertyInfoOverrides.Emplace(MoveTemp(RefTypeInfo));
			}
		}
	}

	virtual bool CanBindToContextStructInternal(const UStruct* InStruct, const int32 InStructIndex) override
	{
		// Output Binding cannot write back to property functions
		if (IsOutputBinding())
		{
			return false;
		}

		// Do not allow to bind directly StateTree nodes
		// @todo: find a way to more specifically call out the context structs, e.g. pass the property path to the callback.
		if (InStruct != nullptr)
		{
			const bool bIsStateTreeNode = GetAccessibleStructs().ContainsByPredicate([InStruct](const TInstancedStruct<FPropertyBindingBindableStructDescriptor>& StructDesc)
			{
				const FStateTreeBindableStructDesc& AccessibleStruct = StructDesc.Get<FStateTreeBindableStructDesc>();
				return AccessibleStruct.DataSource != EStateTreeBindableStructSource::Context
					&& AccessibleStruct.DataSource != EStateTreeBindableStructSource::Parameter
					&& AccessibleStruct.DataSource != EStateTreeBindableStructSource::TransitionEvent
					&& AccessibleStruct.DataSource != EStateTreeBindableStructSource::StateEvent
					&& AccessibleStruct.DataSource != EStateTreeBindableStructSource::PropertyFunction
					&& AccessibleStruct.Struct == InStruct;
			});

			if (bIsStateTreeNode)
			{
				return false;
			}
		}

		const FStateTreeBindableStructDesc& StructDesc = GetBindableStructDescriptor(InStructIndex).Get<FStateTreeBindableStructDesc>();
		// Binding directly into PropertyFunction's struct is allowed if it contains a compatible single output property.
		if (StructDesc.DataSource == EStateTreeBindableStructSource::PropertyFunction)
		{
			const IStateTreeEditorPropertyBindingsOwner* BindingOwner = Cast<IStateTreeEditorPropertyBindingsOwner>(GetOwner());
			FStateTreeDataView DataView;

			// If DataView exists, struct is an instance of already bound function.
			if (BindingOwner == nullptr || BindingOwner->GetBindingDataViewByID(StructDesc.ID, DataView))
			{
				return false;
			}

			if (const FStructProperty* StructProperty = CastField<FStructProperty>(GetPropertyHandle()->GetProperty()))
			{
				// Disallow PropertyRef bound to property functions
				if (StructProperty->Struct && StructProperty->Struct->IsChildOf<FStateTreePropertyRef>())
				{
					return false;
				}
			}

			if (const FProperty* SingleOutputProperty = GetStructSingleOutputProperty(*StructDesc.Struct))
			{
				return CanBindToProperty(SingleOutputProperty, {FBindingChainElement(nullptr, InStructIndex), FBindingChainElement(const_cast<FProperty*>(SingleOutputProperty))});
			}
		}

		return Super::CanBindToContextStructInternal(InStruct, InStructIndex);
	}

	virtual bool CanAcceptPropertyOrChildrenInternal(const FProperty& SourceProperty, TConstArrayView<FBindingChainElement, int> InBindingChain) override
	{
		const int32 SourceStructIndex = InBindingChain[0].ArrayIndex;
		const FStateTreeBindableStructDesc& StructDesc = GetBindableStructDescriptor(SourceStructIndex).Get<FStateTreeBindableStructDesc>();

		const FProperty* TargetProperty = GetPropertyHandle()->GetProperty();

		if (StructDesc.DataSource == EStateTreeBindableStructSource::PropertyFunction)
		{
			IStateTreeEditorPropertyBindingsOwner* BindingOwner = Cast<IStateTreeEditorPropertyBindingsOwner>(GetOwner());
			FStateTreeDataView DataView;
			// If DataView exists, struct is an instance of already bound function.
			if (BindingOwner == nullptr || BindingOwner->GetBindingDataViewByID(StructDesc.ID, DataView))
			{
				return false;
			}

			// To avoid duplicates, PropertyFunction struct's children are not allowed to be bound if it contains a compatible single output property.
			if (const FProperty* SingleOutputProperty = GetStructSingleOutputProperty(*StructDesc.Struct))
			{
				if (CanBindToProperty(SingleOutputProperty, {FBindingChainElement(nullptr, SourceStructIndex), FBindingChainElement(const_cast<FProperty*>(SingleOutputProperty))}))
				{
					return false;
				}
			}

			// Binding to non-output PropertyFunctions properties is not allowed.
			if (InBindingChain.Num() == 1 && GetUsageFromMetaData(&SourceProperty) != EStateTreePropertyUsage::Output)
			{
				return false;
			}
		}

		if (PropertyRefHelpers::IsPropertyRef(*TargetProperty) && !PropertyRefHelpers::IsPropertyAccessibleForPropertyRef(SourceProperty, InBindingChain, StructDesc))
		{
			return false;
		}

		if (IsOutputBinding() && !IsPropertyAccessibleForOutputBinding(InBindingChain))
		{
			return false;
		}

		return true;
	}

	virtual bool DeterminePropertiesCompatibilityInternal(
		const FProperty* InSourceProperty,
		const FProperty* InTargetProperty,
		const void* InSourcePropertyValue,
		const void* InTargetPropertyValue,
		bool& bOutAreCompatible) const override
	{
		// @TODO: Refactor FStateTreePropertyBindings::ResolveCopyType() so that we can use it directly here.

		// output binding copies target into source
		if (IsOutputBinding())
		{
			auto CheckInvalidStructTypes = [&bOutAreCompatible]<typename ...T>(const FProperty* InProperty)
				{
					if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
					{
						if ((... || StructProperty->Struct->IsChildOf<T>()))
						{
							bOutAreCompatible = false;
							return true;
						}
					}

					return false;
				};

			Swap(InSourceProperty, InTargetProperty);
			Swap(InSourcePropertyValue, InTargetPropertyValue);

			// Output binding doesn't support special types
			if (CheckInvalidStructTypes.operator()<FStateTreeDelegateDispatcher, FStateTreeDelegateListener, FStateTreePropertyRef>(InSourceProperty) 
				|| CheckInvalidStructTypes.operator()<FStateTreeDelegateDispatcher, FStateTreeDelegateListener, FStateTreePropertyRef>(InTargetProperty))
			{
				return true;
			}

			// We need to handle compatibility check here because source and target property are reversed
			bOutAreCompatible = (UE::PropertyBinding::GetPropertyCompatibility(InSourceProperty, InTargetProperty) != UE::PropertyBinding::EPropertyCompatibility::Incompatible);
			return true;
		}

		const FStructProperty* TargetStructProperty = CastField<FStructProperty>(InTargetProperty);
		
		// AnyEnums need special handling.
		// It is a struct property but we want to treat it as an enum. We need to do this here, instead of 
		// GetPropertyCompatibility() because the treatment depends on the value too.
		// Note: AnyEnums will need special handling before they can be used for binding.
		if (TargetStructProperty && TargetStructProperty->Struct == FStateTreeAnyEnum::StaticStruct())
		{
			// If the AnyEnum has AllowAnyBinding, allow to bind to any enum.
			const bool bAllowAnyBinding = InTargetProperty->HasMetaData(AllowAnyBindingName);

			check(InTargetPropertyValue);
			const FStateTreeAnyEnum* TargetAnyEnum = static_cast<const FStateTreeAnyEnum*>(InTargetPropertyValue);

			// If the enum class is not specified, allow to bind to any enum, if the class is specified allow only that enum.
			if (const FByteProperty* SourceByteProperty = CastField<FByteProperty>(InSourceProperty))
			{
				if (UEnum* Enum = SourceByteProperty->GetIntPropertyEnum())
				{
					bOutAreCompatible = bAllowAnyBinding || TargetAnyEnum->Enum == Enum;
					return true;
				}
			}
			else if (const FEnumProperty* SourceEnumProperty = CastField<FEnumProperty>(InSourceProperty))
			{
				bOutAreCompatible = bAllowAnyBinding || TargetAnyEnum->Enum == SourceEnumProperty->GetEnum();
				return true;
			}
		}
		else if (TargetStructProperty && TargetStructProperty->Struct == FStateTreeStructRef::StaticStruct())
		{
			FString BaseStructName;
			const UScriptStruct* TargetStructRefBaseStruct = Compiler::GetBaseStructFromMetaData(InTargetProperty, BaseStructName);

			if (const FStructProperty* SourceStructProperty = CastField<FStructProperty>(InSourceProperty))
			{
				if (SourceStructProperty->Struct == TBaseStructure<FStateTreeStructRef>::Get())
				{
					FString SourceBaseStructName;
					const UScriptStruct* SourceStructRefBaseStruct = Compiler::GetBaseStructFromMetaData(SourceStructProperty, SourceBaseStructName);
					bOutAreCompatible = SourceStructRefBaseStruct && SourceStructRefBaseStruct->IsChildOf(TargetStructRefBaseStruct);
					return true;
				}
				else
				{
					bOutAreCompatible = SourceStructProperty->Struct && SourceStructProperty->Struct->IsChildOf(TargetStructRefBaseStruct);
					return true;
				}
			}
		}
		else if (TargetStructProperty && PropertyRefHelpers::IsPropertyRef(*TargetStructProperty))
		{
			check(InTargetPropertyValue);
			bOutAreCompatible = PropertyRefHelpers::IsPropertyRefCompatibleWithProperty(*TargetStructProperty, *InSourceProperty, InTargetPropertyValue, InSourcePropertyValue);
			return true;
		}
		else if (TargetStructProperty && IsDelegateListenerProperty(TargetStructProperty))
		{
			bOutAreCompatible = IsDelegateDispatcherProperty(InSourceProperty);
			return true;
		}

		return false;
	}

	virtual bool DeterminePropertyAndContextStructCompatibleInternal(const UStruct* SourceStruct, const FProperty* TargetProperty) const override
	{
		if (CanTargetBindToTaskCompletion(TargetProperty) && IsSourceTaskCompletionNode(SourceStruct))
		{
			return true;
		}
		return FCachedBindingData::DeterminePropertyAndContextStructCompatibleInternal(SourceStruct, TargetProperty);
	}

	virtual bool GetPropertyFunctionText(FConstStructView InPropertyFunctionStructView, FText& OutText) const override
	{
		const FStateTreeEditorNode& EditorNode = InPropertyFunctionStructView.Get<const FStateTreeEditorNode>();
		if (const FStateTreeNodeBase* Node = EditorNode.GetNode().GetPtr<const FStateTreeNodeBase>())
		{
			IStateTreeEditorPropertyBindingsOwner* BindingOwner = Cast<IStateTreeEditorPropertyBindingsOwner>(GetOwner());
			const FText Description = Node->GetDescription(GetSourcePath().GetStructID(), EditorNode.GetInstance(), FStateTreeBindingLookup(BindingOwner), EStateTreeNodeFormatting::Text);
			if (!Description.IsEmpty())
			{
				OutText = FText::FormatNamed(GetFormatableText(), TEXT("SourceStruct"), Description);
				return true;
			}
		}
		return false;
	}

	virtual bool GetPropertyFunctionTooltipText(FConstStructView InPropertyFunctionStructView, FText& OutText) const override
	{
		const FStateTreeEditorNode& EditorNode = InPropertyFunctionStructView.Get<const FStateTreeEditorNode>();
		if (const FStateTreeNodeBase* Node = EditorNode.GetNode().GetPtr<const FStateTreeNodeBase>())
		{
			IStateTreeEditorPropertyBindingsOwner* BindingOwner = Cast<IStateTreeEditorPropertyBindingsOwner>(GetOwner());
			const FText Description = Node->GetDescription(GetSourcePath().GetStructID(), EditorNode.GetInstance(), FStateTreeBindingLookup(BindingOwner), EStateTreeNodeFormatting::Text);
			if (!Description.IsEmpty())
			{
				OutText = FText::FormatNamed(GetFormatableTooltipText(), TEXT("SourceStruct"), Description);
				return true;
			}
		}
		return false;
	}

	virtual bool GetPropertyFunctionIconColor(FConstStructView InPropertyFunctionStructView, FLinearColor& OutColor) const override
	{
		const FStateTreeEditorNode& EditorNode = InPropertyFunctionStructView.Get<const FStateTreeEditorNode>();
		if (const FStateTreeNodeBase* Node = EditorNode.GetNode().GetPtr<const FStateTreeNodeBase>())
		{
			if (GetStructSingleOutputProperty(*Node->GetInstanceDataType()))
			{
				OutColor = Node->GetIconColor();
				return true;
			}
		}
		return false;
	}

	virtual bool GetPropertyFunctionImage(FConstStructView InPropertyFunctionStructView, const FSlateBrush*& OutImage) const override
	{
		const FStateTreeEditorNode& EditorNode = InPropertyFunctionStructView.Get<const FStateTreeEditorNode>();
		if (const FStateTreeNodeBase* Node = EditorNode.GetNode().GetPtr<const FStateTreeNodeBase>())
		{
			if (GetStructSingleOutputProperty(*Node->GetInstanceDataType()))
			{
				OutImage = UE::StateTreeEditor::EditorNodeUtils::ParseIcon(Node->GetIconName()).GetIcon();
				return true;
			}
		}
		return false;
	}

private:
	bool IsOutputBinding() const
	{
		return HierarchicalPropertyUsage == EStateTreePropertyUsage::Output;
	}

	bool IsPropertyAccessibleForOutputBinding(TConstArrayView<FBindingChainElement> InSourceBindingChain) const
	{
		const FStateTreeBindableStructDesc& SourceStructDesc = GetAccessibleStructs()[InSourceBindingChain[0].ArrayIndex].Get<FStateTreeBindableStructDesc>();

		switch (SourceStructDesc.DataSource)
		{
		case EStateTreeBindableStructSource::Parameter:
		case EStateTreeBindableStructSource::StateParameter:
			return true;
		default:
			return false;
		}
	}

	EStateTreePropertyUsage HierarchicalPropertyUsage = EStateTreePropertyUsage::Invalid;
	// Set before the AddBinding. Use to add TaskCompletion binding.
	TOptional<UE::StateTree::ETaskCompletionCondition> NextCompletionCondition;
};

/* Provides PropertyFunctionNode instance for a property node. */
class FStateTreePropertyFunctionNodeProvider : public IStructureDataProvider
{
public:
	FStateTreePropertyFunctionNodeProvider(IStateTreeEditorPropertyBindingsOwner& InBindingsOwner, FPropertyBindingPath InTargetPath)
		: BindingsOwner(Cast<UObject>(&InBindingsOwner))
		, TargetPath(MoveTemp(InTargetPath))
	{}
	
	virtual bool IsValid() const override
	{
		return GetPropertyFunctionEditorNodeView(BindingsOwner.Get(), TargetPath).IsValid();
	};

	virtual const UStruct* GetBaseStructure() const override
	{
		return FStateTreeEditorNode::StaticStruct();
	}
	
	virtual void GetInstances(TArray<TSharedPtr<FStructOnScope>>& OutInstances, const UStruct* ExpectedBaseStructure) const override
	{
		if (ExpectedBaseStructure)
		{
			const FStructView Node = GetPropertyFunctionEditorNodeView(BindingsOwner.Get(), TargetPath);

			if (Node.IsValid() && Node.GetScriptStruct()->IsChildOf(ExpectedBaseStructure))
			{
				OutInstances.Add(MakeShared<FStructOnScope>(Node.GetScriptStruct(), Node.GetMemory()));
			}
		}
	}

	static bool IsBoundToValidPropertyFunction(UObject& InBindingsOwner, const FPropertyBindingPath& InTargetPath)
	{
		return GetPropertyFunctionEditorNodeView(&InBindingsOwner, InTargetPath).IsValid();
	}
	
private:
	static FStructView GetPropertyFunctionEditorNodeView(UObject* RawBindingsOwner, const FPropertyBindingPath& InTargetPath)
	{
		if(IStateTreeEditorPropertyBindingsOwner* Owner = Cast<IStateTreeEditorPropertyBindingsOwner>(RawBindingsOwner))
		{
			FStateTreeEditorPropertyBindings* EditorBindings = Owner->GetPropertyEditorBindings();
			FPropertyBindingBinding* FoundBinding = EditorBindings->GetMutableBindings().FindByPredicate([&InTargetPath](const FStateTreePropertyPathBinding& Binding)
			{
				return Binding.GetTargetPath() == InTargetPath;
			});

			
			if (FoundBinding)
			{
				const FStructView EditorNodeView = FoundBinding->GetMutablePropertyFunctionNode();
				if (EditorNodeView.IsValid())
				{
					const FStateTreeEditorNode& EditorNode = EditorNodeView.Get<FStateTreeEditorNode>();
					if (EditorNode.GetNode().IsValid() && EditorNode.GetInstance().IsValid())
					{
						return EditorNodeView;
					}
				}
			}
		}

		return FStructView();
	}

	TWeakObjectPtr<UObject> BindingsOwner;
	FPropertyBindingPath TargetPath;
};

} // UE::StateTree::PropertyBinding

TSharedPtr<UE::PropertyBinding::FCachedBindingData> FStateTreeBindingExtension::CreateCachedBindingData(IPropertyBindingBindingCollectionOwner* InBindingsOwner, const FPropertyBindingPath& InTargetPath, const TSharedPtr<IPropertyHandle>& InPropertyHandle, const TConstArrayView<TInstancedStruct<FPropertyBindingBindableStructDescriptor>> InAccessibleStructs) const
{
	return MakeShared<UE::StateTree::PropertyBinding::FStateTreeCachedBindingData>(InBindingsOwner, InTargetPath, InPropertyHandle, InAccessibleStructs);
}

bool FStateTreeBindingExtension::CanBindToProperty(const FPropertyBindingPath& InTargetPath, const IPropertyHandle& InPropertyHandle) const
{
	const FProperty* PropertyToBind = InPropertyHandle.GetProperty();
	if (!PropertyToBind)
	{
		return false;
	}

	const bool bIsPropertyRef = UE::StateTree::PropertyRefHelpers::IsPropertyRef(*PropertyToBind);
	const bool bIsDelegateListener = UE::StateTree::PropertyBinding::IsDelegateListenerProperty(PropertyToBind);

	// Node struct properties are considered as binding target only if their value stay constant at runtime
	if (!bIsPropertyRef && !bIsDelegateListener)
	{
		TSharedPtr<const IPropertyHandle> CurrentPropertyHandle = InPropertyHandle.AsShared();
		while (CurrentPropertyHandle)
		{
			if (UE::StateTree::PropertyBinding::IsNodeStructProperty(CurrentPropertyHandle))
			{
				return false;
			}

			CurrentPropertyHandle = CurrentPropertyHandle->GetParentHandle();
		}
	}

	const EStateTreePropertyUsage Usage = UE::StateTree::GetUsageFromMetaData(InPropertyHandle.GetProperty());
	if (Usage == EStateTreePropertyUsage::Input || Usage == EStateTreePropertyUsage::Context)
	{
		// Allow to bind only to the main level on input and context properties.
		return InTargetPath.GetSegments().Num() == 1;
	}

	return Usage == EStateTreePropertyUsage::Parameter || Usage == EStateTreePropertyUsage::Output;
}

void FStateTreeBindingExtension::UpdateContextStruct(TConstStructView<FPropertyBindingBindableStructDescriptor> InStructDesc, FBindingContextStruct& InOutContextStruct, TMap<FString, FText>& InOutSectionNames) const
{
	using namespace UE::StateTree::Editor;

	const FStateTreeBindableStructDesc& StructDesc = InStructDesc.Get<FStateTreeBindableStructDesc>();
	// Mare sure same section names get exact same FText representation (binding widget uses IsIdentical() to compare the section names).
	if (const FText* SectionText = InOutSectionNames.Find(StructDesc.StatePath))
	{
		InOutContextStruct.Section = *SectionText;
	}
	else
	{
		InOutContextStruct.Section = InOutSectionNames.Add(StructDesc.StatePath, FText::FromString(StructDesc.StatePath));
	}

	// PropertyFunction overrides it's struct's icon color.
	if (StructDesc.DataSource == EStateTreeBindableStructSource::PropertyFunction)
	{
		if (const FProperty* OutputProperty = UE::StateTree::GetStructSingleOutputProperty(*StructDesc.Struct))
		{
			const UEdGraphSchema_StateTree* Schema = GetDefault<UEdGraphSchema_StateTree>();
			check(Schema);

			FEdGraphPinType PinType;
			if (Schema->ConvertPropertyToPinType(OutputProperty, PinType))
			{
				InOutContextStruct.Color = Schema->GetPinTypeColor(PinType);
			}
		}
	}
}

bool FStateTreeBindingExtension::GetPromotionToParameterOverrideInternal(const FProperty& InProperty, bool& bOutOverride) const
{
	if (const FStructProperty* StructProperty = CastField<FStructProperty>(&InProperty))
	{
		// Support Property Refs as even though these aren't bp types, the actual types that would be added are the ones in the meta-data RefType
		if (StructProperty->Struct && StructProperty->Struct->IsChildOf(FStateTreePropertyRef::StaticStruct()))
		{
			bOutOverride = true;
			return true;
		}
	}

	return false;
}

void FStateTreeBindingsChildrenCustomization::CustomizeChildren(IDetailChildrenBuilder& ChildrenBuilder, TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	TArray<UObject*> OuterObjects;
	InPropertyHandle->GetOuterObjects(OuterObjects);
	if (OuterObjects.Num() == 1)
	{
		FPropertyBindingPath TargetPath;
		UE::PropertyBinding::MakeStructPropertyPathFromPropertyHandle(InPropertyHandle, TargetPath);

		using FStateTreePropertyFunctionNodeProvider = UE::StateTree::PropertyBinding::FStateTreePropertyFunctionNodeProvider;
		UObject* BindingsOwner = UE::StateTree::PropertyBinding::FindEditorBindingsOwner(OuterObjects[0]);
		if (BindingsOwner && FStateTreePropertyFunctionNodeProvider::IsBoundToValidPropertyFunction(*BindingsOwner, TargetPath))
		{
			// Bound PropertyFunction takes control over property's children composition.
			const TSharedPtr<FStateTreePropertyFunctionNodeProvider> StructProvider = MakeShared<FStateTreePropertyFunctionNodeProvider>(*CastChecked<IStateTreeEditorPropertyBindingsOwner>(BindingsOwner), MoveTemp(TargetPath));
			// Create unique name to persists expansion state.
			const FName UniqueName = FName(LexToString(TargetPath.GetStructID()) + TargetPath.ToString());
			ChildrenBuilder.AddChildStructure(InPropertyHandle.ToSharedRef(), StructProvider, UniqueName);
		}
	}
}

TNotNull<const UEdGraphSchema_K2*> FStateTreeBindingExtension::GetEdGraphSchema() const
{
	return GetDefault<UE::StateTree::Editor::UEdGraphSchema_StateTree>();
}

void FStateTreeBindingExtension::CustomizeDetailWidgetRow(FDetailWidgetRow& InWidgetRow, const IDetailLayoutBuilder& InDetailBuilder, IPropertyBindingBindingCollectionOwner* InBindingsOwner,
	TSharedPtr<IPropertyHandle> InPropertyHandle, const FPropertyBindingPath& InTargetPath,
	TSharedPtr<UE::PropertyBinding::FCachedBindingData> InCachedBindingData) const
{
	auto IsClipboardSingleBinding = [](const UE::StateTreeEditor::FClipboardEditorData& InClipboard)
		{
			if (!(InClipboard.GetEditorNodesInBuffer().IsEmpty() && InClipboard.GetTransitionsInBuffer().IsEmpty()))
			{
				return false;
			}

			TConstArrayView<FStateTreePropertyPathBinding> Bindings = InClipboard.GetBindingsInBuffer();
			if (Bindings.IsEmpty())
			{
				return false;
			}

			// Every Binding target should either be a property function, or the target node.
			// @todo: detect cyclic reference of property function nodes
			const FGuid TargetNodeID = Bindings[0].GetTargetPath().GetStructID();
			TArray<FGuid, TInlineAllocator<8>> PropertyFunctionIDs;
			for (const FStateTreePropertyPathBinding& Binding : Bindings)
			{
				if (Binding.GetPropertyFunctionNode().IsValid())
				{
					PropertyFunctionIDs.Add(Binding.GetPropertyFunctionNode().Get<const FStateTreeEditorNode>().ID);
				}
			}

			for (const FStateTreePropertyPathBinding& Binding : Bindings)
			{
				const FGuid BindingTargetID = Binding.GetTargetPath().GetStructID();
				if (BindingTargetID != TargetNodeID && !PropertyFunctionIDs.Contains(BindingTargetID))
				{
					return false;
				}
			}

			return true;
		};

	auto ImportTextAsSingleBindingClipboard = [IsClipboardSingleBinding, InBindingsOwner, InPropertyHandle](bool bInProcessBuffer, UE::StateTreeEditor::FClipboardEditorData& OutClipboard)
		{
			TArray<UObject*> OuterObjects;
			InPropertyHandle->GetOuterObjects(OuterObjects);
			if (OuterObjects.Num() != 1)
			{
				return false;
			}

			constexpr UScriptStruct* TargetType = nullptr;
			UE::StateTreeEditor::ImportTextAsClipboardEditorData(TargetType, Cast<UStateTreeEditorData>(InBindingsOwner), UE::StateTree::PropertyBinding::FindStateOrEditorDataOwner(OuterObjects[0]), OutClipboard, bInProcessBuffer);

			return (OutClipboard.IsValid() || !bInProcessBuffer) && IsClipboardSingleBinding(OutClipboard);
		};

	FPropertyBindingExtension::CustomizeDetailWidgetRow(InWidgetRow, InDetailBuilder, InBindingsOwner, InPropertyHandle, InTargetPath, InCachedBindingData);

	// We will refresh the details view when any binding changed, so we can cache the value here
	constexpr FPropertyBindingBindingCollection::ESearchMode SearchMode = FPropertyBindingBindingCollection::ESearchMode::Exact;
	const bool bHasBinding = InCachedBindingData->HasBinding(SearchMode);

	if (bHasBinding)
	{
		// Disable Copy and Paste if it has a binding
		// SDetailSingleItemRow will override the default copy and paste action if FDetailWidgetRow::IsCopyPasteBound == true
		FUIAction EmptyUIAction = FUIAction(
			// create an empty ExecuteAction so that FDetailWidgetRow::IsCopyPasteBound returns true
			FExecuteAction::CreateLambda([](){}),
			FCanExecuteAction::CreateLambda([bHasBinding](){ return false; })
			);

		InWidgetRow.CopyAction(EmptyUIAction);
		InWidgetRow.PasteAction(EmptyUIAction);
	}

	// Copy binding action
	InWidgetRow.AddCustomContextMenuAction(FUIAction(
		FExecuteAction::CreateLambda([InCachedBindingData, SearchMode]()
			{
				using namespace UE::StateTree::PropertyBinding;
				if (const FStateTreeCachedBindingData* StateTreeCachedBindingData = static_cast<FStateTreeCachedBindingData*>(InCachedBindingData.Get()))
				{
					if (const FPropertyBindingBinding* PropertyBinding = StateTreeCachedBindingData->FindBinding(SearchMode))
					{
						if (const UStateTreeEditorData* EditorData = Cast<UStateTreeEditorData>(StateTreeCachedBindingData->GetOwner()))
						{
							UE::StateTreeEditor::FClipboardEditorData Clipboard;
							Clipboard.Append(EditorData, {&PropertyBinding, 1 });

							UE::StateTreeEditor::ExportTextAsClipboardEditorData(Clipboard);
						}
					}
				}
			}),
			FCanExecuteAction::CreateLambda([bHasBinding]()
			{
				return bHasBinding;
			})),

		LOCTEXT("CopyBinding", "Copy Binding"),
		LOCTEXT("CopyBindingTooltip", "Copy the binding of this property"),
		FSlateIcon());

	// Paste binding action
	InWidgetRow.AddCustomContextMenuAction(FUIAction(
		FExecuteAction::CreateLambda([ImportTextAsSingleBindingClipboard, InCachedBindingData, &InDetailBuilder, SearchMode]()
			{
				using namespace UE::StateTree::PropertyBinding;

				if (FStateTreeCachedBindingData* StateTreeCachedBindingData = static_cast<FStateTreeCachedBindingData*>(InCachedBindingData.Get()))
				{
					UStateTreeEditorData* EditorData = Cast<UStateTreeEditorData>(StateTreeCachedBindingData->GetOwner());
					if (!EditorData)
					{
						return;
					}

					UE::StateTreeEditor::FClipboardEditorData Clipboard;

					constexpr bool bProcessBuffer = true;
					if (ImportTextAsSingleBindingClipboard(bProcessBuffer, Clipboard))
					{
						TArrayView<FStateTreePropertyPathBinding> Bindings = Clipboard.GetBindingsInBuffer();
						check(Bindings.Num());

						// This is the direct binding we copy pasted over. All the other bindings are associated with the direct binding(property functions)
						FStateTreePropertyPathBinding& PropertyBinding = Bindings[0];
						FConstStructView BindingPropertyFunctionNodeView = PropertyBinding.GetPropertyFunctionNode();
						TConstArrayView<TInstancedStruct<FPropertyBindingBindableStructDescriptor>> AccessibleStructs = StateTreeCachedBindingData->GetAccessibleStructs();

						bool bBindToPropertyFunctionNode = false;

						int32 SourceStructIndex = INDEX_NONE;

						// if the binding source struct is accessible by the new property
						if (const FStateTreeEditorNode* PropertyFunctionNode = BindingPropertyFunctionNodeView.GetPtr<const FStateTreeEditorNode>())
						{
							bBindToPropertyFunctionNode = true;

							// AccessibleStructDesc stores IDs of "template" property function nodes(ID is made of struct name), which is different from the actual property function nodes(their IDs are instantiated)
							const UScriptStruct* PropertyFunctionNodeStruct = PropertyFunctionNode->GetNode().GetScriptStruct();
							FGuid TemplatePropertyFunctionID;
							EditorData->EnumerateBindablePropertyFunctionNodes(
								[PropertyFunctionNodeStruct, &TemplatePropertyFunctionID](const UScriptStruct* NodeStruct, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)
								{
									if (NodeStruct == PropertyFunctionNodeStruct)
									{
										TemplatePropertyFunctionID = Desc.ID;
										return EStateTreeVisitor::Break;
									}

									return EStateTreeVisitor::Continue;

								});

							SourceStructIndex = AccessibleStructs.IndexOfByPredicate(
								[TemplatePropertyFunctionID](const TInstancedStruct<FPropertyBindingBindableStructDescriptor>& AccessibleStructDesc)
								{
									return AccessibleStructDesc.Get().ID == TemplatePropertyFunctionID;
								});
						}
						else
						{
							SourceStructIndex = AccessibleStructs.IndexOfByPredicate(
								[&PropertyBinding](const TInstancedStruct<FPropertyBindingBindableStructDescriptor>& AccessibleStructDesc)
								{
									return AccessibleStructDesc.Get().ID == PropertyBinding.GetSourcePath().GetStructID();
								});
						}

						// Report an error if Source Struct is not accessible
						if (SourceStructIndex == INDEX_NONE)
						{
							FString InaccessibleSourceStructPathName;
							FString TargetStructPathName;
							// log the inaccessible source struct of the copied binding from editor data, unless its from a different tree
							EditorData->VisitAllNodes(
								[StateTreeCachedBindingData, &PropertyBinding, &InaccessibleSourceStructPathName, &TargetStructPathName](const UStateTreeState* State, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)
								{
									// property function nodes are always accessible, so we don't worry about PropertyFunction SourcePath ID being renewed.
									if (PropertyBinding.GetSourcePath().GetStructID() == Desc.ID)
									{
										InaccessibleSourceStructPathName = Desc.StatePath + TEXT("/") + Desc.Name.ToString();
									}

									if (StateTreeCachedBindingData->GetTargetPath().GetStructID() == Desc.ID)
									{
										TargetStructPathName = Desc.StatePath + TEXT("/") + Desc.Name.ToString();
									}

									if (!InaccessibleSourceStructPathName.IsEmpty() && !TargetStructPathName.IsEmpty())
									{
										return EStateTreeVisitor::Break;
									}

									return EStateTreeVisitor::Continue;
								});

							UE::StateTreeEditor::AddErrorNotification(
								FText::Format(LOCTEXT("InaccessibleBindingSource", "Source Struct {0} is not accessible from Target Struct {1}."), 
								FText::FromString(InaccessibleSourceStructPathName), 
								FText::FromString(TargetStructPathName)));

							return;
						}

						const FStateTreeBindableStructDesc& SourceStructDesc = static_cast<const FStateTreeBindableStructDesc&>(AccessibleStructs[SourceStructIndex].Get());

						TArray<FPropertyBindingPathIndirection> Indirections;
						constexpr FString* OutError = nullptr;
						constexpr bool bHandleRedirects = true;
						if (!PropertyBinding.GetSourcePath().ResolveIndirections(SourceStructDesc.Struct, Indirections, OutError, bHandleRedirects))
						{
							return;
						}

						const FProperty* SourceProperty = Indirections.Num() != 0 ? Indirections.Last().GetProperty() : nullptr;

						TArray<FBindingChainElement> BindingChain;
						MakeBindingChainFromBindingPathIndirections(SourceStructIndex, Indirections, BindingChain);

						const bool bBindToContextStruct = SourceStructDesc.DataSource == EStateTreeBindableStructSource::Context
							|| Indirections.Num() == 0;

						// there are essentially two code paths. One to check if the struct itself is bindable(CanBindToContextStruct).
						// The other to check if property is bindable(need to pass both CanBindToProperty and CanAcceptPropertyOrChildren)
						// The Source Property and Property Chain returned by both functions are different, so we only check CanBindToProperty here
						// All the extra checks we did in CanAcceptPropertyOrChild have already been passed by the original binding we copy from(i.e. they are all agnostic to the target property)
						// @todo: see if we could refactor out the common part of CanBindToProperty and CanAcceptPropertyOrChildren in FCachedBindingData so we don't have to convert back to BindingChain here  
						bool bIsSourcePropertyCompatible = false;
						if (bBindToPropertyFunctionNode || bBindToContextStruct)
						{
							auto IsTaskCompletionBindingValid = [&](const UStruct* Struct)
								{
									if (!PropertyBinding.IsTaskCompletionBinding())
									{
										return false;
									}
									if (Struct == nullptr)
									{
										return false;
									}
									// bound to a task
									return StateTreeCachedBindingData->GetAccessibleStructs().ContainsByPredicate(
										[Struct](const TInstancedStruct<FPropertyBindingBindableStructDescriptor>& StructDesc)
										{
											const FStateTreeBindableStructDesc& AccessibleStruct = StructDesc.Get<FStateTreeBindableStructDesc>();
											const bool bTask = AccessibleStruct.DataSource == EStateTreeBindableStructSource::GlobalTask
												|| AccessibleStruct.DataSource == EStateTreeBindableStructSource::Task;
											return Struct == AccessibleStruct.Struct || bTask;
										});
								};

							bIsSourcePropertyCompatible = StateTreeCachedBindingData->CanBindToContextStruct(SourceStructDesc.Struct, BindingChain[0].ArrayIndex)
								|| IsTaskCompletionBindingValid(SourceStructDesc.Struct);
						}
						else
						{
							check(SourceProperty);
							bIsSourcePropertyCompatible = StateTreeCachedBindingData->CanBindToProperty(SourceProperty, BindingChain);
						}

						// Report an error if properties are not compatible
						if (!bIsSourcePropertyCompatible)
						{
							FText SourceText;
							if (bBindToPropertyFunctionNode)
							{
								FString SourceStructName;
								// We need the class name instead of display name for error reporting property function
								EditorData->EnumerateBindablePropertyFunctionNodes(
									[&SourceStructName, &SourceStructDesc](const UScriptStruct* NodeStruct, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)
									{
										if (SourceStructDesc.ID == Desc.ID)
										{
											SourceStructName = NodeStruct->GetName();
											return EStateTreeVisitor::Break;
										}

										return EStateTreeVisitor::Continue;
									});

								SourceText = FText::FromString(MoveTemp(SourceStructName));
							}
							else if (bBindToContextStruct)
							{
								// For Context Struct, we use the context's name
								SourceText = FText::FromName(SourceStructDesc.Name);
							}
							else
							{
								SourceText = SourceProperty->GetDisplayNameText();
							}

							UE::StateTreeEditor::AddErrorNotification(FText::Format(
								LOCTEXT("IncompatibleSourceProperty", "Source Property {0} is not compatible with Target Property {1}."),
								SourceText,
								StateTreeCachedBindingData->GetPropertyHandle()->GetPropertyDisplayName()));

							return;
						}

						// Add/Replace the binding to property
						FScopedTransaction Transaction(LOCTEXT("StateTreeBindingData_PasteBinding", "Paste Binding"));

						EditorData->Modify();

						StateTreeCachedBindingData->RemoveBinding(SearchMode);

						// Update the target path to be the current property
						PropertyBinding.GetMutableTargetPath() = StateTreeCachedBindingData->GetTargetPath();

						// auto fix the binding to be output binding for output property
						//~ todo: once we support output binding for parameter property, the flag needs to be carried over and validation
						PropertyBinding.SetIsOutputBinding(StateTreeCachedBindingData->HierarchicalPropertyUsage == EStateTreePropertyUsage::Output);
						
						for (int32 Idx = 0; Idx < Bindings.Num(); ++Idx)
						{
							FStateTreePropertyPathBinding& CurrentBinding = Bindings[Idx];
							FPropertyBindingPath SourceBindingPath = CurrentBinding.GetSourcePath();
							FPropertyBindingPath TargetBindingPath = CurrentBinding.GetTargetPath();

							EditorData->GetPropertyEditorBindings()->AddStateTreeBinding(MoveTemp(Bindings[Idx]));
							EditorData->OnPropertyBindingChanged(SourceBindingPath, TargetBindingPath);
						}

						InDetailBuilder.GetPropertyUtilities()->RequestForceRefresh();
					}
				}
			}),
			FCanExecuteAction::CreateLambda([ImportTextAsSingleBindingClipboard]()
			{
				UE::StateTreeEditor::FClipboardEditorData Clipboard;
				constexpr bool bProcessBuffer = false;
				return ImportTextAsSingleBindingClipboard(bProcessBuffer, Clipboard);
			})),
		LOCTEXT("PasteBinding", "Paste Binding"),
		LOCTEXT("PasteBindingTooltip", "Paste the binding from clipboard into this property"),
		FSlateIcon());
}

void FStateTreeBindingExtension::CustomizePropertyBindingWidgetArgs(
	FPropertyBindingWidgetArgs& Args,
	IPropertyBindingBindingCollectionOwner* InBindingsOwner,
	const FPropertyBindingPath& InTargetPath,
	TSharedPtr<UE::PropertyBinding::FCachedBindingData> InCachedBindingData)
{
	// Create a menu for TaskCompletion binding.
	// For each struct that are Task (Global Task), add an entry (grouped by states) with 3 entry for completes, success and failure.
	FOnAddBinding& OnAddBinding = Args.OnAddBinding;
	if (OnAddBinding.IsBound()
		&& InCachedBindingData
		&& UE::StateTree::PropertyBinding::CanTargetBindToTaskCompletion(Args.Property))
	{
		auto AddBinding = [OnAddBinding, InCachedBindingData](int32 StructIndex, UE::StateTree::ETaskCompletionCondition Condition)
			{
				using namespace UE::StateTree::PropertyBinding;

				const FScopedTransaction Transaction(LOCTEXT("BindTaskCompletionDelegate", "Set Task Completion Binding"));

				// Set the condition for AddBinding 
				static_cast<FStateTreeCachedBindingData*>(InCachedBindingData.Get())->SetNextTaskCompletionBinding(Condition);

				TArray<FBindingChainElement> BindingChain;
				FBindingChainElement Element = FBindingChainElement((FProperty*)nullptr, StructIndex);
				BindingChain.Add(Element);
				OnAddBinding.Execute(FName(), BindingChain);

				// Reset (in case the OnAddBinding fails and the binding could not be added).
				static_cast<FStateTreeCachedBindingData*>(InCachedBindingData.Get())->ResetTaskCompletionBinding();
			};

		if (Args.MenuExtender == nullptr)
		{
			Args.MenuExtender = MakeShared<FExtender>();
		}
		Args.MenuExtender->AddMenuExtension(
			TEXT("BindingActions"),
			EExtensionHook::After,
			/*CommandList*/ nullptr,
			FMenuExtensionDelegate::CreateLambda([InCachedBindingData, AddBinding](FMenuBuilder& ExtensionMenuBuilder)
				{
					ExtensionMenuBuilder.AddSubMenu(
						LOCTEXT("TaskCompletion", "Task Completion Dispatcher"),
						LOCTEXT("TaskCompletionTooltip", "Bind a delegate listener to a task completion. When the task completes, it will broadcast the hidden dispatcher and trigger the listener."),
						FNewMenuDelegate::CreateLambda([InCachedBindingData, AddBinding](FMenuBuilder& StructMenuBuilder)
							{
								TConstArrayView<TInstancedStruct<FPropertyBindingBindableStructDescriptor>> AccessibleStructs = InCachedBindingData->GetAccessibleStructs();
								FString CurrentSection;
								bool bSectionOpenned = false;
								for (int32 StructIndex = 0; StructIndex < AccessibleStructs.Num(); ++StructIndex)
								{
									const TInstancedStruct<FPropertyBindingBindableStructDescriptor>& InstancedContextStruct = AccessibleStructs[StructIndex];
									const FStateTreeBindableStructDesc& SourceDesc = InstancedContextStruct.Get<FStateTreeBindableStructDesc>();
									if (UE::StateTree::PropertyBinding::IsSourceTaskCompletionNode(SourceDesc.Struct))
									{
										const FString Section = SourceDesc.GetSection();
										if (!Section.IsEmpty())
										{
											if (CurrentSection != Section)
											{
												if (bSectionOpenned)
												{
													StructMenuBuilder.EndSection();
												}
												CurrentSection = Section;
												StructMenuBuilder.BeginSection(NAME_None, FText::FromString(Section));
												bSectionOpenned = true;
											}
										}

										StructMenuBuilder.AddSubMenu(
											FText::FromName(SourceDesc.Name),
											FText::GetEmpty(),
											FNewMenuDelegate::CreateLambda([StructIndex, AddBinding](FMenuBuilder& ConditionMenuBuilder)
												{
														ConditionMenuBuilder.AddMenuEntry(
															LOCTEXT("ConditionCompleted", "Completes"),
															LOCTEXT("ConditionCompletedTooltip", "Broadcast on either Succeeded or Failed (or Stopped for backward compatibility)."),
															FSlateIcon(),
															FExecuteAction::CreateLambda([StructIndex, AddBinding]()
																{
																	AddBinding(StructIndex, UE::StateTree::ETaskCompletionCondition::Completes);
																}));
														ConditionMenuBuilder.AddMenuEntry(
															LOCTEXT("ConditionSucceeded", "Succeeds"),
															LOCTEXT("ConditionSucceededTooltip", "Broadcast only when task Succeeded"),
															FSlateIcon(),
															FExecuteAction::CreateLambda([StructIndex, AddBinding]()
																{
																	AddBinding(StructIndex, UE::StateTree::ETaskCompletionCondition::Succeeds);
																}));
														ConditionMenuBuilder.AddMenuEntry(
															LOCTEXT("ConditionFailed", "Fails"),
															LOCTEXT("ConditionFailedTooltip", "Broadcast only when task Failed (or Stopped for backward compatibility)."),
															FSlateIcon(),
															FExecuteAction::CreateLambda([StructIndex, AddBinding]()
																{
																	AddBinding(StructIndex, UE::StateTree::ETaskCompletionCondition::Fails);
																}));
												})
											);
									}
								}

								if (bSectionOpenned)
								{
									StructMenuBuilder.EndSection();
								}
							})
					);
				}
			));
	}
}

bool FStateTreeBindingsChildrenCustomization::ShouldCustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle)
{
	TArray<UObject*> OuterObjects;
	InPropertyHandle->GetOuterObjects(OuterObjects);
	if (OuterObjects.Num() == 1)
	{
		// Bound property's children composition gets overridden.
		FPropertyBindingPath TargetPath;
		UE::PropertyBinding::MakeStructPropertyPathFromPropertyHandle(InPropertyHandle, TargetPath);
		IStateTreeEditorPropertyBindingsOwner* BindingOwner = Cast<IStateTreeEditorPropertyBindingsOwner>(UE::StateTree::PropertyBinding::FindEditorBindingsOwner(OuterObjects[0]));
		if (!TargetPath.IsPathEmpty() && BindingOwner)
		{
			if (const FStateTreeEditorPropertyBindings* EditorBindings = BindingOwner->GetPropertyEditorBindings())
			{
				return EditorBindings->HasBinding(TargetPath);
			}
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
