// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailPropertyExtensionHandler.h"
#include "IDetailPropertyChildrenCustomizationHandler.h"
#include "PropertyBindingExtension.h"

enum class EStateTreePropertyUsage : uint8;

class IPropertyHandle;
class IDetailLayoutBuilder;
class IPropertyAccessEditor;
struct FPropertyBindingPath;
class FProperty;

namespace UE::StateTree::PropertyBinding
{
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnStateTreePropertyBindingChanged, const FPropertyBindingPath& /*SourcePath*/, const FPropertyBindingPath& /*TargetPath*/);
	extern STATETREEEDITORMODULE_API FOnStateTreePropertyBindingChanged OnStateTreePropertyBindingChanged;
} // UE::StateTree::PropertyBinding

class FStateTreeBindingExtension : public FPropertyBindingExtension
{
private:
	virtual TSharedPtr<UE::PropertyBinding::FCachedBindingData> CreateCachedBindingData(IPropertyBindingBindingCollectionOwner* InBindingsOwner, const FPropertyBindingPath& InTargetPath, const TSharedPtr<IPropertyHandle>& InPropertyHandle, TConstArrayView<TInstancedStruct<FPropertyBindingBindableStructDescriptor>> InAccessibleStructs) const override;
	virtual bool CanBindToProperty(const FPropertyBindingPath& InTargetPath, const IPropertyHandle& InPropertyHandle) const override;
	virtual bool GetPromotionToParameterOverrideInternal(const FProperty& InProperty, bool& bOutOverride) const override;
	virtual void UpdateContextStruct(TConstStructView<FPropertyBindingBindableStructDescriptor> InStructDesc,FBindingContextStruct& InOutContextStruct, TMap<FString, FText>& InOutSectionNames) const override;
	virtual bool CanBindToArrayElements() const override
	{
		return true;
	}
	virtual TNotNull<const UEdGraphSchema_K2*> GetEdGraphSchema() const override;
	virtual void CustomizeDetailWidgetRow(FDetailWidgetRow& InWidgetRow, const IDetailLayoutBuilder& InDetailBuilder, IPropertyBindingBindingCollectionOwner* InBindingsOwner, TSharedPtr<IPropertyHandle> InPropertyHandle, const FPropertyBindingPath& InTargetPath, TSharedPtr<UE::PropertyBinding::FCachedBindingData> InCachedBindingData) const override;
	virtual void CustomizePropertyBindingWidgetArgs(FPropertyBindingWidgetArgs& Args, IPropertyBindingBindingCollectionOwner* InBindingsOwner, const FPropertyBindingPath& InTargetPath, TSharedPtr<UE::PropertyBinding::FCachedBindingData> InCachedBindingData) override;
};

/* Overrides bound property's children composition. */
class FStateTreeBindingsChildrenCustomization : public IDetailPropertyChildrenCustomizationHandler
{
public:
	virtual bool ShouldCustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle) override;
	virtual void CustomizeChildren(IDetailChildrenBuilder& ChildrenBuilder, TSharedPtr<IPropertyHandle> InPropertyHandle) override;
};
