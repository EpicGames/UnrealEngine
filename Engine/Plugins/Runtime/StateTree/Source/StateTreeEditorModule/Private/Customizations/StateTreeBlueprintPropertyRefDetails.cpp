// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeBlueprintPropertyRefDetails.h"
#include "DetailWidgetRow.h"
#include "SPinTypeSelector.h"
#include "StateTreeEditorNode.h"
#include "StateTreeGraphSchema.h"
#include "StateTreePropertyRef.h"
#include "IDetailChildrenBuilder.h"
#include "DetailLayoutBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeBlueprintPropertyRefDetails)

#define LOCTEXT_NAMESPACE "StateTreeEditor"

namespace UE::StateTree::BlueprintPropertyRef
{
	static bool IsInStateTreeNode(TSharedRef<IPropertyHandle> PropertyHandle)
	{
		TSharedPtr<IPropertyHandle> TestPropertyHandle = PropertyHandle->GetParentHandle();
		while (TestPropertyHandle)
		{
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(TestPropertyHandle->GetProperty()))
			{
				if (StructProperty->Struct == FStateTreeEditorNode::StaticStruct())
				{
					return true;
				}
			}

			TestPropertyHandle = TestPropertyHandle->GetParentHandle();
		}

		return false;
	}
}

TSharedRef<IPropertyTypeCustomization> FStateTreeBlueprintPropertyRefDetails::MakeInstance()
{
	return MakeShared<FStateTreeBlueprintPropertyRefDetails>();
}

void FStateTreeBlueprintPropertyRefDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	// Don't allow to modify property ref internal type if it's a part of State Tree node.
	if (UE::StateTree::BlueprintPropertyRef::IsInStateTreeNode(InPropertyHandle))
	{
		HeaderRow.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		];

		return;
	}

	auto GetPinInfo = [WeakPropertyHandle = TWeakPtr<IPropertyHandle>(InPropertyHandle)]()
	{
		FEdGraphPinType Result;
		if (TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin())
		{
			TSharedPtr<IPropertyHandle> RefTypePropertyHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_STRING_CHECKED(FStateTreeBlueprintPropertyRef, RefType));
			TSharedPtr<IPropertyHandle> IsRefToArrayPropertyHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_STRING_CHECKED(FStateTreeBlueprintPropertyRef, bIsRefToArray));
			TSharedPtr<IPropertyHandle> TypeObjectPropertyHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_STRING_CHECKED(FStateTreeBlueprintPropertyRef, TypeObject));

			if (RefTypePropertyHandle && IsRefToArrayPropertyHandle && TypeObjectPropertyHandle)
			{
				uint8 RefType = 0u;
				bool bIsRefToArray = false;
				UObject* TypeObject = nullptr;

				RefTypePropertyHandle->GetValue(RefType);
				IsRefToArrayPropertyHandle->GetValue(bIsRefToArray);
				TypeObjectPropertyHandle->GetValue(TypeObject);

				FStateTreeBlueprintPropertyRef PropertyRef;
				PropertyRef.RefType = static_cast<EStateTreePropertyRefType>(RefType);
				PropertyRef.bIsRefToArray = bIsRefToArray;
				PropertyRef.TypeObject = TypeObject;
				Result = UE::StateTree::PropertyRefHelpers::GetBlueprintPropertyRefInternalTypeAsPin(PropertyRef);
			}	
		}

		return Result;
	};

	auto PinInfoChanged = [WeakPropertyHandle = TWeakPtr<IPropertyHandle>(InPropertyHandle)](const FEdGraphPinType& PinType)
	{
		if (TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin())
		{
			EStateTreePropertyRefType RefType;
			bool bIsRefToArray = false;
			UObject* TypeObject = nullptr;

			UE::StateTree::PropertyRefHelpers::GetBlueprintPropertyRefInternalTypeFromPin(PinType, RefType, bIsRefToArray, TypeObject);

			TSharedPtr<IPropertyHandle> RefTypePropertyHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_STRING_CHECKED(FStateTreeBlueprintPropertyRef, RefType));
			TSharedPtr<IPropertyHandle> IsRefToArrayPropertyHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_STRING_CHECKED(FStateTreeBlueprintPropertyRef, bIsRefToArray));
			TSharedPtr<IPropertyHandle> TypeObjectPropertyHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_STRING_CHECKED(FStateTreeBlueprintPropertyRef, TypeObject));

			if (RefTypePropertyHandle && IsRefToArrayPropertyHandle && TypeObjectPropertyHandle)
			{
				RefTypePropertyHandle->SetValue(static_cast<uint8>(RefType));
				IsRefToArrayPropertyHandle->SetValue(bIsRefToArray);
				TypeObjectPropertyHandle->SetValue(TypeObject);
			}
		}
	};

	HeaderRow.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(FMargin(0, 0, 6, 0))
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ReferenceTo", "Reference to"))
			.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			.ToolTipText(LOCTEXT("ReferenceTo_Tooltip", "Specifies the type of the referenced property. The referenced property is bound using property binding in the State Tree."))
		]
		
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SPinTypeSelector, FGetPinTypeTree::CreateUObject(GetDefault<UStateTreePropertyRefSchema>(), &UE::StateTree::Editor::UEdGraphSchema_StateTree::GetVariableTypeTree))
				.OnPinTypeChanged_Lambda(PinInfoChanged)
				.TargetPinType_Lambda(GetPinInfo)
				.Schema(GetDefault<UStateTreePropertyRefSchema>())
				.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
				.bAllowArrays(true)
		]
	];
}

void FStateTreeBlueprintPropertyRefDetails::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	if(!UE::StateTree::BlueprintPropertyRef::IsInStateTreeNode(InPropertyHandle))
	{
		TSharedPtr<IPropertyHandle> IsOptionalPropertyHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_STRING_CHECKED(FStateTreeBlueprintPropertyRef, bIsOptional));
		ChildBuilder.AddProperty(IsOptionalPropertyHandle.ToSharedRef());
	}
}

bool UStateTreePropertyRefSchema::SupportsPinTypeContainer(TWeakPtr<const FEdGraphSchemaAction> SchemaAction, const FEdGraphPinType& PinType, const EPinContainerType& ContainerType) const
{
	return ContainerType == EPinContainerType::None || ContainerType == EPinContainerType::Array;
}

#undef LOCTEXT_NAMESPACE
