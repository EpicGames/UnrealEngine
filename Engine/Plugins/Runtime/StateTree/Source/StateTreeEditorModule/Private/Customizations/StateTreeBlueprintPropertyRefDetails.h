// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "EdGraphSchema_K2.h"
#include "StateTreeBlueprintPropertyRefDetails.generated.h"

#define UE_API STATETREEEDITORMODULE_API

class SWidget;
class IPropertyHandle;

/**
 * Type customization for FStateTreeBlueprintPropertyRef.
 */
class FStateTreeBlueprintPropertyRefDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

private:

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
};

/**
 * Specific property ref schema to allow customizing the requirements (e.g. supported containers).
 */
UCLASS(MinimalAPI)
class UStateTreePropertyRefSchema : public UEdGraphSchema_K2
{
	GENERATED_BODY()
public:
	UE_API virtual bool SupportsPinTypeContainer(TWeakPtr<const FEdGraphSchemaAction> SchemaAction, const FEdGraphPinType& PinType, const EPinContainerType& ContainerType) const override;
};

#undef UE_API
