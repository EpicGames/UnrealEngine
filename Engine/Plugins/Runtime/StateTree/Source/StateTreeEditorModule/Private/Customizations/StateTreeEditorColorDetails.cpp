// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeEditorColorDetails.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/Engine.h"
#include "ScopedTransaction.h"
#include "StateTreeEditorData.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SStateTreeEditorColorComboBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"

//----------------------------------------------------------------//
// FStateTreeEditorColorRefDetails
//----------------------------------------------------------------//

TSharedRef<IPropertyTypeCustomization> FStateTreeEditorColorRefDetails::MakeInstance()
{
	return MakeShared<FStateTreeEditorColorRefDetails>();
}

void FStateTreeEditorColorRefDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	UStateTreeEditorData* EditorData = GetEditorData(InStructPropertyHandle);
	if (!EditorData)
	{
		return;
	}

	HeaderRow
		.NameContent()
		[
			InStructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(200.0f)
		[
			SNew(SStateTreeEditorColorComboBox, InStructPropertyHandle, EditorData)
		];
}

UStateTreeEditorData* FStateTreeEditorColorRefDetails::GetEditorData(const TSharedRef<IPropertyHandle>& PropertyHandle) const
{
	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);

	for (UObject* Outer : Objects)
	{
		if (UStateTreeEditorData* EditorData = Cast<UStateTreeEditorData>(Outer))
		{
			return EditorData;
		}

		if (UStateTreeEditorData* EditorData = Outer->GetTypedOuter<UStateTreeEditorData>())
		{
			return EditorData;
		}
	}

	return nullptr;
}

//----------------------------------------------------------------//
// FStateTreeEditorColorDetails
//----------------------------------------------------------------//

TSharedRef<IPropertyTypeCustomization> FStateTreeEditorColorDetails::MakeInstance()
{
	return MakeShared<FStateTreeEditorColorDetails>();
}

void FStateTreeEditorColorDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructPropertyHandle = InStructPropertyHandle;

	ColorPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeEditorColor, Color));

	TSharedPtr<IPropertyHandle> NamePropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeEditorColor, DisplayName));

	HeaderRow
		.WholeRowContent()
		.HAlign(HAlign_Fill)
		.MinDesiredWidth(200.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.MaxWidth(25)
			.VAlign(VAlign_Center)
			[
				SAssignNew(ColorButtonWidget, SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked(this, &FStateTreeEditorColorDetails::OnColorButtonClicked)
				.ContentPadding(2.f)
				[
					SNew(SColorBlock)
					.Color(this, &FStateTreeEditorColorDetails::GetColor)
					.Size(FVector2D(16.0))
				]
			]
			+ SHorizontalBox::Slot()
			.Padding(FMargin(5.f, 0.f, 0.f, 0.f))
			.VAlign(VAlign_Center)
			[
				NamePropertyHandle->CreatePropertyValueWidget(/*bDisplayDefaultPropertyButtons*/false)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			[
				StructPropertyHandle->CreateDefaultPropertyButtonWidgets()
			]
		];
}

FLinearColor* FStateTreeEditorColorDetails::GetColorPtr() const
{
	void* ValueData = nullptr;
	if (ColorPropertyHandle->GetValueData(ValueData) == FPropertyAccess::Result::Success)
	{
		return static_cast<FLinearColor*>(ValueData);
	}
	return nullptr;
}

FLinearColor FStateTreeEditorColorDetails::GetColor() const
{
	if (FLinearColor* ColorPtr = GetColorPtr())
	{
		return *ColorPtr;
	}
	return FLinearColor(ForceInitToZero);
}

void FStateTreeEditorColorDetails::SetColor(FLinearColor Color)
{
	if (FLinearColor* ColorPtr = GetColorPtr())
	{
		ColorPropertyHandle->NotifyPreChange();
		*ColorPtr = Color;
		ColorPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	}
}

void FStateTreeEditorColorDetails::OnColorCommitted(FLinearColor Color)
{
	SetColor(Color);

	// End Transaction
	ColorPickerTransaction.Reset();
}

void FStateTreeEditorColorDetails::OnColorCancelled(FLinearColor Color)
{
	SetColor(Color);

	// Cancel Transaction
	if (ColorPickerTransaction.IsValid())
	{
		ColorPickerTransaction->Cancel();
		ColorPickerTransaction.Reset();
	}
}

FReply FStateTreeEditorColorDetails::OnColorButtonClicked()
{
	CreateColorPickerWindow();
	return FReply::Handled();
}

void FStateTreeEditorColorDetails::CreateColorPickerWindow()
{
	FColorPickerArgs PickerArgs;

	// Begin Transaction
	ColorPickerTransaction = MakeShared<FScopedTransaction>(LOCTEXT("SetStateTreeColorProperty", "Set Color Property"));

	PickerArgs.bOnlyRefreshOnMouseUp = false;
	PickerArgs.ParentWidget = ColorButtonWidget;
	PickerArgs.bUseAlpha = false;
	PickerArgs.bOnlyRefreshOnOk = false;
	PickerArgs.bClampValue = false; // Linear Color
	PickerArgs.DisplayGamma = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma));
	PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &FStateTreeEditorColorDetails::OnColorCommitted);
	PickerArgs.OnColorPickerCancelled = FOnColorPickerCancelled::CreateSP(this, &FStateTreeEditorColorDetails::OnColorCancelled);
	PickerArgs.InitialColor = GetColor();

	OpenColorPicker(PickerArgs);
}

#undef LOCTEXT_NAMESPACE
