// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Math/Color.h"

class UStateTreeEditorData;
class FReply;
class FScopedTransaction;
class SWidget;

/** Property Type Customization for FStateTreeEditorColorRef */
class FStateTreeEditorColorRefDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}

private:
	UStateTreeEditorData* GetEditorData(const TSharedRef<IPropertyHandle>& PropertyHandle) const;
};

/** Property Type Customization for FStateTreeEditorColor */
class FStateTreeEditorColorDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}

private:
	FLinearColor* GetColorPtr() const;

	FLinearColor GetColor() const;
	void SetColor(FLinearColor Color);

	void OnColorCommitted(FLinearColor Color);

	void OnColorCancelled(FLinearColor Color);

	FReply OnColorButtonClicked();

	void CreateColorPickerWindow();

	/** Current Transaction taking place when selecting a color from the picker */
	TSharedPtr<FScopedTransaction> ColorPickerTransaction;

	TSharedPtr<IPropertyHandle> StructPropertyHandle;
	TSharedPtr<IPropertyHandle> ColorPropertyHandle;
	TSharedPtr<SWidget> ColorButtonWidget;
};
