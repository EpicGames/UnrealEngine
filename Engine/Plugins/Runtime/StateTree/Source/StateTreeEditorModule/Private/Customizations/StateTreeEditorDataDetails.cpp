// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeEditorDataDetails.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "PropertyBagDetails.h"
#include "PropertyCustomizationHelpers.h"
#include "StateTreeEditorData.h"
#include "StateTreeEditorNodeUtils.h"
#include "StateTreeEditorSchema.h"
#include "StateTreeEditorStyle.h"
#include "StateTreeEditorUserSettings.h"
#include "StateTreeGraphSchema.h"
#include "StateTreeSchema.h"
#include "StructUtilsMetadata.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"

TSharedRef<IDetailCustomization> FStateTreeEditorDataDetails::MakeInstance()
{
	return MakeShareable(new FStateTreeEditorDataDetails);
}

void FStateTreeEditorDataDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	using namespace UE::StateTree::Editor;

	const TSharedRef<IPropertyUtilities> PropUtils = DetailBuilder.GetPropertyUtilities();
	WeakPropertyUtilities = PropUtils.ToWeakPtr();

	// Find StateTreeEditorData associated with this panel.
	const UStateTreeEditorData* EditorData = nullptr;
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);
	for (TWeakObjectPtr<UObject>& WeakObject : Objects)
	{
		if (UStateTreeEditorData* Object = Cast<UStateTreeEditorData>(WeakObject.Get()))
		{
			EditorData = Object;
			break;
		}
	}
	const UStateTreeSchema* Schema = EditorData ? EditorData->Schema.Get() : nullptr;
	const UStateTreeEditorSchema* EditorSchema = EditorData ? EditorData->EditorSchema.Get() : nullptr;

	// Common category
	IDetailCategoryBuilder& CommonCategory = DetailBuilder.EditCategory(TEXT("Common"), LOCTEXT("EditorDataDetailsCommon", "Common"));
	CommonCategory.SetSortOrder(0);

	// Context category
	IDetailCategoryBuilder& ContextDataCategory = DetailBuilder.EditCategory(TEXT("Context"), LOCTEXT("EditorDataDetailsContext", "Context"));
	ContextDataCategory.SetSortOrder(1);

	// Theme category
	IDetailCategoryBuilder& ThemeCategory = DetailBuilder.EditCategory(TEXT("Theme"));
	ThemeCategory.InitiallyCollapsed(true);

	if (Schema != nullptr)
	{
		for (const FStateTreeExternalDataDesc& ContextData : Schema->GetContextDataDescs())
		{
			if (ContextData.Struct == nullptr)
			{
				continue;
			}
			
			FEdGraphPinType PinType;
			PinType.PinSubCategory = NAME_None;
			if (ContextData.Struct->IsA<UScriptStruct>())
			{
				PinType.PinCategory = UEdGraphSchema_K2::PC_Struct;
			}
			else if (ContextData.Struct->IsA<UClass>())
			{
				PinType.PinCategory = UEdGraphSchema_K2::PC_Object;
			}
			else
			{
				continue;
			}
			PinType.PinSubCategoryObject = const_cast<UStruct*>(ContextData.Struct.Get());

			const UEdGraphSchema_StateTree* EdGraphSchema = GetDefault<UEdGraphSchema_StateTree>();
			const FSlateBrush* Icon = FBlueprintEditorUtils::GetIconFromPin(PinType, true);
			const FLinearColor Color = EdGraphSchema->GetPinTypeColor(PinType);

			const FText DataName = FText::FromName(ContextData.Name);
			const FText DataType = ContextData.Struct != nullptr ? ContextData.Struct->GetDisplayNameText() : FText::GetEmpty();
			
			ContextDataCategory.AddCustomRow(DataName)
				.NameContent()
				[
					SNew(SHorizontalBox)
					
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.Text(DataName)
					]
					
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(4.0f, 0.0f)
					[
						SNew(SBorder)
						.Padding(FMargin(6, 1))
						.BorderImage(FStateTreeEditorStyle::Get().GetBrush("StateTree.Param.Background"))
						[
							SNew(STextBlock)
							.TextStyle(FStateTreeEditorStyle::Get(), "StateTree.Param.Label")
							.ColorAndOpacity(FStyleColors::Foreground)
							.Text(LOCTEXT("LabelContext", "CONTEXT"))
							.ToolTipText(LOCTEXT("ContextSourceTooltip", "This is Context Object, it passed in from where the StateTree is being used."))
						]
					]
				]
				.ValueContent()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(4.0f, 0.0f)
					[
						SNew(SImage)
						.Image(Icon)
						.ColorAndOpacity(Color)
					]
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.Text(DataType)
					]
				];
		}
	}

	if (Schema && Schema->AllowGlobalParameters())
	{
		uint32 GlobalNumParameters = EditorData ? EditorData->GetRootParametersPropertyBag().GetNumPropertiesInBag() : 0;

		// Parameters category
		const FText ParametersDisplayName = EnumHasAllFlags(GetDefault<UStateTreeEditorUserSettings>()->GetDetailsViewDisplayCount(), EStateTreeEditorUserSettingsNodeType::Parameter)
			? FText::Format(LOCTEXT("EditorDataDetailsParametersCounted", "Parameters ({0})"), GlobalNumParameters)
			: LOCTEXT("EditorDataDetailsParameters", "Parameters"); 
		IDetailCategoryBuilder& ParametersCategory = DetailBuilder.EditCategory(TEXT("Parameters"), ParametersDisplayName);
		ParametersCategory.SetSortOrder(2);
		ParametersCategory.SetColor(UE::StateTree::Colors::Blue);
		{
			// Show parameters as a category.
			TSharedPtr<IPropertyHandle> PropertyBagParametersProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeEditorData, RootParameterPropertyBag));
			PropertyBagParametersProperty->MarkHiddenByCustomization();

			if (EditorSchema)
			{
				TOptional<FString> SchemaMetaData = EditorSchema->GetParametersSchemaClass();
				if (SchemaMetaData.IsSet())
				{
					PropertyBagParametersProperty->SetInstanceMetaData(UE::StructUtils::Metadata::PropertyBagSchemaClassName, SchemaMetaData.GetValue());
				}
			}

			const TSharedRef<SHorizontalBox> HeaderContentWidget = SNew(SHorizontalBox)
				.IsEnabled(PropUtils, &IPropertyUtilities::IsPropertyEditingEnabled)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(FMargin(4.f, 0.f, 0.f, 0.f))
				[
					SNew(SImage)
						.ColorAndOpacity(UE::StateTree::Colors::Blue)
						.Image(FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Parameters"))
				]
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(FMargin(4.f, 0.f, 0.f, 0.f))
				.AutoWidth()
				[
					SNew(STextBlock)
					.TextStyle(FStateTreeEditorStyle::Get(), "StateTree.Category")
					.Text(ParametersDisplayName)
				]
				+SHorizontalBox::Slot()
				.FillWidth(1.f)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					FPropertyBagDetails::MakeAddPropertyWidget(PropertyBagParametersProperty, PropUtils, EPropertyBagPropertyType::Bool, FLinearColor(UE::StateTree::Colors::Blue)).ToSharedRef()
				];
			ParametersCategory.HeaderContent(HeaderContentWidget, /*FullRowContent*/true);

			TSharedRef<FPropertyBagInstanceDataDetails> InstanceDetails = MakeShareable(new FPropertyBagInstanceDataDetails(PropertyBagParametersProperty, PropUtils, false));
			ParametersCategory.AddCustomBuilder(InstanceDetails);
		}
	}
	else
	{
		TSharedPtr<IPropertyHandle> PropertyBagParametersProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeEditorData, RootParameterPropertyBag)); // FInstancedPropertyBag
		PropertyBagParametersProperty->MarkHiddenByCustomization();
	}	

	// Evaluators category
	int32 GlobalNumEvaluators = EditorData ? EditorData->Evaluators.Num() : 0;

	TSharedPtr<IPropertyHandle> EvaluatorsProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeEditorData, Evaluators));
	check(EvaluatorsProperty.IsValid());
	const FName EvalCategoryName(TEXT("Evaluators"));
	if (Schema && Schema->AllowEvaluators())
	{
		UE::StateTreeEditor::EditorNodeUtils::MakeArrayCategory(
			DetailBuilder,
			EvaluatorsProperty,
			EvalCategoryName,
			EnumHasAllFlags(GetDefault<UStateTreeEditorUserSettings>()->GetDetailsViewDisplayCount(), EStateTreeEditorUserSettingsNodeType::Evaluator)
				? FText::Format(LOCTEXT("EditorDataDetailsEvaluatorsCounted", "Evaluators ({0})"), GlobalNumEvaluators)
				: LOCTEXT("EditorDataDetailsEvaluators", "Evaluators"),
			FName("StateTreeEditor.Evaluators"),
			UE::StateTree::Colors::Bronze,
			UE::StateTree::Colors::Bronze.WithAlpha(192),
			LOCTEXT("EditorDataDetailsEvaluatorsAddTooltip", "Add new Evaluator"),
			/*SortOrder*/3);
	}
	else
	{
		DetailBuilder.EditCategory(EvalCategoryName).SetCategoryVisibility(false);
	}

	// Global Tasks category
	int32 GlobalNumTasks = EditorData ? EditorData->GlobalTasks.Num() : 0;

	TSharedPtr<IPropertyHandle> GlobalTasksProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeEditorData, GlobalTasks));
	check(GlobalTasksProperty.IsValid());
	const FName GlobalTasksCategoryName(TEXT("Global Tasks"));

	const bool bAllowTasksCompletion = Schema && Schema->AllowTasksCompletion();
	TSharedPtr<IPropertyHandle> GlobalTasksCompletionProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeEditorData, GlobalTasksCompletion));
	GlobalTasksCompletionProperty->MarkHiddenByCustomization();

	IDetailCategoryBuilder& GlobalTasksCategory = UE::StateTreeEditor::EditorNodeUtils::MakeArrayCategoryHeader(
		DetailBuilder,
		GlobalTasksProperty,
		GlobalTasksCategoryName,
		EnumHasAllFlags(GetDefault<UStateTreeEditorUserSettings>()->GetDetailsViewDisplayCount(), EStateTreeEditorUserSettingsNodeType::Task)
			? FText::Format(LOCTEXT("EditorDataDetailsGlobalTasksCounted", "Global Tasks ({0})"), GlobalNumTasks)
			: LOCTEXT("EditorDataDetailsGlobalTasks", "Global Tasks"),
		FName("StateTreeEditor.Tasks"),
		UE::StateTree::Colors::Cyan,
		bAllowTasksCompletion ? GlobalTasksCompletionProperty->CreatePropertyValueWidget(/*bDisplayDefaultPropertyButtons*/false) : TSharedPtr<SWidget>(),
		UE::StateTree::Colors::Cyan.WithAlpha(192),
		LOCTEXT("EditorDataDetailsGlobalTasksAddTooltip", "Add new Global Task"),
		/*SortOrder*/4);
	UE::StateTreeEditor::EditorNodeUtils::MakeArrayItems(GlobalTasksCategory, GlobalTasksProperty);

	// Extensions
	if (EditorSchema && !EditorSchema->AllowExtensions())
	{
		TSharedPtr<IPropertyHandle> ExtensionsProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeEditorData, Extensions));
		check(ExtensionsProperty.IsValid());
		DetailBuilder.EditCategory(ExtensionsProperty->GetDefaultCategoryName()).SetCategoryVisibility(false);
	}

	// Refresh the UI when the Schema changes.
	TSharedPtr<IPropertyHandle> SchemaProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeEditorData, Schema));
	check(SchemaProperty.IsValid());
	SchemaProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([WeakPropertyUtilities = WeakPropertyUtilities] ()
		{
			if (const TSharedPtr<IPropertyUtilities> PropertyUtilities = WeakPropertyUtilities.Pin())
			{
				PropertyUtilities->ForceRefresh();
			}
		}));
	SchemaProperty->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateLambda([WeakPropertyUtilities = WeakPropertyUtilities] ()
		{
			if (const TSharedPtr<IPropertyUtilities> PropertyUtilities = WeakPropertyUtilities.Pin())
			{
				PropertyUtilities->ForceRefresh();
			}
		}));
}

void FStateTreeEditorDataDetails::PendingDelete()
{
	Super::PendingDelete();
	bPendingDelete = true;
}

void FStateTreeEditorDataDetails::MakeArrayCategory(IDetailLayoutBuilder& DetailBuilder, FName CategoryName, const FText& DisplayName, int32 SortOrder, TSharedPtr<IPropertyHandle> PropertyHandle)
{
	IDetailCategoryBuilder& Category = DetailBuilder.EditCategory(CategoryName, DisplayName);
	Category.SetSortOrder(SortOrder);

	TSharedRef<SHorizontalBox> HeaderContentWidget = SNew(SHorizontalBox)
		.IsEnabled(DetailBuilder.GetPropertyUtilities(), &IPropertyUtilities::IsPropertyEditingEnabled);

	HeaderContentWidget->AddSlot()
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Center)
	[
		PropertyHandle->CreateDefaultPropertyButtonWidgets()
	];
	Category.HeaderContent(HeaderContentWidget);

	// Add items inline
	TSharedRef<FDetailArrayBuilder> Builder = MakeShareable(new FDetailArrayBuilder(PropertyHandle.ToSharedRef(), /*InGenerateHeader*/ false, /*InDisplayResetToDefault*/ true, /*InDisplayElementNum*/ false));
	Builder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateLambda([](TSharedRef<IPropertyHandle> PropertyHandle, int32 ArrayIndex, IDetailChildrenBuilder& ChildrenBuilder)
		{
			ChildrenBuilder.AddProperty(PropertyHandle);
		}));
	Category.AddCustomBuilder(Builder, /*bForAdvanced*/ false);
}

void FStateTreeEditorDataDetails::PostUndo(bool bSuccess)
{
	if (bPendingDelete)
	{
		return;
	}

	// Refresh view on undo or redo so that the customization based on e.g. Global tasks bindings, parameters, etc. will be reflected correctly.
	if (const TSharedPtr<IPropertyUtilities> PropertyUtilities = WeakPropertyUtilities.Pin())
	{
		PropertyUtilities->ForceRefresh();
	}
}

void FStateTreeEditorDataDetails::PostRedo(bool bSuccess)
{
	if (bPendingDelete)
	{
		return;
	}

	// Refresh view on undo or redo so that the customization based on e.g. Global tasks bindings, parameters, etc. will be reflected correctly.
	if (const TSharedPtr<IPropertyUtilities> PropertyUtilities = WeakPropertyUtilities.Pin())
	{
		PropertyUtilities->ForceRefresh();
	}
}

#undef LOCTEXT_NAMESPACE
