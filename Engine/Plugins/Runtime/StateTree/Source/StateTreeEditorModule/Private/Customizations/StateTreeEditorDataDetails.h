// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "IDetailCustomization.h"

class IDetailChildrenBuilder;
class IDetailLayoutBuilder;
class IPropertyHandle;
class IPropertyUtilities;

class FStateTreeEditorDataDetails : public IDetailCustomization, FSelfRegisteringEditorUndoClient
{
private:
	using Super = IDetailCustomization;

public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	//~ IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	virtual void PendingDelete() override;

protected:
	void MakeArrayCategory(IDetailLayoutBuilder& DetailBuilder, FName CategoryName, const FText& DisplayName, int32 SortOrder, TSharedPtr<IPropertyHandle> PropertyHandle);

	//~ FEditorUndoClient
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;

	bool bPendingDelete = false;

	TWeakPtr<IPropertyUtilities> WeakPropertyUtilities;
};
