// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeEditorNodeDetails.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "DetailLayoutBuilder.h"
#include "IPropertyUtilities.h"
#include "IDetailChildrenBuilder.h"
#include "StateTree.h"
#include "StateTreeEditor.h"
#include "StateTreeEditorData.h"
#include "StateTreeEditorUserSettings.h"
#include "StateTreePropertyFunctionBase.h"
#include "StateTreePropertyRef.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SStateTreeNodeTypePicker.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Modules/ModuleManager.h"
#include "InstancedStructDetails.h"
#include "StateTreeBindingExtension.h"
#include "StateTreeDelegates.h"
#include "StateTreePropertyHelpers.h"
#include "StateTreeEditorStyle.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Blueprint/StateTreeEvaluatorBlueprintBase.h"
#include "Blueprint/StateTreeTaskBlueprintBase.h"
#include "Blueprint/StateTreeConditionBlueprintBase.h"
#include "Blueprint/StateTreeConsiderationBlueprintBase.h"
#include "Styling/StyleColors.h"
#include "ScopedTransaction.h"
#include "StateTreeEditorNodeUtils.h"
#include "Debugger/StateTreeDebuggerUIExtensions.h"
#include "StateTreeEditingSubsystem.h"
#include "StateTreePropertyBindings.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Styling/SlateTypes.h"
#include "TextStyleDecorator.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "SourceCodeNavigation.h"
#include "StateTreeDelegate.h"
#include "StateTreeEditorDataClipboardHelpers.h"
#include "StateTreeGraphSchema.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"

namespace UE::StateTreeEditor::Internal
{
	/* Returns true if provided property is direct or indirect child of PropertyFunction */
	bool IsOwnedByPropertyFunctionNode(TSharedPtr<IPropertyHandle> Property)
	{
		while (Property)
		{
			if (FStructProperty* StructProperty = CastField<FStructProperty>(Property->GetProperty()))
			{
				if (StructProperty->Struct == FStateTreeEditorNode::StaticStruct())
				{
					if (const FStateTreeEditorNode* Node = UE::StateTreeEditor::EditorNodeUtils::GetCommonNode(Property))
					{
						if (const UScriptStruct* ScriptStruct = Node->GetNode().GetScriptStruct())
						{
							return ScriptStruct->IsChildOf<FStateTreePropertyFunctionBase>();
						}
					}
				}	
			}

			Property = Property->GetParentHandle();
		}

		return false;
	}

	/** @return text describing the pin type, matches SPinTypeSelector. */
	FText GetPinTypeText(const FEdGraphPinType& PinType)
	{
		using namespace UE::StateTree::Editor;

		const FName PinSubCategory = PinType.PinSubCategory;
		const UObject* PinSubCategoryObject = PinType.PinSubCategoryObject.Get();
		if (PinSubCategory != UEdGraphSchema_K2::PSC_Bitmask && PinSubCategoryObject)
		{
			if (const UField* Field = Cast<const UField>(PinSubCategoryObject))
			{
				return Field->GetDisplayNameText();
			}
			return FText::FromString(PinSubCategoryObject->GetName());
		}

		return UEdGraphSchema_StateTree::GetCategoryText(PinType.PinCategory, NAME_None, true);
	}

	/** @return if property is struct property of DelegateDispatcher type. */
	bool IsDelegateDispatcherProperty(const FProperty& Property)
	{
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(&Property))
		{
			return StructProperty->Struct == FStateTreeDelegateDispatcher::StaticStruct();
		}

		return false;
	}

	/** @return UClass or UScriptStruct of class or struct property, nullptr for others. */
	UStruct* GetPropertyStruct(TSharedPtr<IPropertyHandle> PropHandle)
	{
		if (!PropHandle.IsValid())
		{
			return nullptr;
		}

		if (const FStructProperty* StructProperty = CastField<FStructProperty>(PropHandle->GetProperty()))
		{
			return StructProperty->Struct;
		}
		
		if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(PropHandle->GetProperty()))
		{
			return ObjectProperty->PropertyClass;
		}

		return nullptr;
	}

	void ModifyRow(IDetailPropertyRow& ChildRow, const FGuid& ID, UStateTreeEditorData* EditorData)
	{
		using namespace UE::StateTree::Editor;

		FStateTreeEditorPropertyBindings* EditorPropBindings = EditorData ? EditorData->GetPropertyEditorBindings() : nullptr;
		if (!EditorPropBindings)
		{
			return;
		}
		
		TSharedPtr<IPropertyHandle> ChildPropHandle = ChildRow.GetPropertyHandle();
		check(ChildPropHandle.IsValid());
		
		const EStateTreePropertyUsage Usage = UE::StateTree::GetUsageFromMetaData(ChildPropHandle->GetProperty());
		const FProperty* Property = ChildPropHandle->GetProperty();
		
		// Hide output properties for PropertyFunctionNode.
		if (Usage == EStateTreePropertyUsage::Output && UE::StateTreeEditor::Internal::IsOwnedByPropertyFunctionNode(ChildPropHandle))
		{
			ChildRow.Visibility(EVisibility::Hidden);
			return;
		}

		// Conditionally control visibility of the value field of bound properties.
		if (Usage != EStateTreePropertyUsage::Invalid && ID.IsValid())
		{
			// Pass the node ID to binding extension. Since the properties are added using AddChildStructure(), we break the hierarchy and cannot access parent.
			ChildPropHandle->SetInstanceMetaData(UE::PropertyBinding::MetaDataStructIDName, LexToString(ID));

			FPropertyBindingPath Path(ID, *Property->GetFName().ToString());
			TSharedPtr<SWidget> NameWidget;
			TSharedPtr<SWidget> ValueWidget;
			FDetailWidgetRow Row;
			ChildRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);

			const bool bValidUsage = Usage == EStateTreePropertyUsage::Input || Usage == EStateTreePropertyUsage::Output || Usage == EStateTreePropertyUsage::Context;
			const bool bIsDelegateDispatcher = UE::StateTreeEditor::Internal::IsDelegateDispatcherProperty(*Property);

			if (bValidUsage || bIsDelegateDispatcher)
			{
				FEdGraphPinType PinType;
				const UEdGraphSchema_StateTree* Schema = GetDefault<UEdGraphSchema_StateTree>();

				// Show referenced type for property refs.
				if (UE::StateTree::PropertyRefHelpers::IsPropertyRef(*Property))
				{
					// Use internal type to construct PinType if it's property of PropertyRef type.
					FStateTreeDataView TargetDataView;
					if (ensure(EditorData->GetBindingDataViewByID(ID, TargetDataView)))
					{
						TArray<FPropertyBindingPathIndirection> TargetIndirections;
						if (ensure(Path.ResolveIndirectionsWithValue(TargetDataView, TargetIndirections)) && TargetIndirections.Num() > 0)
						{
							const void* PropertyRef = TargetIndirections.Last().GetPropertyAddress();
							PinType = UE::StateTree::PropertyRefHelpers::GetPropertyRefInternalTypeAsPin(*Property, PropertyRef);
						}
					}
				}
				else
				{
					Schema->ConvertPropertyToPinType(Property, PinType);
				}

				auto IsValueVisible = TAttribute<EVisibility>::Create([Path, EditorPropBindings]() -> EVisibility
					{
						return EditorPropBindings->HasBinding(Path, FPropertyBindingBindingCollection::ESearchMode::Exact) ? EVisibility::Collapsed : EVisibility::Visible;
					});

				const FSlateBrush* Icon = FBlueprintEditorUtils::GetIconFromPin(PinType, true);
				FText Text = GetPinTypeText(PinType);
				
				FText ToolTip; 
				FLinearColor IconColor = Schema->GetPinTypeColor(PinType);
				FText Label;
				FText LabelToolTip;
				FSlateColor TextColor = FSlateColor::UseForeground();

				if (bIsDelegateDispatcher)
				{
					Label = LOCTEXT("LabelDelegate", "DELEGATE");
					LabelToolTip = LOCTEXT("DelegateToolTip", "This is Delegate Dispatcher. You can bind to it from listeners.");

					FEdGraphPinType DelegatePinType;
					DelegatePinType.PinCategory = UEdGraphSchema_K2::PC_Struct;
					DelegatePinType.PinSubCategoryObject = FStateTreeDelegateDispatcher::StaticStruct();
					IconColor = Schema->GetPinTypeColor(DelegatePinType);
				}
				else if (Usage == EStateTreePropertyUsage::Input)
				{
					Label = LOCTEXT("LabelInput", "IN");
					LabelToolTip = LOCTEXT("InputToolTip", "This is Input property. It is always expected to be bound to some other property.");
				}
				else if (Usage == EStateTreePropertyUsage::Output)
				{
					Label = LOCTEXT("LabelOutput", "OUT");
					LabelToolTip = LOCTEXT("OutputToolTip", "This is Output property. The node will always set it's value. It can bind to another property to push the value. Other nodes can also bind to it to fetch the value.");
				}
				else if (Usage == EStateTreePropertyUsage::Context)
				{
					Label = LOCTEXT("LabelContext", "CONTEXT");
					LabelToolTip = LOCTEXT("ContextObjectToolTip", "This is Context property. It is automatically connected to one of the Contex objects, or can be overridden with property binding.");

					if (UStruct* Struct = GetPropertyStruct(ChildPropHandle))
					{
						const FStateTreeBindableStructDesc Desc = EditorData->FindContextData(Struct, ChildPropHandle->GetProperty()->GetName());
						if (Desc.IsValid())
						{
							// Show as connected.
							Icon = FCoreStyle::Get().GetBrush("Icons.Link");
							Text = FText::FromName(Desc.Name);
							
							ToolTip = FText::Format(
								LOCTEXT("ToolTipConnected", "Connected to Context {0}."),
									FText::FromName(Desc.Name));
						}
						else
						{
							// Show as unconnected.
							Icon = FCoreStyle::Get().GetBrush("Icons.Warning");
							ToolTip = LOCTEXT("ToolTipNotConnected", "Could not connect Context property automatically.");
						}
					}
					else
					{
						// Mismatching type.
						Text = LOCTEXT("ContextObjectInvalidType", "Invalid type");
						ToolTip = LOCTEXT("ContextObjectInvalidTypeTooltip", "Context properties must be Object references or Structs.");
						Icon = FCoreStyle::Get().GetBrush("Icons.ErrorWithColor");
						IconColor = FLinearColor::White;
					}
				}
				
				ChildRow
					.CustomWidget(true)
					.NameContent()
					[
						SNew(SHorizontalBox)

						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							NameWidget.ToSharedRef()
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(4.0f, 0.0f)
						[
							SNew(SBorder)
							.Padding(FMargin(6.0f, 1.0f))
							.BorderImage(FStateTreeEditorStyle::Get().GetBrush("StateTree.Param.Background"))
							.Visibility(Label.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible)
							[
								SNew(STextBlock)
								.TextStyle(FStateTreeEditorStyle::Get(), "StateTree.Param.Label")
								.ColorAndOpacity(FStyleColors::Foreground)
								.Text(Label)
								.ToolTipText(LabelToolTip)
							]
						]

					]
					.ValueContent()
					[
						SNew(SHorizontalBox)
						.Visibility(IsValueVisible)

						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(4.0f, 0.0f)
						[
							SNew(SImage)
							.Image(Icon)
							.ColorAndOpacity(IconColor)
							.ToolTipText(ToolTip)
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.ColorAndOpacity(TextColor)
							.Text(Text)
							.ToolTipText(ToolTip)
						]
					];
			}
		}
	}

} // UE::StateTreeEditor::Internal

// Customized version of FInstancedStructDataDetails used to hide bindable properties.
class FBindableNodeInstanceDetails : public FInstancedStructDataDetails
{
public:

	FBindableNodeInstanceDetails(TSharedPtr<IPropertyHandle> InStructProperty, const FGuid& InID, UStateTreeEditorData* InEditorData)
		: FInstancedStructDataDetails(InStructProperty)
		, ID(InID)
		, EditorData(InEditorData)
	{
	}

	virtual void OnChildRowAdded(IDetailPropertyRow& ChildRow) override
	{
		UE::StateTreeEditor::Internal::ModifyRow(ChildRow, ID, EditorData.Get());
	}

	FGuid ID;
	TWeakObjectPtr<UStateTreeEditorData> EditorData;
};

////////////////////////////////////

TSharedRef<IPropertyTypeCustomization> FStateTreeEditorNodeDetails::MakeInstance()
{
	return MakeShareable(new FStateTreeEditorNodeDetails);
}

FStateTreeEditorNodeDetails::~FStateTreeEditorNodeDetails()
{
	UE::StateTree::PropertyBinding::OnStateTreePropertyBindingChanged.Remove(OnBindingChangedHandle);
	if (StateTreeViewModel)
	{
		StateTreeViewModel->GetOnAssetChanged().Remove(OnChangedAssetHandle);
	}
}

void FStateTreeEditorNodeDetails::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructProperty = StructPropertyHandle;
	ParentProperty = StructProperty->GetParentHandle();
	ParentArrayProperty = ParentProperty->AsArray();

	PropUtils = StructCustomizationUtils.GetPropertyUtilities();

	NodeProperty = StructProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeEditorNode, Node));
	InstanceProperty = StructProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeEditorNode, Instance));
	InstanceObjectProperty = StructProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeEditorNode, InstanceObject));
	ExecutionRuntimeDataProperty = StructProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeEditorNode, ExecutionRuntimeData));
	ExecutionRuntimeDataObjectProperty = StructProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeEditorNode, ExecutionRuntimeDataObject));
	IDProperty = StructProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeEditorNode, ID));

	IndentProperty = StructProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeEditorNode, ExpressionIndent));
	OperandProperty = StructProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeEditorNode, ExpressionOperand));

	check(NodeProperty.IsValid());
	check(InstanceProperty.IsValid());
	check(InstanceObjectProperty.IsValid());
	check(ExecutionRuntimeDataProperty.IsValid());
	check(ExecutionRuntimeDataObjectProperty.IsValid());
	check(IDProperty.IsValid());
	check(IndentProperty.IsValid());
	check(OperandProperty.IsValid());

	{
		UScriptStruct* BaseScriptStructPtr = nullptr;
		UClass* BaseClassPtr = nullptr;
		UE::StateTreeEditor::EditorNodeUtils::GetNodeBaseScriptStructAndClass(StructProperty, BaseScriptStructPtr, BaseClassPtr);
		BaseScriptStruct = BaseScriptStructPtr;
		BaseClass = BaseClassPtr;
	}

	UE::StateTree::Delegates::OnIdentifierChanged.AddSP(this, &FStateTreeEditorNodeDetails::OnIdentifierChanged);
	OnBindingChangedHandle = UE::StateTree::PropertyBinding::OnStateTreePropertyBindingChanged.AddRaw(this, &FStateTreeEditorNodeDetails::OnBindingChanged);
	FindOuterObjects();
	if (StateTreeViewModel)
	{
		OnChangedAssetHandle = StateTreeViewModel->GetOnAssetChanged().AddSP(this, &FStateTreeEditorNodeDetails::HandleAssetChanged);
	}

	// Don't draw the header if it's a PropertyFunction.
	if (UE::StateTreeEditor::Internal::IsOwnedByPropertyFunctionNode(StructProperty))
	{
		return;
	}

	const FIsResetToDefaultVisible IsResetVisible = FIsResetToDefaultVisible::CreateSP(this, &FStateTreeEditorNodeDetails::ShouldResetToDefault);
	const FResetToDefaultHandler ResetHandler = FResetToDefaultHandler::CreateSP(this, &FStateTreeEditorNodeDetails::ResetToDefault);
	const FResetToDefaultOverride ResetOverride = FResetToDefaultOverride::Create(IsResetVisible, ResetHandler);

	auto IndentColor = [this]() -> FSlateColor
	{
		return (RowBorder && RowBorder->IsHovered()) ? FSlateColor::UseForeground() : FSlateColor(FLinearColor::Transparent);
	};

	TSharedPtr<SBorder> FlagBorder;
	TSharedPtr<SHorizontalBox> DescriptionBox;

	HeaderRow
		.WholeRowContent()
		.VAlign(VAlign_Center)
		[
			// Border to capture mouse clicks on the row (used for right click menu).
			SAssignNew(RowBorder, SBorder)
			.BorderImage(FStyleDefaults::GetNoBrush())
			.Padding(0.0f)
			.ForegroundColor(this, &FStateTreeEditorNodeDetails::GetContentRowColor)
			.OnMouseButtonDown(this, &FStateTreeEditorNodeDetails::OnRowMouseDown)
			.OnMouseButtonUp(this, &FStateTreeEditorNodeDetails::OnRowMouseUp)
			[
				SNew(SHorizontalBox)
				
				// Indent
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.WidthOverride(30.0f)
					.Visibility(this, &FStateTreeEditorNodeDetails::AreIndentButtonsVisible)
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.OnClicked(this, &FStateTreeEditorNodeDetails::HandleIndentPlus)
						.HAlign(HAlign_Center)
						.ContentPadding(FMargin(4.f, 4.f))
						.ToolTipText(LOCTEXT("IncreaseIdentTooltip", "Increment the depth of the expression row controlling parentheses and expression order"))
						[
							SNew(SImage)
							.DesiredSizeOverride(FVector2D(8.f, 8.f))
							.Image(FAppStyle::GetBrush("Icons.Plus"))
							.ColorAndOpacity_Lambda(IndentColor)
						]
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.WidthOverride(this, &FStateTreeEditorNodeDetails::GetIndentSize)
					.Visibility(this, &FStateTreeEditorNodeDetails::AreIndentButtonsVisible)
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.OnClicked(this, &FStateTreeEditorNodeDetails::HandleIndentMinus)
						.HAlign(HAlign_Center)
						.ContentPadding(FMargin(4.f, 4.f))
						.ToolTipText(LOCTEXT("DecreaseIndentTooltip", "Decrement the depth of the expression row controlling parentheses and expression order"))
						[
							SNew(SImage)
							.DesiredSizeOverride(FVector2D(8.f, 8.f))
							.Image(FAppStyle::GetBrush("Icons.Minus"))
							.ColorAndOpacity_Lambda(IndentColor)
						]
					]
				]

				// Operand
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.MinDesiredWidth(30.0f)
					.Padding(FMargin(2.0f, 4.0f, 2.0f, 3.0f))
					.VAlign(VAlign_Center)
					.Visibility(this, &FStateTreeEditorNodeDetails::IsOperandVisible)
					[
						SNew(SComboButton)
						.IsEnabled(TAttribute<bool>(this, &FStateTreeEditorNodeDetails::IsOperandEnabled))
						.ComboButtonStyle(FStateTreeEditorStyle::Get(), "StateTree.Node.Operand.ComboBox")
						.ButtonColorAndOpacity(this, &FStateTreeEditorNodeDetails::GetOperandColor)
						.HasDownArrow(false)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.OnGetMenuContent(this, &FStateTreeEditorNodeDetails::OnGetOperandContent)
						.ButtonContent()
						[
							SNew(STextBlock)
							.TextStyle(FStateTreeEditorStyle::Get(), "StateTree.Node.Operand")
							.Text(this, &FStateTreeEditorNodeDetails::GetOperandText)
						]
					]
				]
				// Open parens
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.Padding(FMargin(FMargin(0.0f, 0.0f, 4.0f, 0.0f)))
					.Visibility(this, &FStateTreeEditorNodeDetails::AreParensVisible)
					[
						SNew(STextBlock)
						.TextStyle(FStateTreeEditorStyle::Get(), "StateTree.Node.Parens")
						.Text(this, &FStateTreeEditorNodeDetails::GetOpenParens)
					]
				]
				// Description
				+ SHorizontalBox::Slot()
				.FillContentWidth(0.0f, 1.0f) // no growing, allow shrink
				.VAlign(VAlign_Center)
				.Padding(FMargin(0.0f, 4.0f))
				[
					SAssignNew(DescriptionBox, SHorizontalBox)
					.Clipping(EWidgetClipping::ClipToBoundsAlways)

					// Icon
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(0.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(SImage)
						.Image(this, &FStateTreeEditorNodeDetails::GetIcon)
						.ColorAndOpacity(this, &FStateTreeEditorNodeDetails::GetIconColor)
						.Visibility(this, &FStateTreeEditorNodeDetails::IsIconVisible)
					]

					// Rich text description and name edit 
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(0.0f, 0.0f, 0.0f, 0.0f)
					[
						SAssignNew(NameSwitcher, SWidgetSwitcher)
						.WidgetIndex(0)
						+ SWidgetSwitcher::Slot()
						[
							SNew(SBox)
							.Padding(FMargin(1.0f,0.0f, 1.0f, 1.0f))
							[
								SNew(SRichTextBlock)
								.Text(this, &FStateTreeEditorNodeDetails::GetNodeDescription)
								.TextStyle(&FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("StateTree.Node.Normal"))
								.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
								.Visibility(this, &FStateTreeEditorNodeDetails::IsNodeDescriptionVisible)
								.ToolTipText(this, &FStateTreeEditorNodeDetails::GetNodeTooltip)
								+SRichTextBlock::Decorator(FTextStyleDecorator::Create(TEXT(""), FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("StateTree.Node.Normal")))
								+SRichTextBlock::Decorator(FTextStyleDecorator::Create(TEXT("b"), FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("StateTree.Node.Bold")))
								+SRichTextBlock::Decorator(FTextStyleDecorator::Create(TEXT("s"), FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("StateTree.Node.Subdued")))
							]
						]
						+ SWidgetSwitcher::Slot()
						[
							SAssignNew(NameEdit, SInlineEditableTextBlock)
							.Style(FStateTreeEditorStyle::Get(), "StateTree.Node.TitleInlineEditableText")
							.Text(this, &FStateTreeEditorNodeDetails::GetName)
							.OnTextCommitted(this, &FStateTreeEditorNodeDetails::HandleNameCommitted)
							.OnVerifyTextChanged(this, &FStateTreeEditorNodeDetails::HandleVerifyNameChanged)
							.Visibility(this, &FStateTreeEditorNodeDetails::IsNodeDescriptionVisible)
						]
					]

					// Flags icons
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(FMargin(4.0f, 0.0f))
					[
						SAssignNew(FlagsContainer, SBorder)
							.BorderImage(FStyleDefaults::GetNoBrush())
							.Visibility(this, &FStateTreeEditorNodeDetails::AreFlagsVisible)
					]

					// Close parens
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.TextStyle(FStateTreeEditorStyle::Get(), "StateTree.Node.Parens")
						.Text(this, &FStateTreeEditorNodeDetails::GetCloseParens)
						.Visibility(this, &FStateTreeEditorNodeDetails::AreParensVisible)
					]
				]

				// Debug and property widgets
				+ SHorizontalBox::Slot()
				.FillContentWidth(1.0f, 0.0f) // grow, no shrinking
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.Padding(FMargin(8.0f, 0.0f, 2.0f, 0.0f))
				[
					SNew(SHorizontalBox)

					// Debugger labels
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						UE::StateTreeEditor::DebuggerExtensions::CreateEditorNodeWidget(StructPropertyHandle, StateTreeViewModel)
					]

					// Browse To source Button
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SBox)
						.Visibility(this, &FStateTreeEditorNodeDetails::IsBrowseToSourceVisible)
						.Padding(FMargin(4.f, 0.f, 0.f, 0.f))
						[
							SNew(SButton)
							.ButtonStyle(FAppStyle::Get(), "SimpleButton")
							.OnClicked(this, &FStateTreeEditorNodeDetails::OnBrowseToSource)
							.ToolTipText(FText::Format(LOCTEXT("GoToCode_ToolTip", "Click to open the node source file in {0}"), FSourceCodeNavigation::GetSelectedSourceCodeIDE()))
							.ContentPadding(0.0f)
							[
								SNew(SImage)
								.Image(FAppStyle::GetBrush("Icons.OpenSourceLocation"))
								.ColorAndOpacity(FSlateColor::UseForeground())
							]
						]
					]

					// Browse To BP Button
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SBox)
						.Visibility(this, &FStateTreeEditorNodeDetails::IsBrowseToNodeBlueprintVisible)
						.Padding(FMargin(4.f, 0.f, 0.f, 0.f))
						[
							SNew(SButton)
							.ButtonStyle(FAppStyle::Get(), "SimpleButton")
							.OnClicked(this, &FStateTreeEditorNodeDetails::OnBrowseToNodeBlueprint)
							.ToolTipText(LOCTEXT("BrowseToCurrentNodeBP", "Browse to the current node blueprint in Content Browser"))
							.ContentPadding(0.0f)
							[
								SNew(SImage)
								.Image(FAppStyle::GetBrush("Icons.BrowseContent"))
								.ColorAndOpacity(FSlateColor::UseForeground())
							]
						]
					]
					// Edit BP Button
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SBox)
						.Visibility(this, &FStateTreeEditorNodeDetails::IsEditNodeBlueprintVisible)
						.Padding(FMargin(4.f, 0.f, 0.f, 0.f))
						[
							SNew(SButton)
							.ButtonStyle(FAppStyle::Get(), "SimpleButton")
							.OnClicked(this, &FStateTreeEditorNodeDetails::OnEditNodeBlueprint)
							.ToolTipText(LOCTEXT("EditCurrentNodeBP", "Edit the current node blueprint in Editor"))
							.ContentPadding(0.0f)
							[
								SNew(SImage)
								.Image(FAppStyle::GetBrush("Icons.Edit"))
								.ColorAndOpacity(FSlateColor::UseForeground())
							]
						]
					]

					// Options
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Right)
					.Padding(FMargin(4.f, 0.f, 0.f, 0.f))
					[
						SNew(SComboButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.OnGetMenuContent(this, &FStateTreeEditorNodeDetails::GenerateOptionsMenu)
						.ToolTipText(LOCTEXT("ItemActions", "Item actions"))
						.HasDownArrow(false)
						.ContentPadding(FMargin(4.0f, 2.0f))
						.ButtonContent()
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("Icons.ChevronDown"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]
				]
			]
		]
		.OverrideResetToDefault(ResetOverride)
		.CopyAction(FUIAction(FExecuteAction::CreateSP(this, &FStateTreeEditorNodeDetails::OnCopyNode)))
		.PasteAction(FUIAction(FExecuteAction::CreateSP(this, &FStateTreeEditorNodeDetails::OnPasteNodes)));

	// Task completion
	bool bShowCompletion = true;
	if (const UStateTreeEditorData* EditorDataPtr = EditorData.Get())
	{
		bShowCompletion = EditorDataPtr->Schema ? EditorDataPtr->Schema->AllowTasksCompletion() : true;
	}
	if (bShowCompletion)
	{
		if (const FStateTreeEditorNode* Node = UE::StateTreeEditor::EditorNodeUtils::GetCommonNode(StructProperty))
		{
			if (const FStateTreeTaskBase* TaskBase = Node->GetNode().GetPtr<FStateTreeTaskBase>())
			{
				DescriptionBox->InsertSlot(0)
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 0.0f, 0.0f)
				[
					// Create the toggle favorites button
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.OnClicked(this, &FStateTreeEditorNodeDetails::HandleToggleCompletionTaskClicked)
					.ToolTipText(this, &FStateTreeEditorNodeDetails::GetToggleCompletionTaskTooltip)
					[
						SNew(SImage)
						.ColorAndOpacity(this, &FStateTreeEditorNodeDetails::GetToggleCompletionTaskColor)
						.Image(this, &FStateTreeEditorNodeDetails::GetToggleCompletionTaskIcon)
					]
					.IsEnabled(UE::StateTreeEditor::EditorNodeUtils::CanEditTaskConsideredForCompletion(*Node))
					.Visibility(this, &FStateTreeEditorNodeDetails::GetToggleCompletionTaskVisibility)
				];
			}
		}
	}

	MakeFlagsWidget();
}

void FStateTreeEditorNodeDetails::MakeFlagsWidget()
{
	if (!FlagsContainer.IsValid())
	{
		return;
	}

	FlagsContainer->SetPadding(FMargin(4.0f));
	FlagsContainer->SetContent(SNullWidget::NullWidget);

	const UStateTree* StateTreePtr = StateTree.Get();

	TArray<void*> RawNodeDatas;
	StructProperty->AccessRawData(RawNodeDatas);
	bool bShowCallTick = false;
	bool bShouldCallTickOnlyOnEvents = false;
	bool bHasTransitionTick = false;
	for (void* RawNodeData : RawNodeDatas)
	{
		if (const FStateTreeEditorNode* EditorNode = reinterpret_cast<const FStateTreeEditorNode*>(RawNodeData))
		{
			bool bUseEditorData = true;
			// Use the compiled version if it exists. It is more accurate (like with BP tasks) but less interactive (the user needs to compile) :(
			if (StateTreePtr)
			{
				if (const FStateTreeTaskBase* CompiledTask = StateTreePtr->GetNode(StateTreePtr->GetNodeIndexFromId(EditorNode->ID).AsInt32()).GetPtr<const FStateTreeTaskBase>())
				{
					if (CompiledTask->bConsideredForScheduling)
					{
						bShowCallTick = bShowCallTick || CompiledTask->bShouldCallTick;
						bShouldCallTickOnlyOnEvents = bShouldCallTickOnlyOnEvents || CompiledTask->bShouldCallTickOnlyOnEvents;
						bHasTransitionTick = bHasTransitionTick || CompiledTask->bShouldAffectTransitions;
					}
					bUseEditorData = false;
				}
			}

			if (bUseEditorData)
			{
				if (const FStateTreeTaskBase* TreeTaskNodePtr = EditorNode->GetNode().GetPtr<FStateTreeTaskBase>())
				{
					if (TreeTaskNodePtr->bConsideredForScheduling)
					{
						bShowCallTick = bShowCallTick || TreeTaskNodePtr->bShouldCallTick;
						bShouldCallTickOnlyOnEvents = bShouldCallTickOnlyOnEvents || TreeTaskNodePtr->bShouldCallTickOnlyOnEvents;
						bHasTransitionTick = bHasTransitionTick || TreeTaskNodePtr->bShouldAffectTransitions;
					}
				}
			}
		}
	}

	if (bShowCallTick || bShouldCallTickOnlyOnEvents || bHasTransitionTick)
	{
		TSharedRef<SHorizontalBox> Box = SNew(SHorizontalBox);
		const bool bShowTickIcon = bShowCallTick || bHasTransitionTick;
		if (bShowTickIcon)
		{
			Box->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Flags.Tick"))
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.ToolTipText(LOCTEXT("TaskTick", "The task ticks at runtime."))
			];
		}

		if (!bShowTickIcon && bShouldCallTickOnlyOnEvents)
		{
			Box->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
					.Image(FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Flags.TickOnEvent"))
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.ToolTipText(LOCTEXT("TaskTickEvent", "The task ticks on event at runtime."))
			];
		}

		FlagsContainer->SetPadding(FMargin(4.0f));
		FlagsContainer->SetContent(Box);
	}
}

FReply FStateTreeEditorNodeDetails::OnRowMouseDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply FStateTreeEditorNodeDetails::OnRowMouseUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
		FSlateApplication::Get().PushMenu(
			NameSwitcher.ToSharedRef(),
			WidgetPath,
			GenerateOptionsMenu(),
			MouseEvent.GetScreenSpacePosition(),
			FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));

		return FReply::Handled();
	}
	
	return FReply::Unhandled();
}

void FStateTreeEditorNodeDetails::OnCopyNode()
{
	UStateTreeEditorData* EditorDataPtr = EditorData.Get();
	if (!EditorDataPtr)
	{
		return;
	}

	void* EditorNodeAddress = nullptr;
	if (StructProperty->GetValueData(EditorNodeAddress) == FPropertyAccess::Success)
	{
		UE::StateTreeEditor::FClipboardEditorData Clipboard;
		Clipboard.Append(EditorDataPtr, TConstArrayView<FStateTreeEditorNode>(static_cast<FStateTreeEditorNode*>(EditorNodeAddress), 1));
		UE::StateTreeEditor::ExportTextAsClipboardEditorData(Clipboard);
	}
}

void FStateTreeEditorNodeDetails::OnCopyAllNodes()
{
	UStateTreeEditorData* EditorDataPtr = EditorData.Get();
	if (!EditorDataPtr)
	{
		return;
	}

	if (ParentArrayProperty)
	{
		void* EditorNodeArrayAddress = nullptr;
		if (ParentProperty->GetValueData(EditorNodeArrayAddress) == FPropertyAccess::Success)
		{
			UE::StateTreeEditor::FClipboardEditorData Clipboard;
			Clipboard.Append(EditorDataPtr, *static_cast<TArray<FStateTreeEditorNode>*>(EditorNodeArrayAddress));
			UE::StateTreeEditor::ExportTextAsClipboardEditorData(Clipboard);
		}
	}
	else
	{
		OnCopyNode();
	}
}

void FStateTreeEditorNodeDetails::OnPasteNodes()
{
	using namespace UE::StateTreeEditor;

	TArray<UObject*> OuterObjects;
	StructProperty->GetOuterObjects(OuterObjects);
	if (OuterObjects.IsEmpty())
	{
		return;
	}

	UStateTreeEditorData* EditorDataPtr = EditorData.Get();
	if (!EditorDataPtr)
	{
		return;
	}

	// In case its multi selected, we need to have a unique copy for each Object
	TArray<FClipboardEditorData, TInlineAllocator<2>> ClipboardEditorDatas;
	ClipboardEditorDatas.AddDefaulted(OuterObjects.Num());

	for (int32 Idx = 0; Idx < OuterObjects.Num(); ++Idx)
	{
		const bool bSuccess = ImportTextAsClipboardEditorData
		(BaseScriptStruct.Get(),
			EditorData.Get(),
			OuterObjects[Idx],
			ClipboardEditorDatas[Idx]);

		if (!bSuccess)
		{
			return;
		}
	}

	// make sure each Clipboard has the same number of nodes
	for (int32 Idx = 0; Idx < ClipboardEditorDatas.Num() - 1; ++Idx)
	{
		check(ClipboardEditorDatas[Idx].GetEditorNodesInBuffer().Num() == ClipboardEditorDatas[Idx + 1].GetEditorNodesInBuffer().Num());
	}

	int32 NumEditorNodesInBuffer = ClipboardEditorDatas[0].GetEditorNodesInBuffer().Num();

	if (NumEditorNodesInBuffer == 0)
	{
		return;
	}

	if (!ParentArrayProperty.IsValid() && NumEditorNodesInBuffer != 1)
	{
		// Node is not in an array. we can't do multi-to-one paste
		return;
	}

	if (OuterObjects.Num() != 1 && NumEditorNodesInBuffer != 1)
	{
		// if multiple selected objects, and we have more than one nodes to paste into
		// Array Handle doesn't support manipulation on multiple objects.
		FNotificationInfo NotificationInfo(FText::GetEmpty());
		NotificationInfo.Text = LOCTEXT("NotSupportedByMultipleObjects", "Operation is not supported for multi-selected objects");
		NotificationInfo.ExpireDuration = 5.0f;
		FSlateNotificationManager::Get().AddNotification(NotificationInfo);

		return;
	}

	{
		FScopedTransaction Transaction(LOCTEXT("PasteNode", "Paste Node"));

		EditorDataPtr->Modify();	// we might modify the bindings on Editor Data
		StructProperty->NotifyPreChange();

		if (ParentArrayProperty)
		{
			// Paste multi nodes into one node
			const int32 StructIndex = StructProperty->GetIndexInArray();
			check(StructIndex != INDEX_NONE);

			uint32 NumArrayElements = 0;
			ParentArrayProperty->GetNumElements(NumArrayElements);
			check(NumArrayElements > 0);	// since we already have at least one element to paste into

			// Insert or append uninitialized elements after the current node to match the number of nodes in the paste buffer and retain the order of elements 
			// The first node in the buffer goes into the current node
			const int32 IndexToInsert = StructIndex + 1;
			const int32 NumElementsToAddOrInsert = ClipboardEditorDatas[0].GetEditorNodesInBuffer().Num() - 1;

			int32 Cnt = 0;
			if (IndexToInsert == NumArrayElements)
			{
				while (Cnt++ < NumElementsToAddOrInsert)
				{
					FPropertyHandleItemAddResult Result = ParentArrayProperty->AddItem();
					if (Result.GetAccessResult() != FPropertyAccess::Success)
					{
						return;
					}
				}
			}
			else
			{
				while (Cnt++ < NumElementsToAddOrInsert)
				{
					FPropertyAccess::Result Result = ParentArrayProperty->Insert(IndexToInsert);
					if (Result != FPropertyAccess::Success)
					{
						return;
					}
				}
			}

			TArray<void*> RawDatasArray;
			ParentProperty->AccessRawData(RawDatasArray);
			check(RawDatasArray.Num() == OuterObjects.Num());
			for (int32 ObjIdx = 0; ObjIdx < OuterObjects.Num(); ++ObjIdx)
			{
				if (TArray<FStateTreeEditorNode>* EditorNodesPtr = static_cast<TArray<FStateTreeEditorNode>*>(RawDatasArray[ObjIdx]))
				{
					TArrayView<FStateTreeEditorNode> EditorNodesClipboardBuffer = ClipboardEditorDatas[ObjIdx].GetEditorNodesInBuffer();
					TArray<FStateTreeEditorNode>& EditorNodesToPasteInto = *EditorNodesPtr;

					for (int32 Idx = 0; Idx < EditorNodesClipboardBuffer.Num(); ++Idx)
					{
						EditorNodesToPasteInto[StructIndex + Idx] = MoveTemp(EditorNodesClipboardBuffer[Idx]);
					}

					for (FStateTreePropertyPathBinding& Binding : ClipboardEditorDatas[ObjIdx].GetBindingsInBuffer())
					{
						EditorDataPtr->GetPropertyEditorBindings()->AddStateTreeBinding(MoveTemp(Binding));
					}
				}
			}
		}
		else
		{
			// Paste single node to a single Node
			TArray<void*> RawDatas;
			StructProperty->AccessRawData(RawDatas);
			check(RawDatas.Num() == OuterObjects.Num());
			for (int32 Idx = 0; Idx < RawDatas.Num(); ++Idx)
			{
				if (FStateTreeEditorNode* CurrentEditorNode = static_cast<FStateTreeEditorNode*>(RawDatas[Idx]))
				{
					*CurrentEditorNode = MoveTemp(ClipboardEditorDatas[Idx].GetEditorNodesInBuffer()[0]);

					for (FStateTreePropertyPathBinding& Binding : ClipboardEditorDatas[Idx].GetBindingsInBuffer())
					{
						EditorDataPtr->GetPropertyEditorBindings()->AddStateTreeBinding(MoveTemp(Binding));
					}
				}
			}
		}

		StructProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
		StructProperty->NotifyFinishedChangingProperties();
	}

	if (PropUtils)
	{
		PropUtils->ForceRefresh();
	}
}

bool FStateTreeEditorNodeDetails::ShouldResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle) const
{
	check(StructProperty);
	
	bool bAnyValid = false;
	
	TArray<const void*> RawNodeData;
	StructProperty->AccessRawData(RawNodeData);
	for (const void* Data : RawNodeData)
	{
		if (const FStateTreeEditorNode* Node = static_cast<const FStateTreeEditorNode*>(Data))
		{
			if (Node->GetNode().IsValid())
			{
				bAnyValid = true;
				break;
			}
		}
	}
	
	// Assume that the default value is empty. Any valid means that some can be reset to empty.
	return bAnyValid;
}


void FStateTreeEditorNodeDetails::ResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle)
{
	UE::StateTreeEditor::EditorNodeUtils::ModifyNodeInTransaction(LOCTEXT("OnTaskEnableToggled", "Toggled Task Enabled"),
		StructProperty,
		[](const TSharedPtr<IPropertyHandle>& StructPropertyHandle)
		{
			TArray<void*> RawNodeData;
			StructPropertyHandle->AccessRawData(RawNodeData);
			for (void* Data : RawNodeData)
			{
				if (FStateTreeEditorNode* Node = static_cast<FStateTreeEditorNode*>(Data))
				{
					Node->Reset();
				}
			}
		});

	if (PropUtils)
	{
		PropUtils->ForceRefresh();
	}
}

void FStateTreeEditorNodeDetails::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	const FStateTreeEditorNode* EditorNodePtr = UE::StateTree::PropertyHelpers::GetStructPtr<FStateTreeEditorNode>(StructProperty);
	if (EditorNodePtr == nullptr)
	{
		// in case StructPropertyHandle is not valid
		return;
	}

	// ID
	if (UE::StateTree::Editor::GbDisplayItemIds)
	{
		// ID
		StructBuilder.AddProperty(IDProperty.ToSharedRef());
	}
	
	UStateTreeEditorData* EditorDataPtr = EditorData.Get();
	auto AddObjectInstance = [this, &StructBuilder, EditorNodePtr, EditorDataPtr](TSharedPtr<IPropertyHandle> ValueProperty)
		{
			if (ValueProperty.IsValid())
			{
				uint32 NumChildren = 0;
				ValueProperty->GetNumChildren(NumChildren);

				// Find visible child properties and sort them so in order: Context, Input, Param, Output.
				struct FSortedChild
				{
					TSharedPtr<IPropertyHandle> PropertyHandle;
					EStateTreePropertyUsage Usage = EStateTreePropertyUsage::Invalid;
				};

				TArray<FSortedChild> SortedChildren;
				for (uint32 Index = 0; Index < NumChildren; Index++)
				{
					if (TSharedPtr<IPropertyHandle> ChildHandle = ValueProperty->GetChildHandle(Index); ChildHandle.IsValid())
					{
						FSortedChild Child;
						Child.PropertyHandle = ChildHandle;
						Child.Usage = UE::StateTree::GetUsageFromMetaData(Child.PropertyHandle->GetProperty());

						// If the property is set to one of these usages, display it even if it is not edit on instance.
						// It is a common mistake to forget to set the "eye" on these properties it and wonder why it does not show up.
						const bool bShouldShowByUsage = Child.Usage == EStateTreePropertyUsage::Input || Child.Usage == EStateTreePropertyUsage::Output || Child.Usage == EStateTreePropertyUsage::Context;
        				const bool bIsEditable = !Child.PropertyHandle->GetProperty()->HasAllPropertyFlags(CPF_DisableEditOnInstance);

						if (bShouldShowByUsage || bIsEditable)
						{
							SortedChildren.Add(Child);
						}
					}
				}

				SortedChildren.StableSort([](const FSortedChild& LHS, const FSortedChild& RHS) { return LHS.Usage < RHS.Usage; });

				for (FSortedChild& Child : SortedChildren)
				{
					IDetailPropertyRow& ChildRow = StructBuilder.AddProperty(Child.PropertyHandle.ToSharedRef());
					UE::StateTreeEditor::Internal::ModifyRow(ChildRow, EditorNodePtr->ID, EditorDataPtr);
				}
			}
		};

	// Node
	TSharedRef<FBindableNodeInstanceDetails> NodeDetails = MakeShareable(new FBindableNodeInstanceDetails(NodeProperty, EditorNodePtr->GetNodeID(), EditorDataPtr));
	StructBuilder.AddCustomBuilder(NodeDetails);

	// Instance
	TSharedRef<FBindableNodeInstanceDetails> InstanceDetails = MakeShareable(new FBindableNodeInstanceDetails(InstanceProperty, EditorNodePtr->ID, EditorDataPtr));
	StructBuilder.AddCustomBuilder(InstanceDetails);

	// InstanceObject
	// Get the actual UObject from the pointer.
	TSharedPtr<IPropertyHandle> InstanceObjectValueProperty = GetInstancedObjectValueHandle(InstanceObjectProperty);
	AddObjectInstance(InstanceObjectValueProperty);

	// ExecutionRuntime Instance
	TSharedRef<FBindableNodeInstanceDetails> ExecutionRuntimeDataDetails = MakeShareable(new FBindableNodeInstanceDetails(ExecutionRuntimeDataProperty, EditorNodePtr->ID, EditorDataPtr));
	ExecutionRuntimeDataProperty->SetInstanceMetaData(UE::PropertyBinding::MetaDataNoBindingName, TEXT("true"));
	StructBuilder.AddCustomBuilder(ExecutionRuntimeDataDetails);

	// ExecutionRuntime Instance Object
	TSharedPtr<IPropertyHandle> ExecutionRuntimeDataObjectValueProperty = GetInstancedObjectValueHandle(ExecutionRuntimeDataObjectProperty);
	ExecutionRuntimeDataObjectProperty->SetInstanceMetaData(UE::PropertyBinding::MetaDataNoBindingName, TEXT("true"));
	AddObjectInstance(ExecutionRuntimeDataObjectValueProperty);
}

TSharedPtr<IPropertyHandle> FStateTreeEditorNodeDetails::GetInstancedObjectValueHandle(TSharedPtr<IPropertyHandle> PropertyHandle)
{
	TSharedPtr<IPropertyHandle> ChildHandle;

	uint32 NumChildren;
	PropertyHandle->GetNumChildren(NumChildren);

	if (NumChildren > 0)
	{
		// when the property is a (inlined) object property, the first child will be
		// the object instance, and its properties are the children underneath that
		ensure(NumChildren == 1);
		ChildHandle = PropertyHandle->GetChildHandle(0);
	}

	return ChildHandle;
}

void FStateTreeEditorNodeDetails::OnIdentifierChanged(const UStateTree& InStateTree)
{
	if (PropUtils && StateTree == &InStateTree)
	{
		PropUtils->ForceRefresh();
	}
}

void FStateTreeEditorNodeDetails::OnBindingChanged(const FPropertyBindingPath& SourcePath, const FPropertyBindingPath& TargetPath)
{
	check(StructProperty);

	UStateTreeEditorData* EditorDataPtr = EditorData.Get();
	if (!EditorDataPtr)
	{
		return;
	}

	TArray<UObject*> OuterObjects;
	StructProperty->GetOuterObjects(OuterObjects);

	TArray<void*> RawNodeData;
	StructProperty->AccessRawData(RawNodeData);

	if (OuterObjects.Num() != RawNodeData.Num())
	{
		return;
	}

	const FStateTreeBindingLookup BindingLookup(EditorDataPtr);

	for (int32 i = 0; i < OuterObjects.Num(); i++)
	{
		FStateTreeEditorNode* EditorNode = static_cast<FStateTreeEditorNode*>(RawNodeData[i]);
		UObject* OuterObject = OuterObjects[i]; // Immediate outer, i.e StateTreeState
		if (EditorNode && OuterObject && EditorNode->ID == TargetPath.GetStructID())
		{
			FStateTreeNodeBase* Node = EditorNode->GetNode().GetPtr<FStateTreeNodeBase>();
			FStateTreeDataView InstanceView = EditorNode->GetInstance(); 

			if (Node && InstanceView.IsValid())
			{
				OuterObject->Modify();
				Node->OnBindingChanged(EditorNode->ID, InstanceView, SourcePath, TargetPath, BindingLookup);
			}
		}
	}
}

void FStateTreeEditorNodeDetails::FindOuterObjects()
{
	check(StructProperty);
	
	EditorData.Reset();
	StateTree.Reset();
	StateTreeViewModel.Reset();

	TArray<UObject*> OuterObjects;
	StructProperty->GetOuterObjects(OuterObjects);
	for (UObject* Outer : OuterObjects)
	{
		UStateTreeEditorData* OuterEditorData = Cast<UStateTreeEditorData>(Outer);
		if (OuterEditorData == nullptr)
		{
			OuterEditorData = Outer->GetTypedOuter<UStateTreeEditorData>();
		}
		
		UStateTree* OuterStateTree = OuterEditorData ? OuterEditorData->GetTypedOuter<UStateTree>() : nullptr;
		if (OuterEditorData && OuterStateTree)
		{
			StateTree = OuterStateTree;
			EditorData = OuterEditorData;
			if (UStateTreeEditingSubsystem* StateTreeEditingSubsystem = GEditor ? GEditor->GetEditorSubsystem<UStateTreeEditingSubsystem>() : nullptr)
			{
				StateTreeViewModel = StateTreeEditingSubsystem->FindOrAddViewModel(OuterStateTree);
			}
			break;
		}
	}
}

FOptionalSize FStateTreeEditorNodeDetails::GetIndentSize() const
{
	return FOptionalSize(static_cast<float>(GetIndent()) * 30.0f);
}

FReply FStateTreeEditorNodeDetails::HandleIndentPlus()
{
	SetIndent(GetIndent() + 1);
	return FReply::Handled();
}

FReply FStateTreeEditorNodeDetails::HandleIndentMinus()
{
	SetIndent(GetIndent() - 1);
	return FReply::Handled();
}

int32 FStateTreeEditorNodeDetails::GetIndent() const
{
	check(IndentProperty);
	
	uint8 Indent = 0;
	IndentProperty->GetValue(Indent);

	return Indent;
}

void FStateTreeEditorNodeDetails::SetIndent(const int32 Indent) const
{
	check(IndentProperty);
	
	IndentProperty->SetValue((uint8)FMath::Clamp(Indent, 0, UE::StateTree::MaxExpressionIndent - 1));
}

bool FStateTreeEditorNodeDetails::IsIndent(const int32 Indent) const
{
	return Indent == GetIndent();
}

bool FStateTreeEditorNodeDetails::IsFirstItem() const
{
	check(StructProperty);
	return StructProperty->GetIndexInArray() == 0;
}

int32 FStateTreeEditorNodeDetails::GetCurrIndent() const
{
	// First item needs to be zero indent to make the parentheses counting to work properly.
	return IsFirstItem() ? 0 : (GetIndent() + 1);
}

int32 FStateTreeEditorNodeDetails::GetNextIndent() const
{
	// Find the intent of the next item by finding the item in the parent array.
	check(StructProperty);
	TSharedPtr<IPropertyHandle> ParentProp = StructProperty->GetParentHandle();
	if (!ParentProp.IsValid())
	{
		return 0;
	}
	TSharedPtr<IPropertyHandleArray> ParentArray = ParentProp->AsArray();
	if (!ParentArray.IsValid())
	{
		return 0;
	}

	uint32 NumElements = 0;
	if (ParentArray->GetNumElements(NumElements) != FPropertyAccess::Success)
	{
		return 0;
	}

	const int32 NextIndex = StructProperty->GetIndexInArray() + 1;
	if (NextIndex >= (int32)NumElements)
	{
		return 0;
	}

	TSharedPtr<IPropertyHandle> NextStructProperty = ParentArray->GetElement(NextIndex);
	if (!NextStructProperty.IsValid())
	{
		return 0;
	}

	TSharedPtr<IPropertyHandle> NextIndentProperty = NextStructProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeEditorNode, ExpressionIndent));
	if (!NextIndentProperty.IsValid())
	{
		return 0;
	}

	uint8 Indent = 0;
	NextIndentProperty->GetValue(Indent);

	return Indent + 1;
}

FText FStateTreeEditorNodeDetails::GetOpenParens() const
{
	check(IndentProperty);

	const int32 CurrIndent = GetCurrIndent();
	const int32 NextIndent = GetNextIndent();
	const int32 DeltaIndent = NextIndent - CurrIndent;
	const int32 OpenParens = FMath::Max(0, DeltaIndent);

	static_assert(UE::StateTree::MaxExpressionIndent == 4);
	switch (OpenParens)
	{
	case 1: return FText::FromString(TEXT("("));
	case 2: return FText::FromString(TEXT("(("));
	case 3: return FText::FromString(TEXT("((("));
	case 4: return FText::FromString(TEXT("(((("));
	}
	return FText::GetEmpty();
}

FText FStateTreeEditorNodeDetails::GetCloseParens() const
{
	check(IndentProperty);

	const int32 CurrIndent = GetCurrIndent();
	const int32 NextIndent = GetNextIndent();
	const int32 DeltaIndent = NextIndent - CurrIndent;
	const int32 CloseParens = FMath::Max(0, -DeltaIndent);

	static_assert(UE::StateTree::MaxExpressionIndent == 4);
	switch (CloseParens)
	{
	case 1: return FText::FromString(TEXT(")"));
	case 2: return FText::FromString(TEXT("))"));
	case 3: return FText::FromString(TEXT(")))"));
	case 4: return FText::FromString(TEXT("))))"));
	}
	return FText::GetEmpty();
}

FSlateColor FStateTreeEditorNodeDetails::GetContentRowColor() const
{
	return UE::StateTreeEditor::DebuggerExtensions::IsEditorNodeEnabled(StructProperty)
		? FSlateColor::UseForeground()
		: FSlateColor::UseSubduedForeground();
}

FText FStateTreeEditorNodeDetails::GetOperandText() const
{
	check(OperandProperty);

	if (IsConditionVisible() == EVisibility::Visible)
	{
		return GetConditionOperandText();
	}
	else if (IsConsiderationVisible() == EVisibility::Visible)
	{
		return GetConsiderationOperandText();
	}

	return FText::GetEmpty();
}

FText FStateTreeEditorNodeDetails::GetConditionOperandText() const
{
	check(OperandProperty);

	// First item does not relate to anything existing, it could be empty. 
	// return IF to indicate that we're building condition and IS for consideration.
	if (IsFirstItem())
	{
		return LOCTEXT("IfOperand", "IF");
	}

	uint8 Value = 0;
	OperandProperty->GetValue(Value);

	switch (EStateTreeExpressionOperand Operand = static_cast<EStateTreeExpressionOperand>(Value))
	{
	case EStateTreeExpressionOperand::And:
		return LOCTEXT("AndOperand", "AND");
	case EStateTreeExpressionOperand::Or:
		return LOCTEXT("OrOperand", "OR");
	case EStateTreeExpressionOperand::Multiply:
	default:
		ensureMsgf(false, TEXT("Unhandled operand %s"), *UEnum::GetValueAsString(Operand));
		return FText::GetEmpty();
	};
}

FText FStateTreeEditorNodeDetails::GetConsiderationOperandText() const
{
	check(OperandProperty);

	// First item does not relate to anything existing, it could be empty. 
	// return IF to indicate that we're building condition and IS for consideration.
	if (IsFirstItem())
	{
		return LOCTEXT("IsOperand", "IS");
	}

	uint8 Value = 0;
	OperandProperty->GetValue(Value);

	switch (EStateTreeExpressionOperand Operand = static_cast<EStateTreeExpressionOperand>(Value))
	{
	case EStateTreeExpressionOperand::And:
		return LOCTEXT("AndOperand", "AND");
	case EStateTreeExpressionOperand::Or:
		return LOCTEXT("OrOperand", "OR");
	case EStateTreeExpressionOperand::Multiply:
		return LOCTEXT("MultiplyOperand", "MULTIPLY");
	default:
		ensureMsgf(false, TEXT("Unhandled operand %s"), *UEnum::GetValueAsString(Operand));
		return FText::GetEmpty();
	};
}

FSlateColor FStateTreeEditorNodeDetails::GetOperandColor() const
{
	check(OperandProperty);

	if (IsFirstItem())
	{
		return FStyleColors::Transparent;
	}

	uint8 Value = 0; 
	OperandProperty->GetValue(Value);

	switch (EStateTreeExpressionOperand Operand = static_cast<EStateTreeExpressionOperand>(Value))
	{
	case EStateTreeExpressionOperand::And:
		return FStyleColors::AccentPink;
	case EStateTreeExpressionOperand::Or:
		return FStyleColors::AccentBlue;
	case EStateTreeExpressionOperand::Multiply:
		return FStyleColors::AccentGreen;
	default:
		ensureMsgf(false, TEXT("Unhandled operand %s"), *UEnum::GetValueAsString(Operand));
		return FStyleColors::Transparent;
	};
}

TSharedRef<SWidget> FStateTreeEditorNodeDetails::OnGetOperandContent() const
{
	if (IsConditionVisible() == EVisibility::Visible)
	{
		return GetConditionOperandContent();
	}
	else //(IsConsiderationVisible() == EVisibility::Visible)
	{
		return GetConsiderationOperandContent();
	}
}

TSharedRef<SWidget> FStateTreeEditorNodeDetails::GetConditionOperandContent() const
{
	FMenuBuilder MenuBuilder(true, NULL);

	FUIAction AndAction(
		FExecuteAction::CreateSP(this, &FStateTreeEditorNodeDetails::SetOperand, EStateTreeExpressionOperand::And),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FStateTreeEditorNodeDetails::IsOperand, EStateTreeExpressionOperand::And));
	MenuBuilder.AddMenuEntry(LOCTEXT("AndOperand", "AND"), TAttribute<FText>(), FSlateIcon(), AndAction, FName(), EUserInterfaceActionType::Check);

	FUIAction OrAction(FExecuteAction::CreateSP(this, &FStateTreeEditorNodeDetails::SetOperand, EStateTreeExpressionOperand::Or),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FStateTreeEditorNodeDetails::IsOperand, EStateTreeExpressionOperand::Or));
	MenuBuilder.AddMenuEntry(LOCTEXT("OrOperand", "OR"), TAttribute<FText>(), FSlateIcon(), OrAction, FName(), EUserInterfaceActionType::Check);

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> FStateTreeEditorNodeDetails::GetConsiderationOperandContent() const
{
	FMenuBuilder MenuBuilder(true, NULL);

	FUIAction AndAction(
		FExecuteAction::CreateSP(this, &FStateTreeEditorNodeDetails::SetOperand, EStateTreeExpressionOperand::And),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FStateTreeEditorNodeDetails::IsOperand, EStateTreeExpressionOperand::And));
	MenuBuilder.AddMenuEntry(LOCTEXT("AndOperand", "AND"), TAttribute<FText>(), FSlateIcon(), AndAction, FName(), EUserInterfaceActionType::Check);

	FUIAction OrAction(FExecuteAction::CreateSP(this, &FStateTreeEditorNodeDetails::SetOperand, EStateTreeExpressionOperand::Or),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FStateTreeEditorNodeDetails::IsOperand, EStateTreeExpressionOperand::Or));
	MenuBuilder.AddMenuEntry(LOCTEXT("OrOperand", "OR"), TAttribute<FText>(), FSlateIcon(), OrAction, FName(), EUserInterfaceActionType::Check);

	FUIAction MultiplyAction(
		FExecuteAction::CreateSP(this, &FStateTreeEditorNodeDetails::SetOperand, EStateTreeExpressionOperand::Multiply),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FStateTreeEditorNodeDetails::IsOperand, EStateTreeExpressionOperand::Multiply));
	MenuBuilder.AddMenuEntry(LOCTEXT("MultiplyOperand", "MULTIPLY"), TAttribute<FText>(), FSlateIcon(), MultiplyAction, FName(), EUserInterfaceActionType::Check);

	return MenuBuilder.MakeWidget();
}

bool FStateTreeEditorNodeDetails::IsOperandEnabled() const
{
	return !IsFirstItem();
}

bool FStateTreeEditorNodeDetails::IsOperand(const EStateTreeExpressionOperand Operand) const
{
	check(OperandProperty);

	uint8 Value = 0; 
	OperandProperty->GetValue(Value);
	const EStateTreeExpressionOperand CurrOperand = static_cast<EStateTreeExpressionOperand>(Value);

	return CurrOperand == Operand;
}

void FStateTreeEditorNodeDetails::SetOperand(const EStateTreeExpressionOperand Operand) const
{
	check(OperandProperty);

	OperandProperty->SetValue(static_cast<uint8>(Operand));
}

EVisibility FStateTreeEditorNodeDetails::IsConditionVisible() const
{
	return UE::StateTreeEditor::EditorNodeUtils::IsConditionVisible(StructProperty);
}

EVisibility FStateTreeEditorNodeDetails::IsConsiderationVisible() const
{
	return UE::StateTreeEditor::EditorNodeUtils::IsConsiderationVisible(StructProperty);
}

EVisibility FStateTreeEditorNodeDetails::IsOperandVisible() const
{
	// Assume the Condition and Consideration's Visibility is either Visible or Collapsed
	if (IsConditionVisible() == EVisibility::Visible || IsConsiderationVisible() == EVisibility::Visible)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

EVisibility FStateTreeEditorNodeDetails::AreIndentButtonsVisible() const
{
	if (IsFirstItem())
	{
		return EVisibility::Collapsed;
	}

	// Assume the Condition and Consideration's Visibility is either Visible or Collapsed
	if (IsConditionVisible() == EVisibility::Visible || IsConsiderationVisible() == EVisibility::Visible)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

EVisibility FStateTreeEditorNodeDetails::AreParensVisible() const
{
	//Assume the Condition and Consideration's Visibility is either Visible or Collapsed
	if (EVisibility::Visible.Value & (IsConditionVisible().Value | IsConsiderationVisible().Value))
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

EVisibility FStateTreeEditorNodeDetails::AreFlagsVisible() const
{
	bool bVisible = EnumHasAllFlags(GetDefault<UStateTreeEditorUserSettings>()->GetStatesViewDisplayNodeType(), EStateTreeEditorUserSettingsNodeType::Flag);
	return bVisible ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FStateTreeEditorNodeDetails::IsIconVisible() const
{
	return UE::StateTreeEditor::EditorNodeUtils::IsIconVisible(StructProperty);
}

const FSlateBrush* FStateTreeEditorNodeDetails::GetIcon() const
{
	return UE::StateTreeEditor::EditorNodeUtils::GetIcon(StructProperty).GetIcon();
}

FSlateColor FStateTreeEditorNodeDetails::GetIconColor() const
{
	return UE::StateTreeEditor::EditorNodeUtils::GetIconColor(StructProperty);
}

FReply FStateTreeEditorNodeDetails::OnDescriptionClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) const
{
	if (NameSwitcher && NameEdit)
	{
		if (NameSwitcher->GetActiveWidgetIndex() == 0)
		{
			// Enter edit mode
			NameSwitcher->SetActiveWidgetIndex(1);

			// Focus on name edit.
			FReply Reply = FReply::Handled();
			Reply.SetUserFocus(NameEdit.ToSharedRef());
			NameEdit->EnterEditingMode();
			return Reply;
		}
	}

	return FReply::Unhandled();
}

FText FStateTreeEditorNodeDetails::GetNodeDescription() const
{
	check(StructProperty);
	const UStateTreeEditorData* EditorDataPtr = EditorData.Get();
	if (!EditorDataPtr)
	{
		return FText::GetEmpty();
	}
	
	// Multiple names do not make sense, just if only one node is selected.
	TArray<void*> RawNodeData;
	StructProperty->AccessRawData(RawNodeData);
	if (RawNodeData.Num() == 1)
	{
		FText Description = LOCTEXT("EmptyNodeRich", "<s>None</>");
		if (const FStateTreeEditorNode* Node = static_cast<FStateTreeEditorNode*>(RawNodeData[0]))
		{
			return EditorDataPtr->GetNodeDescription(*Node, EStateTreeNodeFormatting::RichText);
		}
		return Description;
	}

	return LOCTEXT("MultipleSelectedRich", "<s>Multiple Selected</>");
}

EVisibility FStateTreeEditorNodeDetails::IsNodeDescriptionVisible() const
{
	const UScriptStruct* ScriptStruct = nullptr;
	if (const FStateTreeEditorNode* Node = UE::StateTreeEditor::EditorNodeUtils::GetCommonNode(StructProperty))
	{
		ScriptStruct = Node->GetNode().GetScriptStruct();
	}

	if (ScriptStruct != nullptr && ScriptStruct->IsChildOf(FStateTreeTaskBase::StaticStruct()))
	{
		const UStateTreeEditorData* EditorDataPtr = EditorData.Get();
		const UStateTreeSchema* Schema = EditorDataPtr ? EditorDataPtr->Schema.Get() : nullptr;
		if (Schema && Schema->AllowMultipleTasks() == false)
		{
			// Single task states use the state name as task name.
			return EVisibility::Collapsed;
		}
	}
	
	return EVisibility::Visible;
}

FText FStateTreeEditorNodeDetails::GetNodeTooltip() const
{
	check(StructProperty);

	const UStateTreeEditorData* EditorDataPtr = EditorData.Get();
	if (!EditorDataPtr)
	{
		return FText::GetEmpty();
	}
	
	TArray<void*> RawNodeData;
	StructProperty->AccessRawData(RawNodeData);
	if (RawNodeData.Num() == 1)
	{
		FText NameText;
		FText PathText;
		FText DescText;

		if (const FStateTreeEditorNode* Node = static_cast<FStateTreeEditorNode*>(RawNodeData[0]))
		{
			const UStruct* Struct = Node->GetInstance().GetStruct();
			if (Struct == nullptr || !Struct->IsChildOf<UStateTreeNodeBlueprintBase>())
			{
				Struct = Node->GetNode().GetScriptStruct();
			}

			if (Struct)
			{
				static const FName NAME_Tooltip(TEXT("Tooltip"));
				const FText StructToolTipText = Struct->HasMetaData(NAME_Tooltip) ? Struct->GetToolTipText() : FText::GetEmpty();

				FTextBuilder TooltipBuilder;
				TooltipBuilder.AppendLineFormat(LOCTEXT("NodeTooltip", "{0} ({1})"), Struct->GetDisplayNameText(), FText::FromString(Struct->GetPathName()));

				if (!StructToolTipText.IsEmpty())
				{
					TooltipBuilder.AppendLine();
					TooltipBuilder.AppendLine(StructToolTipText);
				}
				return TooltipBuilder.ToText();
			}
		}
	}

	return FText::GetEmpty();
}

FText FStateTreeEditorNodeDetails::GetName() const
{
	check(StructProperty);

	if (!EditorData.IsValid())
	{
		return FText::GetEmpty();
	}

	// Multiple names do not make sense, just if only one node is selected.
	TArray<void*> RawNodeData;
	StructProperty->AccessRawData(RawNodeData);
	if (RawNodeData.Num() == 1)
	{
		if (const FStateTreeEditorNode* Node = static_cast<FStateTreeEditorNode*>(RawNodeData[0]))
		{
			if (const FStateTreeNodeBase* BaseNode = Node->GetNode().GetPtr<FStateTreeNodeBase>())
			{
				if (!BaseNode->Name.IsNone())
				{
					return FText::FromName(BaseNode->Name);
				}
				const FText Desc = EditorData->GetNodeDescription(*Node, EStateTreeNodeFormatting::Text);
				if (!Desc.IsEmpty())
				{
					return Desc;
				}
			}
		}

		return FText::GetEmpty();
	}

	return LOCTEXT("MultipleSelected", "Multiple Selected");
}

bool FStateTreeEditorNodeDetails::HandleVerifyNameChanged(const FText& InText, FText& OutErrorMessage) const
{
	const FString NewName = FText::TrimPrecedingAndTrailing(InText).ToString();
	if (NewName.Len() >= NAME_SIZE)
	{
		OutErrorMessage = LOCTEXT("VerifyNodeLabelFailed_MaxLength", "Max length exceeded");
		return false;
	}
	return NewName.Len() > 0;
}

void FStateTreeEditorNodeDetails::HandleNameCommitted(const FText& NewText, ETextCommit::Type InTextCommit) const
{
	check(StructProperty);

	if (InTextCommit == ETextCommit::OnEnter || InTextCommit == ETextCommit::OnUserMovedFocus)
	{
		// Remove excess whitespace and prevent categories with just spaces
		const FString NewName = FText::TrimPrecedingAndTrailing(NewText).ToString();
		if (NewName.Len() > 0 && NewName.Len() < NAME_SIZE)
		{
			if (GEditor)
			{
				GEditor->BeginTransaction(LOCTEXT("SetName", "Set Name"));
			}
			StructProperty->NotifyPreChange();

			TArray<void*> RawNodeData;
			StructProperty->AccessRawData(RawNodeData);

			for (void* Data : RawNodeData)
			{
				// Set Name
				if (FStateTreeEditorNode* Node = static_cast<FStateTreeEditorNode*>(Data))
				{
					Node->SetNodeName(FName(NewName));
				}
			}

			StructProperty->NotifyPostChange(EPropertyChangeType::ValueSet);

			if (const UStateTree* StateTreePtr = StateTree.Get())
			{
				UE::StateTree::Delegates::OnIdentifierChanged.Broadcast(*StateTreePtr);
			}

			if (GEditor)
			{
				GEditor->EndTransaction();
			}

			StructProperty->NotifyFinishedChangingProperties();
		}
	}

	// Switch back to rich view.
	NameSwitcher->SetActiveWidgetIndex(0);
}

FReply FStateTreeEditorNodeDetails::HandleToggleCompletionTaskClicked()
{
	UE::StateTreeEditor::EditorNodeUtils::ModifyNodeInTransaction(LOCTEXT("OnCompletionTaskToggled", "Toggled Completion Task"),
		StructProperty,
		[](const TSharedPtr<IPropertyHandle>& StructProperty)
		{
			if (FStateTreeEditorNode* Node = UE::StateTreeEditor::EditorNodeUtils::GetMutableCommonNode(StructProperty))
			{
				if (UE::StateTreeEditor::EditorNodeUtils::IsTaskEnabled(*Node))
				{
					const bool bCurrentValue = UE::StateTreeEditor::EditorNodeUtils::IsTaskConsideredForCompletion(*Node);
					UE::StateTreeEditor::EditorNodeUtils::SetTaskConsideredForCompletion(*Node, !bCurrentValue);
				}
			}
		});

	return FReply::Handled();
}

FText FStateTreeEditorNodeDetails::GetToggleCompletionTaskTooltip() const
{
	if (const FStateTreeEditorNode* Node = UE::StateTreeEditor::EditorNodeUtils::GetCommonNode(StructProperty))
	{
		if (UE::StateTreeEditor::EditorNodeUtils::IsTaskConsideredForCompletion(*Node))
		{
			return LOCTEXT("ToggleTaskCompletionEnabled", "Toggle Completion.\n"
				"The task is considered for state completion.\n"
				"When the task completes, it will stop ticking, and the state can be considered for transition.");
		}
		else
		{
			return LOCTEXT("ToggleTaskCompletionDisabled", "Toggle Completion.\n"
				"The task doesn't affect the state completion.\n"
				"When the task completes, it will stop ticking.");
		}
	}
	return FText::GetEmpty();
}

FSlateColor FStateTreeEditorNodeDetails::GetToggleCompletionTaskColor() const
{
	if (const FStateTreeEditorNode* Node = UE::StateTreeEditor::EditorNodeUtils::GetCommonNode(StructProperty))
	{
		if (UE::StateTreeEditor::EditorNodeUtils::IsTaskConsideredForCompletion(*Node))
		{
			return UE::StateTree::Colors::Cyan;
		}
	}
	return FSlateColor(EStyleColor::Foreground);
}

const FSlateBrush* FStateTreeEditorNodeDetails::GetToggleCompletionTaskIcon() const
{
	if (const FStateTreeEditorNode* Node = UE::StateTreeEditor::EditorNodeUtils::GetCommonNode(StructProperty))
	{
		if (UE::StateTreeEditor::EditorNodeUtils::IsTaskConsideredForCompletion(*Node))
		{
			return FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.TasksCompletion.Enabled");
		}
		else
		{
			return FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.TasksCompletion.Disabled");
		}
	}
	return nullptr;
}

EVisibility FStateTreeEditorNodeDetails::GetToggleCompletionTaskVisibility() const
{
	if (const FStateTreeEditorNode* Node = UE::StateTreeEditor::EditorNodeUtils::GetCommonNode(StructProperty))
	{
		return UE::StateTreeEditor::EditorNodeUtils::IsTaskEnabled(*Node) ? EVisibility::Visible : EVisibility::Hidden;
	}
	return EVisibility::Collapsed;
}

FText FStateTreeEditorNodeDetails::GetNodePickerTooltip() const
{
	check(StructProperty);

	const UStateTreeEditorData* EditorDataPtr = EditorData.Get();
	if (!EditorDataPtr)
	{
		return FText::GetEmpty();
	}

	FTextBuilder TextBuilder;

	// Append full description.
	TArray<void*> RawNodeData;
	StructProperty->AccessRawData(RawNodeData);
	if (RawNodeData.Num() == 1)
	{
		FText Description = LOCTEXT("EmptyNodeStyled", "<s>None</>");
		if (const FStateTreeEditorNode* Node = static_cast<FStateTreeEditorNode*>(RawNodeData[0]))
		{
			TextBuilder.AppendLine(EditorDataPtr->GetNodeDescription(*Node));
		}
	}

	if (TextBuilder.GetNumLines() > 0)
	{
		TextBuilder.AppendLine(FText::GetEmpty());
	}
	
	// Text describing the type.
	if (const FStateTreeEditorNode* Node = UE::StateTreeEditor::EditorNodeUtils::GetCommonNode(StructProperty))
	{
		if (const UScriptStruct* ScriptStruct = Node->GetNode().GetScriptStruct())
		{
			if (ScriptStruct->IsChildOf(FStateTreeBlueprintEvaluatorWrapper::StaticStruct())
				|| ScriptStruct->IsChildOf(FStateTreeBlueprintTaskWrapper::StaticStruct())
				|| ScriptStruct->IsChildOf(FStateTreeBlueprintConditionWrapper::StaticStruct())
				|| ScriptStruct->IsChildOf(FStateTreeBlueprintConsiderationWrapper::StaticStruct()))
			{
				FStateTreeDataView InstanceView = Node->GetInstance();
				if (InstanceView.GetStruct())
				{
					TextBuilder.AppendLine(InstanceView.GetStruct()->GetDisplayNameText());
				}
			}
			else
			{
				TextBuilder.AppendLine(ScriptStruct->GetDisplayNameText());
			}
		}
	}

	return TextBuilder.ToText();
}

namespace UE::StateTree::Editor::Private
{
	const UScriptStruct* GetNodeStruct(const TSharedPtr<IPropertyHandle>& NodeProperty)
	{
		void* Address = nullptr;
		const FPropertyAccess::Result AccessResult = NodeProperty->GetValueData(Address);
		if (AccessResult == FPropertyAccess::Success && Address)
		{
			FInstancedStruct* InstancedStruct = static_cast<FInstancedStruct*>(Address);
			return InstancedStruct->GetScriptStruct();
		}
		return nullptr;
	}

	const UClass* GetInstanceObjectClass(const TSharedPtr<IPropertyHandle>& InstanceObjectProperty)
	{
		const UObject* InstanceObject = nullptr;
		const FPropertyAccess::Result AccessResult = InstanceObjectProperty->GetValue(InstanceObject);
		if (AccessResult == FPropertyAccess::Success && InstanceObject)
		{
			return InstanceObject->GetClass();
		}
		return nullptr;
	}
}

FReply FStateTreeEditorNodeDetails::OnBrowseToSource() const
{
	using namespace UE::StateTree::Editor::Private;
	if (const UScriptStruct* Node = GetNodeStruct(NodeProperty))
	{
		FSourceCodeNavigation::NavigateToStruct(Node);
	}

	return FReply::Handled();
}

FReply FStateTreeEditorNodeDetails::OnBrowseToNodeBlueprint() const
{
	using namespace UE::StateTree::Editor::Private;
	if (const UBlueprintGeneratedClass* BlueprintGeneratedClass = Cast<const UBlueprintGeneratedClass>(GetInstanceObjectClass(InstanceObjectProperty)))
	{
		//If the blueprint asset has been cooked, UBlueprint Object will be set to null and we need to browse to its BlueprintGeneratedClass
		GEditor->SyncBrowserToObject(BlueprintGeneratedClass->ClassGeneratedBy ? BlueprintGeneratedClass->ClassGeneratedBy.Get() : BlueprintGeneratedClass);
	}

	return FReply::Handled();
}

FReply FStateTreeEditorNodeDetails::OnEditNodeBlueprint() const
{
	//Cooked blueprint asset is not editable.
	using namespace UE::StateTree::Editor::Private;
	const UBlueprintGeneratedClass* BlueprintGeneratedClass = Cast<const UBlueprintGeneratedClass>(GetInstanceObjectClass(InstanceObjectProperty));
	if (BlueprintGeneratedClass && BlueprintGeneratedClass->ClassGeneratedBy)
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(BlueprintGeneratedClass->ClassGeneratedBy);
	}

	return FReply::Handled();
}

EVisibility FStateTreeEditorNodeDetails::IsBrowseToSourceVisible() const
{
	using namespace UE::StateTree::Editor::Private;
	if (!Cast<const UBlueprintGeneratedClass>(GetInstanceObjectClass(InstanceObjectProperty)))
	{
		if (const UScriptStruct* Node = GetNodeStruct(NodeProperty))
		{
			if (FSourceCodeNavigation::CanNavigateToStruct(Node))
			{
				return EVisibility::Visible;
			}
		}
	}
	return EVisibility::Collapsed;
}

EVisibility FStateTreeEditorNodeDetails::IsBrowseToNodeBlueprintVisible() const
{
	using namespace UE::StateTree::Editor::Private;
	const UBlueprintGeneratedClass* BlueprintGeneratedClass = Cast<const UBlueprintGeneratedClass>(GetInstanceObjectClass(InstanceObjectProperty));
	return BlueprintGeneratedClass != nullptr ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FStateTreeEditorNodeDetails::IsEditNodeBlueprintVisible() const
{
	//Cooked blueprint asset is not editable
	using namespace UE::StateTree::Editor::Private;
	const UBlueprintGeneratedClass* BlueprintGeneratedClass = Cast<const UBlueprintGeneratedClass>(GetInstanceObjectClass(InstanceObjectProperty));
	return BlueprintGeneratedClass != nullptr && BlueprintGeneratedClass->ClassGeneratedBy ? EVisibility::Visible : EVisibility::Collapsed;
}

void FStateTreeEditorNodeDetails::GeneratePickerMenu(class FMenuBuilder& InMenuBuilder)
{
	// Expand and select currently selected item.
	const UStruct* CommonStruct  = nullptr;
	if (const FStateTreeEditorNode* Node = UE::StateTreeEditor::EditorNodeUtils::GetCommonNode(StructProperty))
	{
		if (const UScriptStruct* ScriptStruct = Node->GetNode().GetScriptStruct())
		{
			if (ScriptStruct->IsChildOf(FStateTreeBlueprintEvaluatorWrapper::StaticStruct())
				|| ScriptStruct->IsChildOf(FStateTreeBlueprintTaskWrapper::StaticStruct())
				|| ScriptStruct->IsChildOf(FStateTreeBlueprintConditionWrapper::StaticStruct())
				|| ScriptStruct->IsChildOf(FStateTreeBlueprintConsiderationWrapper::StaticStruct()))
			{
				if (Node->GetInstance().IsValid())
				{
					CommonStruct = Node->GetInstance().GetStruct();
				}
			}
			else
			{
				CommonStruct = ScriptStruct;
			}
		}
	}

	TSharedRef<SStateTreeNodeTypePicker> Picker = SNew(SStateTreeNodeTypePicker)
		.Schema(EditorData->Schema)
		.BaseScriptStruct(BaseScriptStruct.Get())
		.BaseClass(BaseClass.Get())
		.CurrentStruct(CommonStruct)
		.OnNodeTypePicked(SStateTreeNodeTypePicker::FOnNodeStructPicked::CreateSP(this, &FStateTreeEditorNodeDetails::OnNodePicked));
	
	InMenuBuilder.AddWidget(SNew(SBox)
		.MinDesiredWidth(400.f)
		.MinDesiredHeight(300.f)
		.MaxDesiredHeight(300.f)
		.Padding(2)	
		[
			Picker
		],
		FText::GetEmpty(), /*bNoIdent*/true);
}
	
TSharedRef<SWidget> FStateTreeEditorNodeDetails::GenerateOptionsMenu()
{
	FMenuBuilder MenuBuilder(/*ShouldCloseWindowAfterMenuSelection*/true, /*CommandList*/nullptr);

	MenuBuilder.BeginSection(FName("Type"), LOCTEXT("Type", "Type"));

	// Change type
	MenuBuilder.AddSubMenu(
		LOCTEXT("ReplaceWith", "Replace With"),
		FText::GetEmpty(),
		FNewMenuDelegate::CreateSP(this, &FStateTreeEditorNodeDetails::GeneratePickerMenu));

	MenuBuilder.EndSection();

	MenuBuilder.BeginSection(FName("Edit"), LOCTEXT("Edit", "Edit"));

	// Copy
	MenuBuilder.AddMenuEntry(
		LOCTEXT("CopyItem", "Copy"),
		LOCTEXT("CopyItemTooltip", "Copy this item"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FStateTreeEditorNodeDetails::OnCopyNode),
			FCanExecuteAction()
		));

	// Copy all
	MenuBuilder.AddMenuEntry(
		LOCTEXT("CopyAllItems", "Copy all"),
		LOCTEXT("CopyAllItemsTooltip", "Copy all items"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FStateTreeEditorNodeDetails::OnCopyAllNodes),
			FCanExecuteAction()
		));

	// Paste
	MenuBuilder.AddMenuEntry(
		LOCTEXT("PasteItem", "Paste"),
		LOCTEXT("PasteItemTooltip", "Paste into this item"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Paste"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FStateTreeEditorNodeDetails::OnPasteNodes),
			FCanExecuteAction()
		));

	// Duplicate
	MenuBuilder.AddMenuEntry(
		LOCTEXT("DuplicateItem", "Duplicate"),
		LOCTEXT("DuplicateItemTooltip", "Duplicate this item"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Duplicate"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FStateTreeEditorNodeDetails::OnDuplicateNode),
			FCanExecuteAction()
		));

	// Delete
	MenuBuilder.AddMenuEntry(
		LOCTEXT("DeleteItem", "Delete"),
		LOCTEXT("DeleteItemTooltip", "Delete this item"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Delete"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FStateTreeEditorNodeDetails::OnDeleteNode),
			FCanExecuteAction()
		));

	// Delete All
	MenuBuilder.AddMenuEntry(
		LOCTEXT("DeleteAllItems", "Delete all"),
		LOCTEXT("DeleteAllItemsTooltip", "Delete all items"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Delete"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FStateTreeEditorNodeDetails::OnDeleteAllNodes),
			FCanExecuteAction()
		));

	// Rename
	MenuBuilder.AddMenuEntry(
		LOCTEXT("RenameNode", "Rename"),
		LOCTEXT("RenameNodeTooltip", "Rename this item"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Rename"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FStateTreeEditorNodeDetails::OnRenameNode),
			FCanExecuteAction()
		));

	MenuBuilder.EndSection();

	// Append debugger items.
	UE::StateTreeEditor::DebuggerExtensions::AppendEditorNodeMenuItems(MenuBuilder, StructProperty, StateTreeViewModel);

	return MenuBuilder.MakeWidget();
}

void FStateTreeEditorNodeDetails::OnDeleteNode() const
{
	const int32 Index = StructProperty->GetArrayIndex();
	if (ParentArrayProperty)
	{
		if (UStateTreeEditorData* EditorDataPtr = EditorData.Get())
		{
			FScopedTransaction Transaction(LOCTEXT("DeleteNode", "Delete Node"));

			EditorDataPtr->Modify();

			ParentArrayProperty->DeleteItem(Index);

			EditorDataPtr->RemoveInvalidBindings();
		}
	}
}

void FStateTreeEditorNodeDetails::OnDeleteAllNodes() const
{
	if (const TSharedPtr<IPropertyHandle> ParentHandle = StructProperty->GetParentHandle())
	{
		if (const TSharedPtr<IPropertyHandleArray> ArrayHandle = ParentHandle->AsArray())
		{
			if (UStateTreeEditorData* EditorDataPtr = EditorData.Get())
			{
				FScopedTransaction Transaction(LOCTEXT("DeleteAllNodes", "Delete All Nodes"));

				EditorDataPtr->Modify();

				ArrayHandle->EmptyArray();

				EditorDataPtr->RemoveInvalidBindings();
			}
		}
	}
}

void FStateTreeEditorNodeDetails::OnDuplicateNode() const
{
	TArray<UObject*> OuterObjects;
	StructProperty->GetOuterObjects(OuterObjects);
	if (OuterObjects.IsEmpty())
	{
		return;
	}

	if (OuterObjects.Num() != 1)
	{
		// Array Handle Manipulation doesn't support multiple selected objects
		FNotificationInfo NotificationInfo(FText::GetEmpty());
		NotificationInfo.Text = LOCTEXT("NotSupportedByMultipleObjects", "Operation is not supported for multi-selected objects");
		NotificationInfo.ExpireDuration = 5.0f;
		FSlateNotificationManager::Get().AddNotification(NotificationInfo);

		return;
	}

	if (ParentArrayProperty)
	{
		if (UStateTreeEditorData* EditorDataPtr = EditorData.Get())
		{
			void* NodePtr = nullptr;
			if (StructProperty->GetValueData(NodePtr) == FPropertyAccess::Success)
			{
				UE::StateTreeEditor::FClipboardEditorData Clipboard;
				Clipboard.Append(EditorDataPtr, TConstArrayView<FStateTreeEditorNode>(static_cast<FStateTreeEditorNode*>(NodePtr), 1));
				//UE::StateTreeEditor::ExportTextAsClipboardEditorData()
				Clipboard.ProcessBuffer(nullptr, EditorDataPtr, OuterObjects[0]);

				if (!Clipboard.IsValid())
				{
					return;
				}

				FScopedTransaction Transaction(LOCTEXT("DuplicateNode", "Duplicate Node"));

				// Might modify the bindings data
				EditorDataPtr->Modify();

				const int32 Index = StructProperty->GetArrayIndex();
				ParentArrayProperty->Insert(Index);

				TSharedPtr<IPropertyHandle> InsertedElementHandle = ParentArrayProperty->GetElement(Index);
				void* InsertedNodePtr = nullptr;
				if (InsertedElementHandle->GetValueData(InsertedNodePtr) == FPropertyAccess::Success)
				{
					*static_cast<FStateTreeEditorNode*>(InsertedNodePtr) = MoveTemp(Clipboard.GetEditorNodesInBuffer()[0]);

					for (FStateTreePropertyPathBinding& Binding : Clipboard.GetBindingsInBuffer())
					{
						EditorDataPtr->GetPropertyEditorBindings()->AddStateTreeBinding(MoveTemp(Binding));
					}
				}

				// We reinieitalized item nodes on ArrayProperty operations before the data is completely set up. Reinitialize.
				if (PropUtils)
				{
					PropUtils->ForceRefresh();
				}
			}
		}
	}

	
}

void FStateTreeEditorNodeDetails::OnRenameNode() const
{
	if (NameSwitcher && NameEdit)
	{
		if (NameSwitcher->GetActiveWidgetIndex() == 0)
		{
			// Enter edit mode
			NameSwitcher->SetActiveWidgetIndex(1);

			FSlateApplication::Get().SetKeyboardFocus(NameEdit);
			FSlateApplication::Get().SetUserFocus(0, NameEdit);
			NameEdit->EnterEditingMode();
		}
	}
}

// @todo: refactor it to use FStateTreeEditorDataFixer
void FStateTreeEditorNodeDetails::OnNodePicked(const UStruct* InStruct) const
{
	GEditor->BeginTransaction(LOCTEXT("SelectNode", "Select Node"));

	StructProperty->NotifyPreChange();

	UE::StateTreeEditor::EditorNodeUtils::SetNodeType(StructProperty, InStruct);

	StructProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
	StructProperty->NotifyFinishedChangingProperties();

	GEditor->EndTransaction();

	FSlateApplication::Get().DismissAllMenus();
	
	if (PropUtils)
	{
		PropUtils->ForceRefresh();
	}
}

void FStateTreeEditorNodeDetails::HandleAssetChanged()
{
	MakeFlagsWidget();
}

#undef LOCTEXT_NAMESPACE
