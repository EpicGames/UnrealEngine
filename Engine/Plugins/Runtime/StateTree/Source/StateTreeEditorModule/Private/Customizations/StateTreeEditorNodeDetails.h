// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "IStateTreeEditorHost.h"
#include "Widgets/Views/STreeView.h"
#include "Textures/SlateIcon.h"

struct FObjectKey;
struct FOptionalSize;

class FDetailWidgetRow;
class FStateTreeViewModel;
class IDetailChildrenBuilder;
class IPropertyHandle;
class IPropertyHandleArray;
class IPropertyUtilities;
class SComboButton;
class SWidget;
class SSearchBox;
class SWidgetSwitcher;
class SInlineEditableTextBlock;
class SBorder;
class SMenuAnchor;
class UStateTree;
class UStateTreeState;
class UStateTreeEditorData;
struct EVisibility;
struct FStateTreeEditorNode;
struct FPropertyBindingPath;
enum class EStateTreeExpressionOperand : uint8;

/**
 * Type customization for nodes (Conditions, Evaluators and Tasks) in StateTreeState.
 */
class FStateTreeEditorNodeDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual ~FStateTreeEditorNodeDetails();
	
	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:
	void MakeFlagsWidget();

	bool ShouldResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle) const;
	void ResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle);
	void OnCopyNode();
	void OnCopyAllNodes();
	void OnPasteNodes();
	TSharedPtr<IPropertyHandle> GetInstancedObjectValueHandle(TSharedPtr<IPropertyHandle> PropertyHandle);

	FOptionalSize GetIndentSize() const;
	FReply HandleIndentPlus();
	FReply HandleIndentMinus();
	
	int32 GetIndent() const;
	void SetIndent(const int32 Indent) const;
	bool IsIndent(const int32 Indent) const;
	
	FSlateColor GetContentRowColor() const;

	FText GetOperandText() const;
	FText GetConditionOperandText() const;
	FText GetConsiderationOperandText() const;
	FSlateColor GetOperandColor() const;
	TSharedRef<SWidget> OnGetOperandContent() const;
	TSharedRef<SWidget> GetConditionOperandContent() const;
	TSharedRef<SWidget> GetConsiderationOperandContent() const;
	bool IsOperandEnabled() const;

	void SetOperand(const EStateTreeExpressionOperand Operand) const;
	bool IsOperand(const EStateTreeExpressionOperand Operand) const;

	bool IsFirstItem() const;
	int32 GetCurrIndent() const;
	int32 GetNextIndent() const;
	
	FText GetOpenParens() const;
	FText GetCloseParens() const;

	EVisibility IsConditionVisible() const;
	EVisibility IsConsiderationVisible() const;
	EVisibility IsOperandVisible() const;
	EVisibility AreIndentButtonsVisible() const;
	EVisibility AreParensVisible() const;
	EVisibility AreFlagsVisible() const;
	
	EVisibility IsIconVisible() const;
	const FSlateBrush* GetIcon() const;
	FSlateColor GetIconColor() const;

	FReply OnDescriptionClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) const;
	FText GetNodeDescription() const;
	EVisibility IsNodeDescriptionVisible() const;
	
	FText GetNodeTooltip() const;

	FReply OnRowMouseDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	FReply OnRowMouseUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	FText GetName() const;
	bool HandleVerifyNameChanged(const FText& InText, FText& OutErrorMessage) const;
	void HandleNameCommitted(const FText& NewLabel, ETextCommit::Type CommitType) const;

	FReply HandleToggleCompletionTaskClicked();
	FText GetToggleCompletionTaskTooltip() const;
	FSlateColor GetToggleCompletionTaskColor() const;
	const FSlateBrush* GetToggleCompletionTaskIcon() const;
	EVisibility GetToggleCompletionTaskVisibility() const;

	FText GetNodePickerTooltip() const;
	void OnNodePicked(const UStruct* InStruct) const;

	void OnIdentifierChanged(const UStateTree& StateTree);
	void OnBindingChanged(const FPropertyBindingPath& SourcePath, const FPropertyBindingPath& TargetPath);
	void FindOuterObjects();

	FReply OnBrowseToSource() const;
	FReply OnBrowseToNodeBlueprint() const;
	FReply OnEditNodeBlueprint() const;
	EVisibility IsBrowseToSourceVisible() const;
	EVisibility IsBrowseToNodeBlueprintVisible() const;
	EVisibility IsEditNodeBlueprintVisible() const;

	TSharedRef<SWidget> GenerateOptionsMenu();
	void GeneratePickerMenu(class FMenuBuilder& InMenuBuilder);
	void OnDeleteNode() const;
	void OnDeleteAllNodes() const;
	void OnDuplicateNode() const;
	void OnRenameNode() const;
	void HandleAssetChanged();

	TWeakObjectPtr<UScriptStruct> BaseScriptStruct;
	TWeakObjectPtr<UClass> BaseClass;
	TSharedPtr<SWidgetSwitcher> NameSwitcher;
	TSharedPtr<SInlineEditableTextBlock> NameEdit;
	TSharedPtr<SBorder> RowBorder; 
	TSharedPtr<SBorder> FlagsContainer; 

	TWeakObjectPtr<UStateTreeEditorData> EditorData;
	TWeakObjectPtr<UStateTree> StateTree;
	TSharedPtr<FStateTreeViewModel> StateTreeViewModel;
	
	TSharedPtr<IPropertyUtilities> PropUtils;
	TSharedPtr<IPropertyHandle> StructProperty;
	TSharedPtr<IPropertyHandle> NodeProperty;
	TSharedPtr<IPropertyHandle> InstanceProperty;
	TSharedPtr<IPropertyHandle> InstanceObjectProperty;
	TSharedPtr<IPropertyHandle> ExecutionRuntimeDataProperty;
	TSharedPtr<IPropertyHandle> ExecutionRuntimeDataObjectProperty;
	TSharedPtr<IPropertyHandle> IDProperty;

	TSharedPtr<IPropertyHandle> IndentProperty;
	TSharedPtr<IPropertyHandle> OperandProperty;

	TSharedPtr<IPropertyHandle> ParentProperty;
	TSharedPtr<IPropertyHandleArray> ParentArrayProperty;

	FDelegateHandle OnBindingChangedHandle;
	FDelegateHandle OnChangedAssetHandle;
};
