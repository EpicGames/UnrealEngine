// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeEditorNodeUtils.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "PropertyCustomizationHelpers.h"
#include "Layout/Visibility.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "StateTreeConditionBase.h"
#include "StateTreeConsiderationBase.h"
#include "StateTreeEditorData.h"
#include "StateTreeEditorNode.h"
#include "StateTreeEditorSettings.h"
#include "StateTreeEditorStyle.h"
#include "StateTreeTaskBase.h"
#include "Blueprint/StateTreeConditionBlueprintBase.h"
#include "Blueprint/StateTreeConsiderationBlueprintBase.h"
#include "Blueprint/StateTreeEvaluatorBlueprintBase.h"
#include "Blueprint/StateTreeTaskBlueprintBase.h"
#include "Widgets/SStateTreeNodeTypePicker.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SButton.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"

namespace UE::StateTreeEditor::EditorNodeUtils
{

EStateTreeConditionEvaluationMode GetConditionEvaluationMode(const TSharedPtr<IPropertyHandle>& StructProperty)
{
	if (const FStateTreeEditorNode* Node = GetCommonNode(StructProperty))
	{
		if (const FStateTreeConditionBase* ConditionBase = Node->GetNode().GetPtr<FStateTreeConditionBase>())
		{
			return ConditionBase->EvaluationMode;
		}
	}
	// Evaluate as default value
	return EStateTreeConditionEvaluationMode::Evaluated;
}

bool IsTaskDisabled(const TSharedPtr<IPropertyHandle>& StructProperty)
{
	if (const FStateTreeEditorNode* Node = GetCommonNode(StructProperty))
	{
		return !IsTaskEnabled(*Node);
	}

	return false;
}

bool IsTaskEnabled(const FStateTreeEditorNode& EditorNode)
{
	if (const FStateTreeTaskBase* TaskBase = EditorNode.GetNode().GetPtr<FStateTreeTaskBase>())
	{
		return TaskBase->bTaskEnabled;
	}
	return false;
}

bool IsTaskConsideredForCompletion(const FStateTreeEditorNode& EditorNode)
{
	// We use the Blueprint flag to have a default value that behaves like the other flags. Sadly, it duplicates the flags.
	if (const FStateTreeBlueprintTaskWrapper* TaskWrapper = EditorNode.GetNode().GetPtr<FStateTreeBlueprintTaskWrapper>())
	{
		if (const UStateTreeTaskBlueprintBase* BPTaskBase = EditorNode.GetInstance().GetPtr<const UStateTreeTaskBlueprintBase>())
		{
			return BPTaskBase->bConsideredForCompletion;
		}
	}
	else if (const FStateTreeTaskBase* TaskBase = EditorNode.GetNode().GetPtr<FStateTreeTaskBase>())
	{
		return TaskBase->bConsideredForCompletion;
	}
	return false;
}

void SetTaskConsideredForCompletion(FStateTreeEditorNode& EditorNode, bool bIsConsidered)
{
	if (const FStateTreeBlueprintTaskWrapper* TaskWrapper = EditorNode.GetNode().GetPtr<FStateTreeBlueprintTaskWrapper>())
	{
		if (UStateTreeTaskBlueprintBase* BPTaskBase = EditorNode.GetInstance().GetMutablePtr<UStateTreeTaskBlueprintBase>())
		{
			BPTaskBase->bConsideredForCompletion = bIsConsidered;
		}
	}
	else if (FStateTreeTaskBase* TaskBase = EditorNode.GetNode().GetPtr<FStateTreeTaskBase>())
	{
		TaskBase->bConsideredForCompletion = bIsConsidered;
	}
}

bool CanEditTaskConsideredForCompletion(const FStateTreeEditorNode& EditorNode)
{
	if (const FStateTreeBlueprintTaskWrapper* TaskWrapper = EditorNode.GetNode().GetPtr<FStateTreeBlueprintTaskWrapper>())
	{
		if (const UStateTreeTaskBlueprintBase* BPTaskBase = EditorNode.GetInstance().GetPtr<UStateTreeTaskBlueprintBase>())
		{
			return BPTaskBase->bCanEditConsideredForCompletion;
		}
	}
	else if (const FStateTreeTaskBase* TaskBase = EditorNode.GetNode().GetPtr<FStateTreeTaskBase>())
	{
		return TaskBase->bCanEditConsideredForCompletion;
	}
	return false;
}

void ModifyNodeInTransaction(const FText& Description, const TSharedPtr<IPropertyHandle>& StructProperty, TFunctionRef<void(const TSharedPtr<IPropertyHandle>&)> Func)
{
	check(StructProperty);

	FScopedTransaction ScopedTransaction(Description);

	StructProperty->NotifyPreChange();

	Func(StructProperty);

	StructProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
	StructProperty->NotifyFinishedChangingProperties();
}

EVisibility IsConditionVisible(const TSharedPtr<IPropertyHandle>& StructProperty)
{
	const UScriptStruct* ScriptStruct = nullptr;
	if (const FStateTreeEditorNode* Node = GetCommonNode(StructProperty))
	{
		ScriptStruct = Node->GetNode().GetScriptStruct();
	}

	return ScriptStruct != nullptr && ScriptStruct->IsChildOf(FStateTreeConditionBase::StaticStruct()) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility IsConsiderationVisible(const TSharedPtr<IPropertyHandle>& StructProperty)
{
	const UScriptStruct* ScriptStruct = nullptr;
	if (const FStateTreeEditorNode* Node = GetCommonNode(StructProperty))
	{
		ScriptStruct = Node->GetNode().GetScriptStruct();
	}

	return ScriptStruct != nullptr && ScriptStruct->IsChildOf(FStateTreeConsiderationBase::StaticStruct()) ? EVisibility::Visible : EVisibility::Collapsed;
}

FName GetNodeIconName(const TSharedPtr<IPropertyHandle>& StructProperty)
{
	if (const FStateTreeEditorNode* Node = GetCommonNode(StructProperty))
	{
		if (const FStateTreeNodeBase* BaseNode = Node->GetNode().GetPtr<const FStateTreeNodeBase>())
		{
			return BaseNode->GetIconName();
		}
	}
	
	return FName();
}

FSlateIcon ParseIcon(const FName IconName)
{
	FString IconPath = IconName.ToString();
	constexpr int32 NumOfIconPathNames = 4;
						
	FName IconPathNames[NumOfIconPathNames] = {
		NAME_None, // StyleSetName
		NAME_None, // StyleName
		NAME_None, // SmallStyleName
		NAME_None  // StatusOverlayStyleName
	};

	int32 NameIndex = 0;
	while (!IconPath.IsEmpty() && NameIndex < NumOfIconPathNames)
	{
		FString Left;
		FString Right;

		if (!IconPath.Split(TEXT("|"), &Left, &Right))
		{
			Left = IconPath;
		}

		IconPathNames[NameIndex] = FName(*Left);

		NameIndex++;
		IconPath = Right;
	}

	return FSlateIcon(IconPathNames[0], IconPathNames[1], IconPathNames[2], IconPathNames[3]);	
}

FSlateIcon GetIcon(const TSharedPtr<IPropertyHandle>& StructProperty)
{
	const FName IconName = GetNodeIconName(StructProperty);
	if (!IconName.IsNone())
	{
		return ParseIcon(IconName);
	}
	return {};
}

FSlateColor GetIconColor(const TSharedPtr<IPropertyHandle>& StructProperty)
{
	if (const FStateTreeEditorNode* Node = GetCommonNode(StructProperty))
	{
		if (const FStateTreeNodeBase* BaseNode = Node->GetNode().GetPtr<const FStateTreeNodeBase>())
		{
			return FLinearColor(BaseNode->GetIconColor());
		}
	}
	
	return FSlateColor::UseForeground();
}

EVisibility IsIconVisible(const TSharedPtr<IPropertyHandle>& StructProperty)
{
	const FName IconName = GetNodeIconName(StructProperty);
	return IconName.IsNone() ? EVisibility::Collapsed : EVisibility::Visible;
}

const FStateTreeEditorNode* GetCommonNode(const TSharedPtr<IPropertyHandle>& StructProperty)
{
	if (!StructProperty || !StructProperty->IsValidHandle())
	{
		return nullptr;
	}
	
	TArray<const void*> RawNodeData;
	StructProperty->AccessRawData(RawNodeData);

	const FStateTreeEditorNode* CommonNode = nullptr;

	for (const void* Data : RawNodeData)
	{
		if (const FStateTreeEditorNode* Node = static_cast<const FStateTreeEditorNode*>(Data))
		{
			if (CommonNode == nullptr)
			{
				CommonNode = Node;
			}
			else if (CommonNode != Node)
			{
				CommonNode = nullptr;
				break;
			}
		}
	}

	return CommonNode;
}

FStateTreeEditorNode* GetMutableCommonNode(const TSharedPtr<IPropertyHandle>& StructProperty)
{
	if (!StructProperty || !StructProperty->IsValidHandle())
	{
		return nullptr;
	}

	TArray<void*> RawNodeData;
	StructProperty->AccessRawData(RawNodeData);

	FStateTreeEditorNode* CommonNode = nullptr;

	for (void* Data : RawNodeData)
	{
		if (FStateTreeEditorNode* Node = static_cast<FStateTreeEditorNode*>(Data))
		{
			if (CommonNode == nullptr)
			{
				CommonNode = Node;
			}
			else if (CommonNode != Node)
			{
				CommonNode = nullptr;
				break;
			}
		}
	}

	return CommonNode;
}

void GetNodeBaseScriptStructAndClass(const TSharedPtr<IPropertyHandle>& StructProperty, UScriptStruct*& OutBaseScriptStruct, UClass*& OutBaseClass)
{
	check(StructProperty);
	
	static const FName BaseStructMetaName(TEXT("BaseStruct"));
	static const FName BaseClassMetaName(TEXT("BaseClass"));
	
	const FString BaseStructName = StructProperty->GetMetaData(BaseStructMetaName);
	OutBaseScriptStruct = UClass::TryFindTypeSlow<UScriptStruct>(BaseStructName);

	const FString BaseClassName = StructProperty->GetMetaData(BaseClassMetaName);
	OutBaseClass = UClass::TryFindTypeSlow<UClass>(BaseClassName);
}

struct FNodeStructView
{
	const UStruct* Struct = nullptr;
	void* Memory = nullptr;
	operator bool() const
	{
		return Struct != nullptr && Memory != nullptr;
	}
};

struct FNodeRetainPropertyData
{
	FNodeStructView Node; // FStateTreeNodeBase
	FNodeStructView InstanceData; // the instance data for the node
	FNodeStructView ExecutionRuntimeData; // the execution runtime data for the node (if it exists)
};

FNodeRetainPropertyData GetNodeData(FStateTreeEditorNode& EditorNode)
{
	FNodeRetainPropertyData Data;
	if (FStateTreeNodeBase* NodeBase = EditorNode.GetNode().GetPtr<FStateTreeNodeBase>())
	{
		Data.Node.Struct = EditorNode.GetNode().GetScriptStruct();
		Data.Node.Memory = NodeBase;

		if (const UStruct* InstanceDataType = NodeBase->GetInstanceDataType())
		{
			Data.InstanceData.Struct = EditorNode.GetInstance().GetStruct();
			Data.InstanceData.Memory = EditorNode.GetInstance().GetMutableMemory();
		}
		if (const UStruct* ExecutionRuntimeInstanceDataType = NodeBase->GetExecutionRuntimeDataType())
		{
			Data.ExecutionRuntimeData.Struct = EditorNode.GetExecutionRuntimeData().GetStruct();
			Data.ExecutionRuntimeData.Memory = EditorNode.GetExecutionRuntimeData().GetMutableMemory();
		}
	}

	return Data;
}

void CopyPropertyValues(const UStruct* OldStruct, const void* OldData, const UStruct* NewStruct, void* NewData)
{
	for (FProperty* OldProperty : TFieldRange<FProperty>(OldStruct, EFieldIteratorFlags::IncludeSuper))
	{
		const FProperty* NewProperty = NewStruct->FindPropertyByName(OldProperty->GetFName());
		if (!NewProperty)
		{
			// Let's check if we have the same property present but with(out) the 'b' prefix
			const FBoolProperty* BoolProperty = ExactCastField<const FBoolProperty>(OldProperty);
			if (!BoolProperty)
			{
				continue;
			}

			FString String = OldProperty->GetName();
			if (String.IsEmpty())
			{
				continue;
			}

			if (String[0] == TEXT('b'))
			{
				String.RightChopInline(1, EAllowShrinking::No);
			}
			else
			{
				String.InsertAt(0, TEXT('b'));
			}

			NewProperty = NewStruct->FindPropertyByName(FName(String));
		}

		constexpr uint64 WantedFlags = CPF_Edit;
		constexpr uint64 UnwantedFlags = CPF_DisableEditOnInstance | CPF_EditConst;

		if (NewProperty
			&& OldProperty->HasAllPropertyFlags(WantedFlags)
			&& NewProperty->HasAllPropertyFlags(WantedFlags)
			&& !OldProperty->HasAnyPropertyFlags(UnwantedFlags)
			&& !NewProperty->HasAnyPropertyFlags(UnwantedFlags)
			&& NewProperty->SameType(OldProperty))
		{
			OldProperty->CopyCompleteValue(
				NewProperty->ContainerPtrToValuePtr<void>(NewData),
				OldProperty->ContainerPtrToValuePtr<void>(OldData)
			);
		}
	}
}

void CopyPropertyValues(const FNodeStructView Old, const FNodeStructView New)
{
	CopyPropertyValues(Old.Struct, Old.Memory, New.Struct, New.Memory);
}


void RetainProperties(FStateTreeEditorNode& OldNode, FStateTreeEditorNode& NewNode)
{
	const FNodeRetainPropertyData OldNodeData = GetNodeData(OldNode);
	const FNodeRetainPropertyData NewNodeData = GetNodeData(NewNode);

	// node to ...
	if (OldNodeData.Node)
	{
		if (NewNodeData.Node)
		{
			CopyPropertyValues(OldNodeData.Node, NewNodeData.Node);
		}
		if (NewNodeData.InstanceData)
		{
			CopyPropertyValues(OldNodeData.Node, NewNodeData.InstanceData);
		}
		if (NewNodeData.ExecutionRuntimeData)
		{
			CopyPropertyValues(OldNodeData.Node, NewNodeData.ExecutionRuntimeData);
		}
	}

	// instance data to ... 
	if (OldNodeData.InstanceData)
	{
		if (NewNodeData.Node)
		{
			CopyPropertyValues(OldNodeData.InstanceData, NewNodeData.Node);
		}
		if (NewNodeData.InstanceData)
		{
			CopyPropertyValues(OldNodeData.InstanceData, NewNodeData.InstanceData);
		}
		if (NewNodeData.ExecutionRuntimeData)
		{
			CopyPropertyValues(OldNodeData.InstanceData, NewNodeData.ExecutionRuntimeData);
		}
	}


	// execution runtime data to ... 
	if (OldNodeData.ExecutionRuntimeData)
	{
		if (NewNodeData.Node)
		{
			CopyPropertyValues(OldNodeData.ExecutionRuntimeData, NewNodeData.Node);
		}
		if (NewNodeData.InstanceData)
		{
			CopyPropertyValues(OldNodeData.ExecutionRuntimeData, NewNodeData.InstanceData);
		}
		if (NewNodeData.ExecutionRuntimeData)
		{
			CopyPropertyValues(OldNodeData.ExecutionRuntimeData, NewNodeData.ExecutionRuntimeData);
		}
	}
}

void SetNodeTypeStruct(const TSharedPtr<IPropertyHandle>& StructProperty, const UScriptStruct* InStruct)
{
	TArray<UObject*> OuterObjects;
	TArray<void*> RawNodeData;
	StructProperty->GetOuterObjects(OuterObjects);
	StructProperty->AccessRawData(RawNodeData);

	if (OuterObjects.Num() != RawNodeData.Num())
	{
		return;
	}
	
	for (int32 Index = 0; Index < RawNodeData.Num(); Index++)
	{
		if (UObject* Outer = OuterObjects[Index])
		{
			if (FStateTreeEditorNode* Node = static_cast<FStateTreeEditorNode*>(RawNodeData[Index]))
			{
				const bool bRetainProperties = InStruct && UStateTreeEditorSettings::Get().bRetainNodePropertyValues;
				FStateTreeEditorNode OldNode = bRetainProperties ? *Node : FStateTreeEditorNode();

				if (InStruct)
				{
					Node->InitializeAs(Outer, InStruct);

					if (bRetainProperties)
					{
						RetainProperties(OldNode, *Node);
					}
				}
				else
				{
					Node->Reset();
				}
			}
		}
	}
}

void SetNodeTypeClass(const TSharedPtr<IPropertyHandle>& StructProperty, const UClass* InClass)
{
	TArray<UObject*> OuterObjects;
	TArray<void*> RawNodeData;
	StructProperty->GetOuterObjects(OuterObjects);
	StructProperty->AccessRawData(RawNodeData);

	if (OuterObjects.Num() != RawNodeData.Num())
	{
		return;
	}
		
	for (int32 Index = 0; Index < RawNodeData.Num(); Index++)
	{
		if (UObject* Outer = OuterObjects[Index])
		{
			if (FStateTreeEditorNode* Node = static_cast<FStateTreeEditorNode*>(RawNodeData[Index]))
			{
				bool bRetainProperties = InClass && UStateTreeEditorSettings::Get().bRetainNodePropertyValues;
				FStateTreeEditorNode OldNode = bRetainProperties ? *Node : FStateTreeEditorNode();

				if (InClass)
				{
					if (InClass->IsChildOf(UStateTreeTaskBlueprintBase::StaticClass()))
					{
						FStateTreeBlueprintTaskWrapper TaskWrapper;
						TaskWrapper.TaskClass = const_cast<UClass*>(InClass);
						Node->InitializeAs<FStateTreeBlueprintTaskWrapper>(Outer, MoveTemp(TaskWrapper));
					}
					else if (InClass->IsChildOf(UStateTreeEvaluatorBlueprintBase::StaticClass()))
					{
						FStateTreeBlueprintEvaluatorWrapper EvaluatorWrapper;
						EvaluatorWrapper.EvaluatorClass = const_cast<UClass*>(InClass);
						Node->InitializeAs<FStateTreeBlueprintEvaluatorWrapper>(Outer, MoveTemp(EvaluatorWrapper));
					}
					else if (InClass->IsChildOf(UStateTreeConditionBlueprintBase::StaticClass()))
					{
						FStateTreeBlueprintConditionWrapper ConditionWrapper;
						ConditionWrapper.ConditionClass = const_cast<UClass*>(InClass);
						Node->InitializeAs<FStateTreeBlueprintConditionWrapper>(Outer, MoveTemp(ConditionWrapper));
					}
					else if (InClass->IsChildOf(UStateTreeConsiderationBlueprintBase::StaticClass()))
					{
						FStateTreeBlueprintConsiderationWrapper ConsiderationWrapper;
						ConsiderationWrapper.ConsiderationClass = const_cast<UClass*>(InClass);
						Node->InitializeAs<FStateTreeBlueprintConsiderationWrapper>(Outer, MoveTemp(ConsiderationWrapper));
					}
					else
					{
						// Not retaining properties if we haven't initialized a new node
						bRetainProperties = false;
						Node->Reset();
					}
				}
				else
				{
					bRetainProperties = false;
					Node->Reset();
				}

				if (bRetainProperties)
				{
					RetainProperties(OldNode, *Node);
				}
			}
		}
	}

}

void SetNodeType(const TSharedPtr<IPropertyHandle>& StructProperty, const UStruct* NewType)
{
	if (const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(NewType))
	{
		SetNodeTypeStruct(StructProperty, ScriptStruct);
	}
	else if (const UClass* Class = Cast<UClass>(NewType))
	{
		SetNodeTypeClass(StructProperty, Class);
	}
	else
	{
		// None
		SetNodeTypeStruct(StructProperty, nullptr);
	}	
}

void InstantiateStructSubobjects(UObject& OuterObject, FStructView Struct)
{
	// Empty struct, nothing to do.
	if (!Struct.IsValid())
	{
		return;
	}

	for (TPropertyValueIterator<FProperty> It(Struct.GetScriptStruct(), Struct.GetMemory()); It; ++It)
	{
		if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(It->Key))
		{
			// Duplicate instanced objects.
			if (ObjectProperty->HasAnyPropertyFlags(CPF_InstancedReference | CPF_PersistentInstance))
			{
				if (UObject* Object = ObjectProperty->GetObjectPropertyValue(It->Value))
				{
					UObject* DuplicatedObject = DuplicateObject(Object, &OuterObject);
					ObjectProperty->SetObjectPropertyValue(const_cast<void*>(It->Value), DuplicatedObject);
				}
			}
		}
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(It->Key))
		{
			// If we encounter instanced struct, recursively handle it too.
			if (StructProperty->Struct == TBaseStructure<FInstancedStruct>::Get())
			{
				FInstancedStruct& InstancedStruct = *static_cast<FInstancedStruct*>(const_cast<void*>(It->Value));
				InstantiateStructSubobjects(OuterObject, InstancedStruct);
			}
		}
	}
}

void ConditionalUpdateNodeInstanceData(FStateTreeEditorNode& EditorNode, UObject& InstanceOuter)
{
	const FStateTreeNodeBase* Node = EditorNode.GetNode().GetPtr<FStateTreeNodeBase>();
	if (!Node)
	{
		return;
	}

	const UStruct* CurrentType = EditorNode.GetInstance().GetStruct();
	const UStruct* ExecutionRuntimeCurrentType = EditorNode.GetExecutionRuntimeData().GetStruct();
	const UStruct* DesiredType = Node->GetInstanceDataType();
	const UStruct* ExecutionRuntimeDesiredType = Node->GetExecutionRuntimeDataType();

	// Nothing to upgrade. Instance Data Type is unchanged
	if (CurrentType == DesiredType && ExecutionRuntimeCurrentType == ExecutionRuntimeDesiredType)
	{
		return;
	}

	FStateTreeEditorNode OldEditorNode = EditorNode;

	EditorNode.ReallocInstanceData(&InstanceOuter);

	RetainProperties(OldEditorNode, EditorNode);

	// Ensure that the instanced objects on the nodes are correctly copied over (deep copy)
	UE::StateTreeEditor::EditorNodeUtils::InstantiateStructSubobjects(InstanceOuter, EditorNode.Node);
	if (EditorNode.InstanceObject)
	{
		EditorNode.InstanceObject = DuplicateObject(EditorNode.InstanceObject, &InstanceOuter);
	}
	else
	{
		UE::StateTreeEditor::EditorNodeUtils::InstantiateStructSubobjects(InstanceOuter, EditorNode.Instance);
	}
	if (EditorNode.ExecutionRuntimeDataObject)
	{
		EditorNode.ExecutionRuntimeDataObject = DuplicateObject(EditorNode.ExecutionRuntimeDataObject, &InstanceOuter);
	}
	else
	{
		UE::StateTreeEditor::EditorNodeUtils::InstantiateStructSubobjects(InstanceOuter, EditorNode.ExecutionRuntimeData);
	}
}

void OnArrayNodePicked(const UStruct* InStruct, TSharedPtr<SComboButton> PickerCombo, TSharedPtr<IPropertyHandle> ArrayPropertyHandle, TSharedRef<IPropertyUtilities> PropUtils)
{
	if (const TSharedPtr<IPropertyHandleArray> ArrayHandle = ArrayPropertyHandle->AsArray())
	{
		GEditor->BeginTransaction(LOCTEXT("AddNode", "Add Node"));
		ArrayPropertyHandle->NotifyPreChange();

		// Add new item to the end.
		if (ArrayHandle->AddItem() == FPropertyAccess::Success)
		{
			uint32 NumItems = 0;
			if (ArrayHandle->GetNumElements(NumItems) == FPropertyAccess::Success && NumItems > 0)
			{
				// Initialize the item
				TSharedRef<IPropertyHandle> NewNodeHandle = ArrayHandle->GetElement(NumItems - 1);
				UE::StateTreeEditor::EditorNodeUtils::SetNodeType(NewNodeHandle, InStruct);
				NewNodeHandle->SetExpanded(true);
			}
		}

		// We initialized the new element, so broadcasting an extra callback with ValueSet type, besides the one from AddItem()
		ArrayPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		ArrayPropertyHandle->NotifyFinishedChangingProperties();
		GEditor->EndTransaction();

		PropUtils->ForceRefresh();
	}

	PickerCombo->SetIsOpen(false);
}

TSharedRef<SWidget> GenerateArrayNodePicker(TSharedPtr<SComboButton> PickerCombo, TSharedPtr<IPropertyHandle> ArrayPropertyHandle, TSharedRef<IPropertyUtilities> PropUtils)
{
	check(ArrayPropertyHandle);
	
	UStateTreeEditorData* EditorData = nullptr;
	TArray<UObject*> Objects;
	ArrayPropertyHandle->GetOuterObjects(Objects);
	for (UObject* Object : Objects)
	{
		if (UStateTreeEditorData* OwnerEditorData = Cast<UStateTreeEditorData>(Object))
		{
			EditorData = OwnerEditorData;
			break;
		}
		if (UStateTreeEditorData* OwnerEditorData = Object->GetTypedOuter<UStateTreeEditorData>())
		{
			EditorData = OwnerEditorData;
			break;
		}
	}
	if (!EditorData)
	{
		return SNullWidget::NullWidget;
	}

	UScriptStruct* BaseScriptStruct = nullptr;
	UClass* BaseClass = nullptr;
	UE::StateTreeEditor::EditorNodeUtils::GetNodeBaseScriptStructAndClass(ArrayPropertyHandle, BaseScriptStruct, BaseClass);
	
	TSharedRef<SStateTreeNodeTypePicker> Picker = SNew(SStateTreeNodeTypePicker)
		.Schema(EditorData->Schema)
		.BaseScriptStruct(BaseScriptStruct)
		.BaseClass(BaseClass)
		.OnNodeTypePicked(SStateTreeNodeTypePicker::FOnNodeStructPicked::CreateStatic(OnArrayNodePicked, PickerCombo, ArrayPropertyHandle, PropUtils));
	
	PickerCombo->SetMenuContentWidgetToFocus(Picker->GetWidgetToFocusOnOpen());

	return SNew(SBox)
		.MinDesiredWidth(400.f)
		.MinDesiredHeight(300.f)
		.MaxDesiredHeight(300.f)
		.Padding(2.f)
		[
			Picker
		];
}

TSharedRef<SComboButton> CreateAddNodePickerComboButton(const FText& TooltipText, FLinearColor Color, TSharedPtr<IPropertyHandle> ArrayPropertyHandle, TSharedRef<IPropertyUtilities> PropUtils)
{
	const TSharedRef<SComboButton> PickerCombo = SNew(SComboButton)
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.HasDownArrow(false)
		.ToolTipText(TooltipText)
		.ContentPadding(FMargin(4.f, 2.f))
		.IsEnabled(PropUtils, &IPropertyUtilities::IsPropertyEditingEnabled)
		.ButtonContent()
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
			.ColorAndOpacity(Color)
		];

	PickerCombo->SetOnGetMenuContent(FOnGetContent::CreateStatic(GenerateArrayNodePicker, PickerCombo.ToSharedPtr(), ArrayPropertyHandle, PropUtils));

	return PickerCombo;
}

TSharedRef<SButton> CreateAddItemButton(const FText& TooltipText, FLinearColor Color, TSharedPtr<IPropertyHandle> ArrayPropertyHandle, TSharedRef<IPropertyUtilities> PropUtils)
{
	const TSharedRef<SButton> Button = SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.ToolTipText(TooltipText)
		.OnClicked_Lambda([ArrayPropertyHandle]()
		{
			if (ArrayPropertyHandle && ArrayPropertyHandle->IsValidHandle())
			{
				if (const TSharedPtr<IPropertyHandleArray> ArrayHandle = ArrayPropertyHandle->AsArray())
				{
					if (ArrayHandle->AddItem() == FPropertyAccess::Success)
					{
						uint32 NumElements = 0;
						if (ArrayHandle->GetNumElements(NumElements) == FPropertyAccess::Success && NumElements > 0)
						{
							TSharedRef<IPropertyHandle> NewPropertyHandle = ArrayHandle->GetElement(NumElements-1);
							NewPropertyHandle->SetExpanded(true);
						}
					}
				}
			}
			return FReply::Handled();
		})
		.IsEnabled(PropUtils, &IPropertyUtilities::IsPropertyEditingEnabled)
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
			.ColorAndOpacity(Color)
		];

	return Button;
}

IDetailCategoryBuilder& MakeArrayCategory(
	IDetailLayoutBuilder& DetailBuilder,
	const TSharedPtr<IPropertyHandle>& ArrayPropertyHandle,
	const FName CategoryName,
	const FText& CategoryDisplayName,
	const FName IconName,
	const FLinearColor IconColor,
	const FLinearColor AddIconColor,
	const FText& AddButtonTooltipText,
	const int32 SortOrder)
{
	IDetailCategoryBuilder& Category = MakeArrayCategoryHeader(
		DetailBuilder,
		ArrayPropertyHandle,
		CategoryName,
		CategoryDisplayName,
		IconName,
		IconColor,
		TSharedPtr<SWidget>(),
		AddIconColor,
		AddButtonTooltipText,
		SortOrder
	);
	MakeArrayItems(Category, ArrayPropertyHandle);
	return Category;
}

IDetailCategoryBuilder& MakeArrayCategoryHeader(
	IDetailLayoutBuilder& DetailBuilder,
	const TSharedPtr<IPropertyHandle>& ArrayPropertyHandle,
	const FName CategoryName,
	const FText& CategoryDisplayName,
	const FName IconName,
	const FLinearColor IconColor,
	const TSharedPtr<SWidget> Extension,
	const FLinearColor AddIconColor,
	const FText& AddButtonTooltipText,
	const int32 SortOrder)
{
	IDetailCategoryBuilder& Category = DetailBuilder.EditCategory(CategoryName, CategoryDisplayName);
	Category.SetSortOrder(SortOrder);
	Category.SetColor(IconColor);

	bool bIsNodeArray = false;
	if (const FArrayProperty* ArrayProperty = CastField<const FArrayProperty>(ArrayPropertyHandle->GetProperty()))
	{
		if (const FStructProperty* InnerStruct = CastField<const FStructProperty>(ArrayProperty->Inner))
		{
			bIsNodeArray = InnerStruct->Struct->IsChildOf(TBaseStructure<FStateTreeEditorNode>::Get());
		}
	}

	TSharedPtr<SWidget> AddWidget;
	if (bIsNodeArray)
	{
		// Node array, make the add button a node picker too. 
		AddWidget = CreateAddNodePickerComboButton(AddButtonTooltipText, AddIconColor, ArrayPropertyHandle, DetailBuilder.GetPropertyUtilities());
	}
	else
	{
		// Regular array, just add.
		AddWidget = CreateAddItemButton(AddButtonTooltipText, AddIconColor, ArrayPropertyHandle, DetailBuilder.GetPropertyUtilities());
	}
	
	TSharedRef<SWidget> RemoveAllWidget = PropertyCustomizationHelpers::MakeEmptyButton(
		FSimpleDelegate::CreateLambda([ArrayPropertyHandle]()
			{
				if (ArrayPropertyHandle && ArrayPropertyHandle->IsValidHandle())
				{
					if (const TSharedPtr<IPropertyHandleArray> ArrayHandle = ArrayPropertyHandle->AsArray())
					{
						ArrayHandle->EmptyArray();
					}
				}
			})
		);

	const TSharedRef<SHorizontalBox> HeaderContent = SNew(SHorizontalBox);

	if (!IconName.IsNone())
	{
		HeaderContent->AddSlot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(FMargin(4, 0, 0, 0))
			[
				SNew(SImage)
				.ColorAndOpacity(IconColor)
				.Image(FStateTreeEditorStyle::Get().GetBrush(IconName))
			];
	}
	
	HeaderContent->AddSlot()
		.FillWidth(1.f)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(FMargin(4, 0, 0, 0))
		[
			SNew(STextBlock)
			.TextStyle(FStateTreeEditorStyle::Get(), "StateTree.Category")
			.Text(CategoryDisplayName)
		];

	if (Extension)
	{
		HeaderContent->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				Extension.ToSharedRef()
			];
	}


	HeaderContent->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			AddWidget.ToSharedRef()
		];

	HeaderContent->AddSlot()
		.AutoWidth()
		.Padding(FMargin(0.f, 0.f, 4.f, 0.f))
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			RemoveAllWidget
		];
	
	Category.HeaderContent(SNew(SBox)
		.MinDesiredHeight(30.f)
		[
			HeaderContent
		],
		/*bWholeRowContent*/true);
	return Category;
}

void MakeArrayItems(IDetailCategoryBuilder& Category, const TSharedPtr<IPropertyHandle>& ArrayPropertyHandle)
{
	// Add items inline
	const TSharedRef<FDetailArrayBuilder> Builder = MakeShareable(new FDetailArrayBuilder(ArrayPropertyHandle.ToSharedRef(), /*InGenerateHeader*/ false, /*InDisplayResetToDefault*/ true, /*InDisplayElementNum*/ false));
	Builder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateLambda([](TSharedRef<IPropertyHandle> PropertyHandle, int32 ArrayIndex, IDetailChildrenBuilder& ChildrenBuilder)
	{
		ChildrenBuilder.AddProperty(PropertyHandle);
	}));
	Category.AddCustomBuilder(Builder, /*bForAdvanced*/ false);
}
} // namespace UE::StateTreeEditor::EditorNodeUtils

#undef LOCTEXT_NAMESPACE
