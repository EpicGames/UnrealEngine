// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"
#include "Textures/SlateIcon.h"
#include "Styling/SlateColor.h"

class UStateTreeEditorData;
enum class EStateTreeConditionEvaluationMode : uint8;
struct FStateTreeEditorNode;
struct FStructView;
struct EVisibility;
class FText;
class IPropertyHandle;
class IPropertyUtilities;
class IDetailCategoryBuilder;
class IDetailLayoutBuilder;
template <typename FuncType> class TFunctionRef;
class SComboButton;
class SWidget;

namespace UE::StateTreeEditor::EditorNodeUtils
{
/**
 * @param StructProperty property pointing to a FStateTreeEditorNode.
 * @return StateTree editor node that is common for all edited instances.
 */
const FStateTreeEditorNode* GetCommonNode(const TSharedPtr<IPropertyHandle>& StructProperty);

/**
 * @param StructProperty Property handle pointing to a FStateTreeEditorNode.
 * @return mutable StateTree editor node that is common for all edited instances.
 */
FStateTreeEditorNode* GetMutableCommonNode(const TSharedPtr<IPropertyHandle>& StructProperty);

/**
 * Returns class and/or struct defined in property's "BaseClass" and "BaseStruct" metadata.
 * @param StructProperty Property handle where the meta data is looked at.
 * @param OutBaseScriptStruct ScriptStruct pointer or null if not found or set.
 * @param OutBaseClass Class pointer or null if not found or set.
 */
void GetNodeBaseScriptStructAndClass(const TSharedPtr<IPropertyHandle>& StructProperty, UScriptStruct*& OutBaseScriptStruct, UClass*& OutBaseClass);

/**
 * Returns visibility status depending if the node is a Condition.
 * @param StructProperty Property handle pointing to a FStateTreeEditorNode. 
 * @return visibility status. 
 */
EVisibility IsConditionVisible(const TSharedPtr<IPropertyHandle>& StructProperty);

/**
 * Returns visibility status depending if the node is a Consideration.
 * @param StructProperty Property handle pointing to a FStateTreeEditorNode.
 * @return visibility status.
 */
EVisibility IsConsiderationVisible(const TSharedPtr<IPropertyHandle>& StructProperty);

/**
 * Returns condition evaluation method of a node.
 * @param StructProperty Property handle pointing to a FStateTreeEditorNode. 
 * @return condition evaluation method or Evaluated if not valid node.
 */
EStateTreeConditionEvaluationMode GetConditionEvaluationMode(const TSharedPtr<IPropertyHandle>& StructProperty);

/**
 * @param StructProperty Property handle pointing to a FStateTreeEditorNode. 
 * @return Whether specified task node is marked as disable. Will return false if node is not a valid task.
 */
bool IsTaskDisabled(const TSharedPtr<IPropertyHandle>& StructProperty);

/**
 * @param StructProperty Property handle pointing to a FStateTreeEditorNode.
 * @return Whether the specified task node is marked as enabled. Will return false if node is not a valid task.
 */
bool IsTaskEnabled(const FStateTreeEditorNode& EditorNode);

/** @return Whether the specified task is marked as ConsideredForCompletion.  Will return false if node is not a valid task. */
bool IsTaskConsideredForCompletion(const FStateTreeEditorNode& EditorNode);

/** Set the ConsideredForCompletion flag on the specified task. */
void SetTaskConsideredForCompletion(FStateTreeEditorNode& EditorNode, bool bIsConsidered);

/** @return Whether the ConsideredForCompletion flag can be edited on the specified task. */
bool CanEditTaskConsideredForCompletion(const FStateTreeEditorNode& EditorNode);

/**
 * Execute the provided function within a Transaction 
 * @param Description Description to associate to the transaction
 * @param StructProperty Property handle of the StateTreeEditorNode(s) 
 * @param Func Function that will be execute in the transaction on a valid Editor node property
 */
void ModifyNodeInTransaction(const FText& Description, const TSharedPtr<IPropertyHandle>& StructProperty, TFunctionRef<void(const TSharedPtr<IPropertyHandle>&)> Func);

/**
 * Parses Slate Icon from a name.
 * @param IconName In format StyleSetName | StyleName | [SmallStyleName | StatusOverlayStyleName] 
 * @return Slate icon from parsed name. 
 */
FSlateIcon ParseIcon(const FName IconName);

/**
 * Returns slate icon associated with specified node. 
 * @param StructProperty Property handle pointing to a FStateTreeEditorNode. 
 * @return Slate icon from parsed from node's icon name.
 */
FSlateIcon GetIcon(const TSharedPtr<IPropertyHandle>& StructProperty);

/**
 * Returns color of icon associated with specified node. 
 * @param StructProperty Property handle pointing to a FStateTreeEditorNode. 
 * @return Icon color, or foreground if not set.
 */
FSlateColor GetIconColor(const TSharedPtr<IPropertyHandle>& StructProperty);

/**
 * Return visibility status depending on if the specified node has an icon.   
 * @param StructProperty Property handle pointing to a FStateTreeEditorNode. 
 * @return Visible, if the node has icon.
 */
EVisibility IsIconVisible(const TSharedPtr<IPropertyHandle>& StructProperty);

/**
 * Sets the type of a node. Creates a transaction.
 * @param StructProperty Property handle pointing to a FStateTreeEditorNode. 
 * @param NewType New node type to set. 
 */
void SetNodeType(const TSharedPtr<IPropertyHandle>& StructProperty, const UStruct* NewType);

/**
 * Recursively instantiates instanced objects of a given struct.
 * It is needed to fixup nodes pasted from clipboard, which seem to give shallow copy.
 * @param OuterObject the object the instanced objects should be outered to
 * @param Struct the struct with the instanced objects
 */
void InstantiateStructSubobjects(UObject& OuterObject, FStructView Struct);

/**
 * Handles updating the Node Instance Data if there is a type mismatch
 * @param EditorNode the node to check
 * @param InstanceOuter the outer to use if the instance data is a uobject, or there are instanced uobjects within the instance data
 */
void ConditionalUpdateNodeInstanceData(FStateTreeEditorNode& EditorNode, UObject& InstanceOuter);

/**
 * Creates widget combon button with plus icon (+), which summons node picker and adds the selected node to specified array. 
 * @param TooltipText Tool tip to show on he button.
 * @param Color Color of the icon on the button.
 * @param ArrayPropertyHandle Property handle pointing to a TArray<FStateTreeEditorNode>.
 * @param PropUtils Property utils, used to check edit status and to refresh the array after add.
 * @return shared ref to the created combo button.
 */
TSharedRef<SComboButton> CreateAddNodePickerComboButton(const FText& TooltipText, FLinearColor Color, TSharedPtr<IPropertyHandle> ArrayPropertyHandle, TSharedRef<IPropertyUtilities> PropUtils);

/**
 * Creates a category and sets the contents of the row to: [Icon] [DisplayName]  [+].
 * @param DetailBuilder Detail builder where the category is added.
 * @param ArrayPropertyHandle Property handle pointing to a TArray<FStateTreeEditorNode>.
 * @param CategoryName Name of the category to create.
 * @param CategoryDisplayName Display name of the category.
 * @param IconName Name to the icon resource to show in front of the category display name.
 * @param IconColor Color of icon resource.
 * @param AddIconColor Color of the add icon.
 * @param AddButtonTooltipText Tooltip text to show on the add buttons.
 * @param SortOrder Category sort order (categories with smaller sort order appear earlier in details panel).
 * @return Reference to the category builder of the category just created.
 */
IDetailCategoryBuilder& MakeArrayCategory(
	IDetailLayoutBuilder& DetailBuilder,
	const TSharedPtr<IPropertyHandle>& ArrayPropertyHandle,
	FName CategoryName,
	const FText& CategoryDisplayName,
	FName IconName,
	FLinearColor IconColor,
	FLinearColor AddIconColor,
	const FText& AddButtonTooltipText,
	int32 SortOrder);

/**
 * Creates a category and sets the contents of the row to: [Icon] [DisplayName]  [+].
 * @param DetailBuilder Detail builder where the category is added.
 * @param ArrayPropertyHandle Property handle pointing to a TArray<FStateTreeEditorNode>.
 * @param CategoryName Name of the category to create.
 * @param CategoryDisplayName Display name of the category.
 * @param IconName Name to the icon resource to show in front of the category display name.
 * @param IconColor Color of icon resource.
 * @param AddIconColor Color of the add icon.
 * @param AddButtonTooltipText Tooltip text to show on the add buttons.
 * @param SortOrder Category sort order (categories with smaller sort order appear earlier in details panel).
 * @return Reference to the category builder of the category just created.
 */
IDetailCategoryBuilder& MakeArrayCategoryHeader(
	IDetailLayoutBuilder& DetailBuilder,
	const TSharedPtr<IPropertyHandle>& ArrayPropertyHandle,
	FName CategoryName,
	const FText& CategoryDisplayName,
	FName IconName,
	FLinearColor IconColor,
	const TSharedPtr<SWidget> Extension,
	FLinearColor AddIconColor,
	const FText& AddButtonTooltipText,
	int32 SortOrder);

/**
 * Creates the items of a category.
 * @param Category Builder created by the header.
 * @param ArrayPropertyHandle Property handle pointing to a TArray<FStateTreeEditorNode>.
 */
void MakeArrayItems(IDetailCategoryBuilder& Category, const TSharedPtr<IPropertyHandle>& ArrayPropertyHandle);
};