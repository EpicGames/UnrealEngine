// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeEnumValueScorePairArrayBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "Considerations/StateTreeCommonConsiderations.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "StateTreePropertyHelpers.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"

FStateTreeEnumValueScorePairArrayBuilder::FStateTreeEnumValueScorePairArrayBuilder(TSharedRef<IPropertyHandle> InBasePropertyHandle, const UEnum* InEnumType, bool InGenerateHeader, bool InDisplayResetToDefault, bool InDisplayElementNum)
	: FDetailArrayBuilder(InBasePropertyHandle, InGenerateHeader, InDisplayResetToDefault, InDisplayElementNum)
	, EnumType(InEnumType)
	, PairArrayProperty(InBasePropertyHandle->AsArray())
{
}

void FStateTreeEnumValueScorePairArrayBuilder::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	uint32 NumChildren = 0;
	PairArrayProperty->GetNumElements(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> PairPropertyHandle = PairArrayProperty->GetElement(ChildIndex);

		CustomizePairRowWidget(PairPropertyHandle, ChildrenBuilder);
	}
}

void FStateTreeEnumValueScorePairArrayBuilder::CustomizePairRowWidget(TSharedRef<IPropertyHandle> PairPropertyHandle, IDetailChildrenBuilder& ChildrenBuilder)
{
	TSharedPtr<IPropertyHandle> EnumValuePropertyHandle = PairPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeEnumValueScorePair, EnumValue));
	TSharedPtr<IPropertyHandle> EnumNamePropertyHandle = PairPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeEnumValueScorePair, EnumName));
	TSharedPtr<IPropertyHandle> ScorePropertyHandle = PairPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeEnumValueScorePair, Score));

	IDetailPropertyRow& PropertyRow = ChildrenBuilder.AddProperty(PairPropertyHandle);

	PropertyRow.CustomWidget(false)
		.NameContent()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			SNew(SComboButton)
				.OnGetMenuContent(this, &FStateTreeEnumValueScorePairArrayBuilder::GetEnumEntryComboContent, EnumValuePropertyHandle, EnumNamePropertyHandle)
				.ContentPadding(.0f)
				.ButtonContent()
				[
					SNew(STextBlock)
						.Text(this, &FStateTreeEnumValueScorePairArrayBuilder::GetEnumEntryDescription, PairPropertyHandle)
						.Font(IDetailLayoutBuilder::GetDetailFontBold())
				]
		]
		.ValueContent()
		[
			ScorePropertyHandle->CreatePropertyValueWidget()
		];
}

FText FStateTreeEnumValueScorePairArrayBuilder::GetEnumEntryDescription(TSharedRef<IPropertyHandle> PairPropertyHandle) const
{
	FStateTreeEnumValueScorePair EnumValueScorePair;
	FPropertyAccess::Result Result = UE::StateTree::PropertyHelpers::GetStructValue<FStateTreeEnumValueScorePair>(PairPropertyHandle, EnumValueScorePair);
	if (Result == FPropertyAccess::Success)
	{
		if (EnumType)
		{
			return EnumType->GetDisplayNameTextByValue(int64(EnumValueScorePair.EnumValue));
		}
	}
	else if (Result == FPropertyAccess::MultipleValues)
	{
		return LOCTEXT("MultipleSelected", "Multiple Selected");
	}

	return LOCTEXT("None", "None");
}

TSharedRef<SWidget> FStateTreeEnumValueScorePairArrayBuilder::GetEnumEntryComboContent(TSharedPtr<IPropertyHandle> EnumValuePropertyHandle, TSharedPtr<IPropertyHandle> EnumNamePropertyHandle) const
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection*/true, /*InCommandList*/nullptr);

	if (EnumType)
	{
		const bool bHasMaxValue = EnumType->ContainsExistingMax();
		const int32 NumEnums = bHasMaxValue ? EnumType->NumEnums() - 1 : EnumType->NumEnums();

		for (int32 Index = 0; Index < NumEnums; Index++)
		{

#if WITH_METADATA
			if (EnumType->HasMetaData(TEXT("Hidden"), Index))
			{
				continue;
			}
#endif //WITH_METADATA

			const int64 Value = EnumType->GetValueByIndex(Index);
			MenuBuilder.AddMenuEntry(EnumType->GetDisplayNameTextByIndex(Index), TAttribute<FText>(), FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda(
					[=, this]() {
						if (EnumValuePropertyHandle)
						{
							EnumValuePropertyHandle->SetValue(Value);
							EnumNamePropertyHandle->SetValue(EnumType->GetNameByIndex(Index));
						}
					})
				));
		}
	}
	else
	{
		MenuBuilder.AddMenuEntry(LOCTEXT("None", "None"), TAttribute<FText>(), FSlateIcon(), FUIAction());
	}

	return MenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
