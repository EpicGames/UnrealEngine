// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyCustomizationHelpers.h"

class SWidget;
class IDetailChildrenBuilder;
class IPropertyHandle;
class IPropertyHandleArray;

/** Details customization for arrays composed of <EnumValue, Score> properties with a provided Enum Type.*/
class FStateTreeEnumValueScorePairArrayBuilder : public FDetailArrayBuilder
	, public TSharedFromThis<FStateTreeEnumValueScorePairArrayBuilder>
{
public:
	FStateTreeEnumValueScorePairArrayBuilder(TSharedRef<IPropertyHandle> InBasePropertyHandle, const UEnum* InEnumType, bool InGenerateHeader = true, bool InDisplayResetToDefault = true, bool InDisplayElementNum = true);

	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;
private:
	void CustomizePairRowWidget(TSharedRef<IPropertyHandle> PairPropertyHandle, IDetailChildrenBuilder& ChildrenBuilder);
	FText GetEnumEntryDescription(TSharedRef<IPropertyHandle> PairPropertyHandle) const;
	TSharedRef<SWidget> GetEnumEntryComboContent(TSharedPtr<IPropertyHandle> EnumValuePropertyHandle, TSharedPtr<IPropertyHandle> EnumNamePropertyHandle) const;

private:
	TStrongObjectPtr<const UEnum> EnumType;
	TSharedPtr<IPropertyHandleArray> PairArrayProperty;
};
