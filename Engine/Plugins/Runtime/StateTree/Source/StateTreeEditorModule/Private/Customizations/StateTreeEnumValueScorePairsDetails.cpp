// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeEnumValueScorePairsDetails.h"
#include "Considerations/StateTreeCommonConsiderations.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "StateTreeEnumValueScorePairArrayBuilder.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"

TSharedRef<IPropertyTypeCustomization> FStateTreeEnumValueScorePairsDetails::MakeInstance()
{
	return MakeShareable(new FStateTreeEnumValueScorePairsDetails);
}

void FStateTreeEnumValueScorePairsDetails::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

void FStateTreeEnumValueScorePairsDetails::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	EnumProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeEnumValueScorePairs, Enum));
	PairsProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeEnumValueScorePairs, Data));

	UObject* Object = nullptr; 
	if (EnumProperty->GetValue(Object) == FPropertyAccess::Success)
	{
		const UEnum* EnumType = static_cast<UEnum*>(Object);
		TSharedRef<FStateTreeEnumValueScorePairArrayBuilder> Builder = MakeShareable(
			new FStateTreeEnumValueScorePairArrayBuilder(
				PairsProperty.ToSharedRef(), EnumType, /*InGenerateHeader*/ true, /*InDisplayResetToDefault*/ false, /*InDisplayElementNum*/ true));
			
		StructBuilder.AddCustomBuilder(Builder);
	}
}
#undef LOCTEXT_NAMESPACE
