// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class SWidget;

class IPropertyHandle;

/**
 * Type customization for FStateTreeEnumValueScorePairs.
 */

class FStateTreeEnumValueScorePairsDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

private:

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	FText GetDescription() const;
	TSharedRef<SWidget> OnGetComboContent() const;

	TSharedPtr<IPropertyHandle> EnumProperty;
	TSharedPtr<IPropertyHandle> PairsProperty;
};
