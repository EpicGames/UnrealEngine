// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeEventDescDetails.h"
#include "DetailWidgetRow.h"
#include "StateTreeState.h"
#include "Widgets/Text/STextBlock.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "PropertyBindingExtension.h"
#include "StateTreeBindingExtension.h"
#include "StateTreeEditorStyle.h"
#include "StateTreeGraphSchema.h"
#include "Styling/StyleColors.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Widgets/Images/SImage.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"

/** 
 * Node builder for StateTreeEvent Payload.
 * Draws recursively all children of the selected Payload type without possibility of modifying them.
 */
class FStateTreeEventPayloadDetails : public IDetailCustomNodeBuilder
{
public:
	using FVisitedStructs = TArray<UStruct*, TInlineAllocator<16>>;

	FStateTreeEventPayloadDetails(const FProperty& Property, const FVisitedStructs& InVisitedStructs)
		: Name(Property.GetName())
		, VisitedStructs(InVisitedStructs)
	{
		NameWidget = GenerateNameWidget(Property);
		ValueWidget = GenerateValueWidget(Property);

		if (const FStructProperty* StructProperty = CastField<FStructProperty>(&Property))
		{
			Struct = StructProperty->Struct;
		}
		else if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(&Property))
		{
			Struct = ObjectProperty->PropertyClass;
		}

		if (Struct)
		{
			if (VisitedStructs.Contains(Struct))
			{
				Struct = nullptr;
			}
			else
			{
				VisitedStructs.Add(Struct);
			}
		}
	}

	FStateTreeEventPayloadDetails(FName InName, TSharedRef<SWidget> InNameWidget, TSharedRef<SWidget> InValueWidget, UStruct* InStruct, const FVisitedStructs& InVisitedStructs)
		: Name(InName)
		, NameWidget(InNameWidget)
		, ValueWidget(InValueWidget)	
		, VisitedStructs(InVisitedStructs)
		, Struct(InStruct)
	{
		if (VisitedStructs.Contains(Struct))
		{
			Struct = nullptr;
		}
		else
		{
			VisitedStructs.Add(Struct);
		}
	}

	virtual bool InitiallyCollapsed() const override
	{
		return true;
	}

	virtual void GenerateHeaderRowContent( FDetailWidgetRow& NodeRow ) override 
	{
		NodeRow.ShouldAutoExpand(false);
		NodeRow.NameContent()
		[
			NameWidget.ToSharedRef()
		];

		NodeRow.ValueContent()
		[
			ValueWidget.ToSharedRef()
		];
	}

	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override
	{
		for (TFieldIterator<const FProperty> PropIt(Struct); PropIt; ++PropIt)
		{
			const FProperty& Property = **PropIt;
			if (UE::PropertyBinding::IsPropertyBindable(Property))
			{
				ChildrenBuilder.AddCustomBuilder(MakeShared<FStateTreeEventPayloadDetails>(Property, VisitedStructs));
			}
		}
	}

	FName GetName() const override
	{
		return Name;
	}

private:
	TSharedRef<SWidget> GenerateNameWidget(const FProperty& Property) const
	{
		return SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(Property.GetDisplayNameText())
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.ToolTipText(Property.GetToolTipText())
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(4.0f, 0.0f)
					[
						SNew(SBorder)
						.Padding(FMargin(6, 1))
						.BorderImage(FStateTreeEditorStyle::Get().GetBrush("StateTree.Param.Background"))
						[
							SNew(STextBlock)
							.TextStyle(FStateTreeEditorStyle::Get(), "StateTree.Param.Label")
							.ColorAndOpacity(FStyleColors::Foreground)
							.Text(LOCTEXT("LabelOutput", "OUT"))
						]
					];
	}

	TSharedRef<SWidget> GenerateValueWidget(const FProperty& Property) const
	{
		using namespace UE::StateTree::Editor;

		const UEdGraphSchema_StateTree* Schema = GetDefault<UEdGraphSchema_StateTree>();
		FEdGraphPinType PinType;
		Schema->ConvertPropertyToPinType(&Property, PinType);

		const FSlateBrush* Icon = FBlueprintEditorUtils::GetIconFromPin(PinType, true);
		const FText Text = GetPinTypeText(PinType);
		const FLinearColor IconColor = Schema->GetPinTypeColor(PinType);

		return SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(4.0f, 0.0f)
					[
						SNew(SImage)
							.Image(Icon)
							.ColorAndOpacity(IconColor)
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.Text(Text)
					];
	}

	static FText GetPinTypeText(const FEdGraphPinType& PinType)
	{
		using namespace UE::StateTree::Editor;

		const FName PinSubCategory = PinType.PinSubCategory;
		const UObject* PinSubCategoryObject = PinType.PinSubCategoryObject.Get();
		if (PinSubCategory != UEdGraphSchema_K2::PSC_Bitmask && PinSubCategoryObject)
		{
			if (const UField* Field = Cast<const UField>(PinSubCategoryObject))
			{
				return Field->GetDisplayNameText();
			}
			return FText::FromString(PinSubCategoryObject->GetName());
		}

		return UEdGraphSchema_StateTree::GetCategoryText(PinType.PinCategory, NAME_None, true);
	}

	FName Name;
	TSharedPtr<SWidget> NameWidget;
	TSharedPtr<SWidget> ValueWidget;
	FVisitedStructs VisitedStructs;
	UStruct* Struct = nullptr;
};

TSharedRef<IPropertyTypeCustomization> FStateTreeEventDescDetails::MakeInstance()
{
	return MakeShared<FStateTreeEventDescDetails>();
}

void FStateTreeEventDescDetails::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	HeaderRow
	.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		PropertyHandle->CreatePropertyValueWidget()
	];
}

void FStateTreeEventDescDetails::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyHandle> TagProperty = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeEventDesc, Tag));
	check(TagProperty);
	ChildBuilder.AddProperty(TagProperty.ToSharedRef());

	TSharedPtr<IPropertyHandle> PayloadProperty = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeEventDesc, PayloadStruct));
	check(PayloadProperty);

	UObject* PayloadType = nullptr;
	if (PayloadProperty->GetValue(PayloadType))
	{
		UStruct* PayloadStruct = Cast<UStruct>(PayloadType);
		ChildBuilder.AddCustomBuilder(MakeShared<FStateTreeEventPayloadDetails>(TEXT("EventPayload"), PayloadProperty->CreatePropertyNameWidget(), PayloadProperty->CreatePropertyValueWidget(), PayloadStruct, FStateTreeEventPayloadDetails::FVisitedStructs()));
	}

	PayloadProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([PropUtils = CustomizationUtils.GetPropertyUtilities()]()
	{
		PropUtils->RequestForceRefresh(); 
	}));

	TSharedPtr<IPropertyHandle> ConsumeEventOnSelectProperty = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeEventDesc, bConsumeEventOnSelect));
	check(ConsumeEventOnSelectProperty);
	ChildBuilder.AddProperty(ConsumeEventOnSelectProperty.ToSharedRef());
}

#undef LOCTEXT_NAMESPACE
