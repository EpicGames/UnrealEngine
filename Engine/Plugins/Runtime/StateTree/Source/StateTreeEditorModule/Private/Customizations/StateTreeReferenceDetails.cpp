// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeReferenceDetails.h"

#include "StateTree.h"
#include "StateTreeDelegates.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "IStateTreeSchemaProvider.h"
#include "PropertyBagDetails.h"
#include "PropertyCustomizationHelpers.h"
#include "StateTreeReference.h"
#include "StateTreePropertyHelpers.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"

class FStateTreeRefParametersDetails : public FPropertyBagInstanceDataDetails
{
public:
	FStateTreeRefParametersDetails(const TSharedPtr<IPropertyHandle> InStateTreeRefStructProperty, const TSharedPtr<IPropertyHandle> InParametersStructProperty, const TSharedPtr<IPropertyUtilities>& InPropUtils)
		: FPropertyBagInstanceDataDetails(InParametersStructProperty, InPropUtils, /*bInFixedLayout*/true)
		, StateTreeRefStructProperty(InStateTreeRefStructProperty)
	{
		check(UE::StateTree::PropertyHelpers::IsScriptStruct<FStateTreeReference>(StateTreeRefStructProperty));
	}

protected:
	struct FStateTreeReferenceOverrideProvider : public IPropertyBagOverrideProvider
	{
		FStateTreeReferenceOverrideProvider(FStateTreeReference& InStateTreeRef)
			: StateTreeRef(InStateTreeRef)
		{
		}
		
		virtual bool IsPropertyOverridden(const FGuid PropertyID) const override
		{
			return StateTreeRef.IsPropertyOverridden(PropertyID);
		}
		
		virtual void SetPropertyOverride(const FGuid PropertyID, const bool bIsOverridden) const override
		{
			StateTreeRef.SetPropertyOverridden(PropertyID, bIsOverridden);
		}

	private:
		FStateTreeReference& StateTreeRef;
	};

	virtual bool HasPropertyOverrides() const override
	{
		return true;
	}

	virtual void PreChangeOverrides() override
	{
		check(StateTreeRefStructProperty);
		StateTreeRefStructProperty->NotifyPreChange();
	}

	virtual void PostChangeOverrides() override
	{
		check(StateTreeRefStructProperty);
		StateTreeRefStructProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
		StateTreeRefStructProperty->NotifyFinishedChangingProperties();
	}

	virtual void EnumeratePropertyBags(TSharedPtr<IPropertyHandle> PropertyBagHandle, const EnumeratePropertyBagFuncRef& Func) const override
	{
		check(StateTreeRefStructProperty);
		StateTreeRefStructProperty->EnumerateRawData([Func](void* RawData, const int32 /*DataIndex*/, const int32 /*NumDatas*/)
		{
			if (FStateTreeReference* StateTreeRef = static_cast<FStateTreeReference*>(RawData))
			{
				if (const UStateTree* StateTree = StateTreeRef->GetStateTree())
				{
					const FInstancedPropertyBag& DefaultParameters = StateTree->GetDefaultParameters();
					FInstancedPropertyBag& Parameters = StateTreeRef->GetMutableParameters();
					FStateTreeReferenceOverrideProvider OverrideProvider(*StateTreeRef);
					if (!Func(DefaultParameters, Parameters, OverrideProvider))
					{
						return false;
					}
				}
			}
			return true;
		});
	}

private:
	
	TSharedPtr<IPropertyHandle> StateTreeRefStructProperty;
};


TSharedRef<IPropertyTypeCustomization> FStateTreeReferenceDetails::MakeInstance()
{
	return MakeShareable(new FStateTreeReferenceDetails);
}

void FStateTreeReferenceDetails::CustomizeHeader(const TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	StructPropertyHandle = InStructPropertyHandle;
	PropUtils = InCustomizationUtils.GetPropertyUtilities();

	ParametersHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeReference, Parameters));
	check(ParametersHandle);
	
	// Make sure parameters are synced when displaying StateTreeReference.
	// Associated StateTree asset might have been recompiled after the StateTreeReference was loaded.
	// Note: SyncParameters() will create an undoable transaction, do not try to sync when called during Undo/redo as it would overwrite the undo.   
	if (!GIsTransacting)
	{
		SyncParameters();
	}

	const TSharedPtr<IPropertyHandle> StateTreeProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeReference, StateTree));
	check(StateTreeProperty.IsValid());

	const FString SchemaMetaDataValue = GetSchemaPath(*StructPropertyHandle);

	InHeaderRow
	.NameContent()
	[
		StateTreeProperty->CreatePropertyNameWidget(StructPropertyHandle->GetPropertyDisplayName())
	]
	.ValueContent()
	[
		SNew(SObjectPropertyEntryBox)
		.PropertyHandle(StateTreeProperty)
		.AllowedClass(UStateTree::StaticClass())
		.ThumbnailPool(InCustomizationUtils.GetThumbnailPool())
		.OnShouldFilterAsset_Lambda([SchemaMetaDataValue](const FAssetData& InAssetData)
		{
			return !SchemaMetaDataValue.IsEmpty() && !InAssetData.TagsAndValues.ContainsKeyValue(UE::StateTree::SchemaTag, SchemaMetaDataValue);
		})
	]
	.ShouldAutoExpand(true);
	
	StateTreeProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateSPLambda(AsShared(), [this]()
		{
			SyncParameters();
		}));

	// Registers a delegate to be notified when the associated StateTree asset get successfully recompiled
	// to make sure that the parameters in the StateTreeReference are still valid.
	UE::StateTree::Delegates::OnPostCompile.AddSP(this, &FStateTreeReferenceDetails::OnTreeCompiled);
}

void FStateTreeReferenceDetails::CustomizeChildren(const TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& InChildrenBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	// Show parameter values.
	const TSharedRef<FStateTreeRefParametersDetails> ParameterInstanceDetails = MakeShareable(new FStateTreeRefParametersDetails(StructPropertyHandle, ParametersHandle, PropUtils));
	InChildrenBuilder.AddCustomBuilder(ParameterInstanceDetails);
}

void FStateTreeReferenceDetails::OnTreeCompiled(const UStateTree& StateTree) const
{
	SyncParameters(&StateTree);
}

void FStateTreeReferenceDetails::SyncParameters(const UStateTree* StateTreeToSync) const
{
	check(StructPropertyHandle.IsValid());

	// This function will get called in 3 situations:
	// - auto update when the property customization is created
	// - auto update when a state tree is compiled.
	// - when the associate state tree asset is changed
	
	TArray<UObject*> OuterObjects;
	StructPropertyHandle->GetOuterObjects(OuterObjects);

	TArray<void*> RawData;
	StructPropertyHandle->AccessRawData(RawData);

	bool bDidSync = false;
	
	for (int32 i = 0; i < RawData.Num(); i++)
	{
		if (FStateTreeReference* StateTreeReference = static_cast<FStateTreeReference*>(RawData[i]))
		{
			const UStateTree* StateTreeAsset = StateTreeReference->GetStateTree();

			if ((StateTreeToSync == nullptr || StateTreeAsset == StateTreeToSync) 
				&& StateTreeReference->RequiresParametersSync())
			{
				// Changing the data without Modify().
				// When called on property row creation, we don't expect to dirty the owner.
				// In other cases we expect the outer to already been modified.
				StateTreeReference->SyncParameters();
				bDidSync = true;
			}
		}
	}

	if (bDidSync && PropUtils)
	{
		PropUtils->RequestRefresh();
	}
}

FString FStateTreeReferenceDetails::GetSchemaPath(IPropertyHandle& StructPropertyHandle)
{
	if (StructPropertyHandle.HasMetaData(UE::StateTree::SchemaCanBeOverridenTag))
	{
		TArray<UObject*> OuterObjects;
		StructPropertyHandle.GetOuterObjects(OuterObjects);

		for (const UObject* OuterObject : OuterObjects)
		{
			if (const IStateTreeSchemaProvider* SchemaProvider = Cast<IStateTreeSchemaProvider>(OuterObject))
			{
				if (const UClass* SchemaClass = SchemaProvider->GetSchema().Get())
				{
					return SchemaClass->GetPathName();
				}
			}
		}
	}

	return StructPropertyHandle.GetMetaData(UE::StateTree::SchemaTag);
}

#undef LOCTEXT_NAMESPACE
