// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class IPropertyHandle;
class UStateTree;
class IPropertyUtilities;

/**
 * Type customization for FStateTreeReference.
 */
class FStateTreeReferenceDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& InChildrenBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;

private:
	/**
	 * Callback registered to StateTree post compilation delegate used to synchronize parameters.
	 * @param StateTree The StateTree asset that got compiled.
	 */
	void OnTreeCompiled(const UStateTree& StateTree) const;

	/**
	 * Synchronizes parameters with StateTree asset parameters if needed.
	 * @param StateTreeToSync Optional StateTree asset used to filter which parameters should be synced. A null value indicates that all parameters should be synced.   
	 */
	void SyncParameters(const UStateTree* StateTreeToSync = nullptr) const;

	static FString GetSchemaPath(IPropertyHandle& StructPropertyHandle);

	TSharedPtr<IPropertyHandle> StructPropertyHandle;
	TSharedPtr<IPropertyHandle> ParametersHandle;
	TSharedPtr<IPropertyUtilities> PropUtils;
};
