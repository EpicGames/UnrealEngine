// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeReferenceOverridesDetails.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "StateTreeReference.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"

TSharedRef<IPropertyTypeCustomization> FStateTreeReferenceOverridesDetails::MakeInstance()
{
	return MakeShareable(new FStateTreeReferenceOverridesDetails);
}

void FStateTreeReferenceOverridesDetails::CustomizeHeader(const TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	StructPropertyHandle = InStructPropertyHandle;
	PropUtils = InCustomizationUtils.GetPropertyUtilities();

	OverrideItemsHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeReferenceOverrides, OverrideItems));
	check(OverrideItemsHandle);
	
	InHeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		OverrideItemsHandle->CreatePropertyValueWidget()
	]
	.ShouldAutoExpand(true);
}

void FStateTreeReferenceOverridesDetails::CustomizeChildren(const TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& InChildrenBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	check(OverrideItemsHandle);
	
	const TSharedRef<FDetailArrayBuilder> NestedTreeOverridesBuilder = MakeShareable(new FDetailArrayBuilder(OverrideItemsHandle.ToSharedRef(), /*InGenerateHeader*/false, /*InDisplayResetToDefault*/ true, /*InDisplayElementNum*/ false));
	NestedTreeOverridesBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateLambda(
		[](TSharedRef<IPropertyHandle> PropertyHandle, int32 ArrayIndex, IDetailChildrenBuilder& ChildrenBuilder)
		{

			TSharedPtr<IPropertyHandle> StateTagHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeReferenceOverrideItem, StateTag));
			check(StateTagHandle);
			TSharedPtr<IPropertyHandle> StateTreeReferenceHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeReferenceOverrideItem, StateTreeReference));
			check(StateTreeReferenceHandle);

			IDetailPropertyRow& PropertyRow = ChildrenBuilder.AddProperty(StateTreeReferenceHandle.ToSharedRef());

			TSharedPtr<SWidget> NameWidget;
			TSharedPtr<SWidget> ValueWidget;
			PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, /*bAddWidgetDecoration*/true);

			PropertyRow.CustomWidget(/*bShowChildren*/true)
			.NameContent()
			[
				StateTagHandle->CreatePropertyValueWidgetWithCustomization(nullptr)
			]
			.ValueContent()
			[
				ValueWidget.ToSharedRef()
			];
		}));
	InChildrenBuilder.AddCustomBuilder(NestedTreeOverridesBuilder);}

#undef LOCTEXT_NAMESPACE
