// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class IPropertyHandle;
class IPropertyUtilities;

/**
 * Type customization for FNestedStateTreeReference.
 */
class FStateTreeReferenceOverridesDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& InChildrenBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;

private:
	TSharedPtr<IPropertyHandle> StructPropertyHandle;
	TSharedPtr<IPropertyHandle> OverrideItemsHandle;
	
	TSharedPtr<IPropertyUtilities> PropUtils;
};
