// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeStateDetails.h"
#include "DetailLayoutBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "IPropertyUtilities.h"
#include "PropertyBagDetails.h"
#include "PropertyRestriction.h"
#include "StateTree.h"
#include "StateTreeEditingSubsystem.h"
#include "StateTreeEditor.h"
#include "StateTreeEditorData.h"
#include "StateTreeEditorNodeUtils.h"
#include "StateTreeEditorSchema.h"
#include "StateTreeEditorStyle.h"
#include "StateTreeEditorDataExtension.h"
#include "StateTreeEditorUserSettings.h"
#include "StateTreePropertyHelpers.h"
#include "StateTreeSchema.h"
#include "StateTreeStateParametersDetails.h"
#include "StructUtilsMetadata.h"
#include "Debugger/StateTreeDebuggerUIExtensions.h"
#include "SWarningOrErrorBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"


TSharedRef<IDetailCustomization> FStateTreeStateDetails::MakeInstance()
{
	return MakeShareable(new FStateTreeStateDetails);
}

void FStateTreeStateDetails::PendingDelete()
{
	Super::PendingDelete();
	bIsPendingDelete = true;
}

void FStateTreeStateDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	const TSharedRef<IPropertyUtilities> PropUtils = DetailBuilder.GetPropertyUtilities();
	WeakPropertyUtilities = PropUtils.ToWeakPtr();

	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);

	// Find StateTreeEditorData associated with this panel.
	UStateTreeEditorData* EditorData = nullptr;
	for (TWeakObjectPtr<UObject>& WeakObject : Objects)
	{
		if (UObject* Object = WeakObject.Get())
		{
			if (UStateTreeEditorData* OuterEditorData = Object->GetTypedOuter<UStateTreeEditorData>())
			{
				EditorData = OuterEditorData;
				break;
			}
		}
	}

	// Find StateTreeState associated with this panel.
	UStateTreeState* State = nullptr;
	for (TWeakObjectPtr<UObject>& WeakObject : Objects)
	{
		if (UObject* Object = WeakObject.Get())
		{
			if (UStateTreeState* StateTreeState = Cast<UStateTreeState>(Object))
			{
				State = StateTreeState;
				break;
			}
		}
	}
	WeakState = State;

	const UStateTreeSchema* Schema = EditorData ? EditorData->Schema.Get() : nullptr;
	const UStateTreeEditorSchema* EditorSchema = EditorData ? EditorData->EditorSchema.Get() : nullptr;
	const FString SchemaPath = Schema ? Schema->GetClass()->GetPathName() : FString();
	WeakEditorData = EditorData;

	TSharedPtr<FStateTreeViewModel> ViewModel;
	if (UStateTreeEditingSubsystem* StateTreeEditingSubsystem = EditorData != nullptr ? GEditor->GetEditorSubsystem<UStateTreeEditingSubsystem>() : nullptr)
	{
		ViewModel = StateTreeEditingSubsystem->FindOrAddViewModel(EditorData->GetOuterUStateTree());
	}

	const TSharedPtr<IPropertyHandle> IDProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeState, ID));
	const TSharedPtr<IPropertyHandle> NameProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeState, Name));
	const TSharedPtr<IPropertyHandle> TagProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeState, Tag));
	const TSharedPtr<IPropertyHandle> ColorRefProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeState, ColorRef));
	const TSharedPtr<IPropertyHandle> DescriptionProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeState, Description));
	const TSharedPtr<IPropertyHandle> EnabledProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeState, bEnabled));
	const TSharedPtr<IPropertyHandle> TasksProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeState, Tasks));
	const TSharedPtr<IPropertyHandle> SingleTaskProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeState, SingleTask));
	const TSharedPtr<IPropertyHandle> EnterConditionsProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeState, EnterConditions));
	const TSharedPtr<IPropertyHandle> ConsiderationsProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeState, Considerations));
	const TSharedPtr<IPropertyHandle> TransitionsProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeState, Transitions));
	const TSharedPtr<IPropertyHandle> TypeProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeState, Type));
	const TSharedPtr<IPropertyHandle> LinkedSubtreeProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeState, LinkedSubtree));
	const TSharedPtr<IPropertyHandle> LinkedAssetProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeState, LinkedAsset));
	const TSharedPtr<IPropertyHandle> CustomTickRateProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeState, CustomTickRate));
	const TSharedPtr<IPropertyHandle> ParametersProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeState, Parameters));
	const TSharedPtr<IPropertyHandle> SelectionBehaviorProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeState, SelectionBehavior));
	const TSharedPtr<IPropertyHandle> TasksCompletionProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeState, TasksCompletion));
	const TSharedPtr<IPropertyHandle> RequiredEventToEnterProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeState, RequiredEventToEnter));
	const TSharedPtr<IPropertyHandle> CheckPrerequisitesProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeState, bCheckPrerequisitesWhenActivatingChildDirectly));
	const TSharedPtr<IPropertyHandle> WeightProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeState, Weight));
	const TSharedPtr<IPropertyHandle> CopyParameterBindingsOnTickProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeState, bCopyParameterBindingsOnTick));

	// Never show enabled
	EnabledProperty->MarkHiddenByCustomization();

	// Show ID only for debugging
	if (UE::StateTree::Editor::GbDisplayItemIds == false)
	{
		IDProperty->MarkHiddenByCustomization();
	}

	uint8 StateTypeValue = 0;
	TypeProperty->GetValue(StateTypeValue);
	const EStateTreeStateType StateType = (EStateTreeStateType)StateTypeValue;

	IDetailCategoryBuilder& StateCategory = DetailBuilder.EditCategory(TEXT("State"), LOCTEXT("StateDetailsState", "State"));
	StateCategory.SetSortOrder(0);
	{
		TSharedRef<SHorizontalBox> HeaderContent = SNew(SHorizontalBox)
			.IsEnabled(PropUtils, &IPropertyUtilities::IsPropertyEditingEnabled)
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(SHorizontalBox)

				// Debugger labels
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					UE::StateTreeEditor::DebuggerExtensions::CreateStateWidget(EnabledProperty, ViewModel)
				]
				// Options
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SComboButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.OnGetMenuContent_Lambda([EnabledProperty, ViewModel]()
					{
						FMenuBuilder MenuBuilder(/*ShouldCloseWindowAfterMenuSelection*/true, /*CommandList*/nullptr);
						// Append debugger items.
						UE::StateTreeEditor::DebuggerExtensions::AppendStateMenuItems(MenuBuilder, EnabledProperty, ViewModel);
						return MenuBuilder.MakeWidget();
					})
					.ToolTipText(LOCTEXT("ItemActions", "Item actions"))
					.HasDownArrow(false)
					.ContentPadding(FMargin(4.f, 2.f))
					.ButtonContent()
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Icons.ChevronDown"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			];
		StateCategory.HeaderContent(HeaderContent);
	}

	// Name
	NameProperty->MarkHiddenByCustomization();
	StateCategory.AddProperty(NameProperty);

	// Tag
	TagProperty->MarkHiddenByCustomization();
	StateCategory.AddProperty(TagProperty);

	// Color
	ColorRefProperty->MarkHiddenByCustomization();
	StateCategory.AddProperty(ColorRefProperty);

	// Custom Tick Rate
	CustomTickRateProperty->MarkHiddenByCustomization();
	if (Schema && Schema->IsScheduledTickAllowed())
	{
		StateCategory.AddProperty(CustomTickRateProperty);
	}

	// Description
	DescriptionProperty->MarkHiddenByCustomization();
	StateCategory.AddProperty(DescriptionProperty);

	// Type
	TypeProperty->MarkHiddenByCustomization();
	if (Schema)
	{
		// Restrict IsStateTypeAllowed
		TSharedRef<FPropertyRestriction> SchemaSupported = MakeShared<FPropertyRestriction>(LOCTEXT("StateTypeAllowedRestriction", "The schema restricts the state type."));
		const UEnum* EnumPtr = StaticEnum<EStateTreeStateType>();
		for (int32 Index = 0; Index < EnumPtr->NumEnums(); ++Index)
		{
			const EStateTreeStateType EnumValue = static_cast<EStateTreeStateType>(EnumPtr->GetValueByIndex(Index));
			if (!Schema->IsStateTypeAllowed(EnumValue))
			{
				SchemaSupported->AddHiddenValue(EnumPtr->GetNameStringByIndex(Index));
			}
		}
		TypeProperty->AddRestriction(SchemaSupported);
	}
	StateCategory.AddProperty(TypeProperty);

	// Per state type properties
	SelectionBehaviorProperty->MarkHiddenByCustomization();
	TasksCompletionProperty->MarkHiddenByCustomization();
	LinkedSubtreeProperty->MarkHiddenByCustomization();
	LinkedAssetProperty->MarkHiddenByCustomization();

	if (StateType == EStateTreeStateType::State || StateType == EStateTreeStateType::Subtree)
	{
		// Restrict IsStateSelectionAllowed
		if (Schema)
		{
			TSharedRef<FPropertyRestriction> SchemaSupported = MakeShared<FPropertyRestriction>(LOCTEXT("StateSelectionAllowedRestriction", "The schema restricts the selection behavior."));
			const UEnum* EnumPtr = StaticEnum<EStateTreeStateSelectionBehavior>();
			for (int32 Index = 0; Index < EnumPtr->NumEnums(); ++Index)
			{
				const EStateTreeStateSelectionBehavior EnumValue = static_cast<EStateTreeStateSelectionBehavior>(EnumPtr->GetValueByIndex(Index));
				if (!Schema->IsStateSelectionAllowed(EnumValue))
				{
					SchemaSupported->AddHiddenValue(EnumPtr->GetNameStringByIndex(Index));
				}
			}
			SelectionBehaviorProperty->AddRestriction(SchemaSupported);
		}
		StateCategory.AddProperty(SelectionBehaviorProperty);
	}
	else if (StateType == EStateTreeStateType::Linked)
	{
		StateCategory.AddProperty(LinkedSubtreeProperty);
	}
	else if (StateType == EStateTreeStateType::LinkedAsset)
	{
		if (!OverrideFactoryForLinkedAsset)
		{
			OverrideFactoryForLinkedAsset = TStrongObjectPtr<UStateTreeFactory>(NewObject<UStateTreeFactory>());
			OverrideFactoryForLinkedAsset->SetSchemaClass(Schema->GetClass());
		}

		// Custom widget for the linked asset, to add filtering to the assets.
		IDetailPropertyRow& Row = StateCategory.AddProperty(LinkedAssetProperty);
		Row.CustomWidget()
		.NameContent()
		[
			LinkedAssetProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SObjectPropertyEntryBox)
			.PropertyHandle(LinkedAssetProperty)
			.AllowedClass(UStateTree::StaticClass())
			.AllowCreate(true)
			.NewAssetFactories(TArray<UFactory*>({ OverrideFactoryForLinkedAsset.Get() }))
			.ThumbnailPool(DetailBuilder.GetPropertyUtilities()->GetThumbnailPool())
			.OnShouldFilterAsset_Lambda([SchemaPath](const FAssetData& InAssetData)
			{
				return !SchemaPath.IsEmpty() && !InAssetData.TagsAndValues.ContainsKeyValue(UE::StateTree::SchemaTag, SchemaPath);
			})
		];

		const FButtonStyle* const CloseButtonStyle = &FAppStyle::Get().GetWidgetStyle<FDockTabStyle>("Docking.Tab").CloseButtonStyle;
		StateCategory.AddCustomRow(LOCTEXT("WarningBindingWithLinkedAsset", "WarningBindingWithLinkedAsset"))
			.Visibility(MakeAttributeSP(this, &FStateTreeStateDetails::HandleDisableLinkedAssetOverrideVisibility))
			.WholeRowContent()
			[
				SNew(SWarningOrErrorBox)
					.MessageStyle(EMessageStyle::Warning)
					.IconSize(FVector2D(16.0f, 16.0f))
					.Padding(FMargin(0.0f))
					.Message(LOCTEXT("WarningBindingWithLinkedAssetMess", "Override will be disabled because of the binding(s)."))
			];
	}

	// Parameters category
	int32 NumParameters = WeakState.IsValid() ? WeakState->Parameters.Parameters.GetNumPropertiesInBag() : 0;
	const FText ParametersDisplayName = EnumHasAllFlags(GetDefault<UStateTreeEditorUserSettings>()->GetDetailsViewDisplayCount(), EStateTreeEditorUserSettingsNodeType::Parameter)
		? FText::Format(LOCTEXT("EditorStateDetailsParametersCounted", "Parameters ({0})"), NumParameters)
		: LOCTEXT("EditorStateDetailsParameters", "Parameters");
	IDetailCategoryBuilder& ParametersCategory = DetailBuilder.EditCategory(TEXT("Parameters"), ParametersDisplayName);
	ParametersCategory.SetSortOrder(1);
	ParametersCategory.SetColor(UE::StateTree::Colors::Blue);
	{
		// Show parameters as a category.
		ParametersProperty->MarkHiddenByCustomization();

		TSharedPtr<IPropertyHandle> ParametersParametersProperty = ParametersProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeStateParameters, Parameters)); // FInstancedPropertyBag
		check(ParametersParametersProperty);
		TSharedPtr<IPropertyHandle> ParametersFixedLayoutProperty = ParametersProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeStateParameters, bFixedLayout));
		check(ParametersFixedLayoutProperty);
		TSharedPtr<IPropertyHandle> ParametersIDProperty = ParametersProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeStateParameters, ID));
		check(ParametersIDProperty);

		bool bFixedLayout = false;
		ParametersFixedLayoutProperty->GetValue(bFixedLayout);

		const TSharedRef<SHorizontalBox> HeaderContentWidget = SNew(SHorizontalBox)
			.IsEnabled(PropUtils, &IPropertyUtilities::IsPropertyEditingEnabled);

		HeaderContentWidget->AddSlot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(FMargin(4.f, 0.f, 0.f, 0.f))
			[
				SNew(SImage)
					.ColorAndOpacity(UE::StateTree::Colors::Blue)
					.Image(FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Parameters"))
			];
		HeaderContentWidget->AddSlot()
			.VAlign(VAlign_Center)
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			.AutoWidth()
			[
				SNew(STextBlock)
				.TextStyle(FStateTreeEditorStyle::Get(), "StateTree.Category")
				.Text(ParametersDisplayName)
			];

		if (!bFixedLayout)
		{
			if (EditorSchema)
			{
				TOptional<FString> SchemaMetaData = EditorSchema->GetParametersSchemaClass();
				if (SchemaMetaData.IsSet())
				{
					using namespace UE::StructUtils::Metadata;
					ParametersProperty->SetInstanceMetaData(PropertyBagSchemaClassName, SchemaMetaData.GetValue());
				}
			}

			HeaderContentWidget->AddSlot()
				.FillWidth(1.f)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					FPropertyBagDetails::MakeAddPropertyWidget(
						ParametersParametersProperty,
						PropUtils,
						EPropertyBagPropertyType::Bool,
						FLinearColor(UE::StateTree::Colors::Blue)).ToSharedRef()
				];
		}

		ParametersCategory.HeaderContent(HeaderContentWidget, /*FullRowContent*/true);

		FGuid ID;
		UE::StateTree::PropertyHelpers::GetStructValue<FGuid>(ParametersIDProperty, ID);

		// Show the Value (FInstancedStruct) as child rows.
		TSharedRef<FStateTreeStateParametersInstanceDataDetails> InstanceDetails =
			MakeShareable(new FStateTreeStateParametersInstanceDataDetails(ParametersProperty, ParametersProperty, PropUtils, bFixedLayout, ID, WeakEditorData, WeakState));
		ParametersCategory.AddCustomBuilder(InstanceDetails);

		// Tick Parameter Bindings option
		CopyParameterBindingsOnTickProperty->MarkHiddenByCustomization();
		ParametersCategory.AddProperty(CopyParameterBindingsOnTickProperty);
	}

	// Enter conditions
	const FName EnterConditionsCategoryName(TEXT("Enter Conditions"));
	if (Schema && Schema->AllowEnterConditions())
	{
		int32 NumEnterConditions = WeakState.IsValid() ? WeakState->EnterConditions.Num() : 0;
		IDetailCategoryBuilder& EnterConditionsCategory = UE::StateTreeEditor::EditorNodeUtils::MakeArrayCategory(
			DetailBuilder,
			EnterConditionsProperty,
			EnterConditionsCategoryName,
			EnumHasAllFlags(GetDefault<UStateTreeEditorUserSettings>()->GetDetailsViewDisplayCount(), EStateTreeEditorUserSettingsNodeType::Condition)
				? FText::Format(LOCTEXT("StateDetailsEnterConditionsCounted", "Enter Conditions ({0})"), NumEnterConditions)
				: LOCTEXT("StateDetailsEnterConditions", "Enter Conditions"),
			FName("StateTreeEditor.Conditions"),
			UE::StateTree::Colors::Yellow,
			UE::StateTree::Colors::Yellow.WithAlpha(192),
			LOCTEXT("EnterConditionsAddTooltip", "Add new Enter Condition"),
			/*SortOrder*/2);
		EnterConditionsProperty->MarkHiddenByCustomization();

		// Event
		RequiredEventToEnterProperty->MarkHiddenByCustomization();
		EnterConditionsCategory.AddProperty(RequiredEventToEnterProperty);

		// Check Prerequisites
		CheckPrerequisitesProperty->MarkHiddenByCustomization();
		EnterConditionsCategory.AddProperty(CheckPrerequisitesProperty);
	}
	else
	{
		DetailBuilder.EditCategory(EnterConditionsCategoryName).SetCategoryVisibility(false);
	}

	// Utility
	WeightProperty->MarkHiddenByCustomization();
	ConsiderationsProperty->MarkHiddenByCustomization();
	if (Schema && Schema->AllowUtilityConsiderations())
	{
		const bool bDisplayUtilityConsideration = State
			&& State->Parent
			&& UE::StateTree::IsSelectionBehaviorUsingUtility(State->Parent->SelectionBehavior);
		if (bDisplayUtilityConsideration)
		{
			const FName UtilityCategoryName(TEXT("Selection Utility"));
			int32 NumConsiderations = WeakState.IsValid() ? WeakState->Considerations.Num() : 0;
			FText UtilityCategoryText = EnumHasAllFlags(GetDefault<UStateTreeEditorUserSettings>()->GetDetailsViewDisplayCount(), EStateTreeEditorUserSettingsNodeType::Utility)
				? FText::Format(LOCTEXT("StateDetailsSelectionUtilityCounted", "Selection Utility ({0})"), NumConsiderations)
				: LOCTEXT("StateDetailsSelectionUtility", "Selection Utility");

			IDetailCategoryBuilder& UtilityConsiderationsCategory = UE::StateTreeEditor::EditorNodeUtils::MakeArrayCategory(
				DetailBuilder,
				ConsiderationsProperty,
				UtilityCategoryName,
				UtilityCategoryText,
				FName("StateTreeEditor.Utility"),
				UE::StateTree::Colors::Orange,
				UE::StateTree::Colors::Orange.WithAlpha(192),
				LOCTEXT("UtilityAddTooltip", "Add new Utility Consideration"),
				/*SortOrder*/3);

			// Weight
			UtilityConsiderationsCategory.AddProperty(WeightProperty);
		}
	}

	// Tasks
	if ((StateType == EStateTreeStateType::State || StateType == EStateTreeStateType::Subtree))
	{
		if (Schema && Schema->AllowMultipleTasks())
		{
			int32 NumTasks = WeakState.IsValid() ? WeakState->Tasks.Num() : 0;
			const bool bAllowTasksCompletion = Schema->AllowTasksCompletion();
			const FName TasksCategoryName(TEXT("Tasks"));
			IDetailCategoryBuilder& TasksCategory = UE::StateTreeEditor::EditorNodeUtils::MakeArrayCategoryHeader(
				DetailBuilder,
				TasksProperty,
				TasksCategoryName,
				EnumHasAllFlags(GetDefault<UStateTreeEditorUserSettings>()->GetDetailsViewDisplayCount(), EStateTreeEditorUserSettingsNodeType::Task)
					? FText::Format(LOCTEXT("StateDetailsTasksCounted", "Tasks ({0})"), NumTasks)
					: LOCTEXT("StateDetailsTasks", "Tasks"),
				FName("StateTreeEditor.Tasks"),
				UE::StateTree::Colors::Cyan,
				bAllowTasksCompletion ? TasksCompletionProperty->CreatePropertyValueWidget(/*bDisplayDefaultPropertyButtons*/false) : TSharedPtr<SWidget>(),
				UE::StateTree::Colors::Cyan.WithAlpha(192),
				LOCTEXT("StateDetailsTasksAddTooltip", "Add new Task"),
				/*SortOrder*/4);
			SingleTaskProperty->MarkHiddenByCustomization();
			UE::StateTreeEditor::EditorNodeUtils::MakeArrayItems(TasksCategory, TasksProperty);
		}
		else
		{
			const FName TaskCategoryName(TEXT("Task"));
			IDetailCategoryBuilder& Category = DetailBuilder.EditCategory(TaskCategoryName);
			Category.SetSortOrder(4);

			IDetailPropertyRow& Row = Category.AddProperty(SingleTaskProperty);
			Row.ShouldAutoExpand(true);

			TasksProperty->MarkHiddenByCustomization();
		}
	}
	else
	{
		SingleTaskProperty->MarkHiddenByCustomization();
		TasksProperty->MarkHiddenByCustomization();
	}

	// Transitions
	int32 NumTransitions = WeakState.IsValid() ? WeakState->Transitions.Num() : 0;
	UE::StateTreeEditor::EditorNodeUtils::MakeArrayCategory(
		DetailBuilder,
		TransitionsProperty,
		"Transitions",
		EnumHasAllFlags(GetDefault<UStateTreeEditorUserSettings>()->GetDetailsViewDisplayCount(), EStateTreeEditorUserSettingsNodeType::Transition)
			? FText::Format(LOCTEXT("StateDetailsTransitionsCounted", "Transitions ({0})"), NumTransitions)
			: LOCTEXT("StateDetailsTransitions", "Transitions"),
		FName("StateTreeEditor.Transitions"),
		UE::StateTree::Colors::Magenta,
		UE::StateTree::Colors::Magenta.WithAlpha(192),
		LOCTEXT("StateDetailsTransitionsAddTooltip", "Add new Transition"),
		/*SortOrder*/5);

	// Refresh the UI when the type changes.
	TypeProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([WeakPropertyUtilities = WeakPropertyUtilities]
		{
			if (const TSharedPtr<IPropertyUtilities> PropertyUtilities = WeakPropertyUtilities.Pin())
			{
				PropertyUtilities->ForceRefresh();
			}
		}));

	if (EditorData)
	{
		for (UStateTreeEditorDataExtension* Extension : EditorData->Extensions)
		{
			if (Extension)
			{
				Extension->CustomizeDetails(WeakState.Get(), DetailBuilder);
			}
		}
	}
}

EVisibility FStateTreeStateDetails::HandleDisableLinkedAssetOverrideVisibility() const
{
	const UStateTreeEditorData* EditorData = WeakEditorData.Get();
	const UStateTreeState* State = WeakState.Get();

	if (EditorData
		&& State
		&& State->Type == EStateTreeStateType::LinkedAsset
		&& State->Tag.IsValid())
	{
		const EStateTreeVisitor VisitResult = EditorData->VisitStateBindings(State, [](TNotNull<const UStateTreeState*> State, const FStateTreePropertyPathBinding& Binding, const FStateTreeBindableStructDesc& BindingTargetNodeDesc, const FStateTreeDataView BindingTargetNodeValue)
			{
				if (Binding.GetTargetPath().GetStructID() == State->Parameters.ID || Binding.GetSourcePath().GetStructID() == State->Parameters.ID)
				{
					return EStateTreeVisitor::Break;
				}

				return EStateTreeVisitor::Continue;
			});

		return VisitResult == EStateTreeVisitor::Break ? EVisibility::Visible : EVisibility::Collapsed;
	}

	return EVisibility::Collapsed;
}

void FStateTreeStateDetails::PostUndo(bool bSuccess)
{
	// Prevent redundant calls to force refresh which doubles the number of DetailCustomization 
	if (bIsPendingDelete)
	{
		return;
	}

	// Refresh view on undo or redo so that the customization based on e.g. State type will be reflected correctly.
	if (const TSharedPtr<IPropertyUtilities> PropertyUtilities = WeakPropertyUtilities.Pin())
	{
		PropertyUtilities->ForceRefresh();
	}
}

void FStateTreeStateDetails::PostRedo(bool bSuccess)
{
	// Prevent redundant calls to force refresh which doubles the number of DetailCustomization
	if (bIsPendingDelete)
	{
		return;
	}

	// Refresh view on undo or redo so that the customization based on e.g. State type will be reflected correctly.
	if (const TSharedPtr<IPropertyUtilities> PropertyUtilities = WeakPropertyUtilities.Pin())
	{
		PropertyUtilities->ForceRefresh();
	}
}

#undef LOCTEXT_NAMESPACE
