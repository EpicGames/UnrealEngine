// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "EditorUndoClient.h"
#include "StateTreeEditorData.h"
#include "StateTreeFactory.h"
#include "StateTreeState.h"

class IDetailLayoutBuilder;
class IDetailChildrenBuilder;
class IDetailCategoryBuilder;
class IPropertyHandle;
class IPropertyUtilities;
class SWidget;
class SComboButton;
struct EVisibility;

class FStateTreeStateDetails : public IDetailCustomization, FSelfRegisteringEditorUndoClient
{
private:
	using Super = IDetailCustomization;

public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	//~ IDetailCustomization interface
	virtual void PendingDelete() override;
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
protected:

	//~ FEditorUndoClient
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;

private:
	EVisibility HandleDisableLinkedAssetOverrideVisibility() const;

protected:
	TWeakPtr<IPropertyUtilities> WeakPropertyUtilities;
	TWeakObjectPtr<UStateTreeEditorData> WeakEditorData;
	TWeakObjectPtr<UStateTreeState> WeakState;

	TStrongObjectPtr<UStateTreeFactory> OverrideFactoryForLinkedAsset;

	bool bIsPendingDelete = false;
};
