// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeStateLinkDetails.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "StateTree.h"
#include "StateTreeEditorData.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Images/SImage.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"
#include "StateTreeDescriptionHelpers.h"
#include "StateTreeEditorStyle.h"
#include "StateTreePropertyHelpers.h"
#include "TextStyleDecorator.h"
#include "Widgets/SCompactTreeEditorView.h"
#include "Widgets/Text/SRichTextBlock.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"

TSharedRef<IPropertyTypeCustomization> FStateTreeStateLinkDetails::MakeInstance()
{
	return MakeShareable(new FStateTreeStateLinkDetails);
}

void FStateTreeStateLinkDetails::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructProperty = StructPropertyHandle;
	PropUtils = StructCustomizationUtils.GetPropertyUtilities().Get();

	NameProperty = StructProperty->GetChildHandle(TEXT("Name"));
	IDProperty = StructProperty->GetChildHandle(TEXT("ID"));
	LinkTypeProperty = StructProperty->GetChildHandle(TEXT("LinkType"));

	if (const FProperty* MetaDataProperty = StructProperty->GetMetaDataProperty())
	{
		static const FName NAME_DirectStatesOnly = "DirectStatesOnly";
		static const FName NAME_SubtreesOnly = "SubtreesOnly";
		
		bDirectStatesOnly = MetaDataProperty->HasMetaData(NAME_DirectStatesOnly);
		bSubtreesOnly = MetaDataProperty->HasMetaData(NAME_SubtreesOnly);
	}

	// Store pointer to editor data.
	TArray<UObject*> OuterObjects;
	StructProperty->GetOuterObjects(OuterObjects);
	for (const UObject* Object : OuterObjects)
	{
		if (Object)
		{
			if (const UStateTree* OuterStateTree = Object->GetTypedOuter<UStateTree>())
			{
				WeakEditorData = Cast<UStateTreeEditorData>(OuterStateTree->EditorData);
				if (WeakEditorData.IsValid())
				{
					break;
				}
			}
		}
	}
	
	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.VAlign(VAlign_Center)
		[
			SAssignNew(ComboButton, SComboButton)
			.OnGetMenuContent(this, &FStateTreeStateLinkDetails::GenerateStatePicker)
			.ButtonContent()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0,0,4,0)
				[
					SNew(SImage)
					.ToolTipText(LOCTEXT("MissingState", "The specified state cannot be found."))
					.Visibility_Lambda([this]()
					{
						return IsValidLink() ? EVisibility::Collapsed : EVisibility::Visible;
					})
					.Image(FAppStyle::GetBrush("Icons.ErrorWithColor"))
				]

				+SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(0, 2.0f, 4.0f, 2.0f)
				.AutoWidth()
				[
					SNew(SImage)
					.DesiredSizeOverride(FVector2D(16.0f, 16.0f))
					.Image(this, &FStateTreeStateLinkDetails::GetCurrentStateIcon)
					.ColorAndOpacity(this, &FStateTreeStateLinkDetails::GetCurrentStateColor)
				]
				
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SRichTextBlock)
					.Text(this, &FStateTreeStateLinkDetails::GetCurrentStateDesc)
					.TextStyle(&FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Details.Normal"))
					.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
					+SRichTextBlock::Decorator(FTextStyleDecorator::Create(TEXT(""), FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Details.Normal")))
					+SRichTextBlock::Decorator(FTextStyleDecorator::Create(TEXT("b"), FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Details.Bold")))
					+SRichTextBlock::Decorator(FTextStyleDecorator::Create(TEXT("i"), FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Details.Italic")))
					+SRichTextBlock::Decorator(FTextStyleDecorator::Create(TEXT("s"), FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Details.Subdued")))
				]
			]
		];
}

void FStateTreeStateLinkDetails::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

TSharedRef<SWidget> FStateTreeStateLinkDetails::GenerateStatePicker()
{
	check(ComboButton);
	
	constexpr bool bCloseMenuAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseMenuAfterSelection, nullptr);

	if (!bDirectStatesOnly)
	{
		auto MakeMetaStateWidget = [WeakEditorData = WeakEditorData](EStateTreeTransitionType Type)
		{
			const FStateTreeStateLink Link(Type);
			
			return SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(0, 2.0f, 4.0f, 2.0f)
				.AutoWidth()
				[
					SNew(SImage)
					.DesiredSizeOverride(FVector2D(16.0f, 16.0f))
					.Image(UE::StateTree::Editor::GetStateLinkIcon(WeakEditorData.Get(), Link))
					.ColorAndOpacity(UE::StateTree::Editor::GetStateLinkColor(WeakEditorData.Get(), Link))
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SRichTextBlock)
					.Text(UE::StateTree::Editor::GetStateLinkDesc(WeakEditorData.Get(), Link, EStateTreeNodeFormatting::RichText))
					.TextStyle(&FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Normal.Normal"))
					.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
					+SRichTextBlock::Decorator(FTextStyleDecorator::Create(TEXT(""), FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Normal.Normal")))
					+SRichTextBlock::Decorator(FTextStyleDecorator::Create(TEXT("b"), FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Normal.Bold")))
					+SRichTextBlock::Decorator(FTextStyleDecorator::Create(TEXT("i"), FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Normal.Italic")))
					+SRichTextBlock::Decorator(FTextStyleDecorator::Create(TEXT("s"), FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Normal.Subdued")))
				];
		};

		auto AddMenuEntry = [this, &MakeMetaStateWidget, &MenuBuilder](EStateTreeTransitionType Type, FText Tooltip)
		{
			MenuBuilder.AddMenuEntry(
				FUIAction(
					FExecuteAction::CreateSP(this, &FStateTreeStateLinkDetails::SetTransitionByType, Type),
					FCanExecuteAction(),
					FIsActionChecked::CreateSPLambda(this, [this, Type]
					{
						return GetTransitionType() == Type;
					})),
				MakeMetaStateWidget(Type),
				FName(),
				Tooltip,
				EUserInterfaceActionType::Check);
		};

		AddMenuEntry(EStateTreeTransitionType::None, LOCTEXT("TransitionNoneTooltip", "No transition."));
		AddMenuEntry(EStateTreeTransitionType::Parent, LOCTEXT("TransitionParentTooltip", "Goto the parent state."));
		AddMenuEntry(EStateTreeTransitionType::NextState, LOCTEXT("TransitionNextTooltip", "Goto next sibling State."));
		AddMenuEntry(EStateTreeTransitionType::NextSelectableState, LOCTEXT("TransitionNextSelectableTooltip", "Goto next sibling state, whose enter conditions pass."));
		AddMenuEntry(EStateTreeTransitionType::NextParent, LOCTEXT("TransitionNextParentTooltip", "Goto next sibling of the parent State."));
		AddMenuEntry(EStateTreeTransitionType::NextSelectableParent, LOCTEXT("TransitionNextParentSelectableTooltip", "Goto next sibling of the parent state, whose enter conditions pass."));
		AddMenuEntry(EStateTreeTransitionType::Succeeded, LOCTEXT("TransitionTreeSuccessTooltip", "Complete tree with success."));
		AddMenuEntry(EStateTreeTransitionType::Failed, LOCTEXT("TransitionTreeFailedTooltip", "Complete tree with failure."));
	}

	MenuBuilder.BeginSection("States", LOCTEXT("States", "States"));

	TSharedPtr<UE::StateTree::Editor::SCompactTreeEditorView> StateView;
	
	TSharedRef<SWidget> MenuWidget = 
		SNew(SBox)
		.MinDesiredWidth(300.f)
		.MaxDesiredHeight(400.f)
		.Padding(2)	
		[
			SAssignNew(StateView, UE::StateTree::Editor::SCompactTreeEditorView)
			.StateTreeEditorData(WeakEditorData.Get())
			.SelectionMode(ESelectionMode::Single)
			.SelectableStatesOnly(true)
			.SubtreesOnly(bSubtreesOnly)
			.OnSelectionChanged(this, &FStateTreeStateLinkDetails::OnStateSelected)
		];

	check(StateView);
	
	const EStateTreeTransitionType TransitionType = GetTransitionType().Get(EStateTreeTransitionType::Failed);
	if (TransitionType == EStateTreeTransitionType::GotoState)
	{
		if (const UStateTreeState* State = GetState())
		{
			StateView->SetSelection({ State->ID });
		}
	}

	ComboButton->SetMenuContentWidgetToFocus(StateView->GetWidgetToFocusOnOpen());

	MenuBuilder.AddWidget(MenuWidget, FText::GetEmpty(), /*bInNoIndent*/true);
	
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void FStateTreeStateLinkDetails::OnStateSelected(TConstArrayView<FGuid> SelectedStateIDs)
{
	const UStateTreeState* State = nullptr;
	if (!SelectedStateIDs.IsEmpty())
	{
		if (const UStateTreeEditorData* EditorData = WeakEditorData.Get())
		{
			State = EditorData->GetStateByID(SelectedStateIDs[0]);
		}
	}

	if (State
		&& NameProperty
		&& IDProperty)
	{
		FScopedTransaction Transaction(FText::Format(LOCTEXT("SetPropertyValue", "Set {0}"), StructProperty->GetPropertyDisplayName()));

		LinkTypeProperty->SetValue((uint8)EStateTreeTransitionType::GotoState);

		NameProperty->SetValue(State->Name, EPropertyValueSetFlags::NotTransactable);
		UE::StateTree::PropertyHelpers::SetStructValue<FGuid>(IDProperty, State->ID, EPropertyValueSetFlags::NotTransactable);
	}

	check(ComboButton);
	ComboButton->SetIsOpen(false);
}

void FStateTreeStateLinkDetails::SetTransitionByType(const EStateTreeTransitionType TransitionType)
{
	if (NameProperty
		&& IDProperty)
	{
		FScopedTransaction Transaction(FText::Format(LOCTEXT("SetPropertyValue", "Set {0}"), StructProperty->GetPropertyDisplayName()));

		LinkTypeProperty->SetValue((uint8)TransitionType);

		// Clear name and id.
		NameProperty->SetValue(FName(), EPropertyValueSetFlags::NotTransactable);
		UE::StateTree::PropertyHelpers::SetStructValue<FGuid>(IDProperty, FGuid(), EPropertyValueSetFlags::NotTransactable);
	}

	check(ComboButton);
	ComboButton->SetIsOpen(false);
}

const UStateTreeState* FStateTreeStateLinkDetails::GetState() const
{
	if (const UStateTreeEditorData* EditorData = WeakEditorData.Get())
	{
		FGuid StateID;
		if (UE::StateTree::PropertyHelpers::GetStructValue<FGuid>(IDProperty, StateID) == FPropertyAccess::Success)
		{
			return EditorData->GetStateByID(StateID);
		}
	}
	return nullptr;
}

FText FStateTreeStateLinkDetails::GetCurrentStateDesc() const
{
	if (const FStateTreeStateLink* Link = UE::StateTree::PropertyHelpers::GetStructPtr<FStateTreeStateLink>(StructProperty))
	{
		return UE::StateTree::Editor::GetStateLinkDesc(WeakEditorData.Get(), *Link, EStateTreeNodeFormatting::RichText);
	}
	return LOCTEXT("MultipleSelected", "Multiple Selected");
}

const FSlateBrush* FStateTreeStateLinkDetails::GetCurrentStateIcon() const
{
	if (const FStateTreeStateLink* Link = UE::StateTree::PropertyHelpers::GetStructPtr<FStateTreeStateLink>(StructProperty))
	{
		return UE::StateTree::Editor::GetStateLinkIcon(WeakEditorData.Get(), *Link);
	}
	return nullptr;
}

FSlateColor FStateTreeStateLinkDetails::GetCurrentStateColor() const
{
	if (const FStateTreeStateLink* Link = UE::StateTree::PropertyHelpers::GetStructPtr<FStateTreeStateLink>(StructProperty))
	{
		return UE::StateTree::Editor::GetStateLinkColor(WeakEditorData.Get(), *Link);
	}
	return FSlateColor::UseForeground();
}

bool FStateTreeStateLinkDetails::IsValidLink() const
{
	const EStateTreeTransitionType TransitionType = GetTransitionType().Get(EStateTreeTransitionType::Failed);

	if (TransitionType == EStateTreeTransitionType::GotoState)
	{
		return GetState() != nullptr;
	}

	return true;
}

TOptional<EStateTreeTransitionType> FStateTreeStateLinkDetails::GetTransitionType() const
{
	if (LinkTypeProperty)
	{
		uint8 Value;
		if (LinkTypeProperty->GetValue(Value) == FPropertyAccess::Success)
		{
			return EStateTreeTransitionType(Value);
		}
	}
	return TOptional<EStateTreeTransitionType>();
}

#undef LOCTEXT_NAMESPACE
