// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "StateTreeTypes.h"
#include "UObject/ObjectKey.h"
#include "Widgets/Views/STreeView.h"

enum class EStateTreeTransitionType : uint8;
template <typename OptionalType> struct TOptional;

class IPropertyHandle;
class SWidget;
class SSearchBox;
class SComboButton;
class UStateTreeEditorData;
class UStateTreeState;

/**
 * Type customization for FStateTreeStateLink.
 */

class FStateTreeStateLinkDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:
	TSharedRef<SWidget> GenerateStatePicker();
	FText GetCurrentStateDesc() const;
	const FSlateBrush* GetCurrentStateIcon() const;
	FSlateColor GetCurrentStateColor() const;
	bool IsValidLink() const;
	TOptional<EStateTreeTransitionType> GetTransitionType() const;
	void OnStateSelected(TConstArrayView<FGuid> SelectedStateIDs);
	void SetTransitionByType(const EStateTreeTransitionType TransitionType);
	const UStateTreeState* GetState() const;
	
	TSharedPtr<IPropertyHandle> NameProperty;
	TSharedPtr<IPropertyHandle> IDProperty;
	TSharedPtr<IPropertyHandle> LinkTypeProperty;

	TSharedPtr<SComboButton> ComboButton;

	TWeakObjectPtr<const UStateTreeEditorData> WeakEditorData = nullptr;

	// If set, hide selecting meta states like Next or (tree) Succeeded.
	bool bDirectStatesOnly = false;
	// If set, allow to select only states marked as subtrees.
	bool bSubtreesOnly = false;
	
	class IPropertyUtilities* PropUtils = nullptr;
	TSharedPtr<IPropertyHandle> StructProperty;
};
