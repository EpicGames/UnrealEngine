// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeStateParametersDetails.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Layout/SBox.h"
#include "PropertyBagDetails.h"
#include "StateTree.h"
#include "StateTreeEditorData.h"
#include "StateTreePropertyHelpers.h"
#include "StateTreeBindingExtension.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"


//----------------------------------------------------------------//
// FStateTreeStateParametersInstanceDataDetails
//----------------------------------------------------------------//
struct FStateTreeStateOverrideProvider : public FPropertyBagInstanceDataDetails::IPropertyBagOverrideProvider
{
	FStateTreeStateOverrideProvider(UStateTreeState& InState)
		: State(InState)
	{
	}
		
	virtual bool IsPropertyOverridden(const FGuid PropertyID) const override
	{
		return State.IsParametersPropertyOverridden(PropertyID);
	}
		
	virtual void SetPropertyOverride(const FGuid PropertyID, const bool bIsOverridden) const override
	{
		State.SetParametersPropertyOverridden(PropertyID, bIsOverridden);
	}

private:
	UStateTreeState& State;
};

//----------------------------------------------------------------//
// FStateTreeStateParametersInstanceDataDetails
//----------------------------------------------------------------//

FStateTreeStateParametersInstanceDataDetails::FStateTreeStateParametersInstanceDataDetails(
	const TSharedPtr<IPropertyHandle>& InStructProperty,
	const TSharedPtr<IPropertyHandle>& InParametersStructProperty,
	const TSharedPtr<IPropertyUtilities>& InPropUtils,
	const bool bInFixedLayout,
	FGuid InID,
	TWeakObjectPtr<UStateTreeEditorData> InEditorData,
	TWeakObjectPtr<UStateTreeState> InState)
	: FPropertyBagInstanceDataDetails(InParametersStructProperty, InPropUtils, bInFixedLayout)
	, StructProperty(InStructProperty)
	, WeakEditorData(InEditorData)
	, WeakState(InState)
	, ID(InID)
{
}
	
void FStateTreeStateParametersInstanceDataDetails::OnChildRowAdded(IDetailPropertyRow& ChildRow)
{
	FPropertyBagInstanceDataDetails::OnChildRowAdded(ChildRow);

	EStateTreeStateType Type = EStateTreeStateType::State;
	if (const UStateTreeState* State = WeakState.Get())
	{
		Type = State->Type;
	}

	// Subtree parameters cannot be bound to, they are provided from the linked state.
	const bool bAllowBinding = Type != EStateTreeStateType::Subtree && ID.IsValid(); 

	if (bAllowBinding)
	{
		const TSharedPtr<IPropertyHandle> ChildPropHandle = ChildRow.GetPropertyHandle();
		const FProperty* Property = ChildPropHandle->GetProperty();

		// Set the category to Parameter so that the binding extension will pick it up.
		static const FName CategoryName(TEXT("Category"));
		ChildPropHandle->SetInstanceMetaData(CategoryName, TEXT("Parameter"));
	
		// Conditionally control visibility of the value field of bound properties.

		// Pass the node ID to binding extension. Since the properties are added using AddChildStructure(), we break the hierarchy and cannot access parent.
		ChildPropHandle->SetInstanceMetaData(UE::PropertyBinding::MetaDataStructIDName, LexToString(ID));

		FPropertyBindingPath Path(ID, *Property->GetFName().ToString());
		
		const auto IsValueVisible = TAttribute<EVisibility>::Create([Path, WeakEditorData = WeakEditorData]() -> EVisibility
			{
				bool bHasBinding = false;
				if (UStateTreeEditorData* EditorData = WeakEditorData.Get())
				{
					if (const FStateTreeEditorPropertyBindings* EditorPropBindings = EditorData->GetPropertyEditorBindings())
					{
						bHasBinding = EditorPropBindings->HasBinding(Path); 
					}
				}

				return bHasBinding ? EVisibility::Collapsed : EVisibility::Visible;
			});

		FDetailWidgetDecl* ValueWidgetDecl = ChildRow.CustomValueWidget();
		const TSharedRef<SBox> WrappedValueWidget = SNew(SBox)
			.Visibility(IsValueVisible)
			[
				ValueWidgetDecl->Widget
			];
		ValueWidgetDecl->Widget = WrappedValueWidget;
	}
}

bool FStateTreeStateParametersInstanceDataDetails::HasPropertyOverrides() const
{
	if (const UStateTreeState* State = WeakState.Get())
	{
		return State->Type == EStateTreeStateType::Linked || State->Type == EStateTreeStateType::LinkedAsset;
	}
	return false;
}

void FStateTreeStateParametersInstanceDataDetails::PreChangeOverrides()
{
	check(StructProperty);
	StructProperty->NotifyPreChange();
}

void FStateTreeStateParametersInstanceDataDetails::PostChangeOverrides()
{
	check(StructProperty);
	StructProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
	StructProperty->NotifyFinishedChangingProperties();
}

void FStateTreeStateParametersInstanceDataDetails::EnumeratePropertyBags(TSharedPtr<IPropertyHandle> PropertyBagHandle, const EnumeratePropertyBagFuncRef& Func) const
{
	if (UStateTreeState* State = WeakState.Get())
	{
		if (const FInstancedPropertyBag* DefaultParameters = State->GetDefaultParameters())
		{
			FInstancedPropertyBag& Parameters = State->Parameters.Parameters;
			FStateTreeStateOverrideProvider OverrideProvider(*State);
			Func(*DefaultParameters, Parameters, OverrideProvider);
		}
	}
}


//----------------------------------------------------------------//
// FStateTreeStateParametersDetails
//----------------------------------------------------------------//

TSharedRef<IPropertyTypeCustomization> FStateTreeStateParametersDetails::MakeInstance()
{
	return MakeShareable(new FStateTreeStateParametersDetails);
}

void FStateTreeStateParametersDetails::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructProperty = StructPropertyHandle;
	PropUtils = StructCustomizationUtils.GetPropertyUtilities();

	ParametersProperty = StructProperty->GetChildHandle(TEXT("Parameters"));
	FixedLayoutProperty = StructProperty->GetChildHandle(TEXT("bFixedLayout"));
	IDProperty = StructProperty->GetChildHandle(TEXT("ID"));
	check(ParametersProperty.IsValid());
	check(FixedLayoutProperty.IsValid());
	check(IDProperty.IsValid());

	FindOuterObjects();
	
	bFixedLayout = false;
	FixedLayoutProperty->GetValue(bFixedLayout);

	TSharedPtr<SWidget> ValueWidget = SNullWidget::NullWidget;
	if (!bFixedLayout)
	{
		ValueWidget = FPropertyBagDetails::MakeAddPropertyWidget(ParametersProperty, PropUtils);
	}
	
	HeaderRow
		.NameContent()
		[
			ParametersProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			ValueWidget.ToSharedRef()
		]
		.ShouldAutoExpand(true);
}

void FStateTreeStateParametersDetails::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	FGuid ID;
	UE::StateTree::PropertyHelpers::GetStructValue<FGuid>(IDProperty, ID);

	// Show the Value (FInstancedStruct) as child rows.
	TSharedRef<FStateTreeStateParametersInstanceDataDetails> InstanceDetails = MakeShareable(new FStateTreeStateParametersInstanceDataDetails(StructProperty, ParametersProperty, PropUtils, bFixedLayout, ID, WeakEditorData, WeakState));
	StructBuilder.AddCustomBuilder(InstanceDetails);
}

void FStateTreeStateParametersDetails::FindOuterObjects()
{
	check(StructProperty);
	
	WeakEditorData = nullptr;
	WeakStateTree = nullptr;
	WeakState = nullptr;

	TArray<UObject*> OuterObjects;
	StructProperty->GetOuterObjects(OuterObjects);
	for (UObject* Outer : OuterObjects)
	{
		UStateTreeState* OuterState = Cast<UStateTreeState>(Outer);
		UStateTreeEditorData* OuterEditorData = Outer->GetTypedOuter<UStateTreeEditorData>();
		UStateTree* OuterStateTree = OuterEditorData ? OuterEditorData->GetTypedOuter<UStateTree>() : nullptr;
		if (OuterEditorData && OuterStateTree && OuterState)
		{
			WeakStateTree = OuterStateTree;
			WeakEditorData = OuterEditorData;
			WeakState = OuterState;
			break;
		}
	}
}


#undef LOCTEXT_NAMESPACE
