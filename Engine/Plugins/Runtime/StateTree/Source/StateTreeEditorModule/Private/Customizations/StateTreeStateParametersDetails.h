// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "PropertyBagDetails.h"

class IDetailPropertyRow;
class IPropertyHandle;
class IPropertyUtilities;
class UStateTree;
class UStateTreeState;
class UStateTreeEditorData;

/**
* Type customization for FStateTreeStateParameters.
*/

class FStateTreeStateParametersDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:

	void FindOuterObjects();

	TSharedPtr<IPropertyUtilities> PropUtils;

	TSharedPtr<IPropertyHandle> ParametersProperty;
	TSharedPtr<IPropertyHandle> FixedLayoutProperty;
	TSharedPtr<IPropertyHandle> IDProperty;
	TSharedPtr<IPropertyHandle> StructProperty;

	bool bFixedLayout = false;

	TWeakObjectPtr<UStateTreeEditorData> WeakEditorData = nullptr;
	TWeakObjectPtr<UStateTree> WeakStateTree = nullptr;
	TWeakObjectPtr<UStateTreeState> WeakState = nullptr;
};


class FStateTreeStateParametersInstanceDataDetails : public FPropertyBagInstanceDataDetails
{
public:
	FStateTreeStateParametersInstanceDataDetails(
		const TSharedPtr<IPropertyHandle>& InStructProperty,
		const TSharedPtr<IPropertyHandle>& InParametersStructProperty,
		const TSharedPtr<IPropertyUtilities>& InPropUtils,
		const bool bInFixedLayout,
		FGuid InID,
		TWeakObjectPtr<UStateTreeEditorData> InEditorData,
		TWeakObjectPtr<UStateTreeState> InState);
	
	virtual void OnChildRowAdded(IDetailPropertyRow& ChildRow) override;

	virtual bool HasPropertyOverrides() const override;
	virtual void PreChangeOverrides() override;
	virtual void PostChangeOverrides() override;
	virtual void EnumeratePropertyBags(TSharedPtr<IPropertyHandle> PropertyBagHandle, const EnumeratePropertyBagFuncRef& Func) const override;

private:
	TSharedPtr<IPropertyHandle> StructProperty;
	TWeakObjectPtr<UStateTreeEditorData> WeakEditorData;
	TWeakObjectPtr<UStateTreeState> WeakState;
	FGuid ID;
};
