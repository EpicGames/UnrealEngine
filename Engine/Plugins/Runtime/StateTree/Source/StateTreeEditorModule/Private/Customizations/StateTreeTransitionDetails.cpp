// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeTransitionDetails.h"
#include "Debugger/StateTreeDebuggerUIExtensions.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "ScopedTransaction.h"
#include "StateTreeDescriptionHelpers.h"
#include "StateTreeEditingSubsystem.h"
#include "StateTreeEditor.h"
#include "StateTreeEditorData.h"
#include "StateTreeEditorNodeUtils.h"
#include "StateTreeEditorSchema.h"
#include "StateTreeEditorStyle.h"
#include "StateTreePropertyHelpers.h"
#include "StateTreeEditorDataClipboardHelpers.h"
#include "StateTreeTypes.h"
#include "TextStyleDecorator.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"

TSharedRef<IPropertyTypeCustomization> FStateTreeTransitionDetails::MakeInstance()
{
	return MakeShareable(new FStateTreeTransitionDetails);
}

void FStateTreeTransitionDetails::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructProperty = StructPropertyHandle;
	ParentProperty = StructProperty->GetParentHandle();
	ParentArrayProperty = ParentProperty->AsArray();

	PropUtils = StructCustomizationUtils.GetPropertyUtilities();

	// Find StateTreeEditorData associated with this panel.
	UStateTreeEditorData* EditorData = GetEditorData();
	if (UStateTreeEditingSubsystem* StateTreeEditingSubsystem = EditorData != nullptr ? GEditor->GetEditorSubsystem<UStateTreeEditingSubsystem>() : nullptr)
	{
		StateTreeViewModel = StateTreeEditingSubsystem->FindOrAddViewModel(EditorData->GetOuterUStateTree());
	}

	TriggerProperty = StructProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeTransition, Trigger));
	PriorityProperty = StructProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeTransition, Priority));
	RequiredEventProperty = StructProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeTransition, RequiredEvent));
	DelegateListener = StructProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeTransition, DelegateListener));
	StateProperty = StructProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeTransition, State));
	DelayTransitionProperty = StructProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeTransition, bDelayTransition));
	TransitionReactivateProperty = StructProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeTransition, ReactivateTargetState));
	DelayDurationProperty = StructProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeTransition, DelayDuration));
	DelayRandomVarianceProperty = StructProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeTransition, DelayRandomVariance));
	ConditionsProperty = StructProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeTransition, Conditions));
	IDProperty = StructProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeTransition, ID));

	HeaderRow
		.RowTag(StructProperty->GetProperty()->GetFName())
		.WholeRowContent()
		.VAlign(VAlign_Center)
		[
			// Border to capture mouse clicks on the row (used for right click menu).
			SAssignNew(RowBorder, SBorder)
			.BorderImage(FStyleDefaults::GetNoBrush())
			.Padding(0.f)
			.ForegroundColor(this, &FStateTreeTransitionDetails::GetContentRowColor)
			.OnMouseButtonDown(this, &FStateTreeTransitionDetails::OnRowMouseDown)
			.OnMouseButtonUp(this, &FStateTreeTransitionDetails::OnRowMouseUp)
			[

				SNew(SHorizontalBox)

				// Icon
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(0.f, 0.f, 4.f, 0.f))
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.Image(FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Transition.Goto"))
				]

				// Description
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(0.f, 1.f, 0.f, 0.f))
				[
					SNew(SRichTextBlock)
					.Text(this, &FStateTreeTransitionDetails::GetDescription)
					.TextStyle(&FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Details.Normal"))
					.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
					+SRichTextBlock::Decorator(FTextStyleDecorator::Create(TEXT(""), FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Details.Normal")))
					+SRichTextBlock::Decorator(FTextStyleDecorator::Create(TEXT("b"), FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Details.Bold")))
					+SRichTextBlock::Decorator(FTextStyleDecorator::Create(TEXT("i"), FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Details.Italic")))
					+SRichTextBlock::Decorator(FTextStyleDecorator::Create(TEXT("s"), FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Details.Subdued")))
				]
				// Debug and property widgets
				+ SHorizontalBox::Slot()
				.FillContentWidth(1.0f, 0.0f) // grow, no shrinking
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.Padding(FMargin(8.f, 0.f, 2.f, 0.f))
				[
					SNew(SHorizontalBox)
					// Debugger labels
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						UE::StateTreeEditor::DebuggerExtensions::CreateTransitionWidget(StructPropertyHandle, StateTreeViewModel)
					]

					// Options
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SComboButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.OnGetMenuContent(this, &FStateTreeTransitionDetails::GenerateOptionsMenu)
						.ToolTipText(LOCTEXT("ItemActions", "Item actions"))
						.HasDownArrow(false)
						.ContentPadding(FMargin(4.f, 2.f))
						.ButtonContent()
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("Icons.ChevronDown"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]
				]
			]
		]
		.CopyAction(FUIAction(FExecuteAction::CreateSP(this, &FStateTreeTransitionDetails::OnCopyTransition)))
		.PasteAction(FUIAction(FExecuteAction::CreateSP(this, &FStateTreeTransitionDetails::OnPasteTransitions)));
}

EStateTreeVisitor VisitNestedHierachyIncludingLinks_Internal(const TMap<UStateTreeState*, TArray<UStateTreeState*>>& LinkedByStateMap
	, TSet<UStateTreeState*>& VisitedStates
	, UStateTreeState& NestedStateLeaf
	, TFunctionRef<EStateTreeVisitor(UStateTreeState& State)> InFunc)
{
	bool bContinue = InFunc(NestedStateLeaf) == EStateTreeVisitor::Continue;
	VisitedStates.Add(&NestedStateLeaf);
	if (bContinue)
	{
		const TArray<UStateTreeState*>* LinkedByStates = LinkedByStateMap.Find(&NestedStateLeaf);
		UStateTreeState* ParentState = NestedStateLeaf.Parent;

		if (LinkedByStates)
		{
			for (UStateTreeState* LinkedByState : *LinkedByStates)
			{
				if (!VisitedStates.Contains(LinkedByState) && ensure(LinkedByState))
				{
					bContinue = VisitNestedHierachyIncludingLinks_Internal(LinkedByStateMap, VisitedStates, *LinkedByState, InFunc) == EStateTreeVisitor::Continue;
					if (!bContinue)
					{
						return EStateTreeVisitor::Break;
					}
				}

			}
		}
		if (!VisitedStates.Contains(ParentState) && ParentState)
		{
			bContinue = VisitNestedHierachyIncludingLinks_Internal(LinkedByStateMap, VisitedStates, *ParentState, InFunc) == EStateTreeVisitor::Continue;
			if (!bContinue)
			{
				return EStateTreeVisitor::Break;
			}
		}

		return EStateTreeVisitor::Continue;
	}

	return EStateTreeVisitor::Break;
}

void VisitNestedHierachyIncludingLinks(UStateTreeEditorData* EditorData, UStateTreeState& NestedStateLeaf, TFunctionRef<EStateTreeVisitor(UStateTreeState& State)> InFunc)
{
	// We need to visit our nested state, considering links. Suppose we have:
	// 
	// Root
	// - a -> A
	// - b -> B
	// - C -> C
	// - A -> X
	// - B -> Y
	// - C -> X
	// X 
	// - I
	// -- P -> Root (Infinite loop, this won't compile, but we are editor time so still need consider)
	// -- Q
	// Y 
	// - J
	// 
	// First we construct a LinkedBy list
	// A: {a}
	// B: {b}
	// C: {c} 
	// X: {A, C}
	// Y: {B}
	// Root: {P}
	// 
	// Then we visit Nested & Linked State Parents. 
	// Algo:
	// - Visit Current, add to Visited
	// - For each node in union (LinkedBy[Current], Parent):
	// -- If not in Visited, recurse Algo as new current.
	
	// Note: Ptr's not const since we don't require the callback to be const
	TMap<UStateTreeState*, TArray<UStateTreeState*>> LinkedByStateMap;
	EditorData->VisitHierarchy([&LinkedByStateMap, EditorData](UStateTreeState& State, UStateTreeState* ParentState)
	{
		if (State.Type == EStateTreeStateType::Linked)
		{
			UStateTreeState* LinkedState = EditorData->GetMutableStateByID(State.LinkedSubtree.ID);
			if (ensure(LinkedState))
			{
				LinkedByStateMap.FindOrAdd(LinkedState).Add(&State);
			}
		}
		return EStateTreeVisitor::Continue;
	});

	TSet<UStateTreeState*> VisitedStates;
	VisitNestedHierachyIncludingLinks_Internal(LinkedByStateMap, VisitedStates, NestedStateLeaf, InFunc);
}

void FStateTreeTransitionDetails::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	check(TriggerProperty);
	check(RequiredEventProperty);
	check(DelegateListener);
	check(DelayTransitionProperty);
	check(TransitionReactivateProperty);
	check(DelayDurationProperty);
	check(DelayRandomVarianceProperty);
	check(StateProperty);
	check(ConditionsProperty);
	check(IDProperty);

	UStateTreeState* OwningState = GetOwningStateTreeState();
	TWeakObjectPtr<UStateTreeState> WeakOwningState = OwningState;
	UStateTreeEditorData* EditorData = GetEditorData();
	TWeakObjectPtr<UStateTreeEditorData> WeakEditorData = EditorData;

	TWeakPtr<FStateTreeTransitionDetails> WeakSelf = SharedThis(this);
	auto IsNotCompletionTransition = [WeakSelf]()
	{
		if (const TSharedPtr<FStateTreeTransitionDetails> Self = WeakSelf.Pin())
		{
			return !EnumHasAnyFlags(Self->GetTrigger(), EStateTreeTransitionTrigger::OnStateCompleted) ? EVisibility::Visible : EVisibility::Collapsed;
		}
		return EVisibility::Collapsed;
	};
	auto IsInNestedState = [WeakSelf, WeakOwningState, WeakEditorData]()
	{
		const TSharedPtr<FStateTreeTransitionDetails> Self = WeakSelf.Pin();
		TStrongObjectPtr<UStateTreeState> OwningState = WeakOwningState.Pin();
		TStrongObjectPtr<UStateTreeEditorData> EditorData = WeakEditorData.Pin();

		if (!Self || !OwningState || !EditorData)
		{
			return false;
		}

		// Try to return cached value if it's correct
		FStateTreeStateLink TargetStateLink = Self->GetTargetState();
		if (Self->CachedLastTargetState.ID == TargetStateLink.ID)
		{
			return Self->bCachedIsTargetStateInNestedState;
		}
		else
		{
			Self->CachedLastTargetState = TargetStateLink;
		}

		UStateTree* StateTree = EditorData->GetOuterUStateTree();
		UStateTreeState* CurrentState = OwningState.Get();
		FStateTreeStateLink CurrentStateLink = CurrentState->GetLinkToState();

		bool bIsTargetStateInNestedState = false;
		auto IsTargetState = [TargetStateLink, &bIsTargetStateInNestedState](UStateTreeState& InCurrentState)
		{
			if (TargetStateLink.ID != InCurrentState.GetLinkToState().ID)
			{
				return EStateTreeVisitor::Continue;
			}
			else
			{
				bIsTargetStateInNestedState = true;
				return EStateTreeVisitor::Break;
			}
		};
		VisitNestedHierachyIncludingLinks(EditorData.Get(), *CurrentState, IsTargetState);

		Self->bCachedIsTargetStateInNestedState = bIsTargetStateInNestedState;
		return bIsTargetStateInNestedState;
	};

	if (UE::StateTree::Editor::GbDisplayItemIds)
	{
		StructBuilder.AddProperty(IDProperty.ToSharedRef());
	}

	// Trigger
	StructBuilder.AddProperty(TriggerProperty.ToSharedRef());

	// Show event only when the trigger is set to Event.
	StructBuilder.AddProperty(RequiredEventProperty.ToSharedRef())
		.Visibility(MakeAttributeLambda([WeakSelf]()
		{
			if (const TSharedPtr<FStateTreeTransitionDetails> Self = WeakSelf.Pin())
			{
				return (Self->GetTrigger() == EStateTreeTransitionTrigger::OnEvent) ? EVisibility::Visible : EVisibility::Collapsed;
			}
			return EVisibility::Collapsed;
		}));

	IDetailPropertyRow& DelegateDispatcherRow = StructBuilder.AddProperty(DelegateListener.ToSharedRef())
		.Visibility(MakeAttributeLambda([WeakSelf]()
		{
			if (const TSharedPtr<FStateTreeTransitionDetails> Self = WeakSelf.Pin())
			{
				return (Self->GetTrigger() == EStateTreeTransitionTrigger::OnDelegate) ? EVisibility::Visible : EVisibility::Collapsed;
			}
			return EVisibility::Collapsed;
		}));

	FGuid ID;
	UE::StateTree::PropertyHelpers::GetStructValue<FGuid>(IDProperty, ID);
	DelegateListener->SetInstanceMetaData(UE::PropertyBinding::MetaDataStructIDName, LexToString(ID));

	// State
	StructBuilder.AddProperty(StateProperty.ToSharedRef());

	// Priority
	StructBuilder.AddProperty(PriorityProperty.ToSharedRef())
		.Visibility(MakeAttributeLambda(IsNotCompletionTransition));

	// Delay
	StructBuilder.AddProperty(DelayTransitionProperty.ToSharedRef())
		.Visibility(MakeAttributeLambda(IsNotCompletionTransition));
	StructBuilder.AddProperty(DelayDurationProperty.ToSharedRef())
		.Visibility(MakeAttributeLambda(IsNotCompletionTransition));
	StructBuilder.AddProperty(DelayRandomVarianceProperty.ToSharedRef())
		.Visibility(MakeAttributeLambda(IsNotCompletionTransition));

	// Transition Reactivation
	auto IsTransitionReactiviationSchemaEditable = [WeakEditorData]()
	{
		bool bAllowed = false;
		if (TStrongObjectPtr<UStateTreeEditorData> EditorData = WeakEditorData.Pin())
		{
			UStateTree* StateTree = EditorData->GetOuterUStateTree();
			bAllowed = EnumHasAnyFlags(EditorData->EditorSchema->GetTransitionEditingRules(EditorData.Get()), EStateTreeTransitionEditingRules::AllowReactivation);
		}

		return bAllowed ? EVisibility::Visible : EVisibility::Collapsed;
	};
	StructBuilder.AddProperty(TransitionReactivateProperty.ToSharedRef())
		.EditCondition(MakeAttributeLambda(IsInNestedState), FOnBooleanValueChanged())
		.Visibility(MakeAttributeLambda(IsTransitionReactiviationSchemaEditable));

	// Show conditions always expanded, with simplified header (remove item count)
	IDetailPropertyRow& ConditionsRow = StructBuilder.AddProperty(ConditionsProperty.ToSharedRef());
	ConditionsRow.ShouldAutoExpand(true);

	constexpr bool bShowChildren = true;
	ConditionsRow.CustomWidget(bShowChildren)
		.RowTag(ConditionsProperty->GetProperty()->GetFName())
		.WholeRowContent()
		[
			SNew(SHorizontalBox)

			// Condition text
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(ConditionsProperty->GetPropertyDisplayName())
				.Font(IDetailLayoutBuilder::GetDetailFontBold())
			]

			// Conditions button
			+SHorizontalBox::Slot()
			.FillWidth(1.f)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.Padding(FMargin(0, 0, 3, 0))
				[
					UE::StateTreeEditor::EditorNodeUtils::CreateAddNodePickerComboButton(
						LOCTEXT("TransitionConditionAddTooltip", "Add new Transition Condition"),
						UE::StateTree::Colors::Grey,
						ConditionsProperty,
						PropUtils.ToSharedRef())
				]
			]
		];
}

FSlateColor FStateTreeTransitionDetails::GetContentRowColor() const
{
	return UE::StateTreeEditor::DebuggerExtensions::IsTransitionEnabled(StructProperty)
		? FSlateColor::UseForeground()
		: FSlateColor::UseSubduedForeground();
}

FReply FStateTreeTransitionDetails::OnRowMouseDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply FStateTreeTransitionDetails::OnRowMouseUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
		FSlateApplication::Get().PushMenu(
			RowBorder.ToSharedRef(),
			WidgetPath,
			GenerateOptionsMenu(),
			MouseEvent.GetScreenSpacePosition(),
			FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

TSharedRef<SWidget> FStateTreeTransitionDetails::GenerateOptionsMenu()
{
	FMenuBuilder MenuBuilder(/*ShouldCloseWindowAfterMenuSelection*/true, /*CommandList*/nullptr);

	MenuBuilder.BeginSection(FName("Edit"), LOCTEXT("Edit", "Edit"));

	// Copy
	MenuBuilder.AddMenuEntry(
		LOCTEXT("CopyItem", "Copy"),
		LOCTEXT("CopyItemTooltip", "Copy this item"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FStateTreeTransitionDetails::OnCopyTransition),
			FCanExecuteAction()
		));

	// Copy all
	MenuBuilder.AddMenuEntry(
		LOCTEXT("CopyAllItems", "Copy all"),
		LOCTEXT("CopyAllItemsTooltip", "Copy all items"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FStateTreeTransitionDetails::OnCopyAllTransitions),
			FCanExecuteAction()
		));

	// Paste
	MenuBuilder.AddMenuEntry(
		LOCTEXT("PasteItem", "Paste"),
		LOCTEXT("PasteItemTooltip", "Paste into this item"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Paste"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FStateTreeTransitionDetails::OnPasteTransitions),
			FCanExecuteAction()
		));

	// Duplicate
	MenuBuilder.AddMenuEntry(
		LOCTEXT("DuplicateItem", "Duplicate"),
		LOCTEXT("DuplicateItemTooltip", "Duplicate this item"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Duplicate"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FStateTreeTransitionDetails::OnDuplicateTransition),
			FCanExecuteAction()
		));

	// Delete
	MenuBuilder.AddMenuEntry(
		LOCTEXT("DeleteItem", "Delete"),
		LOCTEXT("DeleteItemTooltip", "Delete this item"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Delete"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FStateTreeTransitionDetails::OnDeleteTransition),
			FCanExecuteAction()
		));

	// Delete all
	MenuBuilder.AddMenuEntry(
		LOCTEXT("DeleteAllItems", "Delete all"),
		LOCTEXT("DeleteAllItemsTooltip", "Delete all items"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Delete"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FStateTreeTransitionDetails::OnDeleteAllTransitions),
			FCanExecuteAction()
		));

	MenuBuilder.EndSection();

	// Append debugger items.
	UE::StateTreeEditor::DebuggerExtensions::AppendTransitionMenuItems(MenuBuilder, StructProperty, StateTreeViewModel);

	return MenuBuilder.MakeWidget();
}

void FStateTreeTransitionDetails::OnDeleteTransition() const
{
	const int32 Index = StructProperty->GetArrayIndex();
	if (const TSharedPtr<IPropertyHandle> ParentHandle = StructProperty->GetParentHandle())
	{
		if (const TSharedPtr<IPropertyHandleArray> ArrayHandle = ParentHandle->AsArray())
		{
			if (UStateTreeEditorData* EditorData = GetEditorData())
			{
				FScopedTransaction Transaction(LOCTEXT("DeleteTransition", "Delete Transition"));
				EditorData->Modify();

				ArrayHandle->DeleteItem(Index);

				EditorData->RemoveInvalidBindings();
			}
		}
	}
}

void FStateTreeTransitionDetails::OnDeleteAllTransitions() const
{
	if (const TSharedPtr<IPropertyHandle> ParentHandle = StructProperty->GetParentHandle())
	{
		if (const TSharedPtr<IPropertyHandleArray> ArrayHandle = ParentHandle->AsArray())
		{
			if (UStateTreeEditorData* EditorData = GetEditorData())
			{
				FScopedTransaction Transaction(LOCTEXT("DeleteAllTransitions", "Delete All Transitions"));
				EditorData->Modify();

				ArrayHandle->EmptyArray();

				EditorData->RemoveInvalidBindings();
			}
		}
	}
}

void FStateTreeTransitionDetails::OnDuplicateTransition() const
{
	TArray<UObject*> OuterObjects;
	StructProperty->GetOuterObjects(OuterObjects);
	if (OuterObjects.IsEmpty())
	{
		return;
	}

	if (OuterObjects.Num() != 1)
	{
		// Array Handle Manipulation doesn't support multiple selected objects
		FNotificationInfo NotificationInfo(FText::GetEmpty());
		NotificationInfo.Text = LOCTEXT("NotSupportedByMultipleObjects", "Operation is not supported for multi-selected objects");
		NotificationInfo.ExpireDuration = 5.0f;
		FSlateNotificationManager::Get().AddNotification(NotificationInfo);

		return;
	}

	if (ParentArrayProperty)
	{
		if (UStateTreeEditorData* EditorDataPtr = GetEditorData())
		{
			void* TransitionPtr = nullptr;
			if (StructProperty->GetValueData(TransitionPtr) == FPropertyAccess::Success)
			{
				UE::StateTreeEditor::FClipboardEditorData Clipboard;
				Clipboard.Append(EditorDataPtr, TConstArrayView<FStateTreeTransition>(static_cast<FStateTreeTransition*>(TransitionPtr), 1));
				Clipboard.ProcessBuffer(nullptr, EditorDataPtr, OuterObjects[0]);

				if (!Clipboard.IsValid())
				{
					return;
				}

				FScopedTransaction Transaction(LOCTEXT("DuplicateTransition", "Duplicate Transition"));

				// Might modify the bindings data
				EditorDataPtr->Modify();

				const int32 Index = StructProperty->GetArrayIndex();
				ParentArrayProperty->Insert(Index);

				TSharedPtr<IPropertyHandle> InsertedElementHandle = ParentArrayProperty->GetElement(Index);
				void* InsertedTransitionPtr = nullptr;
				if (InsertedElementHandle->GetValueData(InsertedTransitionPtr) == FPropertyAccess::Success)
				{
					*static_cast<FStateTreeTransition*>(InsertedTransitionPtr) = MoveTemp(Clipboard.GetTransitionsInBuffer()[0]);

					for (FStateTreePropertyPathBinding& Binding : Clipboard.GetBindingsInBuffer())
					{
						EditorDataPtr->GetPropertyEditorBindings()->AddStateTreeBinding(MoveTemp(Binding));
					}
				}

				// We reinieitalized item nodes on ArrayProperty operations before the data is completely set up. Reinitialize.
				if (PropUtils)
				{
					PropUtils->ForceRefresh();
				}
			}
		}
	}
}

FStateTreeStateLink FStateTreeTransitionDetails::GetTargetState() const
{
	FStateTreeStateLink StateValue;
	FStateTreeStateLink* StateData;
	if (ensure(StateProperty.IsValid()))
	{
		// Note: This can fail. IE: If we are deleting the current transition
		if (StateProperty->GetValueData((void*&)StateData) == FPropertyAccess::Success)
		{
			StateValue = *StateData;
		}
	}
	return StateValue;
}

EStateTreeTransitionTrigger FStateTreeTransitionDetails::GetTrigger() const
{
	EStateTreeTransitionTrigger TriggerValue = EStateTreeTransitionTrigger::None;
	if (ensure(TriggerProperty.IsValid()))
	{
		TriggerProperty->GetValue((uint8&)TriggerValue);
	}
	return TriggerValue;
}

bool FStateTreeTransitionDetails::GetDelayTransition() const
{
	bool bDelayTransition = false;
	if (ensure(DelayTransitionProperty.IsValid()))
	{
		DelayTransitionProperty->GetValue(bDelayTransition);
	}
	return bDelayTransition;
}

FText FStateTreeTransitionDetails::GetDescription() const
{
	check(StateProperty);

	const FStateTreeTransition* Transition = UE::StateTree::PropertyHelpers::GetStructPtr<FStateTreeTransition>(StructProperty);
	if (!Transition)
	{
		return LOCTEXT("MultipleSelected", "Multiple Selected");
	}

	return UE::StateTree::Editor::GetTransitionDesc(GetEditorData(), *Transition, EStateTreeNodeFormatting::RichText);
}

void FStateTreeTransitionDetails::OnCopyTransition() const
{
	UStateTreeEditorData* EditorDataPtr = GetEditorData();
	if (!EditorDataPtr)
	{
		return;
	}

	void* TransitionAddress = nullptr;
	if (StructProperty->GetValueData(TransitionAddress) == FPropertyAccess::Success)
	{
		UE::StateTreeEditor::FClipboardEditorData Clipboard;
		Clipboard.Append(EditorDataPtr, TConstArrayView<FStateTreeTransition>(static_cast<FStateTreeTransition*>(TransitionAddress), 1));

		UE::StateTreeEditor::ExportTextAsClipboardEditorData(Clipboard);
	}
}

void FStateTreeTransitionDetails::OnCopyAllTransitions() const
{
	UStateTreeEditorData* EditorDataPtr = GetEditorData();
	if (!EditorDataPtr)
	{
		return;
	}

	if (ParentArrayProperty)
	{
		void* TransitionArrayAddress = nullptr;
		if (ParentProperty->GetValueData(TransitionArrayAddress) == FPropertyAccess::Success)
		{
			UE::StateTreeEditor::FClipboardEditorData Clipboard;
			Clipboard.Append(EditorDataPtr, *static_cast<TArray<FStateTreeTransition>*>(TransitionArrayAddress));

			UE::StateTreeEditor::ExportTextAsClipboardEditorData(Clipboard);
		}
	}
	else
	{
		OnCopyTransition();
	}
}

UStateTreeState* FStateTreeTransitionDetails::GetOwningStateTreeState() const
{
	TArray<UObject*> OuterObjects;
	StructProperty->GetOuterObjects(OuterObjects);
	for (UObject* Outer : OuterObjects)
	{
		if (UStateTreeState* OuterStateTreeState = Cast<UStateTreeState>(Outer))
		{
			return OuterStateTreeState;
		}
	}

	return nullptr;
}

UStateTreeEditorData* FStateTreeTransitionDetails::GetEditorData() const
{
	TArray<UObject*> OuterObjects;
	StructProperty->GetOuterObjects(OuterObjects);
	for (UObject* Outer : OuterObjects)
	{
		UStateTreeEditorData* OuterEditorData = Cast<UStateTreeEditorData>(Outer);
		if (OuterEditorData == nullptr)
		{
			OuterEditorData = Outer->GetTypedOuter<UStateTreeEditorData>();
		}
		if (OuterEditorData)
		{
			return OuterEditorData;
		}
	}
	return nullptr;
}

void FStateTreeTransitionDetails::OnPasteTransitions() const
{
	using namespace UE::StateTreeEditor;

	TArray<UObject*> OuterObjects;
	StructProperty->GetOuterObjects(OuterObjects);
	if (OuterObjects.IsEmpty())
	{
		return;
	}

	UStateTreeEditorData* EditorDataPtr = GetEditorData();
	if (!EditorDataPtr)
	{
		return;
	}

	// In case its multi selected, we need to have a unique copy for each Object
	TArray<FClipboardEditorData, TInlineAllocator<2>> ClipboardTransitions;
	ClipboardTransitions.AddDefaulted(OuterObjects.Num());

	for (int32 Idx = 0; Idx < OuterObjects.Num(); ++Idx)
	{
		const bool bSuccess = ImportTextAsClipboardEditorData
		(TBaseStructure<FStateTreeTransition>::Get(),
			EditorDataPtr,
			OuterObjects[Idx],
			ClipboardTransitions[Idx]);

		if (!bSuccess)
		{
			return;
		}
	}

	// make sure each Clipboard has the same number of nodes
	for (int32 Idx = 0; Idx < ClipboardTransitions.Num() - 1; ++Idx)
	{
		check(ClipboardTransitions[Idx].GetTransitionsInBuffer().Num() == ClipboardTransitions[Idx + 1].GetTransitionsInBuffer().Num());
	}

	int32 NumTransitionsInBuffer = ClipboardTransitions[0].GetTransitionsInBuffer().Num();

	if (NumTransitionsInBuffer == 0)
	{
		return;
	}

	if (!ParentArrayProperty.IsValid() && NumTransitionsInBuffer != 1)
	{
		// Transition is not in an array. we can't do multi-to-one paste
		return;
	}

	if (OuterObjects.Num() != 1 && NumTransitionsInBuffer != 1)
	{
		// if multiple selected objects, and we have more than one nodes to paste into
		// Array Handle doesn't support manipulation on multiple objects.
		FNotificationInfo NotificationInfo(FText::GetEmpty());
		NotificationInfo.Text = LOCTEXT("NotSupportedByMultipleObjects", "Operation is not supported for multi-selected objects");
		NotificationInfo.ExpireDuration = 5.0f;
		FSlateNotificationManager::Get().AddNotification(NotificationInfo);

		return;
	}

	{
		FScopedTransaction Transaction(LOCTEXT("PasteTransition", "Paste Transition"));

		EditorDataPtr->Modify();	// we might modify the bindings on Editor Data
		StructProperty->NotifyPreChange();

		if (ParentArrayProperty)
		{
			// Paste multi nodes into one node
			const int32 StructIndex = StructProperty->GetIndexInArray();
			check(StructIndex != INDEX_NONE);

			uint32 NumArrayElements = 0;
			ParentArrayProperty->GetNumElements(NumArrayElements);
			check(NumArrayElements > 0);	// since we already have at least one element to paste into

			// Insert or append uninitialized elements after the current node to match the number of nodes in the paste buffer and retain the order of elements 
			// The first node in the buffer goes into the current node
			const int32 IndexToInsert = StructIndex + 1;
			const int32 NumElementsToAddOrInsert = ClipboardTransitions[0].GetTransitionsInBuffer().Num() - 1;

			int32 Cnt = 0;
			if (IndexToInsert == NumArrayElements)
			{
				while (Cnt++ < NumElementsToAddOrInsert)
				{
					FPropertyHandleItemAddResult Result = ParentArrayProperty->AddItem();
					if (Result.GetAccessResult() != FPropertyAccess::Success)
					{
						return;
					}
				}
			}
			else
			{
				while (Cnt++ < NumElementsToAddOrInsert)
				{
					FPropertyAccess::Result Result = ParentArrayProperty->Insert(IndexToInsert);
					if (Result != FPropertyAccess::Success)
					{
						return;
					}
				}
			}

			TArray<void*> RawDatasArray;
			ParentProperty->AccessRawData(RawDatasArray);
			check(RawDatasArray.Num() == OuterObjects.Num());

			for (int32 ObjIdx = 0; ObjIdx < OuterObjects.Num(); ++ObjIdx)
			{
				if (TArray<FStateTreeTransition>* TransitionsPtr = static_cast<TArray<FStateTreeTransition>*>(RawDatasArray[ObjIdx]))
				{
					TArrayView<FStateTreeTransition> TransitionsClipboardBuffer = ClipboardTransitions[ObjIdx].GetTransitionsInBuffer();
					TArray<FStateTreeTransition>& TransitionsToPasteInto = *TransitionsPtr;

					for (int32 Idx = 0; Idx < TransitionsClipboardBuffer.Num(); ++Idx)
					{
						TransitionsToPasteInto[Idx + StructIndex] = MoveTemp(TransitionsClipboardBuffer[Idx]);
					}

					for (FStateTreePropertyPathBinding& Binding : ClipboardTransitions[ObjIdx].GetBindingsInBuffer())
					{
						EditorDataPtr->GetPropertyEditorBindings()->AddStateTreeBinding(MoveTemp(Binding));
					}
				}
			}
		}
		else
		{
			// Paste single node to a single Node
			TArray<void*> RawDatas;
			StructProperty->AccessRawData(RawDatas);
			check(RawDatas.Num() == OuterObjects.Num());

			for (int32 Idx = 0; Idx < RawDatas.Num(); ++Idx)
			{
				if (FStateTreeTransition* CurrentTransition = static_cast<FStateTreeTransition*>(RawDatas[Idx]))
				{
					*CurrentTransition = MoveTemp(ClipboardTransitions[Idx].GetTransitionsInBuffer()[0]);

					for (FStateTreePropertyPathBinding& Binding : ClipboardTransitions[Idx].GetBindingsInBuffer())
					{
						EditorDataPtr->GetPropertyEditorBindings()->AddStateTreeBinding(MoveTemp(Binding));
					}
				}
			}
		}

		StructProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
		StructProperty->NotifyFinishedChangingProperties();
	}

	if (PropUtils)
	{
		PropUtils->ForceRefresh();
	}
}

#undef LOCTEXT_NAMESPACE
