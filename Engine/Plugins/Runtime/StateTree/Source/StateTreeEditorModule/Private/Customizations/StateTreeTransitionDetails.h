// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Input/Reply.h"
#include "StateTreeTypes.h"

class FStateTreeViewModel;
class IPropertyHandle;
class IPropertyHandleArray;
class IPropertyUtilities;
class IDetailChildrenBuilder;
class FDetailWidgetRow;
class UStateTreeEditorData;
enum class EStateTreeTransitionTrigger : uint8;
class SBorder;
class SWidget;
class UStateTreeState;

/**
 * Type customization for FStateTreeTransition.
 */

class FStateTreeTransitionDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:

	UStateTreeState* GetOwningStateTreeState() const;
	UStateTreeEditorData* GetEditorData() const;

	FText GetDescription() const;

	FStateTreeStateLink GetTargetState() const;
	EStateTreeTransitionTrigger GetTrigger() const;
	bool GetDelayTransition() const;

	void OnCopyTransition() const;
	void OnCopyAllTransitions() const;

	void OnPasteTransitions() const;

	FSlateColor GetContentRowColor() const;

	FReply OnRowMouseDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	FReply OnRowMouseUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	TSharedRef<SWidget> GenerateOptionsMenu();
	void OnDeleteTransition() const;
	void OnDeleteAllTransitions() const;
	void OnDuplicateTransition() const;

	TSharedPtr<FStateTreeViewModel> StateTreeViewModel;
	TSharedPtr<IPropertyHandle> TriggerProperty;
	TSharedPtr<IPropertyHandle> PriorityProperty;
	TSharedPtr<IPropertyHandle> RequiredEventProperty;
	TSharedPtr<IPropertyHandle> DelegateListener;
	TSharedPtr<IPropertyHandle> StateProperty;
	TSharedPtr<IPropertyHandle> DelayTransitionProperty;
	TSharedPtr<IPropertyHandle> TransitionReactivateProperty;
	TSharedPtr<IPropertyHandle> DelayDurationProperty;
	TSharedPtr<IPropertyHandle> DelayRandomVarianceProperty;
	TSharedPtr<IPropertyHandle> ConditionsProperty;
	TSharedPtr<IPropertyHandle> IDProperty;

	TSharedPtr<SBorder> RowBorder;

	TSharedPtr<IPropertyUtilities> PropUtils;
	TSharedPtr<IPropertyHandle> StructProperty;

	TSharedPtr<IPropertyHandle> ParentProperty;
	TSharedPtr<IPropertyHandleArray> ParentArrayProperty;

	/** Used to avoid having to re-compute if target state within this state's nested state hierachy. */
	FStateTreeStateLink CachedLastTargetState = {};

	/** True if our last target state was in the nested state hierachy */
	bool bCachedIsTargetStateInNestedState = false;
};
