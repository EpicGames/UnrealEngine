// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/Widgets/SAddTransitionMenu.h"

#include "StateTreeEditorData.h"
#include "StateTreeState.h"
#include "StateTreeTypes.h"
#include "StateTreeViewModel.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"

#define LOCTEXT_NAMESPACE "SAddTransitionMenu"

namespace UE::StateTree::Editor
{

void SAddTransitionMenu::Construct(const FArguments& InArgs)
{
	WeakOwnerState  = InArgs._OwnerState;
	WeakViewModel   = InArgs._ViewModel;

	if (TSharedPtr<FStateTreeViewModel> ViewModel = WeakViewModel.Pin())
	{
		SCompactTreeEditorView::Construct(
			SCompactTreeEditorView::FArguments()
			.SelectionMode(ESelectionMode::Single)
			.StateTreeEditorData(ViewModel->GetStateTreeEditorData())
			.SelectableStatesOnly(true)
			.OnSelectionChanged(SCompactTreeView::FOnSelectionChanged::CreateSP(this, &SAddTransitionMenu::CreateStateTransitionWithDefaultTrigger))
		);
	}
}

TSharedRef<STableRow<TSharedPtr<SCompactTreeView::FStateItem>>> SAddTransitionMenu::GenerateStateItemRowInternal(
	TSharedPtr<FStateItem> Item,
	const TSharedRef<STableViewBase>& OwnerTable,
	TSharedRef<SHorizontalBox> Container)
{
	TSharedRef<STableRow<TSharedPtr<FStateItem>>> Row =
		SCompactTreeEditorView::GenerateStateItemRowInternal(Item, OwnerTable, Container);

	// Chevron image - visual indicator that a trigger submenu is available on hover.
	Container->AddSlot()
	.AutoWidth()
	.VAlign(VAlign_Center)
	.Padding(FMargin(4.f, 0.f, 2.f, 0.f))
	[
		SNew(SImage)
		.Image(FAppStyle::GetBrush("Icons.ChevronRight"))
		.ColorAndOpacity(FStyleColors::Foreground)
	];

	Row->SetOnMouseEnter(FNoReplyPointerEventHandler::CreateSP(this, &SAddTransitionMenu::HandleOnMouseEnterStateRow, Item->StateID));
	return Row;
}

void SAddTransitionMenu::AddTransition(TWeakPtr<FStateTreeViewModel> InViewModel
	, TWeakObjectPtr<UStateTreeState> InWeakOwnerState
	, TNotNull<UStateTreeState*> TargetState
	, EStateTreeTransitionTrigger Trigger)
{
	TSharedPtr<FStateTreeViewModel> ViewModel = InViewModel.Pin();
	UStateTreeState* OwnerState = InWeakOwnerState.Get();
	if (!ViewModel || !OwnerState)
	{
		return;
	}

	ViewModel->AddTransition(OwnerState, Trigger, EStateTreeTransitionType::GotoState, TargetState);
	FSlateApplication::Get().DismissAllMenus();
}

void SAddTransitionMenu::CreateStateTransitionWithDefaultTrigger(TConstArrayView<FGuid> SelectedStateIDs) const
{
	if (SelectedStateIDs.IsEmpty())
	{
		return;
	}

	if (TSharedPtr<FStateTreeViewModel> ViewModel = WeakViewModel.Pin())
	{
		if (UStateTreeState* State = ViewModel->GetMutableStateByID(SelectedStateIDs[0]))
		{
			const FStateTreeTransition DefaultTransition;
			SAddTransitionMenu::AddTransition(ViewModel, WeakOwnerState, State, DefaultTransition.Trigger);
		}
	}
}

void SAddTransitionMenu::HandleOnMouseEnterStateRow(const FGeometry& MyGeometry, const FPointerEvent&, const FGuid InStateID)
{
	if (!ensure(InStateID.IsValid()))
	{
		return;
	}

	// Build entries for the trigger-type flyout. No UEnum iteration since these are enum flags.
	static const TArray<EStateTreeTransitionTrigger> Triggers = {
		EStateTreeTransitionTrigger::OnStateCompleted,
		EStateTreeTransitionTrigger::OnStateSucceeded,
		EStateTreeTransitionTrigger::OnStateFailed,
		EStateTreeTransitionTrigger::OnTick,
		EStateTreeTransitionTrigger::OnEvent,
		EStateTreeTransitionTrigger::OnDelegate,
	};

	// Build the state - trigger action pair.
	FMenuBuilder TransitionTriggerSubMenuBuilder(/*bCloseAfterSelection=*/ true, nullptr);
	for (const EStateTreeTransitionTrigger Trigger : Triggers)
	{
		auto CreateStateTransitionWithTrigger = [InStateID
			, WeakViewModel = WeakViewModel
			, WeakOwnerState = WeakOwnerState
			, Trigger]()
		{
			if (TSharedPtr<FStateTreeViewModel> ViewModel = WeakViewModel.Pin())
			{
				if (UStateTreeState* State = ViewModel->GetMutableStateByID(InStateID))
				{
					SAddTransitionMenu::AddTransition(ViewModel, WeakOwnerState, State, Trigger);
				}
			}
		};

		TransitionTriggerSubMenuBuilder.AddMenuEntry(
			UEnum::GetDisplayValueAsText(Trigger),
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda(CreateStateTransitionWithTrigger)));
	}

	// Position the menu at the right edge of the picker, aligned with the hovered row.
	const FGeometry& PickerGeometry = GetCachedGeometry();
	const FVector2D MenuPosition(
		PickerGeometry.GetAbsolutePosition().X + PickerGeometry.GetAbsoluteSize().X,
		MyGeometry.GetAbsolutePosition().Y);

	TSharedPtr<IMenu> NewMenu = FSlateApplication::Get().PushMenu(
		AsShared(),
		FWidgetPath(),
		TransitionTriggerSubMenuBuilder.MakeWidget(),
		MenuPosition,
		FPopupTransitionEffect(FPopupTransitionEffect::SubMenu),
		/*bFocusImmediately=*/ false);
}

} // namespace UE::StateTree::Editor

#undef LOCTEXT_NAMESPACE
