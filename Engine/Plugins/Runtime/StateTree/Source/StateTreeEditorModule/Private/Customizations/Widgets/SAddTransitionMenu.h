// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Customizations/Widgets/SCompactTreeEditorView.h"
#include "StateTreeTypes.h"

class UStateTreeEditorData;
class UStateTreeState;
class FStateTreeViewModel;
class IMenu;

namespace UE::StateTree::Editor
{

/**
 * State picker popup for creating a transition to a selected state.
 * Extends SCompactTreeEditorView with a per-row trigger-type flyout:
 *
 * - Left-clicking a state row adds a transition with the default trigger and dismisses the menu.
 * - Hovering a state row opens a chevron flyout listing all trigger types at the picker's right edge.
 *   Clicking a trigger type adds a transition with that trigger + state and dismisses the menu.
 */
class SAddTransitionMenu : public SCompactTreeEditorView
{
public:

	SLATE_BEGIN_ARGS(SAddTransitionMenu)
		: _OwnerState(nullptr)
		, _ViewModel(nullptr)
	{}
		SLATE_ARGUMENT(UStateTreeState*, OwnerState)
		SLATE_ARGUMENT(TSharedPtr<FStateTreeViewModel>, ViewModel)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

protected:

	//~ Begin SCompactTreeView Interface
	virtual TSharedRef<STableRow<TSharedPtr<FStateItem>>> GenerateStateItemRowInternal(
		TSharedPtr<FStateItem> Item,
		const TSharedRef<STableViewBase>& OwnerTable,
		TSharedRef<SHorizontalBox> Container) override;
	//~ End SCompactTreeView Interface

private:

	/** Adds a GotoState transition from OwnerState to TargetState with the given trigger, then dismisses all menus. */
	static void AddTransition(TWeakPtr<FStateTreeViewModel> InViewModel
		, TWeakObjectPtr<UStateTreeState> InWeakOwnerState
		, TNotNull<UStateTreeState*> TargetState
		, EStateTreeTransitionTrigger Trigger);

	/** When the user clicks a state row directly, add a transition with the default trigger. */
	void CreateStateTransitionWithDefaultTrigger(TConstArrayView<FGuid> SelectedStateIDs) const;

	/** Callback for when the main state selector row has mouse entry, Spawn submenu on hover. */
	void HandleOnMouseEnterStateRow(const FGeometry& MyGeometry, const FPointerEvent& /*MouseEvent*/, const FGuid InStateID);

	TWeakObjectPtr<UStateTreeState> WeakOwnerState = nullptr;
	TWeakPtr<FStateTreeViewModel> WeakViewModel = nullptr;
};

} // namespace UE::StateTree::Editor
