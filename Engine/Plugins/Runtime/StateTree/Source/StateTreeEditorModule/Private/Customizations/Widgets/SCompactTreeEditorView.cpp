// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCompactTreeEditorView.h"

#include "StateTreeDragDrop.h"
#include "StateTreeEditorData.h"
#include "StateTreeEditorStyle.h"
#include "StateTreeViewModel.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SCompactTreeEditorView)

#define LOCTEXT_NAMESPACE "OutlinerStateTreeView"

namespace UE::StateTree::Editor
{

FSlateColor CompactTreeView::FStateItemLinkData::GetBorderColor() const
{
	if (LinkState == LinkState_None)
	{
		return FLinearColor(0.0f, 0.0f, 0.0f, 0.0f);
	}

	static const FName NAME_LinkingIn = "Colors.StateLinkingIn";
	static const FName NAME_StateLinkedOut = "Colors.StateLinkedOut";
	const FName ColorName = (LinkState & LinkState_LinkingIn) != 0 ? NAME_LinkingIn : NAME_StateLinkedOut;
	return FStateTreeEditorStyle::Get().GetColor(ColorName);
}

void SCompactTreeEditorView::Construct(const FArguments& InArgs, const TSharedPtr<FStateTreeViewModel>& InViewModel)
{
	StateTreeViewModel = InViewModel;
	WeakStateTreeEditorData = InArgs._StateTreeEditorData;
	bSubtreesOnly = InArgs._SubtreesOnly;
	bSelectableStatesOnly = InArgs._SelectableStatesOnly;
	bShowLinkedStates = InArgs._ShowLinkedStates;

	const UStateTree* StateTreeAsset = InViewModel
		? InViewModel->GetStateTree()
		: InArgs._StateTreeEditorData
			? InArgs._StateTreeEditorData->GetTypedOuter<UStateTree>()
			: nullptr;

	if (ensureMsgf(StateTreeAsset != nullptr
		, TEXT("Expecting either a valid StateTreeViewModel or StateTreeEditorData to construct the view")))
	{
		SCompactTreeView::Construct(
			SCompactTreeView::FArguments()
			.SelectionMode(InArgs._SelectionMode)
			.OnSelectionChanged(InArgs._OnSelectionChanged)
			.OnContextMenuOpening(InArgs._OnContextMenuOpening)
			, StateTreeAsset);
	}
}

void SCompactTreeEditorView::CacheStatesInternal()
{
	if (const UStateTreeEditorData* StateTreeEditorData = WeakStateTreeEditorData.Get())
	{
		for (const UStateTreeState* SubTree : StateTreeEditorData->SubTrees)
		{
			CacheState(RootItem, SubTree);
		}
	}
}

void SCompactTreeEditorView::CacheState(TSharedPtr<FStateItem> ParentNode, const UStateTreeState* State)
{
	if (!State)
	{
		return;
	}
	const UStateTreeEditorData* StateTreeEditorData = WeakStateTreeEditorData.Get();
	if (!StateTreeEditorData)
	{
		return;
	}

	bool bShouldAdd = true;
	if (bSubtreesOnly
		&& State->Type != EStateTreeStateType::Subtree)
	{
		bShouldAdd = false;
	}

	if (bSelectableStatesOnly
		&& State->SelectionBehavior == EStateTreeStateSelectionBehavior::None)
	{
		bShouldAdd = false;
	}

	if (bShouldAdd)
	{
		const TSharedRef<FStateItem> StateItem = CreateStateItemInternal();
		CompactTreeView::FStateItemLinkData& CustomData = StateItem->CustomData.GetMutable<CompactTreeView::FStateItemLinkData>();
		StateItem->Desc = FText::FromName(State->Name);
		StateItem->TooltipText = FText::FromString(State->Description);
		StateItem->StateID = State->ID;
		StateItem->bIsEnabled = State->bEnabled;
		CustomData.bIsSubTree = State->Type == EStateTreeStateType::Subtree;
		StateItem->Color = FStateTreeStyle::Get().GetSlateColor("StateTree.CompactView.State");
		if (const FStateTreeEditorColor* FoundColor = StateTreeEditorData->FindColor(State->ColorRef))
		{
			StateItem->Color = FoundColor->Color;
		}

		StateItem->Icon = FStateTreeEditorStyle::GetBrushForSelectionBehaviorType(State->SelectionBehavior, !State->Children.IsEmpty(), State->Type);

		// Linked states
		if (State->Type == EStateTreeStateType::Linked)
		{
			CustomData.bIsLinked = true;
			CustomData.LinkedDesc = FText::FromName(State->LinkedSubtree.Name);
		}
		else if (State->Type == EStateTreeStateType::LinkedAsset)
		{
			CustomData.bIsLinked = true;
			CustomData.LinkedDesc = FText::FromString(GetNameSafe(State->LinkedAsset.Get()));
		}

		ParentNode->Children.Add(StateItem);

		ParentNode = StateItem;
	}

	for (const UStateTreeState* ChildState : State->Children)
	{
		CacheState(ParentNode, ChildState);
	}
}

TSharedRef<SCompactTreeView::FStateItem> SCompactTreeEditorView::CreateStateItemInternal() const
{
	const TSharedRef<FStateItem> StateItem = MakeShared<FStateItem>();
	StateItem->CustomData = TInstancedStruct<CompactTreeView::FStateItemLinkData>::Make({});
	return StateItem;
}

TSharedRef<STableRow<TSharedPtr<SCompactTreeView::FStateItem>>> SCompactTreeEditorView::GenerateStateItemRowInternal(
	TSharedPtr<FStateItem> Item
	, const TSharedRef<STableViewBase>& OwnerTable
	, TSharedRef<SHorizontalBox> Container)
{
	using namespace CompactTreeView;
	const FStateItemLinkData& LinkData = Item->CustomData.Get<FStateItemLinkData>();

	// Link
	if (LinkData.bIsLinked)
	{
		// Link icon
		Container->AddSlot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(4.f, 0.f)
			.AutoWidth()
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Image(FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.StateLinked"))
		];

		// Linked name
		Container->AddSlot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Text(LinkData.LinkedDesc)
			];
	}

	return SNew(STableRow<TSharedPtr<FStateItem>>, OwnerTable)
		.OnDragDetected(this, &SCompactTreeEditorView::HandleDragDetected)
		.OnDragLeave(this, &SCompactTreeEditorView::HandleDragLeave)
		.OnCanAcceptDrop(this, &SCompactTreeEditorView::HandleCanAcceptDrop)
		.OnAcceptDrop(this, &SCompactTreeEditorView::HandleAcceptDrop)
		[
			SNew(SBorder)
				.BorderBackgroundColor_Lambda([Item]
					{
						if (const FStateItem* StateItem = Item.Get())
						{
							return StateItem->CustomData.Get<FStateItemLinkData>().GetBorderColor();
						}
						return FSlateColor(FLinearColor::Transparent);
					})
				[
					Container
				]
		];
}

void SCompactTreeEditorView::OnSelectionChangedInternal(TConstArrayView<TSharedPtr<FStateItem>> SelectedStates)
{
	using namespace CompactTreeView;

	if (bShowLinkedStates && StateTreeViewModel)
	{
		ResetLinkedStates();

		// Find the Linked items
		TArray<FGuid> LinkingIn;
		TArray<FGuid> LinkedOut;
		for (const TSharedPtr<FStateItem>& StateItem : SelectedStates)
		{
			StateTreeViewModel->GetLinkStates(StateItem->StateID, LinkingIn, LinkedOut);
		}

		// Set the outline
		{
			TArray<TSharedPtr<FStateItem>> FoundStates;
			FoundStates.Reserve(LinkingIn.Num());
			FindStatesByIDRecursive(FilteredRootItem, LinkingIn, FoundStates);

			for (const TSharedPtr<FStateItem>& Item : FoundStates)
			{
				PreviousLinkedStates.Add(Item);
				Item->CustomData.GetMutable<FStateItemLinkData>().LinkState |= FStateItemLinkData::LinkState_LinkingIn;
			}
		}
		{
			TArray<TSharedPtr<FStateItem>> FoundStates;
			FoundStates.Reserve(LinkedOut.Num());
			FindStatesByIDRecursive(FilteredRootItem, LinkedOut, FoundStates);

			for (const TSharedPtr<FStateItem>& Item : FoundStates)
			{
				PreviousLinkedStates.AddUnique(Item);
				Item->CustomData.GetMutable<FStateItemLinkData>().LinkState |= FStateItemLinkData::LinkState_LinkedOut;
			}
		}
	}
}

void SCompactTreeEditorView::OnUpdatingFilteredRootInternal()
{
	ResetLinkedStates();
}

void SCompactTreeEditorView::Refresh(const UStateTreeEditorData* NewStateTreeEditorData)
{
	if (NewStateTreeEditorData)
	{
		WeakStateTreeEditorData = NewStateTreeEditorData;
	}

	SCompactTreeView::Refresh();
}

void SCompactTreeEditorView::ResetLinkedStates()
{
	using namespace CompactTreeView;

	for (TWeakPtr<FStateItem>& PreviousLinkedState : PreviousLinkedStates)
	{
		if (const TSharedPtr<FStateItem> Pin = PreviousLinkedState.Pin())
		{
			Pin->CustomData.GetMutable<FStateItemLinkData>().LinkState = FStateItemLinkData::LinkState_None;
		}
	}
	PreviousLinkedStates.Reset();
}

FReply SCompactTreeEditorView::HandleDragDetected(const FGeometry&, const FPointerEvent&) const
{
	return FReply::Handled().BeginDragDrop(FStateTreeSelectedDragDrop::New(StateTreeViewModel));
}

void SCompactTreeEditorView::HandleDragLeave(const FDragDropEvent& DragDropEvent) const
{
	const TSharedPtr<FStateTreeSelectedDragDrop> DragDropOperation = DragDropEvent.GetOperationAs<FStateTreeSelectedDragDrop>();
	if (DragDropOperation.IsValid())
	{
		DragDropOperation->SetCanDrop(false);
	}
}

TOptional<EItemDropZone> SCompactTreeEditorView::HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FStateItem> TargetState) const
{
	if (StateTreeViewModel)
	{
		const TSharedPtr<FStateTreeSelectedDragDrop> DragDropOperation = DragDropEvent.GetOperationAs<FStateTreeSelectedDragDrop>();
		if (DragDropOperation.IsValid())
		{
			DragDropOperation->SetCanDrop(true);

			// Cannot drop on selection or child of selection.
			if (StateTreeViewModel && StateTreeViewModel->IsChildOfSelection(StateTreeViewModel->GetMutableStateByID(TargetState->StateID)))
			{
				DragDropOperation->SetCanDrop(false);
				return TOptional<EItemDropZone>();
			}

			return DropZone;
		}
	}

	return TOptional<EItemDropZone>();
}

FReply SCompactTreeEditorView::HandleAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FStateItem> TargetState) const
{
	if (StateTreeViewModel)
	{
		const TSharedPtr<FStateTreeSelectedDragDrop> DragDropOperation = DragDropEvent.GetOperationAs<FStateTreeSelectedDragDrop>();
		if (DragDropOperation.IsValid())
		{
			if (StateTreeViewModel)
			{
				if (DropZone == EItemDropZone::AboveItem)
				{
					StateTreeViewModel->MoveSelectedStatesBefore(StateTreeViewModel->GetMutableStateByID(TargetState->StateID));
				}
				else if (DropZone == EItemDropZone::BelowItem)
				{
					StateTreeViewModel->MoveSelectedStatesAfter(StateTreeViewModel->GetMutableStateByID(TargetState->StateID));
				}
				else
				{
					StateTreeViewModel->MoveSelectedStatesInto(StateTreeViewModel->GetMutableStateByID(TargetState->StateID));
				}

				return FReply::Handled();
			}
		}
	}

	return FReply::Unhandled();
}

} // UE::StateTree::Editor

#undef LOCTEXT_NAMESPACE
