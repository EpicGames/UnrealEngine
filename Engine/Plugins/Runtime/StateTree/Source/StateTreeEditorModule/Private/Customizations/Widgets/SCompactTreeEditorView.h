// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Views/ITypedTableView.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompactTreeView.h"
#include "SCompactTreeEditorView.generated.h"

namespace ESelectionMode
{
	enum Type : int;
}

enum class EItemDropZone;
class UStateTreeState;
class UStateTreeEditorData;
struct FSlateColor;
class FStateTreeViewModel;

namespace UE::StateTree::Editor
{

namespace CompactTreeView
{
	USTRUCT()
	struct FStateItemLinkData : public FStateItemCustomData
	{
		GENERATED_BODY()

		enum ELinkState
		{
			LinkState_None = 0x00,
			LinkState_LinkingIn = 0x01,
			LinkState_LinkedOut = 0x02,
		};

		FSlateColor GetBorderColor() const;

		uint8 LinkState = LinkState_None;
		bool bIsSubTree = false;
		bool bIsLinked = false;
		FText LinkedDesc;
	};
} // CompactTreeView

/**
 * Widget that displays a list of State Tree nodes which match base types and specified schema.
 * Can be used e.g. in popup menus to select node types.
 */
class SCompactTreeEditorView : public SCompactTreeView
{
public:

	SLATE_BEGIN_ARGS(SCompactTreeEditorView)
		: _StateTreeEditorData(nullptr)
		, _SelectionMode(ESelectionMode::Single)
		, _SelectableStatesOnly(false)
		, _SubtreesOnly(false)
		, _ShowLinkedStates(false)
	{}
		SLATE_ARGUMENT(const UStateTreeEditorData*, StateTreeEditorData)
		SLATE_ARGUMENT(ESelectionMode::Type, SelectionMode)
		SLATE_ARGUMENT(bool, SelectableStatesOnly)
		SLATE_ARGUMENT(bool, SubtreesOnly)
		SLATE_ARGUMENT(bool, ShowLinkedStates)
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)
		SLATE_EVENT(FOnContextMenuOpening, OnContextMenuOpening)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<FStateTreeViewModel>& InViewModel = nullptr);

	void Refresh(const UStateTreeEditorData* NewStateTreeEditorData = nullptr);

protected:

	//~ Begin SCompactStateTreeView interface
	virtual TSharedRef<FStateItem> CreateStateItemInternal() const override;
	virtual void CacheStatesInternal() override;
	virtual TSharedRef<STableRow<TSharedPtr<FStateItem>>> GenerateStateItemRowInternal(TSharedPtr<FStateItem> Item, const TSharedRef<STableViewBase>& OwnerTable, TSharedRef<SHorizontalBox> Container) override;

	virtual void OnSelectionChangedInternal(TConstArrayView<TSharedPtr<FStateItem>> SelectedStates) override;
	virtual void OnUpdatingFilteredRootInternal() override;
	//~ End SCompactStateTreeView interface

private:

	void CacheState(TSharedPtr<FStateItem> ParentNode, const UStateTreeState* State);
	void ResetLinkedStates();

	FReply HandleDragDetected(const FGeometry&, const FPointerEvent&) const;
	void HandleDragLeave(const FDragDropEvent& DragDropEvent) const;
	TOptional<EItemDropZone> HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FStateItem> TargetState) const;
	FReply HandleAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FStateItem> TargetState) const;

	TWeakObjectPtr<const UStateTreeEditorData> WeakStateTreeEditorData = nullptr;
	TSharedPtr<FStateTreeViewModel> StateTreeViewModel;
	TArray<TWeakPtr<FStateItem>> PreviousLinkedStates;

	/** If set, allow to select only states marked as subtrees. */
	bool bSubtreesOnly = false;

	/** If set, states without any selection behavior won't be included in the tree view. */
	bool bSelectableStatesOnly = false;

	/** If set, linked states will be displayed using a colored border in the tree view. */
	bool bShowLinkedStates = false;
};

} // namespace UE::StateTree::Editor