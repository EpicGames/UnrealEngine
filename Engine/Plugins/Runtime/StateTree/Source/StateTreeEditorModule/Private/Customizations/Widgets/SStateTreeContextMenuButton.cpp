// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStateTreeContextMenuButton.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "StateTreeDragDrop.h"
#include "StateTreeState.h"
#include "StateTreeViewModel.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"

void SStateTreeContextMenuButton::Construct(const FArguments& InArgs, const TSharedRef<FStateTreeViewModel>& InStateTreeViewModel, TWeakObjectPtr<UStateTreeState> InOwnerState, const FGuid& InNodeID, bool InbIsTransition)
{
	StateTreeViewModel = InStateTreeViewModel.ToSharedPtr();
	OwnerStateWeak = InOwnerState;
	NodeID = InNodeID;

	bIsStateTransition = false;
	bIsTransition = InbIsTransition;
	bSkipDefaultClickAction = false;
	if (bIsTransition)
	{
		if (UStateTreeState* OwnerState = OwnerStateWeak.Get())
		{
			bIsStateTransition = OwnerState->GetTransitionByID(NodeID) != nullptr;
		}
	}

	TSharedRef<SStackBox> ButtonContent = SNew(SStackBox)
		.Orientation(EOrientation::Orient_Horizontal);
	if (InArgs._bAllowDragAndDrop)
	{
		ButtonContent->AddSlot()
			.Padding(0.0f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.MaxSize(16.0f)
			[
				SNew(SImage)
				.Image(FCoreStyle::Get().GetBrush("VerticalBoxDragIndicatorShort"))
				.Cursor(EMouseCursor::GrabHand)
				.Visibility_Lambda([Weak = AsWeak()]()
					{
						if (const TSharedPtr<SWidget> Button = Weak.Pin())
						{
							return Button->IsHovered() ? EVisibility::Visible : EVisibility::Hidden;
						}
						return EVisibility::Hidden;
					})
			];
	}
	ButtonContent->AddSlot()
		.Padding(0.0f)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoSize()
		[
			InArgs._Content.Widget
		];

	SButton::FArguments ButtonArgs;
	ButtonArgs
	.AllowDragDrop(InArgs._bAllowDragAndDrop)
	.OnClicked(this, &SStateTreeContextMenuButton::HandleButtonClicked)
	.ButtonStyle(InArgs._ButtonStyle)
	.ContentPadding(InArgs._ContentPadding)
	[
		SAssignNew(MenuAnchor, SMenuAnchor)
		.Placement(MenuPlacement_BelowAnchor)
		.OnGetMenuContent(FOnGetContent::CreateSP(this, &SStateTreeContextMenuButton::MakeContextMenu))
		[
			ButtonContent
		]
	];

	SButton::Construct(ButtonArgs);
}

FReply SStateTreeContextMenuButton::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const bool bDoStateTransition = IsEnabled()
		&& bIsStateTransition
		&& MouseEvent.GetModifierKeys().IsAltDown()
		&& (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton || MouseEvent.IsTouchEvent());

	bSkipDefaultClickAction = bDoStateTransition;

	// SButton::OnMouseButtonUp is needed to do extra work
	FReply Reply = SButton::OnMouseButtonUp(MyGeometry, MouseEvent);

	if (IsEnabled() && !Reply.IsEventHandled())
	{
		if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			if (MenuAnchor.IsValid())
			{
				if (MenuAnchor->ShouldOpenDueToClick())
				{
					MenuAnchor->SetIsOpen(true);
				}
				else
				{
					MenuAnchor->SetIsOpen(false);
				}
				Reply = FReply::Handled();
			}
		}
	}
	else if (bDoStateTransition && Reply.IsEventHandled())
	{
		UStateTreeState* OwnerState = OwnerStateWeak.Get();
		UStateTreeState* GotoState = OwnerState ? StateTreeViewModel->GetTransitionToState(OwnerState, NodeID) : nullptr;
		StateTreeViewModel->SetSelection(GotoState);
		//Reply = FReply::Handled(); The Reply from SButton have more infos
	}

	bSkipDefaultClickAction = false;

	return Reply;
}

FReply SStateTreeContextMenuButton::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const bool bIsControlDown = MouseEvent.GetModifierKeys().IsControlDown();
	return FReply::Handled().BeginDragDrop(FStateTreeSelectedDragDrop::New(StateTreeViewModel, NodeID, OwnerStateWeak, bIsControlDown));
}

TSharedRef<SWidget> SStateTreeContextMenuButton::MakeContextMenu() const
{
	FMenuBuilder MenuBuilder(/*ShouldCloseWindowAfterMenuSelection*/true, /*CommandList*/nullptr);

	if (StateTreeViewModel.IsValid() && OwnerStateWeak.IsValid())
	{
		MenuBuilder.BeginSection(FName("View"), LOCTEXT("View", "View"));

		// Navigate
		if (bIsStateTransition)
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("Navigate", "Navigate"),
				LOCTEXT("NavigateTooltip", "Navigate to transition target. (Alt-Click)"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.BrowseContent"),
				FUIAction(
					FExecuteAction::CreateSPLambda(this, [this]()
						{
							UStateTreeState* OwnerState = OwnerStateWeak.Get();
							UStateTreeState* GotoState = OwnerState ? StateTreeViewModel->GetTransitionToState(OwnerState, NodeID) : nullptr;
							StateTreeViewModel->SetSelection(GotoState);
						}),
					FCanExecuteAction::CreateSPLambda(this, [this]()
						{
							UStateTreeState* OwnerState = OwnerStateWeak.Get();
							return OwnerState != nullptr && StateTreeViewModel->GetTransitionToState(OwnerState, NodeID) != nullptr;
						})
				));
		}


		MenuBuilder.EndSection();

		MenuBuilder.BeginSection(FName("Edit"), LOCTEXT("Edit", "Edit"));

		// Copy
		MenuBuilder.AddMenuEntry(
			LOCTEXT("CopyItem", "Copy"),
			LOCTEXT("CopyItemTooltip", "Copy this item"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy"),
			FUIAction(
			FExecuteAction::CreateSPLambda(this, [this]()
			{
				StateTreeViewModel->CopyNode(OwnerStateWeak, NodeID);
			}),
			FCanExecuteAction::CreateSPLambda(this, [this]()
			{
				return !bIsTransition || bIsStateTransition;
			})
		));

		// Copy all
		MenuBuilder.AddMenuEntry(
			LOCTEXT("CopyAllItems", "Copy all"),
			LOCTEXT("CopyAllItemsTooltip", "Copy all items"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy"),
			FUIAction(
				FExecuteAction::CreateSPLambda(this, [this]()
					{
						StateTreeViewModel->CopyAllNodes(OwnerStateWeak, NodeID);
					}),
				FCanExecuteAction::CreateSPLambda(this, [this]()
					{
						return !bIsTransition || bIsStateTransition;
					})
			));

		// Paste
		MenuBuilder.AddMenuEntry(
			LOCTEXT("PasteItem", "Paste"),
			LOCTEXT("PasteItemTooltip", "Paste into this item"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Paste"),
			FUIAction(
			FExecuteAction::CreateSPLambda(this, [this]()
			{
				StateTreeViewModel->PasteNode(OwnerStateWeak, NodeID);
			}),
			FCanExecuteAction::CreateSPLambda(this, [this]()
			{
				return !bIsTransition || bIsStateTransition;
			})
		));

		// Duplicate
		MenuBuilder.AddMenuEntry(
			LOCTEXT("DuplicateItem", "Duplicate"),
			LOCTEXT("DuplicateItemTooltip", "Duplicate this item"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Duplicate"),
			FUIAction(
			FExecuteAction::CreateSPLambda(this, [&]()
			{
				StateTreeViewModel->DuplicateNode(OwnerStateWeak, NodeID);
			}),
			FCanExecuteAction::CreateSPLambda(this, [this]()
			{
				return !bIsTransition || bIsStateTransition;
			})
		));

		// Delete
		MenuBuilder.AddMenuEntry(
			LOCTEXT("DeleteItem", "Delete"),
			LOCTEXT("DeleteItemTooltip", "Delete this item"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Delete"),
			FUIAction(
			FExecuteAction::CreateSPLambda(this, [&]()
			{
				StateTreeViewModel->DeleteNode(OwnerStateWeak, NodeID);
			}),
			FCanExecuteAction::CreateSPLambda(this, [this]()
			{
				return !bIsTransition || bIsStateTransition;
			})
		));

		// Delete All
		MenuBuilder.AddMenuEntry(
			LOCTEXT("DeleteAllItems", "Delete all"),
			LOCTEXT("DeleteAllItemsTooltip", "Delete all items"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Delete"),
			FUIAction(
			FExecuteAction::CreateSPLambda(this, [&]()
			{
				StateTreeViewModel->DeleteAllNodes(OwnerStateWeak, NodeID);
			}),
			FCanExecuteAction::CreateSPLambda(this, [this]()
			{
				return !bIsTransition || bIsStateTransition;
			})
		));

		MenuBuilder.EndSection();
	}

	return MenuBuilder.MakeWidget();
}

FReply SStateTreeContextMenuButton::HandleButtonClicked()
{
	if (!bSkipDefaultClickAction)
	{
		StateTreeViewModel->BringNodeToFocus(OwnerStateWeak.Get(), NodeID);
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
