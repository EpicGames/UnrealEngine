// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Input/SButton.h"

class SMenuAnchor;
class UStateTreeEditorData;
struct FGeometry;
struct FPointerEvent;
class FStateTreeViewModel;
class UStateTreeState;

class SStateTreeContextMenuButton : public SButton
{
public:
	SLATE_BEGIN_ARGS(SStateTreeContextMenuButton)
		: _Content()
		, _ButtonStyle(&FCoreStyle::Get().GetWidgetStyle< FButtonStyle >("Button"))
		, _ContentPadding(FMargin(4.0f, 2.0f))
		, _bAllowDragAndDrop(false)
		{
		}

		/** Slot for this button's content (optional) */
		SLATE_DEFAULT_SLOT(FArguments, Content)

		/** The visual style of the button */
		SLATE_STYLE_ARGUMENT(FButtonStyle, ButtonStyle)

		/** Spacing between button's border and the content. */
		SLATE_ATTRIBUTE(FMargin, ContentPadding)

		/** True if the button should detect drag on mouse down. */
		SLATE_ARGUMENT(bool, bAllowDragAndDrop)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FStateTreeViewModel>& InStateTreeViewModel, const TWeakObjectPtr<UStateTreeState> InOwnerState, const FGuid& InSourceID, bool InbIsTransition = false);

	//~ SWidget interface
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	//~ SWidget overrides

	TSharedRef<SWidget> MakeContextMenu() const;

private:
	FReply HandleButtonClicked();

	TSharedPtr<FStateTreeViewModel> StateTreeViewModel;
	TWeakObjectPtr<UStateTreeState> OwnerStateWeak;
	FGuid NodeID;

	uint8 bIsTransition : 1;
	// We have State Transition, Task Transition and Default Transition back to root
	uint8 bIsStateTransition : 1;

	uint8 bSkipDefaultClickAction : 1 = false;

private:
	TSharedPtr<SMenuAnchor> MenuAnchor;
};
