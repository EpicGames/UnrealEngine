// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStateTreeEditorColorComboBox.h"
#include "DetailLayoutBuilder.h"
#include "GuidStructCustomization.h"
#include "StateTreeEditorData.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"

void SStateTreeEditorColorComboBox::Construct(const FArguments& InArgs, TSharedPtr<IPropertyHandle> InColorRefHandle, UStateTreeEditorData* InEditorData)
{
	WeakEditorData = InEditorData;
	ColorRefHandle = InColorRefHandle;
	ColorIDHandle = InColorRefHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeEditorColorRef, ID));

	ColorRefHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &SStateTreeEditorColorComboBox::UpdatedSelectedColorWidget));

	ChildSlot
	[
		SAssignNew(ColorComboBox, SComboBox<TSharedPtr<FStateTreeEditorColorRef>>)
		.OptionsSource(&ColorRefOptions)
		.OnComboBoxOpening(this, &SStateTreeEditorColorComboBox::RefreshColorOptions)
		.OnGenerateWidget(this, &SStateTreeEditorColorComboBox::GenerateColorOptionWidget)
		.OnSelectionChanged(this, &SStateTreeEditorColorComboBox::OnSelectionChanged)
		[
			SAssignNew(SelectedColorBorder, SBorder)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
		]
	];

	UpdatedSelectedColorWidget();
}

const FStateTreeEditorColor* SStateTreeEditorColorComboBox::FindColorEntry(const FStateTreeEditorColorRef& ColorRef) const
{
	if (UStateTreeEditorData* EditorData = WeakEditorData.Get())
	{
		return EditorData->FindColor(FStateTreeEditorColorRef(ColorRef));
	}
	return nullptr;
}

FText SStateTreeEditorColorComboBox::GetDisplayName(FStateTreeEditorColorRef ColorRef) const
{
	if (const FStateTreeEditorColor* ColorEntry = FindColorEntry(ColorRef))
	{
		return FText::FromString(ColorEntry->DisplayName);
	}
	return FText::GetEmpty();
}

FLinearColor SStateTreeEditorColorComboBox::GetColor(FStateTreeEditorColorRef ColorRef) const
{
	if (const FStateTreeEditorColor* ColorEntry = FindColorEntry(ColorRef))
	{
		return ColorEntry->Color;
	}
	return FLinearColor(ForceInitToZero);
}

TSharedRef<SWidget> SStateTreeEditorColorComboBox::GenerateColorOptionWidget(TSharedPtr<FStateTreeEditorColorRef> ColorRef)
{
	return SNew(SBox)
		.WidthOverride(120.f)
		.Padding(5.f, 1.f)
		[
			GenerateColorWidget(*ColorRef)
		];
}

TSharedRef<SWidget> SStateTreeEditorColorComboBox::GenerateColorWidget(const FStateTreeEditorColorRef& ColorRef)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.MaxWidth(25)
		.VAlign(VAlign_Center)
		[
			SNew(SColorBlock)
			.Color(this, &SStateTreeEditorColorComboBox::GetColor, ColorRef)
			.Size(FVector2D(16.0))
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(5.f, 0.f, 0.f, 0.f))
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(this, &SStateTreeEditorColorComboBox::GetDisplayName, ColorRef)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];
}

void SStateTreeEditorColorComboBox::RefreshColorOptions()
{
	ColorRefOptions.Reset();

	UStateTreeEditorData* EditorData = WeakEditorData.Get();
	if (!EditorData)
	{
		return;
	}

	for (const FStateTreeEditorColor& Color : EditorData->Colors)
	{
		TSharedRef<FStateTreeEditorColorRef> ColorRefOption = MakeShared<FStateTreeEditorColorRef>(Color.ColorRef);
		ColorRefOptions.Add(ColorRefOption);
		if (SelectedColorRef == Color.ColorRef)
		{
			ColorComboBox->SetSelectedItem(ColorRefOption);
		}
	}
}

void SStateTreeEditorColorComboBox::OnSelectionChanged(TSharedPtr<FStateTreeEditorColorRef> SelectedColorRefOption, ESelectInfo::Type SelectInfo)
{
	if (SelectedColorRefOption.IsValid())
	{
		WriteGuidToProperty(ColorIDHandle, SelectedColorRefOption->ID);
		UpdatedSelectedColorWidget();
	}
}

void SStateTreeEditorColorComboBox::UpdatedSelectedColorWidget()
{
	check(SelectedColorBorder.IsValid());

	TArray<const void*> RawData;
	ColorRefHandle->AccessRawData(RawData);

	if (RawData.IsEmpty())
	{
		SelectedColorBorder->ClearContent();
		return;
	}

	SelectedColorRef = *static_cast<const FStateTreeEditorColorRef*>(RawData.Pop(EAllowShrinking::No));

	// Make "Multiple Values" content if there's at least one Color ID that differs
	for (const void* ColorRefRaw : RawData)
	{
		if (SelectedColorRef != *static_cast<const FStateTreeEditorColorRef*>(ColorRefRaw))
		{
			SelectedColorBorder->SetContent(SNew(STextBlock)
				.Text(LOCTEXT("MultipleValues", "Multiple Values"))
				.Font(IDetailLayoutBuilder::GetDetailFont()));
			return;
		}
	}

	SelectedColorBorder->SetContent(GenerateColorWidget(SelectedColorRef));
}

#undef LOCTEXT_NAMESPACE
