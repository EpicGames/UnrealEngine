// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeEditorTypes.h"
#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/SCompoundWidget.h"

class IPropertyHandle;
class SBorder;
class UStateTreeEditorData;
struct FStateTreeEditorColor;
struct FStateTreeEditorColorRef;
template<typename OptionType> class SComboBox;

class SStateTreeEditorColorComboBox : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SStateTreeEditorColorComboBox){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<IPropertyHandle> InColorRefHandle, UStateTreeEditorData* InEditorData);

private:
	const FStateTreeEditorColor* FindColorEntry(const FStateTreeEditorColorRef& ColorRef) const;

	FText GetDisplayName(FStateTreeEditorColorRef ColorRef) const;
	FLinearColor GetColor(FStateTreeEditorColorRef ColorRef) const;

	TSharedRef<SWidget> GenerateColorOptionWidget(TSharedPtr<FStateTreeEditorColorRef> ColorRef);
	TSharedRef<SWidget> GenerateColorWidget(const FStateTreeEditorColorRef& ColorRef);

	void RefreshColorOptions();

	void OnSelectionChanged(TSharedPtr<FStateTreeEditorColorRef> SelectedColorRefOption, ESelectInfo::Type SelectInfo);

	void UpdatedSelectedColorWidget();

	TWeakObjectPtr<UStateTreeEditorData> WeakEditorData;

	TSharedPtr<IPropertyHandle> ColorRefHandle;
	TSharedPtr<IPropertyHandle> ColorIDHandle;

	TSharedPtr<SBorder> SelectedColorBorder;
	TSharedPtr<SComboBox<TSharedPtr<FStateTreeEditorColorRef>>> ColorComboBox;

	TArray<TSharedPtr<FStateTreeEditorColorRef>> ColorRefOptions;

	FStateTreeEditorColorRef SelectedColorRef;
};
