// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStateTreeNodeTypePicker.h"

#include "StateTreeEditorModule.h"
#include "StateTreeNodeBase.h"
#include "StateTreeNodeClassCache.h"
#include "StateTreeSchema.h"
#include "Blueprint/StateTreeNodeBlueprintBase.h"
#include "Customizations/StateTreeEditorNodeUtils.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/Input/SSearchBox.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"

TMap<FObjectKey, SStateTreeNodeTypePicker::FCategoryExpansionState> SStateTreeNodeTypePicker::CategoryExpansionStates;

SStateTreeNodeTypePicker::SStateTreeNodeTypePicker()
{
}

SStateTreeNodeTypePicker::~SStateTreeNodeTypePicker()
{
	
}

void SStateTreeNodeTypePicker::Construct(const SStateTreeNodeTypePicker::FArguments& InArgs)
{
	OnNodeStructPicked = InArgs._OnNodeTypePicked;
	CategoryKey = FObjectKey(InArgs._BaseScriptStruct);
	
	CacheNodeTypes(InArgs._Schema, InArgs._BaseScriptStruct, InArgs._BaseClass);

	NodeTypeTree = SNew(STreeView<TSharedPtr<FStateTreeNodeTypeItem>>)
		.SelectionMode(ESelectionMode::Single)
		.TreeItemsSource(&FilteredRootNode->Children)
		.OnGenerateRow(this, &SStateTreeNodeTypePicker::GenerateNodeTypeRow)
		.OnGetChildren(this, &SStateTreeNodeTypePicker::GetNodeTypeChildren)
		.OnSelectionChanged(this, &SStateTreeNodeTypePicker::OnNodeTypeSelected)
		.OnExpansionChanged(this, &SStateTreeNodeTypePicker::OnNodeTypeExpansionChanged);
	
	// Restore category expansion state from previous use.
	RestoreExpansionState();

	// Expand current selection.
	const TArray<TSharedPtr<FStateTreeNodeTypeItem>> Path = GetPathToItemStruct(InArgs._CurrentStruct);
	if (Path.Num() > 0)
	{
		// Expand all categories up to the selected item.
		bIsRestoringExpansion = true;
		for (const TSharedPtr<FStateTreeNodeTypeItem>& Item : Path)
		{
			NodeTypeTree->SetItemExpansion(Item, true);
		}
		bIsRestoringExpansion = false;
		
		NodeTypeTree->SetItemSelection(Path.Last(), true);
		NodeTypeTree->RequestScrollIntoView(Path.Last());
	}

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		.Padding(4, 2, 4, 2)
		.AutoHeight()
		[
			SAssignNew(SearchBox, SSearchBox)
			.OnTextChanged(this, &SStateTreeNodeTypePicker::OnSearchBoxTextChanged)
		]
		+ SVerticalBox::Slot()
		[
			NodeTypeTree.ToSharedRef()
		]
	];
}

TSharedPtr<SWidget> SStateTreeNodeTypePicker::GetWidgetToFocusOnOpen()
{
	return SearchBox;
}

void SStateTreeNodeTypePicker::SortNodeTypesFunctionItemsRecursive(TArray<TSharedPtr<FStateTreeNodeTypeItem>>& Items)
{
	Items.Sort([](const TSharedPtr<FStateTreeNodeTypeItem>& A, const TSharedPtr<FStateTreeNodeTypeItem>& B)
	{
		if (!A->GetCategoryName().IsEmpty() && !B->GetCategoryName().IsEmpty())
		{
			return A->GetCategoryName() < B->GetCategoryName();
		}
		if (!A->GetCategoryName().IsEmpty() && B->GetCategoryName().IsEmpty())
		{
			return true;
		}
		if (A->GetCategoryName().IsEmpty() && !B->GetCategoryName().IsEmpty())
		{
			return false;
		}
		if (A->Struct != nullptr && B->Struct != nullptr)
		{
			return A->Struct->GetDisplayNameText().CompareTo(B->Struct->GetDisplayNameText()) <= 0;
		}
		return true;
	});

	for (const TSharedPtr<FStateTreeNodeTypeItem>& Item : Items)
	{
		SortNodeTypesFunctionItemsRecursive(Item->Children);
	}
}

TSharedPtr<SStateTreeNodeTypePicker::FStateTreeNodeTypeItem> SStateTreeNodeTypePicker::FindOrCreateItemForCategory(TArray<TSharedPtr<FStateTreeNodeTypeItem>>& Items, TArrayView<FString> CategoryPath)
{
	check(CategoryPath.Num() > 0);

	const FString& CategoryName = CategoryPath.Last();

	int32 Idx = 0;
	for (; Idx < Items.Num(); ++Idx)
	{
		// found item
		if (Items[Idx]->GetCategoryName() == CategoryName)
		{
			return Items[Idx];
		}

		// passed the place where it should have been, break out
		if (Items[Idx]->GetCategoryName() > CategoryName)
		{
			break;
		}
	}

	TSharedPtr<FStateTreeNodeTypeItem> NewItem = Items.Insert_GetRef(MakeShared<FStateTreeNodeTypeItem>(), Idx);
	NewItem->CategoryPath = CategoryPath;
	return NewItem;
}

void SStateTreeNodeTypePicker::AddNode(const UStruct* Struct)
{
	if (!Struct || !RootNode.IsValid())
	{
		return;
	}

	FName IconName;
	FColor IconColor = UE::StateTree::Colors::Grey;

	if (const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(Struct))
	{
		if (ScriptStruct->IsChildOf(TBaseStructure<FStateTreeNodeBase>::Get()))
		{
			FInstancedStruct Temp;
			Temp.InitializeAs(ScriptStruct);
			if (const FStateTreeNodeBase* NodeBase = Temp.GetPtr<FStateTreeNodeBase>())
			{
				IconName = NodeBase->GetIconName();
				IconColor = NodeBase->GetIconColor();
			}
		}
	}
	else if (const UClass* Class = Cast<UClass>(Struct))
	{
		if (Class->IsChildOf(UStateTreeNodeBlueprintBase::StaticClass()))
		{
			if (const UStateTreeNodeBlueprintBase* NodeBase = GetDefault<UStateTreeNodeBlueprintBase>(const_cast<UClass*>(Class)))
			{
				// @todo: get icon from BP class.
			}
		}
	}

	const FText CategoryName = Struct->GetMetaDataText("Category");

	TSharedPtr<FStateTreeNodeTypeItem> ParentItem = RootNode;

	if (!CategoryName.IsEmpty())
	{
		// Split into subcategories and trim
		TArray<FString> CategoryPath;
		CategoryName.ToString().ParseIntoArray(CategoryPath, TEXT("|"));
		for (FString& SubCategory : CategoryPath)
		{
			SubCategory.TrimStartAndEndInline();
		}

		// Create items for the entire category path
		// eg. "Math|Boolean|AND" 
		// Math 
		//   > Boolean
		//     > AND
		for (int32 PathIndex = 0; PathIndex < CategoryPath.Num(); ++PathIndex)
		{
			ParentItem = FindOrCreateItemForCategory(ParentItem->Children, MakeArrayView(CategoryPath.GetData(), PathIndex + 1));
		}
	}
	check(ParentItem);

	const TSharedPtr<FStateTreeNodeTypeItem>& Item = ParentItem->Children.Add_GetRef(MakeShared<FStateTreeNodeTypeItem>());
	Item->Struct = Struct;
	if (!IconName.IsNone())
	{
		Item->Icon = UE::StateTreeEditor::EditorNodeUtils::ParseIcon(IconName);
	}
	Item->IconColor = FLinearColor(IconColor);
}

void SStateTreeNodeTypePicker::CacheNodeTypes(const UStateTreeSchema* Schema, const UScriptStruct* BaseScriptStruct, const UClass* BaseClass)
{
	// Get all usable nodes from the node class cache.
	FStateTreeEditorModule& EditorModule = FModuleManager::GetModuleChecked<FStateTreeEditorModule>(TEXT("StateTreeEditorModule"));
	FStateTreeNodeClassCache* ClassCache = EditorModule.GetNodeClassCache().Get();
	check(ClassCache);

	TArray<TSharedPtr<FStateTreeNodeClassData>> StructNodes;
	TArray<TSharedPtr<FStateTreeNodeClassData>> ObjectNodes;
	
	ClassCache->GetScripStructs(BaseScriptStruct, StructNodes);
	ClassCache->GetClasses(BaseClass, ObjectNodes);

	// Create tree of node types based on category.
	RootNode = MakeShared<FStateTreeNodeTypeItem>();
	
	constexpr bool bSilent = true;
	for (const TSharedPtr<FStateTreeNodeClassData>& Data : StructNodes)
	{
		if (const UScriptStruct* ScriptStruct = Data->GetScriptStruct(bSilent))
		{
			if (ScriptStruct == BaseScriptStruct)
			{
				continue;
			}
			if (ScriptStruct->HasMetaData(TEXT("Hidden")))
			{
				continue;
			}
			if (Schema && !Schema->IsStructAllowed(ScriptStruct))
			{
				continue;
			}

			AddNode(ScriptStruct);
		}
	}

	for (const TSharedPtr<FStateTreeNodeClassData>& Data : ObjectNodes)
	{
		if (const UClass* Class = Data->GetClass(bSilent))
		{
			if (Class == BaseClass)
			{
				continue;
			}
			if (Class->HasAnyClassFlags(CLASS_Abstract | CLASS_Hidden | CLASS_HideDropDown))
			{
				continue;
			}
			if (Class->HasMetaData(TEXT("Hidden")))
			{
				continue;
			}
			if (Schema && !Schema->IsClassAllowed(Class))
			{
				continue;
			}

			AddNode(Class);
		}
	}

	SortNodeTypesFunctionItemsRecursive(RootNode->Children);

	FilteredRootNode = RootNode;
}

TSharedRef<ITableRow> SStateTreeNodeTypePicker::GenerateNodeTypeRow(TSharedPtr<FStateTreeNodeTypeItem> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	FText DisplayName;
	if (Item->IsCategory())
	{
		DisplayName = FText::FromString(Item->GetCategoryName());
	}
	else
	{
		DisplayName = Item->Struct ? Item->Struct->GetDisplayNameText() : LOCTEXT("None", "None");
	}
	
	FText Tooltip = Item->Struct ? Item->Struct->GetMetaDataText("Tooltip") : FText::GetEmpty();
	if (Tooltip.IsEmpty())
	{
		Tooltip = DisplayName;
	}

	const FSlateBrush* Icon = nullptr;
	FSlateColor IconColor;
	if (!Item->IsCategory())
	{
		if (const UClass* ItemClass = Cast<UClass>(Item->Struct))
		{
			if (Item->Icon.IsSet())
			{
				Icon = Item->Icon.GetIcon();
			}
			else
			{
				Icon = FSlateIconFinder::FindIconBrushForClass(ItemClass); 
			}
			IconColor = Item->IconColor;
		}
		else if (const UScriptStruct* ItemScriptStruct = Cast<UScriptStruct>(Item->Struct))
		{
			if (Item->Icon.IsSet())
			{
				Icon = Item->Icon.GetIcon();
			}
			else
			{
				Icon = FSlateIconFinder::FindIconBrushForClass(UScriptStruct::StaticClass());
			}
			IconColor = Item->IconColor;
		}
		else
		{
			// None
			Icon = FSlateIconFinder::FindIconBrushForClass(nullptr);
			IconColor = FSlateColor::UseForeground();
		}
	}

	TSharedRef<STableRow<TSharedPtr<FStateTreeNodeTypeItem>>> Row = SNew(STableRow<TSharedPtr<FStateTreeNodeTypeItem>>, OwnerTable);
	Row->SetContent(
		SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(0, 2.0f, 4.0f, 2.0f)
			.AutoWidth()
			[
				SNew(SImage)
				.Visibility(Icon ? EVisibility::Visible : EVisibility::Collapsed)
				.ColorAndOpacity(IconColor)
				.DesiredSizeOverride(FVector2D(16.0f, 16.0f))
				.Image(Icon)
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Font(Item->IsCategory() ? FAppStyle::Get().GetFontStyle("BoldFont") : FAppStyle::Get().GetFontStyle("NormalText"))
				.Text(DisplayName)
				.ToolTipText(Tooltip)
				.HighlightText_Lambda([this]() { return SearchBox.IsValid() ? SearchBox->GetText() : FText::GetEmpty(); })
			]
		);
	
	return Row;
}

void SStateTreeNodeTypePicker::GetNodeTypeChildren(TSharedPtr<FStateTreeNodeTypeItem> Item, TArray<TSharedPtr<FStateTreeNodeTypeItem>>& OutItems) const
{
	if (Item.IsValid())
	{
		OutItems = Item->Children;
	}
}

void SStateTreeNodeTypePicker::OnNodeTypeSelected(TSharedPtr<FStateTreeNodeTypeItem> SelectedItem, ESelectInfo::Type Type)
{
	// Skip selection set via code, or if Selected Item is invalid
	if (Type == ESelectInfo::Direct || !SelectedItem.IsValid())
	{
		return;
	}

	if (!SelectedItem->IsCategory()
		&& OnNodeStructPicked.IsBound())
	{
		OnNodeStructPicked.Execute(SelectedItem->Struct);
	}
}

void SStateTreeNodeTypePicker::OnNodeTypeExpansionChanged(TSharedPtr<FStateTreeNodeTypeItem> ExpandedItem, bool bInExpanded)
{
	// Do not save expansion state we're restoring expansion state, or when showing filtered results. 
	if (bIsRestoringExpansion || FilteredRootNode != RootNode)
	{
		return;
	}

	if (ExpandedItem.IsValid() && ExpandedItem->CategoryPath.Num() > 0)
	{
		FCategoryExpansionState& ExpansionState = CategoryExpansionStates.FindOrAdd(CategoryKey);
		const FString Path = FString::Join(ExpandedItem->CategoryPath, TEXT("|"));
		if (bInExpanded)
		{
			ExpansionState.CollapsedCategories.Remove(Path);
		}
		else
		{
			ExpansionState.CollapsedCategories.Add(Path);
		}
	}
}

void SStateTreeNodeTypePicker::OnSearchBoxTextChanged(const FText& NewText)
{
	if (!NodeTypeTree.IsValid())
	{
		return;
	}
	
	FilteredRootNode.Reset();

	TArray<FString> FilterStrings;
	NewText.ToString().ParseIntoArrayWS(FilterStrings);
	FilterStrings.RemoveAllSwap([](const FString& String) { return String.IsEmpty(); });
	
	if (FilterStrings.IsEmpty())
	{
		// Show all when there's no filter string.
		FilteredRootNode = RootNode;
		NodeTypeTree->SetTreeItemsSource(&FilteredRootNode->Children);
		RestoreExpansionState();
		NodeTypeTree->RequestTreeRefresh();
		return;
	}

	FilteredRootNode = MakeShared<FStateTreeNodeTypeItem>();
	FilterNodeTypesChildren(FilterStrings, /*bParentMatches*/false, RootNode->Children, FilteredRootNode->Children);

	NodeTypeTree->SetTreeItemsSource(&FilteredRootNode->Children);
	ExpandAll(FilteredRootNode->Children);
	NodeTypeTree->RequestTreeRefresh();
}

int32 SStateTreeNodeTypePicker::FilterNodeTypesChildren(const TArray<FString>& FilterStrings, const bool bParentMatches, const TArray<TSharedPtr<FStateTreeNodeTypeItem>>& SourceArray, TArray<TSharedPtr<FStateTreeNodeTypeItem>>& OutDestArray)
{
	int32 NumFound = 0;

	auto MatchFilter = [&FilterStrings](const TSharedPtr<FStateTreeNodeTypeItem>& SourceItem)
	{
		const FString ItemName = SourceItem->Struct ? SourceItem->Struct->GetDisplayNameText().ToString() : SourceItem->GetCategoryName();
		for (const FString& Filter : FilterStrings)
		{
			if (!ItemName.Contains(Filter))
			{
				return false;
			}
		}
		return true;
	};

	for (const TSharedPtr<FStateTreeNodeTypeItem>& SourceItem : SourceArray)
	{
		// Check if our name matches the filters
		// If bParentMatches is true, the search matched a parent category.
		const bool bMatchesFilters = bParentMatches || MatchFilter(SourceItem);

		int32 NumChildren = 0;
		if (bMatchesFilters)
		{
			NumChildren++;
		}

		// if we don't match, then we still want to check all our children
		TArray<TSharedPtr<FStateTreeNodeTypeItem>> FilteredChildren;
		NumChildren += FilterNodeTypesChildren(FilterStrings, bMatchesFilters, SourceItem->Children, FilteredChildren);

		// then add this item to the destination array
		if (NumChildren > 0)
		{
			TSharedPtr<FStateTreeNodeTypeItem>& NewItem = OutDestArray.Add_GetRef(MakeShared<FStateTreeNodeTypeItem>());
			NewItem->CategoryPath = SourceItem->CategoryPath;
			NewItem->Struct = SourceItem->Struct; 
			NewItem->Children = FilteredChildren;

			NumFound += NumChildren;
		}
	}

	return NumFound;
}

TArray<TSharedPtr<SStateTreeNodeTypePicker::FStateTreeNodeTypeItem>> SStateTreeNodeTypePicker::GetPathToItemStruct(const UStruct* Struct) const
{
	TArray<TSharedPtr<FStateTreeNodeTypeItem>> Path;

	TSharedPtr<FStateTreeNodeTypeItem> CurrentParent = FilteredRootNode;

	if (Struct)
	{
		FText FullCategoryName = Struct->GetMetaDataText("Category");
		if (!FullCategoryName.IsEmpty())
		{
			TArray<FString> CategoryPath;
			FullCategoryName.ToString().ParseIntoArray(CategoryPath, TEXT("|"));

			for (const FString& SubCategory : CategoryPath)
			{
				const FString Trimmed = SubCategory.TrimStartAndEnd();

				TSharedPtr<FStateTreeNodeTypeItem>* FoundItem = 
					CurrentParent->Children.FindByPredicate([&Trimmed](const TSharedPtr<FStateTreeNodeTypeItem>& Item)
					{
						return Item->GetCategoryName() == Trimmed;
					});

				if (FoundItem != nullptr)
				{
					Path.Add(*FoundItem);
					CurrentParent = *FoundItem;
				}
			}
		}
	}

	const TSharedPtr<FStateTreeNodeTypeItem>* FoundItem = 
		CurrentParent->Children.FindByPredicate([Struct](const TSharedPtr<FStateTreeNodeTypeItem>& Item)
		{
			return Item->Struct == Struct;
		});

	if (FoundItem != nullptr)
	{
		Path.Add(*FoundItem);
	}

	return Path;
}

void SStateTreeNodeTypePicker::ExpandAll(const TArray<TSharedPtr<FStateTreeNodeTypeItem>>& Items)
{
	for (const TSharedPtr<FStateTreeNodeTypeItem>& Item : Items)
	{
		NodeTypeTree->SetItemExpansion(Item, true);
		ExpandAll(Item->Children);
	}
}

void SStateTreeNodeTypePicker::RestoreExpansionState()
{
	FCategoryExpansionState& ExpansionState = CategoryExpansionStates.FindOrAdd(CategoryKey);

	TSet<TSharedPtr<FStateTreeNodeTypeItem>> CollapseNodes;
	for (const FString& Category : ExpansionState.CollapsedCategories)
	{
		TArray<FString> CategoryPath;
		Category.ParseIntoArray(CategoryPath, TEXT("|"));

		TSharedPtr<FStateTreeNodeTypeItem> CurrentParent = RootNode;

		for (const FString& SubCategory : CategoryPath)
		{
			TSharedPtr<FStateTreeNodeTypeItem>* FoundItem = 
				CurrentParent->Children.FindByPredicate([&SubCategory](const TSharedPtr<FStateTreeNodeTypeItem>& Item)
				{
					return Item->GetCategoryName() == SubCategory;
				});

			if (FoundItem != nullptr)
			{
				CollapseNodes.Add(*FoundItem);
				CurrentParent = *FoundItem;
			}
		}
	}

	if (NodeTypeTree.IsValid())
	{
		bIsRestoringExpansion = true;

		ExpandAll(RootNode->Children);
		for (const TSharedPtr<FStateTreeNodeTypeItem>& Node : CollapseNodes)
		{
			NodeTypeTree->SetItemExpansion(Node, false);
		}

		bIsRestoringExpansion = false;
	}
}

#undef LOCTEXT_NAMESPACE