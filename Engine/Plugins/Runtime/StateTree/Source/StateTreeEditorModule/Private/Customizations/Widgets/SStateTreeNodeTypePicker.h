// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"
#include "Textures/SlateIcon.h"
#include "UObject/ObjectKey.h"

class UStruct;
class UScriptStruct;
class UClass;
class UStateTreeSchema;
class SSearchBox;

/**
 * Widget that displays a list of State Tree nodes which match base types and specified schema.
 * Can be used e.g. in popup menus to select node types.
 */
class SStateTreeNodeTypePicker : public SCompoundWidget
{
public:

	DECLARE_DELEGATE_OneParam(FOnNodeStructPicked, const UStruct*);
	
	SLATE_BEGIN_ARGS(SStateTreeNodeTypePicker)
		: _CurrentStruct(nullptr)
		, _BaseScriptStruct(nullptr)
		, _BaseClass(nullptr)
		, _Schema(nullptr)
	{}
		/** Currently selected struct, initially highlighted. */
		SLATE_ARGUMENT(const UStruct*, CurrentStruct)
		/** Base struct of the node */
		SLATE_ARGUMENT(const UScriptStruct*, BaseScriptStruct)
		/** Base class of the node */
		SLATE_ARGUMENT(const UClass*, BaseClass)
		/** Schema used to filter allowed types. */
		SLATE_ARGUMENT(const UStateTreeSchema*, Schema)
		/** Callback to call when a type is selected. */
		SLATE_ARGUMENT(FOnNodeStructPicked, OnNodeTypePicked)
	SLATE_END_ARGS()

	SStateTreeNodeTypePicker();
	virtual ~SStateTreeNodeTypePicker() override;

	void Construct(const FArguments& InArgs);

	/** @returns widget to focus (search box) when the picker is opened. */
	TSharedPtr<SWidget> GetWidgetToFocusOnOpen();

private:
	// Stores a category path segment, or a node type.
	struct FStateTreeNodeTypeItem
	{
		bool IsCategory() const { return CategoryPath.Num() > 0; }
		FString GetCategoryName() { return CategoryPath.Num() > 0 ? CategoryPath.Last() : FString(); }

		TArray<FString> CategoryPath;
		const UStruct* Struct = nullptr;
		FSlateIcon Icon;
		FSlateColor IconColor;
		TArray<TSharedPtr<FStateTreeNodeTypeItem>> Children;
	};

	// Stores per session node expansion state for a node type.
	struct FCategoryExpansionState
	{
		TSet<FString> CollapsedCategories;
	};

	static void SortNodeTypesFunctionItemsRecursive(TArray<TSharedPtr<FStateTreeNodeTypeItem>>& Items);
	static TSharedPtr<FStateTreeNodeTypeItem> FindOrCreateItemForCategory(TArray<TSharedPtr<FStateTreeNodeTypeItem>>& Items, TArrayView<FString> CategoryPath);
	void AddNode(const UStruct* Struct);
	void CacheNodeTypes(const UStateTreeSchema* Schema, const UScriptStruct* BaseScriptStruct, const UClass* BaseClass);

	TSharedRef<ITableRow> GenerateNodeTypeRow(TSharedPtr<FStateTreeNodeTypeItem> Item, const TSharedRef<STableViewBase>& OwnerTable);
	void GetNodeTypeChildren(TSharedPtr<FStateTreeNodeTypeItem> Item, TArray<TSharedPtr<FStateTreeNodeTypeItem>>& OutItems) const;
	void OnNodeTypeSelected(TSharedPtr<FStateTreeNodeTypeItem> SelectedItem, ESelectInfo::Type);
	void OnNodeTypeExpansionChanged(TSharedPtr<FStateTreeNodeTypeItem> ExpandedItem, bool bInExpanded);
	void OnSearchBoxTextChanged(const FText& NewText);
	int32 FilterNodeTypesChildren(const TArray<FString>& FilterStrings, const bool bParentMatches, const TArray<TSharedPtr<FStateTreeNodeTypeItem>>& SourceArray, TArray<TSharedPtr<FStateTreeNodeTypeItem>>& OutDestArray);
	void ExpandAll(const TArray<TSharedPtr<FStateTreeNodeTypeItem>>& Items);
	TArray<TSharedPtr<FStateTreeNodeTypeItem>> GetPathToItemStruct(const UStruct* Struct) const;

	void RestoreExpansionState();

	TSharedPtr<FStateTreeNodeTypeItem> RootNode;
	TSharedPtr<FStateTreeNodeTypeItem> FilteredRootNode;
	
	TSharedPtr<SSearchBox> SearchBox;
	TSharedPtr<STreeView<TSharedPtr<FStateTreeNodeTypeItem>>> NodeTypeTree;
	bool bIsRestoringExpansion = false;

	FOnNodeStructPicked OnNodeStructPicked;
	
	FObjectKey CategoryKey;

	// Save expansion state for each base node type. The expansion state does not persist between editor sessions. 
	static TMap<FObjectKey, FCategoryExpansionState> CategoryExpansionStates;
};