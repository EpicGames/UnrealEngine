// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStateTreeDebuggerEventTimelineView.h"
#include "Framework/Application/SlateApplication.h"
#include "Rendering/DrawElements.h"
#include "SSimpleTimeSlider.h"
#include "Fonts/FontMeasure.h"

#define LOCTEXT_NAMESPACE "SStateTreeDebuggerEventTimelineView"


void SStateTreeDebuggerEventTimelineView::Construct(const FArguments& InArgs)
{
	ViewRange = InArgs._ViewRange;
	DesiredSize = InArgs._DesiredSize;
	EventData = InArgs._EventData;

	EventBrush = FAppStyle::GetBrush("Sequencer.KeyDiamond");
	EventBorderBrush = FAppStyle::GetBrush("Sequencer.KeyDiamondBorder");
}

int32 SStateTreeDebuggerEventTimelineView::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, const int32 LayerId, const FWidgetStyle& InWidgetStyle, const bool bParentEnabled) const
{
	const int32 NewLayer = PaintEvents(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	return FMath::Max(NewLayer, SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, NewLayer, InWidgetStyle, ShouldBeEnabled(bParentEnabled)));
}

FVector2D SStateTreeDebuggerEventTimelineView::ComputeDesiredSize(float) const
{
	return DesiredSize.Get();
}

int32 SStateTreeDebuggerEventTimelineView::PaintEvents(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const TRange<double> DebugTimeRange = ViewRange.Get();

	const SSimpleTimeSlider::FScrubRangeToScreen RangeToScreen(DebugTimeRange, AllottedGeometry.GetLocalSize());

	if (const TSharedPtr<FTimelineEventData> TimelineEventData = EventData.Get())
	{
		const FSlateBrush* Brush = FAppStyle::GetBrush("Sequencer.SectionArea.Background");

		const int NumWindows = TimelineEventData->Windows.Num();
		if (NumWindows > 0)
		{
			for (int i = 0; i < NumWindows; i++)
			{
				const FTimelineEventData::EventWindow& Window = TimelineEventData->Windows[i];

				FVector2f EventSize = EventBrush->GetImageSize();

				const float XStart = RangeToScreen.InputToLocalX(static_cast<float>(Window.TimeStart));
				const float XEnd = RangeToScreen.InputToLocalX(static_cast<float>(Window.TimeEnd));
				const float Y = (AllottedGeometry.Size.Y - EventSize.Y) / 2.f;

				// Window bar
				constexpr float HSizeReduction = 2.f;
				constexpr float VSizeReduction = 2.f;
				constexpr float VerticalOffset = 0.5f * VSizeReduction;

				// Ensure at least 1 pixel remains after horizontal size reduction, else don't reduce size
				const float XDelta = (XEnd - XStart);
				const float Width = XDelta - 1 > HSizeReduction ? XDelta - HSizeReduction : XDelta;
				FSlateDrawElement::MakeBox
				(OutDrawElements,
					LayerId++,
					AllottedGeometry.ToPaintGeometry(FVector2f(Width, EventSize.Y - VSizeReduction), FSlateLayoutTransform(FVector2f(XStart, Y + VerticalOffset))),
					Brush,
					ESlateDrawEffect::None,
					Window.Color);

				if (Window.Description.IsEmpty() == false)
				{
					FString Description = Window.Description.ToString();					
					FSlateFontInfo FontInfo = FCoreStyle::GetDefaultFontStyle("Regular", 8);
					const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
					constexpr int32 TextLeftOffset = 8;
					const int32 HorizontalOffset = FMath::RoundToInt(XDelta - TextLeftOffset);
					const int32 LastWholeCharacterIndex = FontMeasureService->FindLastWholeCharacterIndexBeforeOffset(Description, FontInfo, HorizontalOffset);

					if (LastWholeCharacterIndex >= 0)
					{
						Description = Description.Left(LastWholeCharacterIndex + 1);

						FSlateDrawElement::MakeText(
							OutDrawElements,
							LayerId,
							AllottedGeometry.ToPaintGeometry(EventSize, FSlateLayoutTransform(FVector2D(XStart + TextLeftOffset, Y))),
							Description,
							FontInfo,
							ESlateDrawEffect::None,
							FLinearColor::Black);
					}
				}
			}
		}

		const int NumPoints = TimelineEventData->Points.Num();

		double PrevPointTime = 0;
		int OverlappingPointCount = 0;

		if (NumPoints > 0)
		{
			for (int i = 0; i < NumPoints; i++)
			{
				const FTimelineEventData::EventPoint& Point = TimelineEventData->Points[i];

				FVector2D EventSize = EventBrush->GetImageSize();

				float X = RangeToScreen.InputToLocalX(static_cast<float>(Point.Time));
				X = X - static_cast<float>(EventSize.X) / 2.f;
				float Y = (AllottedGeometry.GetLocalSize().Y - static_cast<float>(EventSize.Y)) / 2.f;
				if (Point.Time == PrevPointTime)
				{
					OverlappingPointCount++;

					static constexpr float OverlapOffsetAmount = 2.0f;
					Y += OverlapOffsetAmount * static_cast<float>(OverlappingPointCount);
				}
				else
				{
					OverlappingPointCount = 0;
				}

				PrevPointTime = Point.Time;

				FSlateDrawElement::MakeBox
				(OutDrawElements,
					LayerId,
					AllottedGeometry.ToPaintGeometry(EventSize, FSlateLayoutTransform(FVector2f(X, Y))),
					EventBrush,
					ESlateDrawEffect::None,
					Point.Color);

				FSlateDrawElement::MakeBox
				(OutDrawElements,
					LayerId,
					AllottedGeometry.ToPaintGeometry(EventSize, FSlateLayoutTransform(FVector2f(X, Y))),
					EventBorderBrush,
					ESlateDrawEffect::None,
					FLinearColor::Black);
			}
		}

		LayerId++;
	}

	return LayerId;
}

#undef LOCTEXT_NAMESPACE
