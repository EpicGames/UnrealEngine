// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FPaintArgs;
class FSlateWindowElementList;
class FTimeSliderController;


/**
 * Displays a sequence of points and windows for a statetree debugger instance timeline.
 * This is a slightly modified version of the SEventTimelineView used by RewindDebugger
 * so it should make it easier to eventually merge the tools.
 */
class SStateTreeDebuggerEventTimelineView : public SCompoundWidget
{
public:
	struct FTimelineEventData
	{
	public:
		struct EventPoint
		{
			double Time;
			FText Type;
			FText Description;
			FLinearColor Color;
		};
		
		struct EventWindow
		{
			double TimeStart;
			double TimeEnd;
			FText Type;
			FText Description;
			FLinearColor Color;
		};

		TArray<EventPoint> Points;
		TArray<EventWindow> Windows;
	};
	
	SLATE_BEGIN_ARGS(SStateTreeDebuggerEventTimelineView)
		: _ViewRange(TRange<double>(0,10))
		, _DesiredSize(FVector2D(100.f,20.f))
	{}
    	/** View time range */
    	SLATE_ATTRIBUTE(TRange<double>, ViewRange);

		/** Data for events to render */
		SLATE_ATTRIBUTE(TSharedPtr<FTimelineEventData>, EventData);

		/** Desired widget size */
		SLATE_ATTRIBUTE(FVector2D, DesiredSize);
	
	SLATE_END_ARGS()


	/**
	 * Construct the widget
	 * 
	 * @param InArgs A declaration from which to construct the widget
	 */
	void Construct( const FArguments& InArgs );

protected:
	// SWidget interface
	virtual FVector2D ComputeDesiredSize(float) const override;
	virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	
	int32 PaintEvents(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const;
	
	TAttribute<TRange<double>> ViewRange;
	TAttribute<TSharedPtr<FTimelineEventData>> EventData;
	TAttribute<FVector2D> DesiredSize;
	
	const FSlateBrush* EventBrush = nullptr;
	const FSlateBrush* EventBorderBrush = nullptr;
};

