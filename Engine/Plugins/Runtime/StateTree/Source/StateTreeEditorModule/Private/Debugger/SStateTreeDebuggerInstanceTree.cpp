// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_STATETREE_TRACE_DEBUGGER

#include "Debugger/SStateTreeDebuggerInstanceTree.h"
#include "RewindDebuggerTrack.h"
#include "StateTreeDebuggerTrack.h"

#include "Widgets/Images/SLayeredImage.h"

//----------------------------------------------------------------------//
// SStateTreeInstanceTree
//----------------------------------------------------------------------//
SStateTreeDebuggerInstanceTree::SStateTreeDebuggerInstanceTree()
{
}

SStateTreeDebuggerInstanceTree::~SStateTreeDebuggerInstanceTree()
{
}

TSharedRef<ITableRow> SStateTreeDebuggerInstanceTree::GenerateTreeRow(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> Item, const TSharedRef<STableViewBase>& OwnerTable) const
{
	const FSlateIcon ObjectIcon = Item->GetIcon();

	const TSharedRef<SLayeredImage> LayeredIcons = SNew(SLayeredImage)
				.DesiredSizeOverride(FVector2D(16, 16))
				.Image(ObjectIcon.GetIcon());

	if (ObjectIcon.GetOverlayIcon())
	{
		LayeredIcons->AddLayer(ObjectIcon.GetOverlayIcon());
	}
	
	return SNew(STableRow<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>, OwnerTable)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot().AutoWidth().Padding(2.f)
			[
				LayeredIcons
			]
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(1.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(Item->GetDisplayName())
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.ColorAndOpacity_Lambda([Item]()
				{
					if (const FStateTreeDebuggerBaseTrack* DebuggerTrack = static_cast<FStateTreeDebuggerBaseTrack*>(Item.Get()))
					{
						if (DebuggerTrack != nullptr && DebuggerTrack->IsStale())
						{
							return FSlateColor::UseSubduedForeground();
						}
					}
					return FSlateColor::UseForeground();
				})
			]
		];
}

void SStateTreeDebuggerInstanceTree::TreeExpansionChanged(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> Item, const bool bShouldBeExpanded) const
{
	Item->SetIsExpanded(bShouldBeExpanded);
	OnExpansionChanged.ExecuteIfBound();
}

TSharedPtr<RewindDebugger::FRewindDebuggerTrack> SStateTreeDebuggerInstanceTree::GetSelection() const
{
	TArray<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>> SelectedItems;
	const int32 NumSelected = TreeView->GetSelectedItems(SelectedItems);

	// TreeView uses 'SelectionMode(ESelectionMode::Single)' so number of selected items is 0 or 1
	return NumSelected > 0 ? SelectedItems.Top() : nullptr;	
}

void SStateTreeDebuggerInstanceTree::SetSelection(const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>& SelectedItem) const
{
	TreeView->SetSelection(SelectedItem);
}

void SStateTreeDebuggerInstanceTree::ScrollTo(const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>& SelectedItem) const
{
	TreeView->RequestScrollIntoView(SelectedItem);
}

void SStateTreeDebuggerInstanceTree::ScrollTo(const double ScrollOffset) const
{
	TreeView->SetScrollOffset(static_cast<float>(ScrollOffset));
}

void SStateTreeDebuggerInstanceTree::Construct(const FArguments& InArgs)
{
	InstanceTracks = InArgs._InstanceTracks;

	TreeView = SNew(STreeView<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>)
				.TreeItemsSource(InstanceTracks)
				.OnGenerateRow(this, &SStateTreeDebuggerInstanceTree::GenerateTreeRow)
				.OnGetChildren_Lambda([](const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>& Item, TArray<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>& OutChildren)
					{
						Item->IterateSubTracks([&OutChildren](const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>& Track)
						{
							if (Track->IsVisible())
							{
								OutChildren.Add(Track);
							}
						});
					})
				.OnExpansionChanged(this, &SStateTreeDebuggerInstanceTree::TreeExpansionChanged)
				.SelectionMode(ESelectionMode::Single)
				.OnSelectionChanged(InArgs._OnSelectionChanged)
				.OnMouseButtonDoubleClick(InArgs._OnMouseButtonDoubleClick)
				.ExternalScrollbar(InArgs._ExternalScrollBar)
				.AllowOverscroll(EAllowOverscroll::No)
				.OnTreeViewScrolled(InArgs._OnScrolled)
				.ScrollbarDragFocusCause(EFocusCause::SetDirectly)
				.OnContextMenuOpening(InArgs._OnContextMenuOpening);

	ChildSlot
	[
		TreeView.ToSharedRef()
	];

	OnExpansionChanged = InArgs._OnExpansionChanged;
}

namespace UE::StateTreeDebugger
{
static void RestoreExpansion(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> Track, TSharedPtr<STreeView<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>>& TreeView)
{
	TreeView->SetItemExpansion(Track, Track->GetIsExpanded());
	Track->IterateSubTracks([&TreeView](TSharedPtr<RewindDebugger::FRewindDebuggerTrack> SubTrack)
	{
		RestoreExpansion(SubTrack, TreeView);
	});
}
} // UE::StateTreeDebugger

void SStateTreeDebuggerInstanceTree::RestoreExpansion()
{
	for (const auto& Track : *InstanceTracks)
	{
		UE::StateTreeDebugger::RestoreExpansion(Track, TreeView);
	}
}

void SStateTreeDebuggerInstanceTree::Refresh()
{
	TreeView->RebuildList();

	if (InstanceTracks)
	{
		// make sure any newly added TreeView nodes are created expanded
		RestoreExpansion();
	}
}

#endif // WITH_STATETREE_TRACE_DEBUGGER