// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_STATETREE_TRACE_DEBUGGER

#include "Widgets/Views/STreeView.h"

namespace RewindDebugger
{
	class FRewindDebuggerTrack;
}

struct FStateTreeDebugger;

class SStateTreeDebuggerInstanceTree : public SCompoundWidget
{
	using FOnSelectionChanged = STreeView<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>::FOnSelectionChanged;
	using FOnMouseButtonDoubleClick = STreeView<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>::FOnMouseButtonDoubleClick;

public:
	SLATE_BEGIN_ARGS(SStateTreeDebuggerInstanceTree) { }
	SLATE_ARGUMENT(TArray<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>*, InstanceTracks);
	SLATE_ARGUMENT(TSharedPtr< SScrollBar >, ExternalScrollBar);
	SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)
	SLATE_EVENT(FOnMouseButtonDoubleClick, OnMouseButtonDoubleClick)
	SLATE_EVENT(FOnContextMenuOpening, OnContextMenuOpening)
	SLATE_EVENT(FSimpleDelegate, OnExpansionChanged)
	SLATE_EVENT(FOnTableViewScrolled, OnScrolled)
	SLATE_END_ARGS()

	SStateTreeDebuggerInstanceTree();
	virtual ~SStateTreeDebuggerInstanceTree();

	void Construct(const FArguments& InArgs);

	void Refresh();
	void RestoreExpansion();

	TSharedPtr<RewindDebugger::FRewindDebuggerTrack> GetSelection() const;
	void SetSelection(const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>& SelectedItem) const;
	void ScrollTo(const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>& SelectedItem) const;
	void ScrollTo(double ScrollOffset) const;

private:
	TSharedRef<ITableRow> GenerateTreeRow(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> Item, const TSharedRef<STableViewBase>& OwnerTable) const;
	void TreeExpansionChanged(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> Item, bool bShouldBeExpanded) const;
	
	TArray<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>* InstanceTracks = nullptr;

	TSharedPtr<STreeView<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>> TreeView;

	FSimpleDelegate OnExpansionChanged;	
};


#endif // WITH_STATETREE_TRACE_DEBUGGER