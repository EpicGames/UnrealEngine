// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_STATETREE_TRACE_DEBUGGER

#include "Debugger/SStateTreeDebuggerTimelines.h"
#include "Widgets/Layout/SSpacer.h"
#include "RewindDebuggerTrack.h"

//#define LOCTEXT_NAMESPACE "StateTreeEditor"


//----------------------------------------------------------------------//
// SRewindDebuggerTimelineTableRow
//----------------------------------------------------------------------//
class SRewindDebuggerTimelineTableRow : public STableRow<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>
{
	public:
	
	SLATE_BEGIN_ARGS(SRewindDebuggerTimelineTableRow) {}
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		// @todo: why not passing InArgs ???
		STableRow::Construct(STableRow::FArguments(), InOwnerTableView);

		SetExpanderArrowVisibility(EVisibility::Collapsed);
	}

	virtual int32 GetIndentLevel() const override
	{
		// don't indent timeline tracks
		return 0;
	}
};


//----------------------------------------------------------------------//
// SStateTreeInstanceTimelines
//----------------------------------------------------------------------//
SStateTreeDebuggerTimelines::SStateTreeDebuggerTimelines() 
	: SSimpleTimeSlider()
{ 
}

SStateTreeDebuggerTimelines::~SStateTreeDebuggerTimelines() 
{
}

void SStateTreeDebuggerTimelines::SetSelection(const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>& SelectedItem)
{
	InstanceTreeView->SetSelection(SelectedItem);
}

void SStateTreeDebuggerTimelines::ScrollTo(const double ScrollOffset)
{
	InstanceTreeView->SetScrollOffset(static_cast<float>(ScrollOffset));
}

TSharedRef<ITableRow> SStateTreeDebuggerTimelines::GenerateTreeRow(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	const TSharedPtr<SRewindDebuggerTimelineTableRow> Track = SNew(SRewindDebuggerTimelineTableRow, OwnerTable);
	TSharedPtr<SWidget> TimelineView = InItem->GetTimelineView();

	if (!TimelineView.IsValid())
	{
		TimelineView = SNew(SSpacer).Size(FVector2D(100, 20));
	}

	Track->SetContent(TimelineView.ToSharedRef());

	return Track.ToSharedRef();
}

namespace UE::StateTreeDebugger
{
void TimelineViewGetChildren(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> InItem, TArray<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>& OutChildren)
{
	InItem->IterateSubTracks([&OutChildren](TSharedPtr<RewindDebugger::FRewindDebuggerTrack> Track)
	{
		if (Track->IsVisible())
		{
			OutChildren.Add(Track);
		}
	});
}
} // UE::StateTreeDebugger

void SStateTreeDebuggerTimelines::TreeExpansionChanged(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> InItem, bool bShouldBeExpanded)
{
	InItem->SetIsExpanded(bShouldBeExpanded);
	OnExpansionChanged.ExecuteIfBound();
}

void SStateTreeDebuggerTimelines::Construct(const FArguments& InArgs)
{
	SSimpleTimeSlider::Construct(SSimpleTimeSlider::FArguments()
		.ViewRange(InArgs._ViewRange)
		.OnViewRangeChanged(InArgs._OnViewRangeChanged)
		.ClampRange(InArgs._ClampRange)
		.ScrubPosition(InArgs._ScrubPosition)
		.OnScrubPositionChanged(InArgs._OnScrubPositionChanged));

	InstanceTracks = InArgs._InstanceTracks;

	OnExpansionChanged = InArgs._OnExpansionChanged;

	InstanceTreeView = SNew(STreeView<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>)
									.OnSelectionChanged(InArgs._OnSelectionChanged)
									.TreeItemsSource(InstanceTracks)
									.OnGenerateRow(this, &SStateTreeDebuggerTimelines::GenerateTreeRow)
									.OnGetChildren_Static(&UE::StateTreeDebugger::TimelineViewGetChildren)
									.OnExpansionChanged(this, &SStateTreeDebuggerTimelines::TreeExpansionChanged)
									.SelectionMode(ESelectionMode::Single)
									.AllowOverscroll(EAllowOverscroll::No)
									.ExternalScrollbar(InArgs._ExternalScrollbar)
									.OnTreeViewScrolled(InArgs._OnScrolled);

	ChildSlot
	[
		InstanceTreeView.ToSharedRef()
	];
}

int32 SStateTreeDebuggerTimelines::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	const TRange<double> LocalViewRange = ViewRange.Get();
	const FScrubRangeToScreen RangeToScreen( LocalViewRange, AllottedGeometry.GetLocalSize() );

	// Paint Child Widgets (Tracks) 
	LayerId = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, ShouldBeEnabled(bParentEnabled));

	// Paint Major Time Marks
	FDrawTickArgs TickArgs;
	TickArgs.AllottedGeometry = AllottedGeometry;
	TickArgs.bOnlyDrawMajorTicks = true;
	TickArgs.TickColor = FLinearColor(0.2f,0.2f,0.2f,0.2f);
	TickArgs.ClippingRect = MyCullingRect;
	TickArgs.DrawEffects = ESlateDrawEffect::None;
	TickArgs.StartLayer = ++LayerId;
	TickArgs.TickOffset = 0;
	TickArgs.MajorTickHeight = AllottedGeometry.Size.Y;
	DrawTicks(OutDrawElements, RangeToScreen, TickArgs);
    	
	// Paint Cursor
	const float XPos = RangeToScreen.InputToLocalX(static_cast<float>(ScrubPosition.Get()));

	const TArray<FVector2D> Points {{XPos, 0}, { XPos, AllottedGeometry.Size.Y}};

	FSlateDrawElement::MakeLines(
				OutDrawElements,
						++LayerId,
						AllottedGeometry.ToPaintGeometry(),
						Points,
						ESlateDrawEffect::None,
						FLinearColor::White,
						false
						);

	return LayerId;
}

FVector2D SStateTreeDebuggerTimelines::ComputeDesiredSize(float X) const
{
	FVector2D DefaultSize = SSimpleTimeSlider::ComputeDesiredSize(X);
	if (InstanceTracks && InstanceTracks->Num() > 0)
	{
		return FVector2D(InstanceTracks->Num() * DefaultSize[0], InstanceTracks->Num() * DefaultSize[1]);
	}
	return DefaultSize;
}

namespace UE::StateTreeDebugger
{
static void RestoreTimelineExpansion(const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>& Track, TSharedPtr<STreeView<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>>& TreeView)
{
	TreeView->SetItemExpansion(Track, Track->GetIsExpanded());
	Track->IterateSubTracks([&TreeView](const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>& SubTrack)
		{
			RestoreTimelineExpansion(SubTrack, TreeView);
		});
}
} // UE::StateTreeDebugger

void SStateTreeDebuggerTimelines::RestoreExpansion()
{
	for (const auto& Track : *InstanceTracks)
	{
		UE::StateTreeDebugger::RestoreTimelineExpansion(Track, InstanceTreeView);
	}
}

void SStateTreeDebuggerTimelines::Refresh()
{
	InstanceTreeView->RebuildList();

	if (InstanceTracks)
	{
		// make sure any newly added TreeView nodes are created expanded
		RestoreExpansion();
	}
}

#endif // WITH_STATETREE_TRACE_DEBUGGER