// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_STATETREE_TRACE_DEBUGGER

#include "SSimpleTimeSlider.h"
#include "Widgets/Views/STreeView.h"

namespace RewindDebugger
{
	class FRewindDebuggerTrack;
}

struct FStateTreeDebugger;

class SStateTreeDebuggerTimelines : public SSimpleTimeSlider
{
	using FOnSelectionChanged = STreeView<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>::FOnSelectionChanged;
public:
	SLATE_BEGIN_ARGS(SStateTreeDebuggerTimelines) {}
	 
		SLATE_ARGUMENT( TArray< TSharedPtr< RewindDebugger::FRewindDebuggerTrack > >*, InstanceTracks );
		SLATE_ARGUMENT( TSharedPtr<SScrollBar>, ExternalScrollbar)
		SLATE_EVENT( FOnSelectionChanged, OnSelectionChanged )
		SLATE_EVENT( FOnTableViewScrolled, OnScrolled)
		SLATE_EVENT( FSimpleDelegate, OnExpansionChanged )

	
		/** The scrub position */
    	SLATE_ATTRIBUTE(double, ScrubPosition);
    
    	/** View time range */
    	SLATE_ATTRIBUTE(TRange<double>, ViewRange);
    
    	/** Clamp time range */
    	SLATE_ATTRIBUTE(TRange<double>, ClampRange);
    
		/** Called when the scrub position changes */
		SLATE_EVENT(FOnScrubPositionChanged, OnScrubPositionChanged);
	
		/** Called right before the scrubber begins to move */
		SLATE_EVENT(FSimpleDelegate, OnBeginScrubberMovement);
	
		/** Called right after the scrubber handle is released by the user */
		SLATE_EVENT(FSimpleDelegate, OnEndScrubberMovement);
	
    	/** Called when the view range changes */
    	SLATE_EVENT(FOnRangeChanged, OnViewRangeChanged);
    	
	SLATE_END_ARGS()

	/**
	* Default constructor.
	*/
	SStateTreeDebuggerTimelines();
	virtual ~SStateTreeDebuggerTimelines();

	/**
	* Constructs the application.
	*
	* @param InArgs The Slate argument list.
	*/
	void Construct(const FArguments& InArgs);

	void Refresh();
	void RestoreExpansion();

	void SetSelection(const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>& SelectedItem);
	void ScrollTo(double ScrollOffset);
	
	virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;

protected:
	virtual FVector2D ComputeDesiredSize(float) const override;

private:
	TSharedRef<ITableRow> GenerateTreeRow(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> InItem, const TSharedRef<STableViewBase>& OwnerTable);
	void TreeExpansionChanged(TSharedPtr<RewindDebugger::FRewindDebuggerTrack> InItem, bool bShouldBeExpanded);
	
	TArray<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>* InstanceTracks = nullptr;
	TSharedPtr<STreeView<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>>> InstanceTreeView;
	
	FSimpleDelegate OnExpansionChanged;
};

#endif // WITH_STATETREE_TRACE_DEBUGGER