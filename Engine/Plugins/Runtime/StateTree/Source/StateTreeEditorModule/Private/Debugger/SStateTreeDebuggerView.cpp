// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_STATETREE_TRACE_DEBUGGER

#include "SStateTreeDebuggerView.h"
#include "Debugger/SStateTreeFrameEventsView.h"
#include "Editor.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IGameplayProvider.h"
#include "Common/ProviderLock.h"
#include "IRewindDebuggerModule.h"
#include "Kismet2/DebuggerCommands.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "SStateTreeDebuggerInstanceTree.h"
#include "SStateTreeDebuggerTimelines.h"
#include "StateTreeDebuggerCommands.h"
#include "StateTreeDebuggerTrack.h"
#include "StateTreeDelegates.h"
#include "StateTreeEditorData.h"
#include "StateTreeEditorSettings.h"
#include "StateTreeEditorStyle.h"
#include "StateTreeEditorUserSettings.h"
#include "StateTreeModule.h"
#include "StateTreeViewModel.h"
#include "SWarningOrErrorBox.h"
#include "Toolkits/ToolkitManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SScrollBox.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"

//----------------------------------------------------------------------//
// SStateTreeDebuggerView
//----------------------------------------------------------------------//
SStateTreeDebuggerView::~SStateTreeDebuggerView()
{
	if (StateTreeViewModel)
	{
		StateTreeViewModel->RemoveAllBreakpoints();
	}

	UE::StateTree::Delegates::OnTracingStateChanged.RemoveAll(this);

	check(Debugger);
	Debugger->OnScrubStateChanged.Unbind();
	Debugger->OnBreakpointHit.Unbind();
	Debugger->OnNewSession.Unbind();
	Debugger->OnNewInstance.Unbind();

	Debugger->ClearSelection();

	FEditorDelegates::BeginPIE.Remove(BeginPIEHandle);
	FEditorDelegates::PausePIE.Remove(PausePIEHandle);
	FEditorDelegates::ResumePIE.Remove(ResumePIEHandle);
	FEditorDelegates::SingleStepPIE.Remove(SingleStepPIEHandle);
	FEditorDelegates::ShutdownPIE.Remove(ShutdownPIEHandle);
}

void SStateTreeDebuggerView::StartRecording()
{
	if (CanStartRecording())
	{
		check(Debugger);
		Debugger->ClearSelection();

		// Stop the current analysis so we have a chance to reconnect to an existing instance
		// if there is existing data, and it is still active.
		Debugger->StopSessionAnalysis();

		// We give priority to the Editor actions even if an analysis was active (remote process)
		// This will stop current analysis and connect to the new live trace.
		bRecording = Debugger->RequestAnalysisOfEditorSession();
	}
}

void SStateTreeDebuggerView::StopRecording()
{
	if (CanStopRecording())
	{
		// Calling StopTraces on the module will notify all registered views through OnTracesStateChanged delegate.
		IStateTreeModule& StateTreeModule = FModuleManager::GetModuleChecked<IStateTreeModule>("StateTreeModule");
		StateTreeModule.StopTraces();
	}
}

void SStateTreeDebuggerView::HandleTracesStateChanged(const EStateTreeTraceStatus TraceStatus)
{
	if (TraceStatus == EStateTreeTraceStatus::TracesStarted)
	{
		bRecording = true;
	}
	else
	{
		HandleTracesStopped();
	}
}

void SStateTreeDebuggerView::HandleTracesStopped()
{
	bRecording = false;

	// Update max duration from current recording until track data gets reset
	check(Debugger);
	MaxTrackRecordingDuration = FMath::Max(MaxTrackRecordingDuration, ExtrapolatedRecordedWorldTime);

	// Mark all tracks from the stopped session as stale to have different look.
	for (const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>& DebugTrack : InstanceOwnerTracks)
	{
		if (FStateTreeDebuggerBaseTrack* StateTreeTrack = static_cast<FStateTreeDebuggerBaseTrack*>(DebugTrack.Get()))
		{
			StateTreeTrack->MarkAsStale(ExtrapolatedRecordedWorldTime);
		}
	}

	// Reset the extrapolated time for next runs
	ExtrapolatedRecordedWorldTime = 0;
}

bool SStateTreeDebuggerView::CanResumeDebuggerAnalysis() const
{
	check(Debugger);
	return Debugger->IsAnalysisSessionPaused() && Debugger->HasHitBreakpoint();
}

void SStateTreeDebuggerView::ResumeDebuggerAnalysis() const
{
	check(Debugger);
	if (Debugger->IsAnalysisSessionPaused())
	{
		Debugger->ResumeSessionAnalysis();
	}

	// Resume PIE session if possible
	if (FPlayWorldCommands::Get().ResumePlaySession.IsValid())
	{
		if (CommandList->CanExecuteAction(FPlayWorldCommands::Get().ResumePlaySession.ToSharedRef()))
		{
			CommandList->ExecuteAction(FPlayWorldCommands::Get().ResumePlaySession.ToSharedRef());
		}
	}
}

bool SStateTreeDebuggerView::CanResetTracks() const
{
	return !InstanceOwnerTracks.IsEmpty();
}

void SStateTreeDebuggerView::ResetTracks()
{
	InstanceOwnerTracks.Reset();

	check(Debugger);
	Debugger->ResetEventCollections();

	MaxTrackRecordingDuration = 0;

	// Refresh tree view
	if (InstancesTreeView)
	{
		InstancesTreeView->Refresh();
	}

	if (InstanceTimelinesTreeView)
	{
		InstanceTimelinesTreeView->Refresh();
	}
}

void SStateTreeDebuggerView::OnPIEStarted(const bool bIsSimulating)
{
	if (UStateTreeEditorSettings::Get().bEnableLegacyDebuggerWindow
		&& UStateTreeEditorSettings::Get().bShouldDebuggerAutoRecordOnPIE)
	{
		StartRecording();
	}
}

void SStateTreeDebuggerView::OnPIEStopped(const bool bIsSimulating)
{
	if (UStateTreeEditorSettings::Get().bEnableLegacyDebuggerWindow)
	{
		StopRecording();
	}

	check(Debugger);
	Debugger->StopSessionAnalysis();
}

void SStateTreeDebuggerView::OnPIEPaused(const bool bIsSimulating) const
{
	check(Debugger);
	Debugger->PauseSessionAnalysis();
}

void SStateTreeDebuggerView::OnPIEResumed(const bool bIsSimulating) const
{
	check(Debugger);
	Debugger->ResumeSessionAnalysis();
}

void SStateTreeDebuggerView::OnPIESingleStepped(bool bSimulating) const
{
	check(Debugger);
	Debugger->SyncToCurrentSessionDuration();
}

void SStateTreeDebuggerView::Construct(const FArguments& InArgs
	, const TNotNull<const UStateTree*> InStateTree
	, const TSharedRef<FStateTreeViewModel>& InStateTreeViewModel
	, const TSharedRef<FUICommandList>& InCommandList)
{
	CommandList = InCommandList;

	StateTreeViewModel = InStateTreeViewModel;
	StateTree = InStateTree;
	StateTreeEditorData = Cast<UStateTreeEditorData>(InStateTree->EditorData.Get());
	Debugger = InStateTreeViewModel->GetDebugger();
	check(Debugger);

	const IStateTreeModule& StateTreeModule = FModuleManager::GetModuleChecked<IStateTreeModule>("StateTreeModule");
	bRecording = StateTreeModule.IsTracing();
	UE::StateTree::Delegates::OnTracingStateChanged.AddSP(this, &SStateTreeDebuggerView::HandleTracesStateChanged);

	Debugger->OnBreakpointHit.BindSP(this, &SStateTreeDebuggerView::OnBreakpointHit, InCommandList);

	// Put debugger in proper simulation state when view is constructed after PIE/SIE was started
	if (FPlayWorldCommandCallbacks::HasPlayWorldAndPaused())
	{
		Debugger->PauseSessionAnalysis();
	}

	BeginPIEHandle = FEditorDelegates::BeginPIE.AddSP(this, &SStateTreeDebuggerView::OnPIEStarted);
	PausePIEHandle = FEditorDelegates::PausePIE.AddSP(this, &SStateTreeDebuggerView::OnPIEPaused);
	ResumePIEHandle = FEditorDelegates::ResumePIE.AddSP(this, &SStateTreeDebuggerView::OnPIEResumed);
	SingleStepPIEHandle = FEditorDelegates::SingleStepPIE.AddSP(this, &SStateTreeDebuggerView::OnPIESingleStepped);
	// Using ShutdownPIE instead of EndPIE to make sure all traces emitted during world EndPlay get processed before disabling channels
	ShutdownPIEHandle = FEditorDelegates::ShutdownPIE.AddSP(this, &SStateTreeDebuggerView::OnPIEStopped);

	if (GetDefault<UStateTreeEditorSettings>()->bEnableLegacyDebuggerWindow)
	{
		ConstructLegacyView(InCommandList);
	}
	else
	{
		ConstructView(InCommandList);
	}

	if (bRecording && !Debugger->IsAnalysisSessionActive())
	{
		// Auto-select session if there is only one available and that we are tracing
		// Do that after creating all our widgets in case we receive a callback
		TArray<FStateTreeDebugger::FTraceDescriptor> TraceDescriptors;
		Debugger->GetLiveTraces(TraceDescriptors);
		if (TraceDescriptors.Num() == 1)
		{
			Debugger->RequestSessionAnalysis(TraceDescriptors.Last());
		}
	}
}

void SStateTreeDebuggerView::ConstructLegacyView(const TSharedRef<FUICommandList>& InCommandList)
{
	// Registers a delegate to be notified when the associated StateTree asset get successfully recompiled
	// to clear any previous recorded data that could mismatch new compiled tree data version.
	UE::StateTree::Delegates::OnPostCompile.AddSPLambda(this, [this](const UStateTree& RecompiledStateTree)
		{
			if (StateTree == &RecompiledStateTree)
			{
				ResetTracks();
			}
		});

	// Bind callbacks to the debugger delegates
	Debugger->OnNewSession.BindSP(this, &SStateTreeDebuggerView::OnNewSession);
	Debugger->OnNewInstance.BindSP(this, &SStateTreeDebuggerView::OnNewInstance);
	Debugger->OnScrubStateChanged.BindSP(this, &SStateTreeDebuggerView::OnDebuggerScrubStateChanged);

	// Bind our scrub time attribute to follow the value computed by the debugger
	ScrubTimeAttribute = TAttribute<double>(StateTreeViewModel->GetDebugger(), &FStateTreeDebugger::GetScrubTime);

	// Add & Bind commands
	BindDebuggerToolbarCommands(InCommandList);

	// Register the play world commands
	InCommandList->Append(FPlayWorldCommands::GlobalPlayWorldActions.ToSharedRef());

	// Toolbars
	FSlimHorizontalToolBarBuilder LeftToolbar(InCommandList, FMultiBoxCustomization::None, /*InExtender*/ nullptr, /*InForceSmallIcons*/ true);
	LeftToolbar.BeginSection(TEXT("Debugging"));
	{
		LeftToolbar.BeginStyleOverride(FName("Toolbar.BackplateLeft"));
		const FPlayWorldCommands& PlayWorldCommands = FPlayWorldCommands::Get();
		LeftToolbar.AddToolBarButton(PlayWorldCommands.RepeatLastPlay);
		LeftToolbar.AddToolBarButton(PlayWorldCommands.PausePlaySession,
					NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlayWorld.PausePlaySession.Small"));
		LeftToolbar.AddToolBarButton(PlayWorldCommands.ResumePlaySession,
					NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlayWorld.ResumePlaySession.Small"));
		LeftToolbar.BeginStyleOverride(FName("Toolbar.BackplateRight"));
		LeftToolbar.AddToolBarButton(PlayWorldCommands.StopPlaySession,
					NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlayWorld.StopPlaySession.Small"));
		LeftToolbar.EndStyleOverride();

		LeftToolbar.AddSeparator();

		const FStateTreeDebuggerCommands& DebuggerCommands = FStateTreeDebuggerCommands::Get();
		LeftToolbar.AddToolBarButton(DebuggerCommands.StartRecording);
		LeftToolbar.AddToolBarButton(DebuggerCommands.StopRecording);

		LeftToolbar.AddSeparator();

		LeftToolbar.AddToolBarButton(DebuggerCommands.ResumeDebuggerAnalysis);

		LeftToolbar.AddToolBarButton(DebuggerCommands.PreviousFrameWithStateChange);
		LeftToolbar.AddToolBarButton(DebuggerCommands.PreviousFrameWithEvents);
		LeftToolbar.AddToolBarButton(DebuggerCommands.NextFrameWithEvents);
		LeftToolbar.AddToolBarButton(DebuggerCommands.NextFrameWithStateChange);

		LeftToolbar.AddToolBarButton(DebuggerCommands.ResetTracks);
	}
	LeftToolbar.EndSection();

	FSlimHorizontalToolBarBuilder RightToolbar(nullptr, FMultiBoxCustomization::None);
	RightToolbar.BeginSection("Auto-Scroll");
	{
		FUIAction AutoScrollToggleButtonAction;
		AutoScrollToggleButtonAction.GetActionCheckState.BindSPLambda(this, [&bAutoScroll=bAutoScroll]
		{
			return bAutoScroll ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		});
		AutoScrollToggleButtonAction.ExecuteAction.BindSPLambda(this, [&bAutoScroll=bAutoScroll]
		{
			bAutoScroll = !bAutoScroll;
		});

		RightToolbar.AddToolBarButton
		(
			AutoScrollToggleButtonAction,
			NAME_None,
			TAttribute<FText>(),
			LOCTEXT("AutoScrollToolTip", "Auto-Scroll"),
			FSlateIcon(FStateTreeEditorStyle::Get().GetStyleSetName(), "StateTreeEditor.AutoScroll"),
			EUserInterfaceActionType::ToggleButton);
	}
	RightToolbar.EndSection();

	// Placeholder toolbar for now but the intent to add more functionalities (e.g. Search)
	FSlimHorizontalToolBarBuilder FrameDetailsToolbar(nullptr, FMultiBoxCustomization::None);
	FrameDetailsToolbar.BeginSection("FrameDetails");
	{
		FrameDetailsToolbar.AddWidget(SNew(STextBlock).Text(FText::FromString(GetNameSafe(StateTree.Get()))));
	}
	FrameDetailsToolbar.EndSection();

	// Trace selection combo
	const TSharedRef<SWidget> TraceSelectionBox = SNew(SComboButton)
		.OnGetMenuContent(this, &SStateTreeDebuggerView::OnGetDebuggerTracesMenu)
		.ButtonContent()
		[
			SNew(STextBlock)
			.ToolTipText(LOCTEXT("SelectTraceSession", "Pick trace session to debug"))
			.Text_Lambda([Debugger = Debugger]()
			{
				check(Debugger);
				return Debugger->GetSelectedTraceDescription();
			})
		];

	const TSharedPtr<SScrollBar> ScrollBar = SNew(SScrollBar);

	// Instances TreeView
	InstancesTreeView = SNew(SStateTreeDebuggerInstanceTree)
		.ExternalScrollBar(ScrollBar)
		.OnExpansionChanged_Lambda([this]() { InstanceTimelinesTreeView->RestoreExpansion(); })
		.OnScrolled_Lambda([this](double ScrollOffset)
		{
			InstanceTimelinesTreeView->ScrollTo(ScrollOffset);
		})
		.InstanceTracks(&InstanceOwnerTracks)
		.OnSelectionChanged_Lambda([this](TSharedPtr<RewindDebugger::FRewindDebuggerTrack> SelectedItem, ESelectInfo::Type SelectInfo)
			{
				InstanceTimelinesTreeView->SetSelection(SelectedItem);

				if (FStateTreeDebuggerBaseTrack* StateTreeTrack = static_cast<FStateTreeDebuggerBaseTrack*>(SelectedItem.Get()))
				{
					StateTreeTrack->OnSelected();
				}
			});

	// Timelines TreeView
	InstanceTimelinesTreeView = SNew(SStateTreeDebuggerTimelines)
		.ExternalScrollbar(ScrollBar)
		.OnExpansionChanged_Lambda([this]() { InstancesTreeView->RestoreExpansion(); })
		.OnScrolled_Lambda([this](double ScrollOffset) { InstancesTreeView->ScrollTo(ScrollOffset); })
		.InstanceTracks(&InstanceOwnerTracks)
		.ViewRange_Lambda([this]() { return ViewRange; })
		.ClampRange_Lambda([this]() { return TRange<double>(0, MaxTrackRecordingDuration); })
		.OnViewRangeChanged_Lambda([this](TRange<double> NewRange) { ViewRange = NewRange; })
		.ScrubPosition(ScrubTimeAttribute)
		.OnScrubPositionChanged_Lambda([this](double NewScrubTime, bool bIsScrubbing) { OnTimeLineScrubPositionChanged(NewScrubTime, bIsScrubbing); });

	ChildSlot
	[
		SNew(SBorder)
		.Padding(4.0f)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				SNew(SSplitter)
				.Orientation(Orient_Horizontal)
				+ SSplitter::Slot()
				[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Fill)
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.HAlign(HAlign_Left)
							.FillContentWidth(1.f)
							[
								LeftToolbar.MakeWidget()
							]
							+ SHorizontalBox::Slot()
							.HAlign(HAlign_Right)
							.FillContentWidth(1.f)
							[
								RightToolbar.MakeWidget()
							]
						]
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SAssignNew(HeaderSplitter, SSplitter)
							.Orientation(Orient_Horizontal)
							+ SSplitter::Slot()
							.Value(0.2f)
							.Resizable(false)
							[
								TraceSelectionBox
							]
							+ SSplitter::Slot()
							.Resizable(false)
							[
								SNew(SSimpleTimeSlider)
								.DesiredSize({100, 24})
								.ClampRangeHighlightSize(0.15f)
								.ClampRangeHighlightColor(FLinearColor::Red.CopyWithNewOpacity(0.5f))
								.ScrubPosition(ScrubTimeAttribute)
								.ViewRange_Lambda([this]() { return ViewRange; })
								.OnViewRangeChanged_Lambda([this](TRange<double> NewRange) { ViewRange = NewRange; })
								.ClampRange_Lambda([this]() { return TRange<double>(0, MaxTrackRecordingDuration); })
								.OnScrubPositionChanged_Lambda([this](double NewScrubTime, bool bIsScrubbing) { OnTimeLineScrubPositionChanged(NewScrubTime, bIsScrubbing); })
							]
						]
						+ SVerticalBox::Slot()
						.FillHeight(1.0f)
						[
							SAssignNew(TreeViewsSplitter, SSplitter)
							.Orientation(Orient_Horizontal)
							+ SSplitter::Slot()
							.Value(0.2f)
							.OnSlotResized_Lambda([this](float Size)
								{
									// Sync both header and content
									TreeViewsSplitter->SlotAt(0).SetSizeValue(Size);
									HeaderSplitter->SlotAt(0).SetSizeValue(Size);
								})
							[
								SNew(SScrollBox)
								.Orientation(Orient_Horizontal)
								+ SScrollBox::Slot()
								.FillSize(1.0f)
								[
									InstancesTreeView.ToSharedRef()
								]
							]
							+ SSplitter::Slot()
							.OnSlotResized_Lambda([this](float Size)
								{
									TreeViewsSplitter->SlotAt(1).SetSizeValue(Size);
									HeaderSplitter->SlotAt(1).SetSizeValue(Size);
								})
							[
								SNew(SOverlay)
								+ SOverlay::Slot()
								[
									InstanceTimelinesTreeView.ToSharedRef()
								]
								+ SOverlay::Slot().HAlign(HAlign_Right)
								[
									ScrollBar.ToSharedRef()
								]
							]
						]
				]
				+ SSplitter::Slot()
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Fill)
					[
						FrameDetailsToolbar.MakeWidget()
					]
					+ SVerticalBox::Slot()
					.FillHeight(1.0f)
					[
						SAssignNew(EventsView, UE::StateTreeDebugger::SFrameEventsView, StateTree.Get())
					]
				]
			]
		]
	];

	if (Debugger->IsAnalysisSessionActive())
	{
		// Creating the view for an already active analysis session
		// so rebuild track views for all instances and scrub to the analysis current time

		TArray<const TSharedRef<const UE::StateTreeDebugger::FInstanceDescriptor>> InstanceDescriptors;
		Debugger->GetSessionInstanceDescriptors(InstanceDescriptors);
		for (const TSharedRef<const UE::StateTreeDebugger::FInstanceDescriptor>& InstanceDescriptor : InstanceDescriptors)
		{
			OnNewInstance(InstanceDescriptor->Id);
		}

		Debugger->SetScrubTime(Debugger->GetScrubTime());
	}
}

void SStateTreeDebuggerView::ConstructView(const TSharedRef<FUICommandList>& InCommandList)
{
	// Add & Bind commands
	BindDebuggingToolbarCommands(InCommandList);

	// Register the play world commands (to pause/resume PIE when hitting breakpoint)
	InCommandList->Append(FPlayWorldCommands::GlobalPlayWorldActions.ToSharedRef());

	// Toolbars
	FSlimHorizontalToolBarBuilder LeftToolbar(InCommandList, FMultiBoxCustomization::None, /*InExtender*/ nullptr, /*InForceSmallIcons*/ true);
	LeftToolbar.BeginSection(TEXT("Debugging"));
	{
		const FStateTreeDebuggerCommands& DebuggerCommands = FStateTreeDebuggerCommands::Get();

		LeftToolbar.SetLabelVisibility(EVisibility::Visible);
		LeftToolbar.AddToolBarButton(DebuggerCommands.OpenRewindDebugger);
		LeftToolbar.SetLabelVisibility(EVisibility::Collapsed);
		LeftToolbar.AddToolBarButton(DebuggerCommands.ResumeDebuggerAnalysis);
	}
	LeftToolbar.EndSection();

	// Instance selection combo
	const TSharedRef<SWidget> InstanceSelectionBox = SNew(SComboButton)
		.OnGetMenuContent_Lambda([Debugger = Debugger]
			{
				TArray<const TSharedRef<const UE::StateTreeDebugger::FInstanceDescriptor>> InstanceDescriptors;
				Debugger->GetSessionInstanceDescriptors(InstanceDescriptors);

				FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection*/true, /*InCommandList*/nullptr);
				for (const TSharedRef<const UE::StateTreeDebugger::FInstanceDescriptor>& InstanceDescriptor : InstanceDescriptors)
				{
					// Add instance to the list if the asset associated to the view is the instance's main asset, or one of its linked assets.
					if (InstanceDescriptor.Get().StateTree != Debugger->GetAsset()
						&& !InstanceDescriptor.Get().LinkedStateTrees.Contains(Debugger->GetAsset()))
					{
						continue;
					}

					const FText Label = Debugger->DescribeInstance(InstanceDescriptor.Get());

					FUIAction ItemAction(FExecuteAction::CreateSPLambda(Debugger.ToSharedRef(), [Debugger, InstanceId = InstanceDescriptor.Get().Id]
						{
							Debugger->SelectInstance(InstanceId);

							// Make sure to open RewindDebugger to create the global instance
							// or to bring focus on the selected instance
							if (CanOpenRewindDebugger())
							{
								OpenRewindDebugger();
							}

							IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
							if (RewindDebugger == nullptr)
							{
								UE_LOGF(LogStateTree, Error, "Unable to access or create instance of RewindDebugger");
								return;
							}

							// By default, we set the debug object associated to the StateTree instance
							RewindDebugger::FObjectId ObjectToDebug{InstanceId.ToUint64()};

							// Then if possible we traverse its outer chain until the last outer before the level or world.
							if (const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession())
							{
								const IGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<IGameplayProvider>("GameplayProvider");
								TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
								const FClassInfo* LevelClass = GameplayProvider->FindClassInfo(TEXT("/Script/Engine.Level"));
								const FClassInfo* WorldClass = GameplayProvider->FindClassInfo(TEXT("/Script/Engine.World"));
								const FObjectInfo* ObjectInfo = GameplayProvider->FindObjectInfo(RewindDebugger::FObjectId{InstanceId.ToUint64()});
								while (ObjectInfo != nullptr)
								{
									if ((LevelClass != nullptr && ObjectInfo->ClassId == LevelClass->Id)
										|| (WorldClass != nullptr && ObjectInfo->ClassId == WorldClass->Id))
									{
										break;
									}

									ObjectToDebug = ObjectInfo->GetId();
									ObjectInfo = GameplayProvider->FindObjectInfo(ObjectInfo->GetOuterId());
								}
							}
							RewindDebugger->SetObjectToDebug(ObjectToDebug);
							RewindDebugger->SelectTrack(RewindDebugger::FObjectId{InstanceId.ToUint64()});
						}));
					MenuBuilder.AddMenuEntry(Label, TAttribute<FText>(), FSlateIcon(), ItemAction);
				}

				// Failsafe when no match
				if (InstanceDescriptors.Num() == 0)
				{
					const FText Desc = LOCTEXT("NoActiveInstances", "Can't find active instances");
					MenuBuilder.AddMenuEntry(Desc, TAttribute<FText>(), FSlateIcon(), FUIAction());
				}

				return MenuBuilder.MakeWidget();
			})
		.ButtonContent()
		[
			SNew(STextBlock)
				.ToolTipText(LOCTEXT("SelectInstance", "Pick instance to debug"))
				.Text_Lambda([Debugger = Debugger]
					{
						check(Debugger);
						const TSharedPtr<const UE::StateTreeDebugger::FInstanceDescriptor> DebuggedInstance = Debugger->GetSelectedDescriptor();
						return Debugger->DescribeInstance(DebuggedInstance ? *DebuggedInstance : UE::StateTreeDebugger::FInstanceDescriptor{});
					})
		];

	// Deprecation warning message. Only visible when bDisplayMigrationToRewindDebuggerWarning.
	TSharedRef<SWidget> MigrationToRewindDebugger = SNullWidget::NullWidget;
	bDisplayMigrationToRewindDebuggerWarning = GetDefault<UStateTreeEditorUserSettings>()->GetDisplayMigrationToRewindDebuggerWarning();
	if (bDisplayMigrationToRewindDebuggerWarning)
	{
		const FButtonStyle* const CloseButtonStyle = &FAppStyle::Get().GetWidgetStyle<FDockTabStyle>("Docking.Tab").CloseButtonStyle;
		MigrationToRewindDebugger = SNew(SWarningOrErrorBox)
			.MessageStyle(EMessageStyle::Warning)
			.Message(LOCTEXT("MigrationToRewindDebugger",
				"Note that recording and visualization of the recorded data has been migrated to RewindDebugger.\n"
				"It is still possible to use the previous visualization by using 'Enable Legacy Debugger Window'"
				" under the StateTree Editor plugin section of the Editor Preferences."))
			.Visibility(MakeAttributeSPLambda(AsShared(), [Self = this]() -> EVisibility
				{
					return Self->bDisplayMigrationToRewindDebuggerWarning ? EVisibility::Visible : EVisibility::Collapsed;
				}))
			.ContentVAlign(EVerticalAlignment::VAlign_Top)
			[
				SNew(SButton)
					.ButtonStyle(CloseButtonStyle)
					.ContentPadding(FMargin(0.0f, 1.5f, 0.0f, 0.0f))
					.ToolTipText(LOCTEXT("MigrationToRewindDebuggerCloseTooltip", "Close the warning box."))
					.OnClicked(FOnClicked::CreateSPLambda(AsShared(), [Self = this]() -> FReply
						{
							Self->bDisplayMigrationToRewindDebuggerWarning = false;
							GetMutableDefault<UStateTreeEditorUserSettings>()->SetDisplayMigrationToRewindDebuggerWarning(false);
							return FReply::Handled();
						}))
					[
						SNew(SSpacer)
						.Size(CloseButtonStyle->Normal.ImageSize)
					]
			];
	}

	ChildSlot
	[
		SNew(SBorder)
		.Padding(4.0f)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.MinHeight(48.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					LeftToolbar.MakeWidget()
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					InstanceSelectionBox
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				MigrationToRewindDebugger
			]
		]
	];
}

FReply SStateTreeDebuggerView::OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	// We consider the key as handled regardless if the action can be executed or not since we don't want
	// some of them affecting other widgets once the action can no longer be executed
	// (e.g. can no longer scrub once reaching beginning or end of the timeline)
	// This is why we test the input manually instead of relying on ProcessCommandBindings.
	const FInputChord InputChord(
		InKeyEvent.GetKey(),
		EModifierKey::FromBools(InKeyEvent.IsControlDown(), InKeyEvent.IsAltDown(), InKeyEvent.IsShiftDown(), InKeyEvent.IsCommandDown()));

	const TSharedPtr<FUICommandInfo> CommandInfo = FInputBindingManager::Get().FindCommandInContext(
		FStateTreeDebuggerCommands::Get().GetContextName(), InputChord, /*bCheckDefault*/false);

	if (CommandInfo.IsValid())
	{
		CommandList->TryExecuteAction(CommandInfo.ToSharedRef());
		return FReply::Handled();
	}

	return SCompoundWidget::OnPreviewKeyDown(MyGeometry, InKeyEvent);
}

void SStateTreeDebuggerView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_StateTreeDebuggerView_TickView);

	check(Debugger);

	const double LastProcessedRecordedWorldTime = Debugger->GetLastProcessedRecordedWorldTime();
	const bool bHasMoreRecentData = LastUpdatedTrackRecordingDuration != LastProcessedRecordedWorldTime;
	LastUpdatedTrackRecordingDuration = LastProcessedRecordedWorldTime;

	if (bHasMoreRecentData)
	{
		// Prevent extrapolated value from going back, can only catch up
		ExtrapolatedRecordedWorldTime = FMath::Max(LastProcessedRecordedWorldTime, ExtrapolatedRecordedWorldTime);
	}

	MaxTrackRecordingDuration = FMath::Max(MaxTrackRecordingDuration, MaxTrackRecordingDuration);

	if ((Debugger->IsAnalysisSessionActive() && !Debugger->IsAnalysisSessionPaused())
		|| bHasMoreRecentData)
	{
		// When no events are processed this frame we force timeline to progress to make UI feel more responsive,
		// This is useful when debugging a live recording where events are produced in realtime and can't be read faster from a saved recording.
		// Note that we update regardless of autoscroll to maintain a valid value if it gets enabled.
		if (!bHasMoreRecentData && bRecording)
		{
			ExtrapolatedRecordedWorldTime += InDeltaTime;
		}

		if (bAutoScroll)
		{
			// Stick to most recent data if auto scroll is enabled.
			// Autoscroll is disabled when paused.
			// This allows the user to pause the analysis, inspect the data, and continue and the autoscroll will catch up with latest.
			// Complementary logic in OnTimeLineScrubPositionChanged().
			Debugger->SetScrubTime(ExtrapolatedRecordedWorldTime);
		}
		else
		{
			// Set scrub time to self to request update the UI.
			Debugger->SetScrubTime(Debugger->GetScrubTime());
		}
	}

	RefreshTracks();
}

void SStateTreeDebuggerView::RefreshTracks()
{
	bool bChanged = false;
	for (const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>& DebugTrack : InstanceOwnerTracks)
	{
		bChanged = DebugTrack->Update() || bChanged;
	}

	if (bChanged)
	{
		if (InstancesTreeView)
		{
			InstancesTreeView->Refresh();
		}

		if (InstanceTimelinesTreeView)
		{
			InstanceTimelinesTreeView->Refresh();
		}

		TrackCursor();
	}
}

void SStateTreeDebuggerView::BindDebuggerToolbarCommands(const TSharedRef<FUICommandList>& ToolkitCommands)
{
	const FStateTreeDebuggerCommands& Commands = FStateTreeDebuggerCommands::Get();

	ToolkitCommands->MapAction(
		Commands.StartRecording,
		FExecuteAction::CreateSP(this, &SStateTreeDebuggerView::StartRecording),
		FCanExecuteAction::CreateSP(this, &SStateTreeDebuggerView::CanStartRecording),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateLambda([this]() { return !IsRecording();}));

	ToolkitCommands->MapAction(
		Commands.StopRecording,
		FExecuteAction::CreateSP(this, &SStateTreeDebuggerView::StopRecording),
		FCanExecuteAction::CreateSP(this, &SStateTreeDebuggerView::CanStopRecording),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateRaw(this, &SStateTreeDebuggerView::CanStopRecording));

	ToolkitCommands->MapAction(
		Commands.PreviousFrameWithStateChange,
		FExecuteAction::CreateSP(this, &SStateTreeDebuggerView::StepBackToPreviousStateChange),
		FCanExecuteAction::CreateSP(this, &SStateTreeDebuggerView::CanStepBackToPreviousStateChange));

	ToolkitCommands->MapAction(
		Commands.PreviousFrameWithEvents,
		FExecuteAction::CreateSP(this, &SStateTreeDebuggerView::StepBackToPreviousStateWithEvents),
		FCanExecuteAction::CreateSP(this, &SStateTreeDebuggerView::CanStepBackToPreviousStateWithEvents));

	ToolkitCommands->MapAction(
		Commands.NextFrameWithEvents,
		FExecuteAction::CreateSP(this, &SStateTreeDebuggerView::StepForwardToNextStateWithEvents),
		FCanExecuteAction::CreateSP(this, &SStateTreeDebuggerView::CanStepForwardToNextStateWithEvents));

	ToolkitCommands->MapAction(
		Commands.NextFrameWithStateChange,
		FExecuteAction::CreateSP(this, &SStateTreeDebuggerView::StepForwardToNextStateChange),
		FCanExecuteAction::CreateSP(this, &SStateTreeDebuggerView::CanStepForwardToNextStateChange));

	ToolkitCommands->MapAction(
		Commands.ResumeDebuggerAnalysis,
		FExecuteAction::CreateSP(this, &SStateTreeDebuggerView::ResumeDebuggerAnalysis),
		FCanExecuteAction::CreateSP(this, &SStateTreeDebuggerView::CanResumeDebuggerAnalysis));

	ToolkitCommands->MapAction(
		Commands.ResetTracks,
		FExecuteAction::CreateSP(this, &SStateTreeDebuggerView::ResetTracks),
		FCanExecuteAction::CreateSP(this, &SStateTreeDebuggerView::CanResetTracks));
}

void SStateTreeDebuggerView::BindDebuggingToolbarCommands(const TSharedRef<FUICommandList> ToolkitCommands)
{
	const FStateTreeDebuggerCommands& Commands = FStateTreeDebuggerCommands::Get();

	ToolkitCommands->MapAction(
		Commands.OpenRewindDebugger,
		FExecuteAction::CreateStatic(&SStateTreeDebuggerView::OpenRewindDebugger),
		FCanExecuteAction::CreateStatic(&SStateTreeDebuggerView::CanOpenRewindDebugger));

	ToolkitCommands->MapAction(
		Commands.ResumeDebuggerAnalysis,
		FExecuteAction::CreateSP(this, &SStateTreeDebuggerView::ResumeDebuggerAnalysis),
		FCanExecuteAction::CreateSP(this, &SStateTreeDebuggerView::CanResumeDebuggerAnalysis));
}

bool SStateTreeDebuggerView::CanOpenRewindDebugger()
{
	if (const FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor"))
	{
		if (const TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule->GetLevelEditorTabManager())
		{
			if (const IRewindDebuggerModule* RewindDebuggerModule = FModuleManager::GetModulePtr<IRewindDebuggerModule>("RewindDebugger"))
			{
				return LevelEditorTabManager->FindTabSpawnerFor(RewindDebuggerModule->GetMainTabName()).IsValid();
			}
		}
	}
	return false;
}

void SStateTreeDebuggerView::OpenRewindDebugger()
{
	if (const FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor"))
	{
		if (const TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule->GetLevelEditorTabManager())
		{
			if (const IRewindDebuggerModule* RewindDebuggerModule = FModuleManager::GetModulePtr<IRewindDebuggerModule>("RewindDebugger"))
			{
				if (const TSharedPtr<SDockTab> Tab = LevelEditorTabManager->GetOwnerTab())
				{
					Tab->ActivateInParent(SetDirectly);
				}

				if (const TSharedPtr<SDockTab> Tab = LevelEditorTabManager->TryInvokeTab(RewindDebuggerModule->GetMainTabName()))
				{
					LevelEditorTabManager->DrawAttention(Tab.ToSharedRef());
				}

				if (const TSharedPtr<SDockTab> Tab = LevelEditorTabManager->TryInvokeTab(RewindDebuggerModule->GetDetailsTabName()))
				{
					LevelEditorTabManager->DrawAttention(Tab.ToSharedRef());
				}
			}
		}
	}
}

bool SStateTreeDebuggerView::CanUseScrubButtons() const
{
	// Nothing preventing use of scrub buttons on the Editor side at the moment.
	return true;
}

bool SStateTreeDebuggerView::CanStepBackToPreviousStateWithEvents() const
{
	check(Debugger);
	return CanUseScrubButtons() && Debugger->CanStepBackToPreviousStateWithEvents();
}

void SStateTreeDebuggerView::StepBackToPreviousStateWithEvents()
{
	check(Debugger);
	Debugger->StepBackToPreviousStateWithEvents();
	bAutoScroll = false;
}

bool SStateTreeDebuggerView::CanStepForwardToNextStateWithEvents() const
{
	check(Debugger);
	return CanUseScrubButtons() && Debugger->CanStepForwardToNextStateWithEvents();
}

void SStateTreeDebuggerView::StepForwardToNextStateWithEvents()
{
	check(Debugger);
	Debugger->StepForwardToNextStateWithEvents();
	bAutoScroll = false;
}

bool SStateTreeDebuggerView::CanStepBackToPreviousStateChange() const
{
	check(Debugger);
	return CanUseScrubButtons() && Debugger->CanStepBackToPreviousStateChange();
}

void SStateTreeDebuggerView::StepBackToPreviousStateChange()
{
	check(Debugger);
	Debugger->StepBackToPreviousStateChange();
	bAutoScroll = false;
}

bool SStateTreeDebuggerView::CanStepForwardToNextStateChange() const
{
	check(Debugger);
	return CanUseScrubButtons() && Debugger->CanStepForwardToNextStateChange();
}

void SStateTreeDebuggerView::StepForwardToNextStateChange()
{
	check(Debugger);
	Debugger->StepForwardToNextStateChange();
	bAutoScroll = false;
}

void SStateTreeDebuggerView::OnTimeLineScrubPositionChanged(double Time, bool bIsScrubbing)
{
	check(Debugger);
	// Disable auto scroll when scrubbing.
	// But, do not disable it if the analysis is in progress but paused.
	// This allows the user to pause the analysis, inspect the data, and continue and the autoscroll will catch up with latest.
	// Complementary logic in Tick().
	if (Debugger->IsAnalysisSessionActive() && !Debugger->IsAnalysisSessionPaused())
	{
		bAutoScroll = false;
	}
	Debugger->SetScrubTime(Time);
}

void SStateTreeDebuggerView::OnDebuggerScrubStateChanged(const UE::StateTreeDebugger::FScrubState& ScrubState)
{
	TrackCursor();

	EventsView->RequestRefresh(ScrubState);
}

void SStateTreeDebuggerView::OnBreakpointHit(const FStateTreeInstanceDebugId InstanceId, const FStateTreeDebuggerBreakpoint Breakpoint, const TSharedRef<FUICommandList> ActionList) const
{
	// Pause PIE session if possible
	if (FPlayWorldCommands::Get().PausePlaySession.IsValid())
	{
		if (ActionList->CanExecuteAction(FPlayWorldCommands::Get().PausePlaySession.ToSharedRef()))
		{
			ActionList->ExecuteAction(FPlayWorldCommands::Get().PausePlaySession.ToSharedRef());
		}
	}

	// Extract associated UStateTreeState to focus on it.
	check(StateTreeViewModel);
	if (UStateTreeState* AssociatedState = StateTreeViewModel->FindStateAssociatedToBreakpoint(Breakpoint))
	{
		StateTreeViewModel->SetSelection(AssociatedState);
	}

	// Find matching event in the tree view and select it
	if (EventsView.IsValid())
	{
		EventsView->SelectByPredicate([Breakpoint](const FStateTreeTraceEventVariantType& Event)
			{
				return Breakpoint.IsMatchingEvent(Event);
			});
	}

	// Bring attention to the Editor associated to the breakpoint
	if (const TSharedPtr<IToolkit> FoundAssetEditor = FToolkitManager::Get().FindEditorForAsset(StateTree.Get()))
	{
		FoundAssetEditor->BringToolkitToFront();
	}
}

void SStateTreeDebuggerView::OnNewSession()
{
	// We clear tracks:
	//  - analysis is not for an Editor session
	//  - explicitly set in the settings for Editor sessions
	//  - if previous analysis was not an Editor session
	check(Debugger);
	if (!Debugger->IsAnalyzingEditorSession()
		|| (UStateTreeEditorSettings::Get().bEnableLegacyDebuggerWindow
			&& UStateTreeEditorSettings::Get().bShouldDebuggerResetDataOnNewPIESession)
		|| !Debugger->WasAnalyzingEditorSession())
	{
		ResetTracks();
	}
	else if (Debugger->GetSelectedInstanceId().IsInvalid())
	{
		// In PIE it is possible to stop/start the recording multiple times during the same game session,
		// in this case we try to reselect the currently selected instance in case it can be reactivated.
		if (FStateTreeDebuggerBaseTrack* DebuggerBaseTrack = static_cast<FStateTreeDebuggerBaseTrack*>(InstancesTreeView->GetSelection().Get()))
		{
			DebuggerBaseTrack->OnSelected();
		}
	}

	// Restore automatic scroll to most recent data.
	bAutoScroll = true;
}

void SStateTreeDebuggerView::OnNewInstance(FStateTreeInstanceDebugId InstanceId)
{
	check(Debugger);
	const TSharedPtr<const UE::StateTreeDebugger::FInstanceDescriptor> FoundDescriptor = Debugger->GetDescriptor(InstanceId);
	if (!ensureMsgf(FoundDescriptor != nullptr, TEXT("This callback is from the Debugger so we expect to be able to find matching descriptor.")))
	{
		return;
	}

	const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>* ExistingOwnerTrack = InstanceOwnerTracks.FindByPredicate(
		[FoundDescriptor](const TSharedPtr<RewindDebugger::FRewindDebuggerTrack>& Track)
		{
			return Track.Get()->GetName() == FoundDescriptor->Name;
		});

	if (ExistingOwnerTrack == nullptr)
	{
		ExistingOwnerTrack = &InstanceOwnerTracks.Add_GetRef(MakeShared<FStateTreeDebuggerOwnerTrack>(FText::FromString(FoundDescriptor->Name)));
	}

	if (FStateTreeDebuggerOwnerTrack* OwnerTrack = static_cast<FStateTreeDebuggerOwnerTrack*>(ExistingOwnerTrack->Get()))
	{
		const FString TrackName = FString::Printf(TEXT("Execution #%d"), OwnerTrack->NumSubTracks() + 1);
		const TSharedPtr<FStateTreeDebuggerInstanceTrack> SubTrack = MakeShared<FStateTreeDebuggerInstanceTrack>(SharedThis(this), Debugger.ToSharedRef(), InstanceId, FText::FromString(TrackName), ViewRange);
		OwnerTrack->AddSubTrack(SubTrack);

		// Look at current selection; if nothing selected or stale track then select new track
		const TSharedPtr<RewindDebugger::FRewindDebuggerTrack> Selection = InstancesTreeView->GetSelection();
		if (const FStateTreeDebuggerBaseTrack* DebuggerBaseTrack = static_cast<FStateTreeDebuggerBaseTrack*>(Selection.Get()))
		{
			if (DebuggerBaseTrack->IsStale())
			{
				InstancesTreeView->SetSelection(SubTrack);
				InstancesTreeView->ScrollTo(SubTrack);
			}
		}
		else
		{
			InstancesTreeView->SetSelection(SubTrack);
		}
	}

	InstancesTreeView->Refresh();
	InstanceTimelinesTreeView->Refresh();
}

TSharedRef<SWidget> SStateTreeDebuggerView::OnGetDebuggerTracesMenu() const
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection*/true, /*InCommandList*/nullptr);

	TArray<FStateTreeDebugger::FTraceDescriptor> TraceDescriptors;

	check(Debugger);
	Debugger->GetLiveTraces(TraceDescriptors);

	for (const FStateTreeDebugger::FTraceDescriptor& TraceDescriptor : TraceDescriptors)
	{
		const FText Desc = Debugger->DescribeTrace(TraceDescriptor);

		FUIAction ItemAction(FExecuteAction::CreateSPLambda(Debugger.ToSharedRef(), [Debugger = Debugger, TraceDescriptor]()
			{
				// Request new analysis only if user picked a different trace (we don't want to clear the tracks)
				if (Debugger->GetSelectedTraceDescriptor() != TraceDescriptor)
				{
					Debugger->RequestSessionAnalysis(TraceDescriptor);
				}
			}));
		MenuBuilder.AddMenuEntry(Desc, TAttribute<FText>(), FSlateIcon(), ItemAction);
	}

	// Failsafe when no match
	if (TraceDescriptors.Num() == 0)
	{
		const FText Desc = LOCTEXT("NoLiveSessions", "Can't find live trace sessions");
		FUIAction ItemAction(FExecuteAction::CreateSPLambda(Debugger.ToSharedRef(), [Debugger = Debugger]()
			{
				Debugger->RequestSessionAnalysis(FStateTreeDebugger::FTraceDescriptor());
			}));
		MenuBuilder.AddMenuEntry(Desc, TAttribute<FText>(), FSlateIcon(), ItemAction);
	}

	return MenuBuilder.MakeWidget();
}

void SStateTreeDebuggerView::TrackCursor()
{
	const double ScrubTime = ScrubTimeAttribute.Get();
	TRange<double> CurrentViewRange = ViewRange;
	const double ViewRangeDuration = CurrentViewRange.GetUpperBoundValue() - CurrentViewRange.GetLowerBoundValue();

	static constexpr double LeadingMarginFraction = 0.05;
	static constexpr double TrailingMarginFraction = 0.01;

	if (ScrubTime > (CurrentViewRange.GetUpperBoundValue() - (ViewRangeDuration * LeadingMarginFraction)))
	{
		CurrentViewRange.SetUpperBound(ScrubTime + (ViewRangeDuration * LeadingMarginFraction));
		CurrentViewRange.SetLowerBound(CurrentViewRange.GetUpperBoundValue() - ViewRangeDuration);
	}

	if (ScrubTime < (CurrentViewRange.GetLowerBoundValue() + (ViewRangeDuration * TrailingMarginFraction)))
	{
		CurrentViewRange.SetLowerBound(ScrubTime - (ViewRangeDuration * TrailingMarginFraction));
		CurrentViewRange.SetUpperBound(CurrentViewRange.GetLowerBoundValue() + ViewRangeDuration);
	}

	ViewRange = CurrentViewRange;
}

#undef LOCTEXT_NAMESPACE

#endif // WITH_STATETREE_TRACE_DEBUGGER
