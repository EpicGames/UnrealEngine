// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Debugger/StateTreeDebugger.h"

#if WITH_STATETREE_TRACE_DEBUGGER

#include "Widgets/Views/STreeView.h"
#include "Widgets/SCompoundWidget.h"

namespace RewindDebugger
{
	class FRewindDebuggerTrack;
}
namespace UE::StateTreeDebugger
{
	struct FFrameSpan;
	class SFrameEventsView;
}

enum class EStateTreeBreakpointType : uint8;

struct FSlateIcon;
struct FStateTreeDebugger;
struct FStateTreeInstanceDebugId;
struct FStateTreeDebuggerBreakpoint;
class FStateTreeEditor;
class FStateTreeViewModel;
class FUICommandList;
class UStateTree;
class SStateTreeDebuggerTimelines;
class SStateTreeDebuggerInstanceTree;
class UStateTreeEditorData;


/**
 * Widget holding the timelines for all statetree instances matching a given asset
 * in addition to some frame details panels.
 */
class SStateTreeDebuggerView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SStateTreeDebuggerView) {}
	SLATE_END_ARGS()

	~SStateTreeDebuggerView();

	void Construct(const FArguments& InArgs, TNotNull<const UStateTree*> InStateTree, const TSharedRef<FStateTreeViewModel>& InStateTreeViewModel, const TSharedRef<FUICommandList>& InCommandList);

	[[nodiscard]] TSharedPtr<FStateTreeDebugger> GetDebugger() const
	{
		return Debugger;
	}

	[[nodiscard]] double GetExtrapolatedRecordedWorldTime() const
	{
		return ExtrapolatedRecordedWorldTime;
	}

private:
	void ConstructLegacyView(const TSharedRef<FUICommandList>& InCommandList);
	void ConstructView(const TSharedRef<FUICommandList>& InCommandList);

	virtual FReply OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	void RefreshTracks();

	void TrackCursor();

	TSharedRef<SWidget> OnGetDebuggerTracesMenu() const;

	void OnPIEStarted(bool bIsSimulating);
	void OnPIEStopped(bool bIsSimulating);
	void OnPIEPaused(bool bIsSimulating) const;
	void OnPIEResumed(bool bIsSimulating) const;
	void OnPIESingleStepped(bool bIsSimulating) const;

	void OnBreakpointHit(const FStateTreeInstanceDebugId InstanceId, const FStateTreeDebuggerBreakpoint Breakpoint, const TSharedRef<FUICommandList> ActionList) const;
	void OnNewSession();
	void OnNewInstance(FStateTreeInstanceDebugId InstanceId);

	/** Legacy debugger commands */
	void BindDebuggerToolbarCommands(const TSharedRef<FUICommandList>& ToolkitCommands);

	/** RewindDebugger sync commands */
	void BindDebuggingToolbarCommands(const TSharedRef<FUICommandList> ToolkitCommands);

	static bool CanOpenRewindDebugger();
	static void OpenRewindDebugger();

	bool CanUseScrubButtons() const;

	bool CanStartRecording() const
	{
		return !IsRecording();
	}

	void StartRecording();

	bool IsRecording() const
	{
		return bRecording;
	}

	bool CanStopRecording() const
	{
		return IsRecording();
	}

	void StopRecording();

	void HandleTracesStateChanged(EStateTreeTraceStatus TraceStatus);
	void HandleTracesStopped();

	bool CanResumeDebuggerAnalysis() const;
	void ResumeDebuggerAnalysis() const;

	bool CanResetTracks() const;
	void ResetTracks();

	bool CanStepBackToPreviousStateWithEvents() const;
	void StepBackToPreviousStateWithEvents();

	bool CanStepForwardToNextStateWithEvents() const;
	void StepForwardToNextStateWithEvents();

	bool CanStepBackToPreviousStateChange() const;
	void StepBackToPreviousStateChange();

	bool CanStepForwardToNextStateChange() const;
	void StepForwardToNextStateChange();

	/** Callback from timeline widgets to update the debugger scrub state. */
	void OnTimeLineScrubPositionChanged(double Time, bool bIsScrubbing);

	/** Callback used to reflect debugger scrub state in the UI. */
	void OnDebuggerScrubStateChanged(const UE::StateTreeDebugger::FScrubState& ScrubState);

	TSharedPtr<FStateTreeDebugger> Debugger;
	TSharedPtr<FStateTreeViewModel> StateTreeViewModel;
	TWeakObjectPtr<const UStateTree> StateTree;
	TWeakObjectPtr<UStateTreeEditorData> StateTreeEditorData;
	TSharedPtr<FUICommandList> CommandList;

	/** Tracks for all statetree instance owners producing trace events for the associated state tree asset. */
	TArray<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>> InstanceOwnerTracks;

	/**
	 * Tree view displaying the instance names and synced with InstanceTimelinesTreeView.
	 * Note that this is currently used as a list view but kept the tree view to be close
	 * to the rewind debugger track implementation.
	 */
	TSharedPtr<SStateTreeDebuggerInstanceTree> InstancesTreeView;

	/**
	 * Tree view displaying the instance timelines and synced with InstancesTreeView.
	 * Note that this is currently used as a list view but kept the tree view to be close
	 * to the rewind debugger track implementation.
	 */
	TSharedPtr<SStateTreeDebuggerTimelines> InstanceTimelinesTreeView;

	/** Splitter between instances selector and simple time slider. Used with TreeViewsSplitter to keep header and content in sync */
	TSharedPtr<SSplitter> HeaderSplitter;

	/** Splitter between instances names and their timelines. Used with HeaderSplitter to keep header and content in sync */
	TSharedPtr<SSplitter> TreeViewsSplitter;

	/** Tree view displaying the frame events of the instance associated to the selected track. */
	TSharedPtr<UE::StateTreeDebugger::SFrameEventsView> EventsView;

	/** Attribute provided by the debugger scrub position to control cursor and timelines positions. */
	TAttribute<double> ScrubTimeAttribute;

	/** Range controlled by the timeline widgets and used to adjust cursor position and track content. */
	TRange<double> ViewRange = TRange<double>(0.0, 10.0);

	/** In case tracks are not reset when a new analysis session is started we keep track of the longest duration to adjust our clamp range. */
	double MaxTrackRecordingDuration = 0.0;

	/** The recording duration time the UI was last updated from the debugger. Used to detect if new data has been collect while the UI was inactive. */
	double LastUpdatedTrackRecordingDuration = 0.0;

	/**
	 * Extrapolated world time used to make the timeline progress until the debugger
	 * receives a new event that will update its last processed world time.
	 */
	double ExtrapolatedRecordedWorldTime = 0.0;

	/** Whether the "Migration To Rewind Debugger" is visible. */
	bool bDisplayMigrationToRewindDebuggerWarning = true;

private:

	FDelegateHandle BeginPIEHandle;
	FDelegateHandle PausePIEHandle;
	FDelegateHandle ResumePIEHandle;
	FDelegateHandle SingleStepPIEHandle;
	FDelegateHandle ShutdownPIEHandle;

	/** Indicates that a live session was started (record button or auto record in PIE) to generate StateTree traces. */
	bool bRecording = false;

	/** True if the timelines scrolls automatically. */
	bool bAutoScroll = true;
};

#endif // WITH_STATETREE_TRACE_DEBUGGER