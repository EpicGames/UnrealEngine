// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_STATETREE_TRACE_DEBUGGER

#include "StateTreeDebuggerCommands.h"

#include "StateTreeEditorStyle.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "StateTreeDebugger"

FStateTreeDebuggerCommands::FStateTreeDebuggerCommands()
	: TCommands
	(
		"StateTreeEditor.Debugger",								// Context name for fast lookup
		LOCTEXT("StateTreeDebugger", "StateTree Debugger"),		// Localized context name for displaying
		NAME_None,												// Parent context name
		FStateTreeEditorStyle::Get().GetStyleSetName()
	)
{
}

void FStateTreeDebuggerCommands::RegisterCommands()
{
	UI_COMMAND(StartRecording, "Start Recording", "Start a new trace session.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::R));
	UI_COMMAND(StopRecording, "Stop Recording", "Stop the current trace session.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::S));

	UI_COMMAND(PreviousFrameWithStateChange, "Previous State Change", "Jump to previous state changed frame", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::Left));
	UI_COMMAND(PreviousFrameWithEvents, "Previous Frame", "Step one frame back", EUserInterfaceActionType::Button, FInputChord(EKeys::Left));
	UI_COMMAND(NextFrameWithEvents, "Next Frame", "Step one frame forward", EUserInterfaceActionType::Button, FInputChord(EKeys::Right));
	UI_COMMAND(NextFrameWithStateChange, "Next State Change", "Jump to next state changed frame", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::Right));
	UI_COMMAND(ResumeDebuggerAnalysis, "Resume Analysis", "Resume trace analysis after hitting breakpoint", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ResetTracks, "Reset Tracks", "Clear current tracks and associated events.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(OpenRewindDebugger, "Open RewindDebugger", "Open RewindDebugger window and its details tab", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE

#endif // WITH_STATETREE_TRACE_DEBUGGER