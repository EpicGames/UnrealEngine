// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_STATETREE_TRACE_DEBUGGER

#include "Framework/Commands/Commands.h"

/**
 * StateTree Debugger command set.
 */
class FStateTreeDebuggerCommands : public TCommands<FStateTreeDebuggerCommands>
{
public:
	FStateTreeDebuggerCommands();

	//~ TCommands<> overrides
	virtual void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> StartRecording;
	TSharedPtr<FUICommandInfo> StopRecording;
	TSharedPtr<FUICommandInfo> PreviousFrameWithStateChange;
	TSharedPtr<FUICommandInfo> PreviousFrameWithEvents;
	TSharedPtr<FUICommandInfo> NextFrameWithEvents;
	TSharedPtr<FUICommandInfo> NextFrameWithStateChange;
	TSharedPtr<FUICommandInfo> ResumeDebuggerAnalysis;
	TSharedPtr<FUICommandInfo> ResetTracks;
	TSharedPtr<FUICommandInfo> OpenRewindDebugger;
};

#endif // WITH_STATETREE_TRACE_DEBUGGER