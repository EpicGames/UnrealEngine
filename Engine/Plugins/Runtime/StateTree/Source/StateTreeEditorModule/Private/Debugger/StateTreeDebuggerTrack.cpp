// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_STATETREE_TRACE_DEBUGGER

#include "Debugger/StateTreeDebuggerTrack.h"
#include "Debugger/StateTreeDebugger.h"
#include "SStateTreeDebuggerView.h"

namespace UE::StateTreeDebugger
{

//----------------------------------------------------------------------//
// FInstanceTrackHelper
//----------------------------------------------------------------------//
FInstanceTrackHelper::FInstanceTrackHelper(const FStateTreeInstanceDebugId InInstanceId, const TRange<double>& InViewRange)
	: InstanceId(InInstanceId)
	, ViewRange(InViewRange)
{
	TimelineData = MakeShared<SStateTreeDebuggerEventTimelineView::FTimelineEventData>();
}

bool FInstanceTrackHelper::RebuildEventData(
	const TNotNull<const UStateTree*> InStateTree
	, const FInstanceEventCollection& InEventCollection
	, const double InRecordingDuration
	, const double InScrubTime
	, FStateTreeTraceActiveStates* OutLastActiveStates
	, const bool bInIsStaleTrack) const
{
	if (InEventCollection.IsInvalid())
	{
		return false;
	}

	const int32 PrevNumPoints = TimelineData->Points.Num();
	const int32 PrevNumWindows = TimelineData->Windows.Num();

	TimelineData->Points.SetNum(0, EAllowShrinking::No);
	TimelineData->Windows.SetNum(0);

	auto MakeRandomColor = [bActive = !bInIsStaleTrack](const uint32 InSeed)->FLinearColor
		{
			const FRandomStream Stream(InSeed);
			const uint8 Hue = static_cast<uint8>(Stream.FRand() * 255.0f);
			const uint8 SatVal = bActive ? 196 : 128;
			return FLinearColor::MakeFromHSV8(Hue, SatVal, SatVal);
		};

	const TConstArrayView<FFrameSpan> Spans = InEventCollection.FrameSpans;
	const TConstArrayView<FStateTreeTraceEventVariantType> Events = InEventCollection.Events;
	const uint32 NumStateChanges = InEventCollection.ActiveStatesChanges.Num();

	TArray<FInstanceEventCollection::FContiguousTraceInfo> TracesInfo = InEventCollection.ContiguousTracesData;
	// Append the ongoing trace info to "stopped" previous trace
	if (NumStateChanges > 0)
	{
		TracesInfo.Emplace(InEventCollection.ActiveStatesChanges.Last().SpanIndex);
	}

	int32 StateChangeEndIndex = INDEX_NONE;
	for (int32 TraceIndex = 0; TraceIndex < TracesInfo.Num(); TraceIndex++)
	{
		FInstanceEventCollection::FContiguousTraceInfo TraceInfo = TracesInfo[TraceIndex];
		// Start at first event for the first trace or from the end index of the previous trace 
		const int32 StateChangeBeginIndex = (StateChangeEndIndex == INDEX_NONE) ? 0 : StateChangeEndIndex;

		// Find the starting index of the next trace to stop our iteration
		StateChangeEndIndex = InEventCollection.ActiveStatesChanges.IndexOfByPredicate(
			[LastSpanIndex = TraceInfo.LastSpanIndex](const FInstanceEventCollection::FActiveStatesChangePair& Pair)
			{
				return Pair.SpanIndex > LastSpanIndex;
			});

		// When not found means we are processing the last (or the only) trace
		if (StateChangeEndIndex == INDEX_NONE)
		{
			StateChangeEndIndex = NumStateChanges;
		}

		for (int32 StateChangeIndex = StateChangeBeginIndex; StateChangeIndex < StateChangeEndIndex; ++StateChangeIndex)
		{
			const uint32 SpanIndex = InEventCollection.ActiveStatesChanges[StateChangeIndex].SpanIndex;
			const uint32 EventIndex = InEventCollection.ActiveStatesChanges[StateChangeIndex].EventIndex;
			const FStateTreeTraceActiveStatesEvent& Event = Events[EventIndex].Get<FStateTreeTraceActiveStatesEvent>();

			FString StatePath = Event.GetValueString(*InStateTree);
			FFrameSpan Span = InEventCollection.FrameSpans[SpanIndex];

			// Only update active states with the last received even before timeline scrub time
			if (OutLastActiveStates
				&& Span.GetWorldTimeStart() <= InScrubTime)
			{
				*OutLastActiveStates = Event.ActiveStates;
			}

			SStateTreeDebuggerEventTimelineView::FTimelineEventData::EventWindow& Window = TimelineData->Windows.AddDefaulted_GetRef();
			Window.Color = MakeRandomColor(CityHash32(reinterpret_cast<const char*>(*StatePath), StatePath.Len() * sizeof(FString::ElementType)));
			Window.Description = FText::FromString(StatePath);
			Window.TimeStart = Span.GetWorldTimeStart();

			// For the last received event we use either the current recording duration if the instance is still active or the last recorded frame time
			if (StateChangeIndex == (NumStateChanges - 1))
			{
				Window.TimeEnd = InRecordingDuration;
			}
			else
			{
				// When there is another state change after the current one in the list we use it to close the window.
				// If the event is not the last of that specific trace then we use the start time of the next span.
				// Otherwise, we use the end time of the last frame that was part of that trace.
				const int32 NextStateChangeSpanIndex = InEventCollection.ActiveStatesChanges[StateChangeIndex + 1].SpanIndex;
				Window.TimeEnd = (StateChangeIndex < (StateChangeEndIndex - 1))
					? InEventCollection.FrameSpans[NextStateChangeSpanIndex].GetWorldTimeStart()
					: InEventCollection.FrameSpans[NextStateChangeSpanIndex - 1].GetWorldTimeEnd(); //TraceInfo.LastRecordingTime;
			}
		}
	}

	for (int32 SpanIndex = 0; SpanIndex < Spans.Num(); SpanIndex++)
	{
		const FFrameSpan& Span = Spans[SpanIndex];

		const int32 StartIndex = Span.EventIdx;
		const int32 MaxIndex = (SpanIndex + 1 < Spans.Num()) ? Spans[SpanIndex + 1].EventIdx : Events.Num();
		for (int EventIndex = StartIndex; EventIndex < MaxIndex; ++EventIndex)
		{
			if (Events[EventIndex].IsType<FStateTreeTraceLogEvent>())
			{
				SStateTreeDebuggerEventTimelineView::FTimelineEventData::EventPoint Point;
				Point.Time = Span.GetWorldTimeStart();
				Point.Color = FColorList::Salmon;
				TimelineData->Points.Add(Point);
			}
		}
	}

	return (PrevNumPoints != TimelineData->Points.Num() || PrevNumWindows != TimelineData->Windows.Num());
}

} // namespace UE::StateTreeDebugger

//----------------------------------------------------------------------//
// FStateTreeDebuggerInstanceTrack
//----------------------------------------------------------------------//
FStateTreeDebuggerInstanceTrack::FStateTreeDebuggerInstanceTrack(
	const TSharedRef<SStateTreeDebuggerView>& InDebuggerView,
	const TSharedRef<FStateTreeDebugger>& InDebugger,
	const FStateTreeInstanceDebugId InInstanceId,
	const FText& InName,
	const TRange<double>& InViewRange
	)
	: FStateTreeDebuggerBaseTrack(FSlateIcon("StateTreeEditorStyle", "StateTreeEditor.Debugger.InstanceTrack", "StateTreeEditor.Debugger.InstanceTrack"), InName)
	, StateTreeDebuggerView(InDebuggerView)
	, StateTreeDebugger(InDebugger)
	, InstanceTrackHelper(new UE::StateTreeDebugger::FInstanceTrackHelper(InInstanceId, InViewRange))
	, Descriptor(InDebugger.Get().GetDescriptor(InInstanceId))
{
}

void FStateTreeDebuggerInstanceTrack::OnSelected()
{
	if (const TSharedPtr<FStateTreeDebugger> Debugger = StateTreeDebugger.Pin())
	{
		Debugger->SelectInstance(InstanceTrackHelper.Get().GetInstanceId());
	}
}

bool FStateTreeDebuggerInstanceTrack::UpdateInternal()
{
	const TSharedPtr<FStateTreeDebugger> Debugger = StateTreeDebugger.Pin();
	const TSharedPtr<SStateTreeDebuggerView> DebuggerView = StateTreeDebuggerView.Pin();
	if (!Debugger.IsValid()
		|| !DebuggerView.IsValid())
	{
		return false;
	}

	double InstanceLastActiveTime = DebuggerView->GetExtrapolatedRecordedWorldTime();
	const bool bIsStaleTrack = IsStale();
	if (bIsStaleTrack)
	{
		// If the instance was no longer active when the track was marked as stale we use its lifetime to build the data
		// otherwise we build up to the last time the track was active.
		InstanceLastActiveTime = Descriptor->IsInstanceActive() ? StaleTime : Descriptor.Get()->Lifetime.GetUpperBoundValue();
	}
	else if (!Descriptor->IsInstanceActive())
	{
		// If the instance is no longer active we don't want to build data past that time
		InstanceLastActiveTime = Descriptor.Get()->Lifetime.GetUpperBoundValue();
	}

	const bool bChanged = InstanceTrackHelper.Get().RebuildEventData(Debugger->GetAsset()
		, Debugger->GetEventCollection(InstanceTrackHelper.Get().GetInstanceId())
		, InstanceLastActiveTime
		, Debugger->GetScrubTime()
		, /*OutLastActiveStates*/ nullptr
		, bIsStaleTrack);

	// Tracks can be reactivated when multiple recordings are made in a single PIE session.
	if (bChanged && IsStale())
	{
		MarkAsActive();
	}

	return bChanged;
}

TSharedPtr<SWidget> FStateTreeDebuggerInstanceTrack::GetTimelineViewInternal()
{
	return InstanceTrackHelper.Get().CreateTimelineView();
}

//----------------------------------------------------------------------//
// FStateTreeDebuggerOwnerTrack
//----------------------------------------------------------------------//
FStateTreeDebuggerOwnerTrack::FStateTreeDebuggerOwnerTrack(const FText& InInstanceName)
	: FStateTreeDebuggerBaseTrack(FSlateIcon("StateTreeEditorStyle", "StateTreeEditor.Debugger.OwnerTrack", "StateTreeEditor.Debugger.OwnerTrack"), InInstanceName)
{
}

bool FStateTreeDebuggerOwnerTrack::UpdateInternal()
{
	bool bChanged = false;
	for (const TSharedPtr<FStateTreeDebuggerInstanceTrack>& Track : SubTracks)
	{
		bChanged = Track->Update() || bChanged;
	}

	return bChanged;
}

TConstArrayView<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>> FStateTreeDebuggerOwnerTrack::GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const
{
	return MakeConstArrayView<TSharedPtr<FRewindDebuggerTrack>>(reinterpret_cast<const TSharedPtr<FRewindDebuggerTrack>*>(SubTracks.GetData()), SubTracks.Num());
}

void FStateTreeDebuggerOwnerTrack::MarkAsStale(const double InStaleTime)
{
	for (TSharedPtr<FStateTreeDebuggerInstanceTrack>& Track : SubTracks)
	{
		if (FStateTreeDebuggerInstanceTrack* InstanceTrack = Track.Get())
		{
			InstanceTrack->MarkAsStale(InStaleTime);
		}
	}
}

bool FStateTreeDebuggerOwnerTrack::IsStale() const
{
	// Considered stale only if all sub tracks are stale
	if (SubTracks.IsEmpty())
	{
		return false;
	}

	for (const TSharedPtr<FStateTreeDebuggerInstanceTrack>& Track : SubTracks)
	{
		const FStateTreeDebuggerInstanceTrack* InstanceTrack = Track.Get();
		if (InstanceTrack && InstanceTrack->IsStale() == false)
		{
			return false;
		}
	}

	return true;
}

#endif // WITH_STATETREE_TRACE_DEBUGGER
