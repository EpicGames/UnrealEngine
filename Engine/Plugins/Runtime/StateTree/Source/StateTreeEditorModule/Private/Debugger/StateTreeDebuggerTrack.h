// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_STATETREE_TRACE_DEBUGGER

#include "RewindDebuggerTrack.h"
#include "SStateTreeDebuggerEventTimelineView.h"
#include "Debugger/StateTreeDebuggerTypes.h"

class SStateTreeDebuggerView;
struct FStateTreeDebugger;

namespace UE::StateTreeDebugger
{

/**
 * Helper struct for debuggers relying on StateTree instance tracks
 */
struct FInstanceTrackHelper : public TSharedFromThis<FInstanceTrackHelper>
{
	explicit FInstanceTrackHelper(const FStateTreeInstanceDebugId InInstanceId, const TRange<double>& InViewRange);
	
	TSharedPtr<SStateTreeDebuggerEventTimelineView::FTimelineEventData> GetTimelineData() const
	{
		return TimelineData;
	}

	/**
	 * Rebuild the timeline data from the provided Event collection
	 * @return Whether the number of points, or windows, in the timeline data
	 * changed since the last update.
	 * */
	bool RebuildEventData(
		TNotNull<const UStateTree*> InStateTree
		, const FInstanceEventCollection& InEventCollection
		, const double InRecordingDuration
		, const double InScrubTime
		, FStateTreeTraceActiveStates* OutLastActiveStates = nullptr
		, const bool bInIsStaleTrack = false) const;

	FStateTreeInstanceDebugId GetInstanceId() const
	{
		return InstanceId;
	}

	const FStateTreeTraceActiveStates& GetActiveStates() const
	{
		return ActiveStates;
	}

	void SetActiveStates(const FStateTreeTraceActiveStates& InActiveStates)
	{
		ActiveStates = InActiveStates;
	}

	TSharedPtr<SWidget> CreateTimelineView()
	{
		return SNew(SStateTreeDebuggerEventTimelineView)
			.ViewRange_Lambda([WeakHelper = AsWeak()]
				{
					if (const TSharedPtr<FInstanceTrackHelper> Helper = WeakHelper.Pin())
					{
						return Helper->ViewRange;
					}
					return TRange<double>{};
				})
			.EventData_Lambda([WeakHelper = AsWeak()]()
				{
					if (const TSharedPtr<FInstanceTrackHelper> Helper = WeakHelper.Pin())
					{
						return Helper->TimelineData;
					}
					return TSharedPtr<SStateTreeDebuggerEventTimelineView::FTimelineEventData>{};
				});
	}

protected:

	FStateTreeInstanceDebugId InstanceId;
	TSharedPtr<SStateTreeDebuggerEventTimelineView::FTimelineEventData> TimelineData;
	const TRange<double>& ViewRange;
	FStateTreeTraceActiveStates ActiveStates;
};

} // UE::StateTreeDebugger


/** Base struct for Debugger tracks to append some functionalities not available in RewindDebuggerTrack */
struct FStateTreeDebuggerBaseTrack : RewindDebugger::FRewindDebuggerTrack
{
	explicit FStateTreeDebuggerBaseTrack(const FSlateIcon& Icon, const FText& TrackName)
		: Icon(Icon)
		, TrackName(TrackName)
	{
	}

	virtual bool IsStale() const
	{
		return false;
	}

	virtual void MarkAsStale(const double InStaleTime)
	{
	}

	virtual void OnSelected()
	{
	}

protected:
	virtual FSlateIcon GetIconInternal() override
	{
		return Icon;
	}

	virtual FName GetNameInternal() const override
	{
		return FName(TrackName.ToString());
	}

	virtual FText GetDisplayNameInternal() const override
	{
		return TrackName;
	}

	FSlateIcon Icon;
	FText TrackName;
};

/** Track used to represent timeline events for a single StateTree instance. */
struct FStateTreeDebuggerInstanceTrack : FStateTreeDebuggerBaseTrack
{
	explicit FStateTreeDebuggerInstanceTrack(
		const TSharedRef<SStateTreeDebuggerView>& InDebuggerView,
		const TSharedRef<FStateTreeDebugger>& InDebugger,
		const FStateTreeInstanceDebugId InInstanceId,
		const FText& InName,
		const TRange<double>& InViewRange);

	virtual void OnSelected() override;

	void MarkAsActive()
	{
		StaleTime = UE::StateTreeDebugger::FInstanceDescriptor::ActiveInstanceEndTime;
	}

	virtual void MarkAsStale(const double InStaleTime) override
	{
		// Do not update timestamp for already stale tracks
		if (!IsStale())
		{
			StaleTime = InStaleTime;
		}
	}

	virtual bool IsStale() const override
	{
		return StaleTime != UE::StateTreeDebugger::FInstanceDescriptor::ActiveInstanceEndTime;
	}

protected:
	virtual bool UpdateInternal() override;
	virtual TSharedPtr<SWidget> GetTimelineViewInternal() override;

private:
	
	TWeakPtr<SStateTreeDebuggerView> StateTreeDebuggerView;
	TWeakPtr<FStateTreeDebugger> StateTreeDebugger;
	TSharedRef<UE::StateTreeDebugger::FInstanceTrackHelper> InstanceTrackHelper;
	TSharedPtr<const UE::StateTreeDebugger::FInstanceDescriptor> Descriptor;
	double StaleTime = UE::StateTreeDebugger::FInstanceDescriptor::ActiveInstanceEndTime;
};


/** Parent track of all the statetree instance tracks sharing the same execution context owner */
struct FStateTreeDebuggerOwnerTrack : FStateTreeDebuggerBaseTrack
{
	explicit FStateTreeDebuggerOwnerTrack(const FText& InInstanceName);

	void AddSubTrack(const TSharedPtr<FStateTreeDebuggerInstanceTrack>& InSubTrack)
	{
		SubTracks.Emplace(InSubTrack);
	}

	int32 NumSubTracks() const
	{
		return SubTracks.Num();
	}

	virtual void MarkAsStale(double InStaleTime) override;
	virtual bool IsStale() const override;

protected:
	virtual bool UpdateInternal() override;
	virtual TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const override;

private:
	TArray<TSharedPtr<FStateTreeDebuggerInstanceTrack>> SubTracks;
};

#endif // WITH_STATETREE_TRACE_DEBUGGER