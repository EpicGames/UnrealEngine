// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeDebuggerUIExtensions.h"
#include "Customizations/StateTreeEditorNodeUtils.h"
#include "DetailLayoutBuilder.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IPropertyUtilities.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "StateTreeConditionBase.h"
#include "StateTreeEditorData.h"
#include "StateTreeEditorNode.h"
#include "StateTreeEditorStyle.h"
#include "StateTreeTaskBase.h"
#include "StateTreeViewModel.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SOverlay.h"


#define LOCTEXT_NAMESPACE "StateTreeEditor"

namespace UE::StateTreeEditor::DebuggerExtensions
{

void OnConditionEvaluationModeChanged(TSharedPtr<IPropertyHandle> StructProperty, const EStateTreeConditionEvaluationMode Mode)
{
	EditorNodeUtils::ModifyNodeInTransaction(LOCTEXT("OnEvaluationModeChanged", "Condition Evaluation Mode Changed"),
		StructProperty,
		[Mode](const TSharedPtr<IPropertyHandle>& StructProperty)
		{
			if (FStateTreeEditorNode* EditorNode = EditorNodeUtils::GetMutableCommonNode(StructProperty))
			{
				if (FStateTreeConditionBase* ConditionBase = EditorNode->GetNode().GetPtr<FStateTreeConditionBase>())
				{
					ConditionBase->EvaluationMode = Mode;
				}
			}
		});
}

void OnTaskEnableToggled(TSharedPtr<IPropertyHandle> StructProperty)
{
	EditorNodeUtils::ModifyNodeInTransaction(LOCTEXT("OnTaskEnableToggled", "Toggled Task Enabled"),
		StructProperty,
		[](const TSharedPtr<IPropertyHandle>& StructProperty)
		{
			if (FStateTreeEditorNode* EditorNode = EditorNodeUtils::GetMutableCommonNode(StructProperty))
			{
				if (FStateTreeTaskBase* TaskBase = EditorNode->GetNode().GetPtr<FStateTreeTaskBase>())
				{
					TaskBase->bTaskEnabled = !TaskBase->bTaskEnabled;
				}
			}
		});
}

void OnStateEnableToggled(TSharedPtr<IPropertyHandle> EnabledProperty)
{
	FScopedTransaction Transaction(LOCTEXT("OnStateEnableToggled", "Toggled State Enabled"));

	bool bEnabled = false;
	const FPropertyAccess::Result Result = EnabledProperty->GetValue(bEnabled);

	if (Result == FPropertyAccess::MultipleValues || bEnabled == false)
	{
		EnabledProperty->SetValue(true);
	}
	else
	{
		EnabledProperty->SetValue(false);
	}
}

bool HasStateBreakpoint(const TArray<TWeakObjectPtr<>>& StatesBeingCustomized, const UStateTreeEditorData* EditorData, const EStateTreeBreakpointType Type)
{
#if WITH_STATETREE_TRACE_DEBUGGER
	if (EditorData == nullptr)
	{
		return false;
	}
	for (const TWeakObjectPtr<>& WeakStateObject : StatesBeingCustomized)
	{
		if (const UStateTreeState* State = static_cast<const UStateTreeState*>(WeakStateObject.Get()))
		{
			if (EditorData->HasBreakpoint(State->ID, Type))
			{
				return true;
			}
		}
	}
#endif // WITH_STATETREE_TRACE_DEBUGGER
	return false;
}

void OnStateBreakpointToggled(const TArray<TWeakObjectPtr<>>& States, const TSharedPtr<FStateTreeViewModel>& StateTreeViewModel, const EStateTreeBreakpointType Type)
{
#if WITH_STATETREE_TRACE_DEBUGGER
	if (StateTreeViewModel != nullptr)
	{
		StateTreeViewModel->ToggleStateBreakpoints(States, Type);
	}
#endif // WITH_STATETREE_TRACE_DEBUGGER
}

bool HasTaskBreakpoint(TSharedPtr<IPropertyHandle> StructProperty, const TSharedPtr<FStateTreeViewModel>& StateTreeViewModel, const EStateTreeBreakpointType Type)
{
#if WITH_STATETREE_TRACE_DEBUGGER
	const FStateTreeEditorNode* Node = EditorNodeUtils::GetCommonNode(StructProperty);
	if (StateTreeViewModel != nullptr && Node != nullptr)
	{
		return StateTreeViewModel->HasBreakpoint(Node->ID, Type);
	}
#endif // WITH_STATETREE_TRACE_DEBUGGER
	return false;
}

void OnTaskBreakpointToggled(TSharedPtr<IPropertyHandle> StructProperty, const TSharedPtr<FStateTreeViewModel>& StateTreeViewModel, const EStateTreeBreakpointType Type)
{
#if WITH_STATETREE_TRACE_DEBUGGER
	const FStateTreeEditorNode* Node = EditorNodeUtils::GetMutableCommonNode(StructProperty);
	if (StateTreeViewModel != nullptr && Node != nullptr)
	{
		StateTreeViewModel->ToggleTaskBreakpoint(Node->ID, Type);
	}
#endif // WITH_STATETREE_TRACE_DEBUGGER
}

ECheckBoxState GetBreakpointCheckState(
	const TSharedPtr<IPropertyHandle>& StructPropertyHandle,
	const UStateTreeEditorData* EditorData,
	TFunctionRef<ECheckBoxState(const UObject& OuterObject, const IPropertyHandle& PropertyHandle, const UStateTreeEditorData&)> Callback)
{
	ECheckBoxState CommonState = ECheckBoxState::Unchecked;
#if WITH_STATETREE_TRACE_DEBUGGER
	if (StructPropertyHandle.IsValid() == false
		|| StructPropertyHandle->IsValidHandle() == false
		|| EditorData == nullptr)
	{
		return CommonState;
	}

	TArray<UObject*> OuterObjects;
	StructPropertyHandle->GetOuterObjects(OuterObjects);

	CommonState = ECheckBoxState::Undetermined;
	for (const UObject* OuterObject : OuterObjects)
	{
		if (OuterObject != nullptr)
		{
			const ECheckBoxState State = Callback(*OuterObject, *StructPropertyHandle.Get(), *EditorData);
			if (CommonState == ECheckBoxState::Undetermined)
			{
				CommonState = State;
			}
			else if (CommonState != State)
			{
				CommonState = ECheckBoxState::Undetermined;
				break;
			}
		}
	}
#endif // WITH_STATETREE_TRACE_DEBUGGER
	return CommonState;
}

ECheckBoxState GetTransitionBreakpointCheckState(const TSharedPtr<IPropertyHandle>& StructPropertyHandle, const UStateTreeEditorData* EditorData)
{
#if WITH_STATETREE_TRACE_DEBUGGER
	return GetBreakpointCheckState(StructPropertyHandle, EditorData,
		[](const UObject& OuterObject, const IPropertyHandle& PropertyHandle, const UStateTreeEditorData& EditorData)
		{
			const UStateTreeState* TreeState = Cast<UStateTreeState>(&OuterObject);
			const int32 IndexInArray = PropertyHandle.GetIndexInArray();
			if (TreeState->Transitions.IsValidIndex(IndexInArray))
			{
				return EditorData.HasBreakpoint(TreeState->Transitions[IndexInArray].ID, EStateTreeBreakpointType::OnTransition)
					? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}

			return ECheckBoxState::Unchecked;
		});
#else
	return  ECheckBoxState::Unchecked;
#endif // WITH_STATETREE_TRACE_DEBUGGER
}

ECheckBoxState GetTransitionEnabledCheckState(const TSharedPtr<IPropertyHandle>& StructPropertyHandle)
{
	ECheckBoxState CommonState = ECheckBoxState::Checked;

	if (!StructPropertyHandle.IsValid() || !StructPropertyHandle->IsValidHandle())
	{
		return CommonState;
	}

	const TSharedPtr<IPropertyHandle> EnabledProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeTransition, bTransitionEnabled));
	if (ensureMsgf(EnabledProperty.IsValid() && EnabledProperty->IsValidHandle(),
		TEXT("The property is missing keywords in its UPROPERTY macro to be exposed to the Editor, or doesn't have a UPROPERTY macro.")))
	{
		bool bEnabled = false;

		switch (EnabledProperty->GetValue(bEnabled))
		{
		case FPropertyAccess::MultipleValues:
			CommonState = ECheckBoxState::Undetermined;
			break;
		case FPropertyAccess::Success:
			CommonState = bEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			break;
		case FPropertyAccess::Fail:
		default:
			break;
		}
	}
	return CommonState;
}

void OnTransitionEnableToggled(const TSharedPtr<IPropertyHandle>& StructPropertyHandle)
{
	if (!StructPropertyHandle.IsValid() || !StructPropertyHandle->IsValidHandle())
	{
		return;
	}

	const TSharedPtr<IPropertyHandle> EnabledProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FStateTreeTransition, bTransitionEnabled));
	if (ensureMsgf(EnabledProperty.IsValid() && EnabledProperty->IsValidHandle(),
		TEXT("The property is missing keywords in its UPROPERTY macro to be exposed to the Editor, or doesn't have a UPROPERTY macro.")))
	{
		FScopedTransaction Transaction(LOCTEXT("OnStateEnableToggled", "Toggled State Enabled"));

		bool bEnabled = false;
		const FPropertyAccess::Result Result = EnabledProperty->GetValue(bEnabled);

		if (Result == FPropertyAccess::MultipleValues || bEnabled == false)
		{
			EnabledProperty->SetValue(true);
		}
		else
		{
			EnabledProperty->SetValue(false);
		}
	}
}

void OnTransitionBreakpointToggled(const TSharedPtr<IPropertyHandle>& StructPropertyHandle, const TSharedPtr<FStateTreeViewModel>& StateTreeViewModel)
{
#if WITH_STATETREE_TRACE_DEBUGGER
	if (StructPropertyHandle.IsValid() == false
		|| StructPropertyHandle->IsValidHandle() == false
		|| StateTreeViewModel == nullptr)
	{
		return;
	}

	// Determine desired toggled state. Undermined will switch all to checked (i.e. Enable breakpoint)
	const ECheckBoxState CommonState = GetTransitionBreakpointCheckState(StructPropertyHandle, StateTreeViewModel->GetStateTreeEditorData());
	const ECheckBoxState ToggledState = (CommonState == ECheckBoxState::Checked) ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
	const IPropertyHandle& PropertyHandle = *(StructPropertyHandle.Get());

	TArray<UObject*> OuterObjects;
	TArray<TNotNull<const FStateTreeTransition*>> Transitions;
	PropertyHandle.GetOuterObjects(OuterObjects);

	if (OuterObjects.Num() > 0)
	{
		for (const UObject* OuterObject : OuterObjects)
		{
			if (const UStateTreeState* TreeState = Cast<UStateTreeState>(OuterObject))
			{
				const int32 IndexInArray = PropertyHandle.GetIndexInArray();

				if (TreeState->Transitions.IsValidIndex(IndexInArray))
				{
					Transitions.Push(&TreeState->Transitions[IndexInArray]);
				}
			}
		}
	}

	StateTreeViewModel->ToggleTransitionBreakpoint(Transitions, ToggledState);

#endif // WITH_STATETREE_TRACE_DEBUGGER
}

TArray<TWeakObjectPtr<UObject>> GetWeakOuterObjects(const TSharedPtr<IPropertyHandle>& PropertyHandle)
{
	TArray<UObject*> OuterObjects;
	PropertyHandle->GetOuterObjects(OuterObjects);

	TArray<TWeakObjectPtr<UObject>> WeakOuterObjects;
	for (UObject* Object : OuterObjects)
	{
		WeakOuterObjects.Add(Object);
	}
	return WeakOuterObjects;
}

TSharedRef<SWidget> CreateStateWidget(TSharedPtr<IPropertyHandle> StateEnabledProperty, const TSharedPtr<FStateTreeViewModel>& StateTreeViewModel)
{
	TWeakObjectPtr<const UStateTreeEditorData> WeakTreeData = StateTreeViewModel != nullptr ? StateTreeViewModel->GetStateTreeEditorData() : nullptr;
	TArray<TWeakObjectPtr<UObject>> StatesBeingCustomized = GetWeakOuterObjects(StateEnabledProperty);

	const TSharedRef<SHorizontalBox> HeaderContentWidget = SNew(SHorizontalBox)
		.Clipping(EWidgetClipping::OnDemand)

		// Disabled Label
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(4.0f, 0.0f)
		[
			SNew(SBorder)
			.Padding(FMargin(6, 1))
			.BorderImage(FStateTreeEditorStyle::Get().GetBrush("StateTree.Node.Label"))
			.Visibility_Lambda([Property = StateEnabledProperty]
				{
					bool bEnabled = false;
					Property->GetValue(bEnabled);
					return bEnabled ? EVisibility::Collapsed : EVisibility::Visible;
				})
			[
				SNew(STextBlock)
				.TextStyle(FStateTreeEditorStyle::Get(), "StateTree.Param.Label")
				.ColorAndOpacity(FStyleColors::Foreground)
				.Text(LOCTEXT("LabelDisabled", "DISABLED"))
				.ToolTipText(LOCTEXT("DisabledStateTooltip", "This state has been disabled."))
			]
		]

		// Break on enter Label
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(4.0f, 0.0f)
		[
			SNew(SBorder)
			.Padding(FMargin(6, 2))
			.BorderImage(FStateTreeEditorStyle::Get().GetBrush("StateTree.Node.Label"))
			.Visibility_Lambda([StatesBeingCustomized, WeakTreeData]
				{
					return HasStateBreakpoint(StatesBeingCustomized, WeakTreeData.Get(), EStateTreeBreakpointType::OnEnter) ? EVisibility::Visible : EVisibility::Collapsed;
				})
			[
				SNew(STextBlock)
				.TextStyle(FStateTreeEditorStyle::Get(), "StateTree.Param.Label")
				.ColorAndOpacity(FStyleColors::Foreground)
				.Text(LOCTEXT("LabelOnEnterBreakpoint", "BREAK ON ENTER"))
				.ToolTipText(LOCTEXT("BreakOnEnterTaskTooltip", "Break when entering this task."))
			]
		]

		// Break on exit Label
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(4.0f, 0.0f)
		[
			SNew(SBorder)
			.Padding(FMargin(6, 2))
			.BorderImage(FStateTreeEditorStyle::Get().GetBrush("StateTree.Node.Label"))
			.Visibility_Lambda([StatesBeingCustomized, WeakTreeData]
				{
					return HasStateBreakpoint(StatesBeingCustomized, WeakTreeData.Get(), EStateTreeBreakpointType::OnExit) ? EVisibility::Visible : EVisibility::Collapsed;
				})
			[
				SNew(STextBlock)
				.TextStyle(FStateTreeEditorStyle::Get(), "StateTree.Param.Label")
				.ColorAndOpacity(FStyleColors::Foreground)
				.Text(LOCTEXT("LabelOnExitBreakpoint", "BREAK ON EXIT"))
				.ToolTipText(LOCTEXT("BreakOnExitTaskTooltip", "Break when exiting this task."))
			]
		];

	return HeaderContentWidget;
}

void AppendStateMenuItems(FMenuBuilder& InMenuBuilder, TSharedPtr<IPropertyHandle> StateEnabledProperty, const TSharedPtr<FStateTreeViewModel>& StateTreeViewModel)
{
	TWeakObjectPtr WeakTreeData = StateTreeViewModel != nullptr ? StateTreeViewModel->GetStateTreeEditorData() : nullptr;
	TArray<TWeakObjectPtr<UObject>> StatesBeingCustomized = GetWeakOuterObjects(StateEnabledProperty);

	InMenuBuilder.BeginSection(FName("Options"), LOCTEXT("StateOptions", "State Debug Options"));
	InMenuBuilder.AddMenuEntry
	(
		LOCTEXT("ToggleStateEnabled", "State Enabled"),
		LOCTEXT("ToggleStateEnabled_ToolTip", "Enables or disables selected state(s). StateTree must be recompiled to take effect."),
		FSlateIcon(),
		FUIAction
		(
			FExecuteAction::CreateLambda([StateEnabledProperty]
				{
					OnStateEnableToggled(StateEnabledProperty);
				}),
			FCanExecuteAction(),
			FGetActionCheckState::CreateLambda([StateEnabledProperty]
				{
					bool bEnabled = false;
					if (StateEnabledProperty && StateEnabledProperty->IsValidHandle())
					{
						const FPropertyAccess::Result Result = StateEnabledProperty->GetValue(bEnabled);
						if (Result == FPropertyAccess::MultipleValues)
						{
							return ECheckBoxState::Undetermined;
						}
					}

					return bEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
		),
		NAME_None,
		EUserInterfaceActionType::Check
	);
	InMenuBuilder.AddSeparator();
	InMenuBuilder.AddMenuEntry
	(
		LOCTEXT("ToggleOnEnterStateBreakpoint", "Break on Enter"),
		LOCTEXT("ToggleOnEnterStateBreakpoint_ToolTip", "Enables or disables breakpoint when entering selected state(s). (Debugger Window required)"),
		FSlateIcon(),
		FUIAction
		(
			FExecuteAction::CreateLambda([StatesBeingCustomized, StateTreeViewModel]
				{
					OnStateBreakpointToggled(StatesBeingCustomized, StateTreeViewModel, EStateTreeBreakpointType::OnEnter);
				}),
			FCanExecuteAction::CreateLambda([StateTreeViewModel]
				{
					return StateTreeViewModel != nullptr && StateTreeViewModel->CanProcessBreakpoints();
				}),
			FGetActionCheckState::CreateLambda([StatesBeingCustomized, WeakTreeData]
				{
					return HasStateBreakpoint(StatesBeingCustomized, WeakTreeData.Get(), EStateTreeBreakpointType::OnEnter) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
		),
		NAME_None,
		EUserInterfaceActionType::Check
	);
	InMenuBuilder.AddMenuEntry
	(
		LOCTEXT("ToggleOnExitStateBreakpoint", "Break on Exit"),
		LOCTEXT("ToggleOnExitStateBreakpoint_ToolTip", "Enables or disables breakpoint when exiting selected state(s). (Debugger Window required)"),
		FSlateIcon(),
		FUIAction
		(
			FExecuteAction::CreateLambda([StatesBeingCustomized, StateTreeViewModel]
				{
					OnStateBreakpointToggled(StatesBeingCustomized, StateTreeViewModel, EStateTreeBreakpointType::OnExit);
				}),
			FCanExecuteAction::CreateLambda([StateTreeViewModel]
				{
					return StateTreeViewModel != nullptr && (StateTreeViewModel->CanAddStateBreakpoint(EStateTreeBreakpointType::OnExit)
							|| StateTreeViewModel->CanRemoveStateBreakpoint(EStateTreeBreakpointType::OnExit));
				}),
			FGetActionCheckState::CreateLambda([StatesBeingCustomized, WeakTreeData]
				{
					return HasStateBreakpoint(StatesBeingCustomized, WeakTreeData.Get(), EStateTreeBreakpointType::OnExit) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
		),
		NAME_None,
		EUserInterfaceActionType::Check
	);
	InMenuBuilder.EndSection();

}

TSharedRef<SWidget> CreateEditorNodeWidget(const TSharedPtr<IPropertyHandle>& StructPropertyHandle, const TSharedPtr<FStateTreeViewModel>& StateTreeViewModel)
{
	using namespace EditorNodeUtils;

	bool bIsTask = false;
	bool bIsCondition = false;
	if (const FStructProperty* TmpStructProperty = CastField<FStructProperty>(StructPropertyHandle->GetProperty()))
	{
		if (TmpStructProperty->Struct->IsChildOf(TBaseStructure<FStateTreeEditorNode>::Get()))
		{
			const UScriptStruct* ScriptStruct = nullptr;
			if (const FStateTreeEditorNode* EditorNode = GetCommonNode(StructPropertyHandle))
			{
				ScriptStruct = EditorNode->GetNode().GetScriptStruct();
			}

			if (ScriptStruct != nullptr)
			{
				bIsTask = ScriptStruct->IsChildOf(TBaseStructure<FStateTreeTaskBase>::Get());
				bIsCondition = ScriptStruct->IsChildOf(TBaseStructure<FStateTreeConditionBase>::Get());
			}
		}
	}

	// In case there is no common Editor node or not associated to a condition or a task we don't need to add any widget.
	if (bIsCondition == false && bIsTask == false)
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.HAlign(HAlign_Right)
		[
			SNew(SHorizontalBox)
			.Clipping(EWidgetClipping::OnDemand)

			// Disabled Label
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(4.0f, 0.0f)
			[
				SNew(SBorder)
				.Padding(FMargin(6.0f, 2.0f))
				.BorderImage(FStateTreeEditorStyle::Get().GetBrush("StateTree.Node.Label"))
				.Visibility_Lambda([StructPropertyHandle]
					{
						return IsEditorNodeEnabled(StructPropertyHandle) ? EVisibility::Collapsed : EVisibility::Visible;
					})
				[
					SNew(STextBlock)
					.TextStyle(FStateTreeEditorStyle::Get(), "StateTree.Param.Label")
					.ColorAndOpacity(FStyleColors::Foreground)
					.Text(LOCTEXT("LabelDisabled", "DISABLED"))
					.ToolTipText(LOCTEXT("DisabledTaskTooltip", "This task has been disabled."))
				]
			]

			// Break on enter Label
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(4.0f, 0.0f)
			[
				SNew(SBorder)
				.Padding(FMargin(6.0f, 2.0f))
				.BorderImage(FStateTreeEditorStyle::Get().GetBrush("StateTree.Node.Label"))
				.Visibility_Lambda([StructPropertyHandle, StateTreeViewModel]
					{
						return HasTaskBreakpoint(StructPropertyHandle, StateTreeViewModel, EStateTreeBreakpointType::OnEnter) ? EVisibility::Visible : EVisibility::Collapsed;
					})
				[
					SNew(STextBlock)
					.TextStyle(FStateTreeEditorStyle::Get(), "StateTree.Param.Label")
					.ColorAndOpacity(FStyleColors::Foreground)
					.Text(LOCTEXT("LabelOnEnterBreakpoint", "BREAK ON ENTER"))
					.ToolTipText(LOCTEXT("BreakOnEnterTaskTooltip", "Break when entering this task."))
				]
			]

			// Break on exit Label
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(4.0f, 0.0f)
			[
				SNew(SBorder)
				.Padding(FMargin(6.0f, 2.0f))
				.BorderImage(FStateTreeEditorStyle::Get().GetBrush("StateTree.Node.Label"))
				.Visibility_Lambda([StructPropertyHandle, StateTreeViewModel]
					{
						return HasTaskBreakpoint(StructPropertyHandle, StateTreeViewModel, EStateTreeBreakpointType::OnExit) ? EVisibility::Visible : EVisibility::Collapsed;
					})
				[
					SNew(STextBlock)
					.TextStyle(FStateTreeEditorStyle::Get(), "StateTree.Param.Label")
					.ColorAndOpacity(FStyleColors::Foreground)
					.Text(LOCTEXT("LabelOnExitBreakpoint", "BREAK ON EXIT"))
					.ToolTipText(LOCTEXT("BreakOnExitTaskTooltip", "Break when exiting this task."))
				]
			]

			// Force True / Force False Label
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(4.0f, 0.0f)
			[
				SNew(SBorder)
				.Padding(FMargin(6.0f, 2.0f))
				.BorderImage(FStateTreeEditorStyle::Get().GetBrush("StateTree.Node.Label"))
				.BorderBackgroundColor_Lambda([StructPropertyHandle]
					{
						return GetConditionEvaluationMode(StructPropertyHandle) == EStateTreeConditionEvaluationMode::ForcedTrue
								   ? FStyleColors::AccentGreen.GetSpecifiedColor()
								   : FStyleColors::AccentRed.GetSpecifiedColor();
					})
				.Visibility_Lambda([StructPropertyHandle]
					{
						return GetConditionEvaluationMode(StructPropertyHandle) != EStateTreeConditionEvaluationMode::Evaluated ? EVisibility::Visible : EVisibility::Collapsed;
					})
				[
					SNew(SOverlay)
					+ SOverlay::Slot()
					[
						SNew(STextBlock)
						.TextStyle(FStateTreeEditorStyle::Get(), "StateTree.Param.Label")
						.Text(LOCTEXT("ConditionForcedTrue", "TRUE"))
						.ToolTipText(LOCTEXT("ForcedTrueConditionTooltip", "This condition is not evaluated and result forced to 'true'."))
						.Visibility_Lambda([StructPropertyHandle]
							{
								return GetConditionEvaluationMode(StructPropertyHandle) == EStateTreeConditionEvaluationMode::ForcedTrue ? EVisibility::Visible : EVisibility::Collapsed;
							})
					]
					+ SOverlay::Slot()
					[
						SNew(STextBlock)
						.TextStyle(FStateTreeEditorStyle::Get(), "StateTree.Param.Label")
						.Text(LOCTEXT("ConditionForcedFalse", "FALSE"))
						.ToolTipText(LOCTEXT("ForcedFalseConditionTooltip", "This condition is not evaluated and result forced to 'false'."))
						.Visibility_Lambda([StructPropertyHandle]
							{
								return GetConditionEvaluationMode(StructPropertyHandle) == EStateTreeConditionEvaluationMode::ForcedFalse ? EVisibility::Visible : EVisibility::Collapsed;
							})
					]
				]
			]
		];
}

void AppendEditorNodeMenuItems(FMenuBuilder& InMenuBuilder, const TSharedPtr<IPropertyHandle>& StructPropertyHandle, const TSharedPtr<FStateTreeViewModel>& StateTreeViewModel)
{
	using namespace EditorNodeUtils;

	bool bIsTask = false;
	bool bIsCondition = false;
	if (const FStructProperty* TmpStructProperty = CastField<FStructProperty>(StructPropertyHandle->GetProperty()))
	{
		if (TmpStructProperty->Struct->IsChildOf(TBaseStructure<FStateTreeEditorNode>::Get()))
		{
			const UScriptStruct* ScriptStruct = nullptr;
			if (const FStateTreeEditorNode* EditorNode = GetCommonNode(StructPropertyHandle))
			{
				ScriptStruct = EditorNode->GetNode().GetScriptStruct();
			}

			if (ScriptStruct != nullptr)
			{
				bIsTask = ScriptStruct->IsChildOf(TBaseStructure<FStateTreeTaskBase>::Get());
				bIsCondition = ScriptStruct->IsChildOf(TBaseStructure<FStateTreeConditionBase>::Get());
			}
		}
	}

	// In case there is no common Editor node or not associated to a condition or a task we don't need to add any widget.
	if (bIsCondition == false && bIsTask == false)
	{
		return;
	}

	if (bIsTask)
	{
		InMenuBuilder.BeginSection(FName("Options"), LOCTEXT("TaskOptions", "Task Debug Options"));
		InMenuBuilder.AddMenuEntry
		(
			LOCTEXT("ToggleTaskEnabled", "Task Enabled"),
			LOCTEXT("ToggleTaskEnabled_ToolTip", "Enables or disables selected task(s). StateTree must be recompiled to take effect."),
			FSlateIcon(),
			FUIAction
			(
				FExecuteAction::CreateLambda([StructPropertyHandle] { OnTaskEnableToggled(StructPropertyHandle); }),
				FCanExecuteAction(),
				FGetActionCheckState::CreateLambda([StructPropertyHandle]
					{
						return IsTaskDisabled(StructPropertyHandle) ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
					})
			),
			NAME_None,
			EUserInterfaceActionType::Check
		);
		InMenuBuilder.AddSeparator();
		InMenuBuilder.AddMenuEntry
		(
			LOCTEXT("ToggleOnEnterTaskBreakpoint", "Break on Enter"),
			LOCTEXT("ToggleOnEnterTaskBreakpoint_ToolTip", "Enables or disables breakpoint when entering selected task(s). (Debugger Window required)"),
			FSlateIcon(),
			FUIAction
			(
				FExecuteAction::CreateLambda([StructPropertyHandle, StateTreeViewModel]
					{
						OnTaskBreakpointToggled(StructPropertyHandle, StateTreeViewModel, EStateTreeBreakpointType::OnEnter);
					}),
				FCanExecuteAction::CreateLambda([StateTreeViewModel]
					{
						return StateTreeViewModel != nullptr && StateTreeViewModel->CanProcessBreakpoints();
					}),
				FGetActionCheckState::CreateLambda([StructPropertyHandle, StateTreeViewModel]
					{
						return HasTaskBreakpoint(StructPropertyHandle, StateTreeViewModel, EStateTreeBreakpointType::OnEnter) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
			),
			NAME_None,
			EUserInterfaceActionType::Check
		);
		InMenuBuilder.AddMenuEntry
		(
			LOCTEXT("ToggleOnExitTaskBreakpoint", "Break on Exit"),
			LOCTEXT("ToggleOnExitTaskBreakpoint_ToolTip", "Enables or disables breakpoint when exiting selected task(s). (Debugger Window required)"),
			FSlateIcon(),
			FUIAction
			(
				FExecuteAction::CreateLambda([StructPropertyHandle, StateTreeViewModel]
					{
						OnTaskBreakpointToggled(StructPropertyHandle, StateTreeViewModel, EStateTreeBreakpointType::OnExit);
					}),
				FCanExecuteAction::CreateLambda([StateTreeViewModel]
					{
						return StateTreeViewModel != nullptr && StateTreeViewModel->CanProcessBreakpoints();
					}),
				FGetActionCheckState::CreateLambda([StructPropertyHandle, StateTreeViewModel]
					{
						return HasTaskBreakpoint(StructPropertyHandle, StateTreeViewModel, EStateTreeBreakpointType::OnExit) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
			),
			NAME_None,
			EUserInterfaceActionType::Check
		);
		InMenuBuilder.EndSection();
	}

	if (bIsCondition)
	{
		InMenuBuilder.BeginSection(FName("Options"), LOCTEXT("ConditionOptions", "Condition Debug Options"));
		InMenuBuilder.AddMenuEntry
		(
			LOCTEXT("Evaluate", "Evaluate"),
			LOCTEXT("Evaluate_ToolTip", "Condition result is evaluated (normal behavior)"),
			FSlateIcon(),
			FUIAction
			(
				FExecuteAction::CreateLambda([StructPropertyHandle]
					{
						OnConditionEvaluationModeChanged(StructPropertyHandle, EStateTreeConditionEvaluationMode::Evaluated);
					}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([StructPropertyHandle]
					{
						return GetConditionEvaluationMode(StructPropertyHandle) == EStateTreeConditionEvaluationMode::Evaluated;
					})
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
		InMenuBuilder.AddMenuEntry
		(
			LOCTEXT("ForceTrue", "Force True"),
			LOCTEXT("ForceTrue_ToolTip", "Result is forced to 'true' (condition is not evaluated)"),
			FSlateIcon(),
			FUIAction
			(
				FExecuteAction::CreateLambda([StructPropertyHandle]
					{
						OnConditionEvaluationModeChanged(StructPropertyHandle, EStateTreeConditionEvaluationMode::ForcedTrue);
					}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([StructPropertyHandle]
					{
						return GetConditionEvaluationMode(StructPropertyHandle) == EStateTreeConditionEvaluationMode::ForcedTrue;
					})
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
		InMenuBuilder.AddMenuEntry
		(
			LOCTEXT("ForceFalse", "Force False"),
			LOCTEXT("ForceFalse_ToolTip", "Result is forced to 'false' (condition is not evaluated)"),
			FSlateIcon(),
			FUIAction
			(
				FExecuteAction::CreateLambda([StructPropertyHandle]
					{
						OnConditionEvaluationModeChanged(StructPropertyHandle, EStateTreeConditionEvaluationMode::ForcedFalse);
					}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([StructPropertyHandle]
					{
						return GetConditionEvaluationMode(StructPropertyHandle) == EStateTreeConditionEvaluationMode::ForcedFalse;
					})
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		InMenuBuilder.EndSection();
	}
}

bool IsEditorNodeEnabled(const TSharedPtr<IPropertyHandle>& StructPropertyHandle)
{
	using namespace EditorNodeUtils;

	if (const FStateTreeEditorNode* Node = GetCommonNode(StructPropertyHandle))
	{
		if (const FStateTreeTaskBase* TaskBase = Node->GetNode().GetPtr<FStateTreeTaskBase>())
		{
			return TaskBase->bTaskEnabled;
		}
	}

	return true;
}

TSharedRef<SWidget> CreateTransitionWidget(const TSharedPtr<IPropertyHandle>& StructPropertyHandle, const TSharedPtr<FStateTreeViewModel>& StateTreeViewModel)
{
	using namespace EditorNodeUtils;

	TWeakObjectPtr WeakTreeData = StateTreeViewModel != nullptr ? StateTreeViewModel->GetStateTreeEditorData() : nullptr;

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.HAlign(HAlign_Right)
		[
			SNew(SHorizontalBox)
			.Clipping(EWidgetClipping::OnDemand)

			// Disabled Label
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(4.0f, 0.0f)
			[
				SNew(SBorder)
				.Padding(FMargin(6, 2))
				.BorderImage_Lambda([StructPropertyHandle]
					{
						return GetTransitionEnabledCheckState(StructPropertyHandle) == ECheckBoxState::Unchecked
						? FStateTreeEditorStyle::Get().GetBrush("StateTree.Node.Label")
						: FStateTreeEditorStyle::Get().GetBrush("StateTree.Node.Label.Mixed");
					})
				.Visibility_Lambda([StructPropertyHandle]
					{
						return IsTransitionEnabled(StructPropertyHandle) ? EVisibility::Collapsed : EVisibility::Visible;
					})
				[
					SNew(STextBlock)
					.TextStyle(FStateTreeEditorStyle::Get(), "StateTree.Param.Label")
					.ColorAndOpacity(FStyleColors::Foreground)
					.Text(LOCTEXT("LabelDisabled", "DISABLED"))
					.ToolTipText(LOCTEXT("DisabledTransitionTooltip", "This transition has been disabled."))
				]
			]

			// Break on transition Label
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(4.0f, 0.0f)
			[
				SNew(SBorder)
				.Padding(FMargin(6, 2))
				.BorderImage_Lambda([StructPropertyHandle, WeakTreeData]
					{
						return GetTransitionBreakpointCheckState(StructPropertyHandle, WeakTreeData.Get()) == ECheckBoxState::Checked
						? FStateTreeEditorStyle::Get().GetBrush("StateTree.Node.Label")
						: FStateTreeEditorStyle::Get().GetBrush("StateTree.Node.Label.Mixed");
					})
				.Visibility_Lambda([StructPropertyHandle, WeakTreeData]
					{
						return GetTransitionBreakpointCheckState(StructPropertyHandle, WeakTreeData.Get()) == ECheckBoxState::Unchecked ? EVisibility::Collapsed : EVisibility::Visible;
					})
				[
					SNew(STextBlock)
					.TextStyle(FStateTreeEditorStyle::Get(), "StateTree.Param.Label")
					.ColorAndOpacity(FStyleColors::Foreground)
					.Text(LOCTEXT("LabelTransitionBreakpoint", "BREAK ON TRANSITION"))
					.ToolTipText(LOCTEXT("BreakTransitionTooltip", "Break when executing this transition."))
				]
			]
		];
}

void AppendTransitionMenuItems(FMenuBuilder& InMenuBuilder, const TSharedPtr<IPropertyHandle>& StructPropertyHandle, const TSharedPtr<FStateTreeViewModel>& StateTreeViewModel)
{
	using namespace EditorNodeUtils;

	TWeakObjectPtr WeakTreeData = StateTreeViewModel != nullptr ? StateTreeViewModel->GetStateTreeEditorData() : nullptr;

	InMenuBuilder.BeginSection(FName("Options"), LOCTEXT("TransitionOptions", "Transition Debug Options"));
	InMenuBuilder.AddMenuEntry
	(
		LOCTEXT("ToggleTransitionEnabled", "Transition Enabled"),
		LOCTEXT("ToggleTransitionEnabled_ToolTip", "Enables or disables selected transition(s). StateTree must be recompiled to take effect."),
		FSlateIcon(),
		FUIAction
		(
			FExecuteAction::CreateLambda([StructPropertyHandle] { OnTransitionEnableToggled(StructPropertyHandle); }),
			FCanExecuteAction(),
			FGetActionCheckState::CreateLambda([StructPropertyHandle] { return GetTransitionEnabledCheckState(StructPropertyHandle); })
		),
		NAME_None,
		EUserInterfaceActionType::Check
	);
	InMenuBuilder.AddSeparator();
	InMenuBuilder.AddMenuEntry
	(
		LOCTEXT("ToggleTransitionBreakpoint", "Break on transition"),
		LOCTEXT("ToggleTransitionBreakpoint_ToolTip", "Enables or disables breakpoint when executing the transition(s). (Debugger Window required)"),
		FSlateIcon(),
		FUIAction
		(
			FExecuteAction::CreateLambda([StructPropertyHandle, StateTreeViewModel]
			{
				OnTransitionBreakpointToggled(StructPropertyHandle, StateTreeViewModel);
			}),
			FCanExecuteAction::CreateLambda([StateTreeViewModel]
				{
					return StateTreeViewModel != nullptr && StateTreeViewModel->CanProcessBreakpoints();
				}),
			FGetActionCheckState::CreateLambda([StructPropertyHandle, WeakTreeData]
				{
					return GetTransitionBreakpointCheckState(StructPropertyHandle, WeakTreeData.Get());
				})
		),
		NAME_None,
		EUserInterfaceActionType::Check
	);
	InMenuBuilder.EndSection();
}


bool IsTransitionEnabled(const TSharedPtr<IPropertyHandle>& StructPropertyHandle)
{
	return GetTransitionEnabledCheckState(StructPropertyHandle) == ECheckBoxState::Checked;
}

} // namespace UE::StateTreeEditor::DebuggerExtensions

#undef LOCTEXT_NAMESPACE
