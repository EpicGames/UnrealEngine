// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"

struct FStateTreeEditorNode;
enum class EStateTreeConditionEvaluationMode : uint8;

class SWidget;
class IPropertyHandle;
class IDetailLayoutBuilder;
class FDetailWidgetRow;
class FMenuBuilder;
class FStateTreeViewModel;
class UStateTreeEditorData;

namespace UE::StateTreeEditor::DebuggerExtensions
{

TSharedRef<SWidget> CreateStateWidget(TSharedPtr<IPropertyHandle> StateEnabledProperty, const TSharedPtr<FStateTreeViewModel>& InStateTreeViewModel);
void AppendStateMenuItems(FMenuBuilder& InMenuBuilder, TSharedPtr<IPropertyHandle> StateEnabledProperty, const TSharedPtr<FStateTreeViewModel>& InStateTreeViewModel);

TSharedRef<SWidget> CreateEditorNodeWidget(const TSharedPtr<IPropertyHandle>& StructPropertyHandle, const TSharedPtr<FStateTreeViewModel>& InStateTreeViewModel);
void AppendEditorNodeMenuItems(FMenuBuilder& InMenuBuilder, const TSharedPtr<IPropertyHandle>& StructPropertyHandle, const TSharedPtr<FStateTreeViewModel>& InStateTreeViewModel);
bool IsEditorNodeEnabled(const TSharedPtr<IPropertyHandle>& StructPropertyHandle);

TSharedRef<SWidget> CreateTransitionWidget(const TSharedPtr<IPropertyHandle>& StructPropertyHandle, const TSharedPtr<FStateTreeViewModel>& InStateTreeViewModel);
void AppendTransitionMenuItems(FMenuBuilder& InMenuBuilder, const TSharedPtr<IPropertyHandle>& StructPropertyHandle, const TSharedPtr<FStateTreeViewModel>& InStateTreeViewModel);
bool IsTransitionEnabled(const TSharedPtr<IPropertyHandle>& StructPropertyHandle);

}; // UE::StateTreeEditor::DebuggerExtensions
