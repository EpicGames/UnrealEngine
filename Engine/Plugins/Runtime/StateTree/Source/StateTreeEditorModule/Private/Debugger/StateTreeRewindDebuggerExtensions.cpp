// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debugger/StateTreeRewindDebuggerExtensions.h"

#if WITH_STATETREE_TRACE_DEBUGGER
#include "Debugger/StateTreeTraceTypes.h"
#include "StateTreeDelegates.h"
#include "IRewindDebugger.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/Frames.h"

namespace UE::StateTreeDebugger
{

//----------------------------------------------------------------------//
// FRewindDebuggerPlaybackExtension
//----------------------------------------------------------------------//
void FRewindDebuggerPlaybackExtension::Update(float DeltaTime, IRewindDebugger* RewindDebugger)
{
	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();

	bool bFrameFound;
	TraceServices::FFrame Frame;
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
		const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(*AnalysisSession);
		bFrameFound = FrameProvider.GetFrameFromTime(TraceFrameType_Game, RewindDebugger->CurrentTraceTime(), Frame);
	}

	// Require some debug frame to exist before doing any processing, currently the frame itself is not used
	if (bFrameFound)
	{
		auto SetScrubberTimeline = [&]()
		{
			if (RewindDebugger->IsPIESimulating())
			{
				return;
			}

			const double CurrentScrubTime = RewindDebugger->GetScrubTime();
			if (LastScrubTime != CurrentScrubTime)
			{
				StateTree::Delegates::OnTracingTimelineScrubbed.Broadcast(CurrentScrubTime);
				LastScrubTime = CurrentScrubTime;
			}
		};
		SetScrubberTimeline();
	}
}

void FRewindDebuggerPlaybackExtension::AnalysisSessionOpened(IRewindDebugger* RewindDebugger)
{
	StateTree::Delegates::OnTraceAnalysisStateChanged.Broadcast(EStateTreeTraceAnalysisStatus::Started);
}

void FRewindDebuggerPlaybackExtension::AnalysisSessionClosed(IRewindDebugger* RewindDebugger)
{
	StateTree::Delegates::OnTraceAnalysisStateChanged.Broadcast(EStateTreeTraceAnalysisStatus::Stopped);
}

void FRewindDebuggerPlaybackExtension::Clear(IRewindDebugger* RewindDebugger)
{
	StateTree::Delegates::OnTraceAnalysisStateChanged.Broadcast(EStateTreeTraceAnalysisStatus::Cleared);
}

} // UE::StateTreeDebugger

#endif // WITH_STATETREE_TRACE_DEBUGGER
