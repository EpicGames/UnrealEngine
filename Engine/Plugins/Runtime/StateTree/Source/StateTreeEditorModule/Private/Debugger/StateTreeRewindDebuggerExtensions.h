// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_STATETREE_TRACE_DEBUGGER
#include "IRewindDebuggerExtension.h"

class IRewindDebugger;

namespace UE::StateTreeDebugger
{

class FRewindDebuggerPlaybackExtension final : public IRewindDebuggerExtension
{
	virtual void Update(float DeltaTime, IRewindDebugger* RewindDebugger) override;
	virtual void AnalysisSessionOpened(IRewindDebugger* RewindDebugger) override;
	virtual void AnalysisSessionClosed(IRewindDebugger* RewindDebugger) override;
	virtual void Clear(IRewindDebugger* RewindDebugger) override;

	/** Last scrub time we received. Used to avoid redundant updates. */
	double LastScrubTime = 0.0;
};

} // UE::StateTreeDebugger

#endif // WITH_STATETREE_TRACE_DEBUGGER
