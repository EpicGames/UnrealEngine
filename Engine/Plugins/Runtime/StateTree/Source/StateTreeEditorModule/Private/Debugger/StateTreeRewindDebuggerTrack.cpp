// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeRewindDebuggerTrack.h"
#include "StateTreeEditorSettings.h"
#include "Debugger/SCompactTreeDebuggerView.h"
#include "Debugger/SStateTreeFrameEventsView.h"
#include "Debugger/StateTreeDebugger.h"
#include "Debugger/StateTreeTraceProvider.h"
#include "Editor.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Styling/SlateIconFinder.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "StateTreeRewindDebuggerTrack"

namespace UE::StateTreeDebugger
{

static const FLazyName StateTreeInstancesName("StateTreeInstances");

//----------------------------------------------------------------------//
// FRewindDebuggerTrackCreator
//----------------------------------------------------------------------//
void FRewindDebuggerTrackCreator::GetTrackTypesInternal(TArray<RewindDebugger::FRewindDebuggerTrackType>& Types) const
{
	Types.Add({StateTreeInstancesName, LOCTEXT("StateTree", "StateTreeInstance")});
}

TSharedPtr<RewindDebugger::FRewindDebuggerTrack> FRewindDebuggerTrackCreator::CreateTrackInternal(const RewindDebugger::FObjectId& InObjectId) const
{
	return MakeShared<FRewindDebuggerTrack>(InObjectId);
}

bool FRewindDebuggerTrackCreator::HasDebugInfoInternal(const RewindDebugger::FObjectId& InObjectId) const
{
	if (const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance())
	{
		if (const TraceServices::IAnalysisSession* Session = RewindDebugger->GetAnalysisSession())
		{
			if (const IStateTreeTraceProvider* StateTreeTraceProvider = Session->ReadProvider<IStateTreeTraceProvider>(FStateTreeTraceProvider::ProviderName))
			{
				TraceServices::FProviderReadScopeLock ProviderReadScope(*StateTreeTraceProvider);
				if (StateTreeTraceProvider->GetInstanceDescriptor(FStateTreeInstanceDebugId(InObjectId.GetMainId())).IsValid())
				{
					return true;
				}
			}
		}
	}
	return false;
}

bool FRewindDebuggerTrackCreator::IsCreatingPrimaryChildTrackInternal() const
{
	return true;
}

FName FRewindDebuggerTrackCreator::GetTargetTypeNameInternal() const
{
	return FStateTreeInstanceData::StaticStruct()->GetFName();
}

//----------------------------------------------------------------------//
// FRewindDebuggerTrack
//----------------------------------------------------------------------//
FRewindDebuggerTrack::FRewindDebuggerTrack(const RewindDebugger::FObjectId& InObjectId)
	: EventCollection(FStateTreeInstanceDebugId(InObjectId.GetMainId()))
	, InstanceTrackHelper(new FInstanceTrackHelper(EventCollection.InstanceId, IRewindDebugger::Instance()->GetCurrentViewRange()))
	, Icon(FSlateIconFinder::FindIconForClass(UStateTree::StaticClass()))
	, ObjectId(InObjectId)
{
	if (const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance())
	{
		if (const TraceServices::IAnalysisSession* Session = RewindDebugger->GetAnalysisSession())
		{
			if (const IStateTreeTraceProvider* StateTreeTraceProvider = Session->ReadProvider<IStateTreeTraceProvider>(FStateTreeTraceProvider::ProviderName))
			{
				TraceServices::FProviderReadScopeLock ProviderReadScope(*StateTreeTraceProvider);
				Descriptor = StateTreeTraceProvider->GetInstanceDescriptor(EventCollection.InstanceId);
			}
		}
	}

	if (!ensureMsgf(Descriptor.IsValid(), TEXT("Track should not be created unless the instance is available in the StateTreeTraceProvider")))
	{
		return;
	}

	SAssignNew(CompactTreeView, UE::StateTree::Editor::SCompactTreeDebuggerView, Descriptor->StateTree.Get())
		.ActiveStates_Lambda([WeakTrackHelper = InstanceTrackHelper->AsWeak()]
			{
				if (const TSharedPtr<FInstanceTrackHelper> TrackHelper = WeakTrackHelper.Pin())
				{
					return TrackHelper->GetActiveStates();
				}
				return FStateTreeTraceActiveStates{};
			});

	SAssignNew(EventsView, UE::StateTreeDebugger::SFrameEventsView, Descriptor->StateTree.Get());
}

FText FRewindDebuggerTrack::GetDisplayNameInternal() const
{
	if (Descriptor.IsValid())
	{
		if (UStateTreeEditorSettings::Get().DebuggerTrackNameVerbosity == EStateTreeDebuggerTrackNameVerbosity::Verbose)
		{
			return FText::FromString(FString::Printf(TEXT("%s StateTree (%s)")
				, *Descriptor->Name
				, *GetNameSafe(Descriptor->StateTree.Get())));
		}
		return FText::FromString(GetNameSafe(Descriptor->StateTree.Get()));
	}

	return LOCTEXT("StateTreeRewindDebuggerTrackName", "StateTreeInstance");
}

FSlateIcon FRewindDebuggerTrack::GetIconInternal()
{
	return Icon;
}

FName FRewindDebuggerTrack::GetNameInternal() const
{
	if (Descriptor.IsValid())
	{
		return *Descriptor->Name;
	}

	return "StateTreeDebuggerTrack";
}

uint64 FRewindDebuggerTrack::GetObjectIdInternal() const
{
	return ObjectId.GetMainId();
}

FInstanceEventCollection* FRewindDebuggerTrack::GetOrCreateEventCollection(const FStateTreeInstanceDebugId InstanceId)
{
	ensureMsgf(EventCollection.IsValid(), TEXT("Event collection is expected to be initialized"));
	if (EventCollection.InstanceId == InstanceId)
	{
		return &EventCollection;
	}

	return nullptr;
}

bool FRewindDebuggerTrack::UpdateInternal()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStateTreeRewindDebuggerTrack::UpdateInternal);

	constexpr bool bChildrenChanged = false;
	if (!Descriptor.IsValid())
	{
		return bChildrenChanged;
	}

	const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	if (const TraceServices::IAnalysisSession* Session = RewindDebugger ? RewindDebugger->GetAnalysisSession() : nullptr)
	{
		const UStateTree* StateTree = Descriptor->StateTree.Get();

		ReadTrace(*Session
			, RewindDebugger->CurrentTraceTime()
			, /*ITraceReader*/this
			, FTraceFilter(InstanceTrackHelper.Get().GetInstanceId())
			, LastTraceReadTime);

		if (!FMath::IsNearlyEqual(LastUpdateScrubTime, RewindDebugger->GetScrubTime()))
		{
			const double InstanceLastActiveTime = Descriptor->IsInstanceActive() ? RewindDebugger->GetRecordingDuration() : Descriptor->Lifetime.GetUpperBoundValue();
			const double ScrubTime = RewindDebugger->GetScrubTime();
			LastUpdateScrubTime = ScrubTime;

			FStateTreeTraceActiveStates ActiveStates;
			const bool bTimelineDataChanged = InstanceTrackHelper.Get().RebuildEventData(StateTree
				, EventCollection
				, InstanceLastActiveTime
				, ScrubTime
				, &ActiveStates);

			const bool bActivateStatesChanged = ActiveStates != InstanceTrackHelper.Get().GetActiveStates();
			if (bActivateStatesChanged)
			{
				InstanceTrackHelper.Get().SetActiveStates(ActiveStates);
			}

			// Events need to be updated for a new scrub time
			FScrubState ScrubState(&EventCollection);
			ScrubState.SetScrubTime(ScrubTime, /*ForceRefresh*/true);
			EventsView->RequestRefresh(ScrubState);

			// Tree view only need to be updated on new timeline data or new active states
			if (bTimelineDataChanged
				|| bActivateStatesChanged)
			{
				CompactTreeView->Refresh();
			}
		}
	}

	return bChildrenChanged;
}

TSharedPtr<SWidget> FRewindDebuggerTrack::GetTimelineViewInternal()
{
	return InstanceTrackHelper.Get().CreateTimelineView();
}

TSharedPtr<SWidget> FRewindDebuggerTrack::GetDetailsViewInternal()
{
	if (!Descriptor.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Fill)
		[
			CompactTreeView.ToSharedRef()
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			EventsView.ToSharedRef()
		];
}

bool FRewindDebuggerTrack::HandleDoubleClickInternal()
{
	if (Descriptor.IsValid())
	{
		if (const UStateTree* StateTreeAsset = Descriptor->StateTree.Get())
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(StateTreeAsset->GetPathName());
			return true;
		}
	}

	return false;
}

FText FRewindDebuggerTrack::GetStepCommandTooltipInternal(const RewindDebugger::EStepMode StepMode) const
{
	switch (StepMode)
	{
	case RewindDebugger::EStepMode::Forward:
		return LOCTEXT("StepForwardTooltip", "Step to next event in StateTree track");
	case RewindDebugger::EStepMode::Backward:
		return LOCTEXT("StepBackwardTooltip", "Step to previous event in StateTree track");
	}

	return {};
}

TOptional<double> FRewindDebuggerTrack::GetStepFrameTimeInternal(const RewindDebugger::EStepMode StepMode, const FScrubTimeInformation& ScrubTimeInformation) const
{
	FScrubState ScrubState(&EventCollection);
	const double CurrentScrubTime = ScrubTimeInformation.ElapsedTime;
	ScrubState.SetScrubTime(CurrentScrubTime, /*ForceRefresh*/true);
	if (ScrubState.IsInBounds())
	{
		const FFrameSpan& Span = EventCollection.FrameSpans[ScrubState.GetFrameSpanIndex()];

		if (StepMode == RewindDebugger::EStepMode::Backward
			&& CurrentScrubTime > Span.GetWorldTimeStart())
		{
			return Span.GetWorldTimeStart();
		}

		if (StepMode == RewindDebugger::EStepMode::Forward
			&& CurrentScrubTime < Span.GetWorldTimeEnd())
		{
			return Span.GetWorldTimeEnd();
		}
	}

	if (StepMode == RewindDebugger::EStepMode::Backward &&
		ScrubState.HasPreviousFrame())
	{
		return ScrubState.GotoPreviousFrame();
	}

	if (StepMode == RewindDebugger::EStepMode::Forward
		&& ScrubState.HasNextFrame())
	{
		return ScrubState.GotoNextFrame();
	}

	return {};
}

} // namespace UE::StateTreeDebugger
#undef LOCTEXT_NAMESPACE