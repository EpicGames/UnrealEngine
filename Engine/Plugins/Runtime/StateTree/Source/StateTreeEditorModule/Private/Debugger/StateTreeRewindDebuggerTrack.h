// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Debugger/StateTreeDebugger.h"
#include "Debugger/StateTreeDebuggerTypes.h"
#include "IRewindDebuggerTrackCreator.h"
#include "RewindDebuggerTrack.h"
#include "StateTreeDebuggerTrack.h"

namespace UE::StateTree::Editor
{
class SCompactTreeDebuggerView;
}

namespace UE::StateTreeDebugger
{
class SFrameEventsView;

#if WITH_ENGINE
/**
 * RewindDebugger track creator for StateTree instances
 */
struct FRewindDebuggerTrackCreator final : RewindDebugger::IRewindDebuggerTrackCreator
{
protected:
	virtual FName GetTargetTypeNameInternal() const override;
	virtual FName GetNameInternal() const override
	{
		return "StateTreeInstanceTrack";
	}
	virtual void GetTrackTypesInternal(TArray<RewindDebugger::FRewindDebuggerTrackType>& Types) const override;
	virtual TSharedPtr<RewindDebugger::FRewindDebuggerTrack> CreateTrackInternal(const RewindDebugger::FObjectId& InObjectId) const override;
	virtual bool HasDebugInfoInternal(const RewindDebugger::FObjectId& InObjectId) const override;
	virtual bool IsCreatingPrimaryChildTrackInternal() const override;
};
#endif // WITH_ENGINE

/**
 * RewindDebugger track representing a single StateTree instance
 */
struct FRewindDebuggerTrack : RewindDebugger::FRewindDebuggerTrack, ITraceReader
{
	explicit FRewindDebuggerTrack(const RewindDebugger::FObjectId& InObjectId);

private:
	//~ Begin UE::StateTreeDebugger::ITraceReader interface
	virtual FInstanceEventCollection* GetOrCreateEventCollection(FStateTreeInstanceDebugId InstanceId) override;
	//~ End UE::StateTreeDebugger::ITraceReader interface

	//~ Begin RewindDebugger::FRewindDebuggerTrack interface
	virtual bool UpdateInternal() override;
	virtual TSharedPtr<SWidget> GetTimelineViewInternal() override;
	virtual TSharedPtr<SWidget> GetDetailsViewInternal() override;
	virtual FText GetDisplayNameInternal() const override;
	virtual FSlateIcon GetIconInternal() override;
	virtual FName GetNameInternal() const override;
	virtual uint64 GetObjectIdInternal() const override;
	virtual bool HandleDoubleClickInternal() override;
	virtual FText GetStepCommandTooltipInternal(RewindDebugger::EStepMode) const override;
	virtual TOptional<double> GetStepFrameTimeInternal(RewindDebugger::EStepMode, const FScrubTimeInformation&) const override;
	//~ End RewindDebugger::FRewindDebuggerTrack interface

	FInstanceEventCollection EventCollection;
	TSharedRef<FInstanceTrackHelper> InstanceTrackHelper;
	TSharedPtr<const FInstanceDescriptor> Descriptor;
	TSharedPtr<StateTree::Editor::SCompactTreeDebuggerView> CompactTreeView;
	TSharedPtr<SFrameEventsView> EventsView;
	FSlateIcon Icon;
	RewindDebugger::FObjectId ObjectId;

	/** Last time in the recording that we used to fetch events, and that we will use for the next read. */
	double LastTraceReadTime = 0;

	/** Last scrub time used to rebuild the event data. */
	double LastUpdateScrubTime = 0;
};

} // UE::StateTreeDebugger