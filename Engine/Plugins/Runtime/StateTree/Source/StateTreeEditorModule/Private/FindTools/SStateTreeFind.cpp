// Copyright Epic Games, Inc. All Rights Reserved.

#include "FindTools/SStateTreeFind.h"

#include "Containers/Ticker.h"
#include "Customizations/StateTreeEditorNodeUtils.h"
#include "Editor.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IStateTreeEditorHost.h"
#include "StateTree.h"
#include "StateTreeEditorNode.h"
#include "StateTreeEditingSubsystem.h"
#include "StateTreeState.h"
#include "StateTreeViewModel.h"

#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "StateTreeEditorStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "StateTreeFindInAsset"

namespace UE::StateTreeEditor
{

namespace Private
{
struct FFoundItemStack
{
	const UStruct* Struct = nullptr;
	const void* Data = nullptr;
};

static bool IsPropertyIndexable(const TPropertyValueIterator<FProperty>& It, const FProperty* Property)
{
	// Don't index transient properties.
	if (Property->HasAnyPropertyFlags(CPF_Transient))
	{
		return false;
	}

	if (const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(Property))
	{
		if (const UObject* Object = ObjectProperty->GetObjectPropertyValue(It.Value()))
		{
			if (Object->HasAnyFlags(RF_Transient))
			{
				// Don't do anything with transient objects.
				return false;
			}
		}
	}

	return true;
};

static void IterateProperties(TArray<FFoundItemStack>& Stack, const UStruct* InStruct, const void* InStructValue, TFunctionRef<void(const TPropertyValueIterator<FProperty>& /*Property*/, const FString& /*Value*/)> Callback)
{
	Stack.Add(FFoundItemStack{ .Struct = InStruct, .Data = InStructValue });

	FString ValueExported;
	for (TPropertyValueIterator<FProperty> It(InStruct, InStructValue, EPropertyValueIteratorFlags::FullRecursion, EFieldIteratorFlags::ExcludeDeprecated); It; ++It)
	{
		ValueExported.Reset();

		const FProperty* Property = It.Key();

		// Don't index a transient property.
		if (!IsPropertyIndexable(It, Property))
		{
			It.SkipRecursiveProperty();
			continue;
		}

		constexpr EClassCastFlags ContainerCastFlags = CASTCLASS_FArrayProperty | CASTCLASS_FMapProperty | CASTCLASS_FSetProperty | CASTCLASS_FOptionalProperty;
		if ((Property->GetClass()->GetCastFlags() & ContainerCastFlags) != 0)
		{
			// Don't export the container itself.
			continue;
		}

		bool bExport = true;
		if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property))
		{
			bExport = false;
			if (const UObject* Object = ObjectProperty->GetObjectPropertyValue(It.Value()))
			{
				Object->GetName(ValueExported);
				if (Property->HasAllPropertyFlags(CPF_ExportObject) && Object->GetClass()->HasAllClassFlags(CLASS_EditInlineNew))
				{
					// Add the inner properties of this instanced object.
					IterateProperties(Stack, Object->GetClass(), Object, Callback);
				}
				else if (Property->HasAllPropertyFlags(CPF_InstancedReference))
				{
					// Add the inner properties of this instanced object.
					IterateProperties(Stack, Object->GetClass(), Object, Callback);
				}
			}
		}
		else if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			// Wants to know when we go inside a struct.
			IterateProperties(Stack, StructProperty->Struct, It.Value(), Callback);

			It.SkipRecursiveProperty();
			continue;
		}
		else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
		{
			if (const UEnum* Enum = ByteProperty->Enum)
			{
				bExport = false;
				const int64 Value = ByteProperty->GetSignedIntPropertyValue(It.Value());
				ValueExported = Enum->GetDisplayNameTextByValue(Value).ToString();
			}
		}
		else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
		{
			bExport = false;
			const int64 Value = EnumProperty->GetUnderlyingProperty()->GetSignedIntPropertyValue(It.Value());
			ValueExported = EnumProperty->GetEnum()->GetDisplayNameTextByValue(Value).ToString();
		}

		if (bExport)
		{
			const void* PropertyValue = It.Value();
			constexpr void* DefaultValut = nullptr;
			constexpr UObject* Parent = nullptr;
			Property->ExportTextItem_Direct(ValueExported, PropertyValue, DefaultValut, Parent, (int32)PPF_DebugDump);
		}

		if (!ValueExported.IsEmpty())
		{
			Callback(It, ValueExported);
		}
	}

	Stack.Pop();
}
} // namespace Private

void SFindInAsset::Construct(const FArguments& InArgs, TSharedPtr<IStateTreeEditorHost> InEditorHost)
{
	EditorHost = InEditorHost;

	RegisterCommands();

	this->ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(8.f, 5.f, 8.f, 5.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SAssignNew(SearchTextField, SSearchBox)
					.HintText(LOCTEXT("SearchHint", "Enter a task name or a property value to find references..."))
					.OnTextCommitted(this, &SFindInAsset::HandleSearchTextCommitted)
					.Visibility(InArgs._bShowSearchBar ? EVisibility::Visible : EVisibility::Collapsed)
					.DelayChangeNotificationsWhileTyping(false)
				]
			]

			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
				.Padding(FMargin(8.f, 8.f, 4.f, 0.f))
				[
					SAssignNew(TreeView, STreeViewType)
					.TreeItemsSource(&ItemsFound)
					.OnGenerateRow(this, &SFindInAsset::HandleTreeGenerateRow)
					.OnGetChildren(this, &SFindInAsset::HandleGetTreeChildren)
					.OnMouseButtonDoubleClick(this, &SFindInAsset::HandleTreeSelectionDoubleClicked)
					.SelectionMode(ESelectionMode::Multi)
					.OnContextMenuOpening(this, &SFindInAsset::HandleTreeContextMenuOpening)
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding( FMargin( 16.f, 8.f ) )
			[
				SNew(SHorizontalBox)

				// Text
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font( FAppStyle::Get().GetFontStyle("Text.Large") )
					.Text( LOCTEXT("SearchResults", "Searching...") )
					.Visibility(this, &SFindInAsset::HandleGetSearchingWidgetVisiblity)
				]

				// Throbber
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(12.f, 8.f, 16.f, 8.f))
				.VAlign(VAlign_Center)
				[
					SNew(SThrobber)
					.Visibility(this, &SFindInAsset::HandleGetSearchingWidgetVisiblity)
				]
			]
		]
	];
}

void SFindInAsset::RegisterCommands()
{
	CommandList = MakeShareable(new FUICommandList());

	CommandList->MapAction(FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &SFindInAsset::HandleCopyAction));

	CommandList->MapAction(FGenericCommands::Get().SelectAll,
		FExecuteAction::CreateSP(this, &SFindInAsset::HandleSelectAllAction));
}

void SFindInAsset::HandleSelectAllAction()
{
	for (const TSharedPtr<FFindResult>& Item : ItemsFound)
	{
		RecursiveSelectAllAction(Item);
	}
}

void SFindInAsset::RecursiveSelectAllAction(const TSharedPtr<FFindResult>& Item)
{
	// Iterates over all children and recursively selects all items in the results
	TreeView->SetItemSelection(Item, true);

	for (const TSharedPtr<FFindResult>& Child : Item->Children)
	{
		RecursiveSelectAllAction(Child);
	}
}

void SFindInAsset::HandleCopyAction()
{
	TArray<TSharedPtr<FFindResult>> SelectedItems = TreeView->GetSelectedItems();

	TStringBuilder<256> CopyText;
	for (const TSharedPtr<FFindResult>& SelectedItem : SelectedItems)
	{
		// Add indents for each layer into the tree the item is
		for (auto ParentItem = SelectedItem->Parent; ParentItem.IsValid(); ParentItem = ParentItem.Pin()->Parent)
		{
			CopyText << TEXT("\t");
		}

		// Add the display string
		switch (SelectedItem->Type)
		{
		case FFindResult::EResultType::StateTree:
			if (const UStateTree* StateTree = SelectedItem->StateTree.Get())
			{
				StateTree->GetPathName(nullptr, CopyText);
			}
			break;
		case FFindResult::EResultType::State:
		case FFindResult::EResultType::Node:
			CopyText << SelectedItem->Name;
			break;
		case FFindResult::EResultType::Value:
			CopyText << SelectedItem->PropertyName.ToString();
			CopyText << TEXT(" = ");
			CopyText << SelectedItem->Value;
			break;
		default:
			check(false);
			break;
		}

		// Line terminator so the next item will be on a new line
		CopyText << LINE_TERMINATOR;
	}

	// Copy text to clipboard
	FPlatformApplicationMisc::ClipboardCopy(*CopyText);
}

void SFindInAsset::MakeQueryDelayed()
{
	if (bSearching)
	{
		if (UStateTree* StateTreePtr = Cast<UStateTree>(StateTreesToProcess.Pop().ResolveObjectPtr()))
		{
			ProcessedStateTrees.Add(StateTreePtr);

			TArray<Private::FFoundItemStack> StackItems;
			auto ProcessProperty = [Self = this, &StackItems, StateTreePtr](const TPropertyValueIterator<FProperty>& Iterator, const FString& Value)
			{
				const FProperty* Property = Iterator.Key();
				bool bMatches = Value.Contains(*Self->SearchString);

				if (!bMatches)
				{
					const FString& PropertyName =  Property->GetDisplayNameText().ToString();
					bMatches = PropertyName.Contains(*Self->SearchString);
				}

				if (bMatches)
				{
					// Find the State/Node item. If the item doesn't exist, then create it (with the parents).
					// Add a Value to the State/Node item
					TSharedPtr<FFindResult> NewResult = MakeShared<FFindResult>();
					NewResult->Name = Property->GetFName();
					NewResult->PropertyName = Property->GetDisplayNameText();
					NewResult->Value = Value;
					NewResult->Type = FFindResult::EResultType::Value;

					// Find the root
					TSharedPtr<FFindResult>* CurrentParentNode = Self->ItemsFound.FindByPredicate([StateTreePtr](const TSharedPtr<FFindResult>& Item)
					{
						check(Item->Type == FFindResult::EResultType::StateTree);
						return Item->StateTree == StateTreePtr;
					});
					if (!CurrentParentNode)
					{
						CurrentParentNode = &(Self->ItemsFound.Add_GetRef(MakeShared<FFindResult>()));
						(*CurrentParentNode)->Type = FFindResult::EResultType::StateTree;
						(*CurrentParentNode)->Name = StateTreePtr->GetFName();
						(*CurrentParentNode)->StateTree = StateTreePtr;
					}

					// Start the index at one to skip the EditorData object
					for (int32 Index = 1; Index < StackItems.Num(); ++Index)
					{
						auto CreateNode = [&CurrentParentNode, StateTreePtr](FGuid ID, FName Name, const FSlateBrush* Brush, FSlateColor Color, FFindResult::EResultType ResultType)
						{
							// Find the owner
							TSharedPtr<FFindResult>* NewParentParentNode = (*CurrentParentNode)->Children.FindByPredicate([ID](const TSharedPtr<FFindResult>& Item)
								{
									return Item->ID == ID;
								});
							if (!NewParentParentNode)
							{
								NewParentParentNode = &(*CurrentParentNode)->Children.Add_GetRef(MakeShared<FFindResult>());
								(*NewParentParentNode)->Parent = *CurrentParentNode;
								(*NewParentParentNode)->Name = Name;
								(*NewParentParentNode)->ID = ID;
								(*NewParentParentNode)->IconBrush = Brush;
								(*NewParentParentNode)->IconColor = Color;
								(*NewParentParentNode)->Type = ResultType;
							}
							CurrentParentNode = NewParentParentNode;
						};

						if (StackItems[Index].Struct == FStateTreeEditorNode::StaticStruct())
						{
							const FStateTreeEditorNode* EditorNode = reinterpret_cast<const FStateTreeEditorNode*>(StackItems[Index].Data);
							check(EditorNode);
							const FSlateBrush* Brush = nullptr;
							FSlateColor BrushColor = FSlateColor::UseForeground();
							if (const FStateTreeNodeBase* BaseNode = EditorNode->GetNode().GetPtr<const FStateTreeNodeBase>())
							{
								Brush = UE::StateTreeEditor::EditorNodeUtils::ParseIcon(BaseNode->GetIconName()).GetIcon();
								BrushColor = FLinearColor(BaseNode->GetIconColor());
							}
							CreateNode(EditorNode->ID, EditorNode->GetName(), Brush, BrushColor, FFindResult::EResultType::Node);
						}
						else if (StackItems[Index].Struct == UStateTreeState::StaticClass())
						{
							const UStateTreeState* State = reinterpret_cast<const UStateTreeState*>(StackItems[Index].Data);
							check(State);
							const FSlateBrush* Brush = FStateTreeEditorStyle::GetBrushForSelectionBehaviorType(State->SelectionBehavior, !State->Children.IsEmpty(), State->Type);
							const FSlateColor BrushColor = FSlateColor::UseForeground();
							CreateNode(State->ID, State->Name, Brush, BrushColor, FFindResult::EResultType::State);
						}
					}

					NewResult->Parent = *CurrentParentNode;
					(*CurrentParentNode)->Children.Add(NewResult);
				}

				// Add linked StateTree
				if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property))
				{
					if (const UStateTreeState* State = Cast<const UStateTreeState>(ObjectProperty->GetObjectPropertyValue(Iterator.Value())))
					{
						if (State->Type == EStateTreeStateType::LinkedAsset && State->LinkedAsset)
						{
							if (!Self->ProcessedStateTrees.Contains(State->LinkedAsset))
							{
								Self->StateTreesToProcess.Add(State->LinkedAsset);
							}
						}
					}
				}
			};

			Private::IterateProperties(StackItems, StateTreePtr->EditorData->GetClass(), StateTreePtr->EditorData, ProcessProperty);
		}
	}

	if (StateTreesToProcess.IsEmpty())
	{
		bSearching = false;
		TreeView->RequestTreeRefresh();
		for (const TSharedPtr<FFindResult>& Entry : ItemsFound)
		{
			ExpandAll(Entry);
		}
	}
	else
	{
		TriggerQueryDelayed();
	}
}

void SFindInAsset::ExpandAll(const TSharedPtr<FFindResult>& Entry)
{
	TreeView->SetItemExpansion(Entry, true);

	for (const TSharedPtr<FFindResult>& Child : Entry->Children)
	{
		ExpandAll(Child);
	}
}

void SFindInAsset::TriggerQueryDelayed()
{
	FTSTicker::GetCoreTicker().AddTicker(TEXT("SFindInAsset::MakeQuery"), 0.1f,
		[WeakThis = TWeakPtr<SFindInAsset>(SharedThis(this))](float)
		{
			if (TSharedPtr<SFindInAsset> This = WeakThis.Pin())
			{
				This->MakeQueryDelayed();
			}
			return false;
		});
}

void SFindInAsset::MakeQuery(FStringView InSearchString)
{
	SearchString = InSearchString;

	// Reset the UI
	ClearResults();

	// Start the search on next frame
	if (SearchString.Len() > 0)
	{
		if (TSharedPtr<IStateTreeEditorHost> EditorHostPinned = EditorHost.Pin())
		{
			const UStateTree* StateTree = EditorHostPinned->GetStateTree();
			StateTreesToProcess.Add(StateTree);
			bSearching = true;
			TriggerQueryDelayed();
		}
	}
}

void SFindInAsset::ClearResults()
{
	bSearching = false;
	ProcessedStateTrees.Reset();
	StateTreesToProcess.Reset();
	ItemsFound.Empty();
	HighlightText = FText::FromString(SearchString);
	SearchTextField->SetText(HighlightText);
	TreeView->RequestTreeRefresh();
}

EVisibility SFindInAsset::HandleGetSearchingWidgetVisiblity() const
{
	return bSearching ? EVisibility::Visible : EVisibility::Collapsed;
}

void SFindInAsset::HandleSearchTextCommitted(const FText& Text, ETextCommit::Type CommitType)
{
	if (CommitType == ETextCommit::OnEnter)
	{
		MakeQuery(Text.ToString());
	}
}

TSharedRef<ITableRow> SFindInAsset::HandleTreeGenerateRow(TSharedPtr<FFindResult> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	if (InItem->Type == FFindResult::EResultType::StateTree)
	{
		return SNew(STableRow< TSharedPtr<FFindResult>>, OwnerTable)
		.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("ShowParentsTableView.Row"))
		.Padding(FMargin(2.f, 3.f, 2.f, 3.f))
		[
			SNew(STextBlock)
			.Text(FText::FromName(InItem->Name))
		];
	}
	else if (InItem->Type == FFindResult::EResultType::Value)
	{
		return SNew(STableRow< TSharedPtr<FFindResult> >, OwnerTable)
		.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("ShowParentsTableView.Row"))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(InItem->PropertyName)
				.HighlightText(HighlightText)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.f)
			[
				SNew(STextBlock)
				.Text(FText::FromString(FString::Printf(TEXT(" = %s"), *InItem->Value)))
				.HighlightText(HighlightText)
			]
		];
	}
	else
	{
		return SNew(STableRow< TSharedPtr<FFindResult> >, OwnerTable)
		.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("ShowParentsTableView.Row"))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SImage)
				.Image(InItem->IconBrush)
				.ColorAndOpacity(InItem->IconColor)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.f)
			[
				SNew(STextBlock)
				.Text(FText::FromName(InItem->Name))
				.HighlightText(HighlightText)
			]
		];
	}
}

void SFindInAsset::HandleGetTreeChildren(TSharedPtr<FFindResult> InItem, TArray< TSharedPtr<FFindResult> >& OutChildren)
{
	OutChildren += InItem->Children;
}

void SFindInAsset::HandleTreeSelectionDoubleClicked(TSharedPtr<FFindResult> Item)
{
	if (Item.IsValid())
	{
		TSharedPtr<FFindResult> StateTreeParent = Item;
		FGuid NodeID;
		FGuid StateID;
		while (StateTreeParent && StateTreeParent->Type != FFindResult::EResultType::StateTree)
		{
			if (StateTreeParent->Type == FFindResult::EResultType::Node && !NodeID.IsValid())
			{
				NodeID = StateTreeParent->ID;
			}
			if (StateTreeParent->Type == FFindResult::EResultType::State && !StateID.IsValid())
			{
				StateID = StateTreeParent->ID;
			}
			StateTreeParent = StateTreeParent->Parent.Pin();
		}

		if (UStateTree* StateTree = StateTreeParent ? StateTreeParent->StateTree.Get() : nullptr)
		{
			if (UStateTreeEditingSubsystem* StateTreeEditingSubsystem = GEditor->GetEditorSubsystem<UStateTreeEditingSubsystem>())
			{
				TSharedRef<FStateTreeViewModel> ViewModel = StateTreeEditingSubsystem->FindOrAddViewModel(StateTree);
				ViewModel->BringNodeToFocus(ViewModel->GetMutableStateByID(StateID), NodeID);
			}
		}
	}
}

TSharedPtr<SWidget> SFindInAsset::HandleTreeContextMenuOpening()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

	MenuBuilder.BeginSection("BasicOperations");
	{
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().SelectAll);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
	}

	return MenuBuilder.MakeWidget();
}

FReply SFindInAsset::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList.IsValid())
	{
		if (CommandList->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}
	}
	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

} //namespace UE::StateTreeEditor

#undef LOCTEXT_NAMESPACE
