// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "UObject/ObjectKey.h"

class FUICommandList;
class ITableRow;
class SSearchBox;
class STableViewBase;
template<typename ItemType> class STreeView;
class UStateTree;
class IStateTreeEditorHost;

namespace UE::StateTreeEditor
{

/**
 * Widget to display and find inside a StateTree asset.
 */
class SFindInAsset : public SCompoundWidget
{
private:
	/* Item that matched the search results */
	class FFindResult
	{
	public:
		TArray<TSharedPtr<FFindResult>> Children;
		TWeakPtr<FFindResult> Parent;

		/* The brush for the icon. */
		const FSlateBrush* IconBrush = nullptr;
		FSlateColor IconColor;

		/* Valid when the type is State, Node or Value. */
		FName Name;

		/* Valid when the type is Value. */
		FText PropertyName;
		FString Value;

		/* Valid when type is State or Node */
		FGuid ID;

		/* Valid when type is StateTree */
		TWeakObjectPtr<UStateTree> StateTree;

		enum class EResultType : uint8
		{
			StateTree,
			State,
			Node,
			Value,
		};
		/** The type of the result. */
		EResultType Type = EResultType::StateTree;
	};
	using STreeViewType = STreeView<TSharedPtr<FFindResult>>;

public:
	SLATE_BEGIN_ARGS(SFindInAsset) {}
		SLATE_ARGUMENT_DEFAULT(bool, bShowSearchBar) = true;
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<IStateTreeEditorHost> EditorHost);

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	void MakeQuery(FStringView SearchString);

	void ClearResults();

private:
	/** Delay the query to allow for async or multiple assets search. */
	void MakeQueryDelayed();
	void TriggerQueryDelayed();

	/** Register commands used by the widget. */
	void RegisterCommands();
	void HandleCopyAction();
	void HandleSelectAllAction();
	void RecursiveSelectAllAction(const TSharedPtr<FFindResult>&);

	/** Celled to check the visibility of the result. */
	EVisibility HandleGetSearchingWidgetVisiblity() const;

	/* Called when user changes commits text to the search box */
	void HandleSearchTextCommitted(const FText& Text, ETextCommit::Type CommitType);

	///** Called when the user clicks the global find results button */
	//FReply HandleOpenGlobalFindResults();

	/** Called when the lock button is clicked in a global find results tab */
	FReply HandleLockButtonClicked();

	/** Returns the image used for the lock button in a global find results tab */
	const FSlateBrush* HandleGetLockButtonImage() const;

	/* Get the children of a row */
	void HandleGetTreeChildren(TSharedPtr<FFindResult> InItem, TArray<TSharedPtr<FFindResult>>& OutChildren);

	/* Called when user double clicks on a new result */
	void HandleTreeSelectionDoubleClicked(TSharedPtr<FFindResult> Item);

	/* Called when a new row is being generated */
	TSharedRef<ITableRow> HandleTreeGenerateRow(TSharedPtr<FFindResult> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Callback to build the context menu when right clicking in the tree */
	TSharedPtr<SWidget> HandleTreeContextMenuOpening();

	/** Expand all item in the tree. */
	void ExpandAll(const TSharedPtr<FFindResult>& Entry);

private:
	TWeakPtr<IStateTreeEditorHost> EditorHost;

	/** Tree processed during . */
	TSet<FObjectKey> ProcessedStateTrees;

	/** Object to search. */
	TArray<FObjectKey> StateTreesToProcess;

	FString SearchString;
	bool bSearching = false;

	/** The search text box. */
	TSharedPtr<SSearchBox> SearchTextField;

	/* The tree view displays the results. */
	TSharedPtr<STreeViewType> TreeView;

	/** Commands handled by this widget. */
	TSharedPtr<FUICommandList> CommandList;

	/* This buffer stores the currently displayed results. */
	TArray<TSharedPtr<FFindResult>> ItemsFound;

	/* The string to search for. */
	FString	SearchValue;

	/* The string to highlight in the results. */
	FText HighlightText;
};

} //namespace UE::StateTreeEditor