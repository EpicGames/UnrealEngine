// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_StateTreeBlueprintPropertyRef.h"
#include "StateTreePropertyRef.h"
#include "Blueprint/StateTreeNodeBlueprintBase.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "EdGraphUtilities.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "K2Node_VariableGet.h"
#include "StateTreeGraphSchema.h"
#include "StateTreePropertyRefHelpers.h"
#include "KismetCompiledFunctionContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_StateTreeBlueprintPropertyRef)

#define LOCTEXT_NAMESPACE "K2Node_StateTreeBlueprintPropertyRef"

namespace UE::StateTree::BlueprintPropertyRef
{
	const FName PropertyRefPinName(TEXT("PropertyRef"));

	const FName SucceededExecPinName(TEXT("Succeeded"));
	const FName FailedExecPinName(TEXT("Failed"));

	const FName OutReferencePinName(TEXT("Reference"));
}

void UK2Node_StateTreeBlueprintPropertyRef::AllocateDefaultPins()
{
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Struct, FStateTreeBlueprintPropertyRef::StaticStruct(), UE::StateTree::BlueprintPropertyRef::PropertyRefPinName);

	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UE::StateTree::BlueprintPropertyRef::SucceededExecPinName);
    CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UE::StateTree::BlueprintPropertyRef::FailedExecPinName);

	UEdGraphNode::FCreatePinParams OutPinParams;
	OutPinParams.bIsReference = true;
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Wildcard, UE::StateTree::BlueprintPropertyRef::OutReferencePinName, OutPinParams);
}

FText UK2Node_StateTreeBlueprintPropertyRef::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("NodeTitle", "Get Property Reference");
}

FText UK2Node_StateTreeBlueprintPropertyRef::GetTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Returns bound property reference.");
}

FText UK2Node_StateTreeBlueprintPropertyRef::GetMenuCategory() const
{
	return LOCTEXT("NodeCategory", "StateTree|PropertyRef");
}

void UK2Node_StateTreeBlueprintPropertyRef::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	Super::GetMenuActions(ActionRegistrar);
	UClass* Action = GetClass();
	if (ActionRegistrar.IsOpenForRegistration(Action))
    {
        UBlueprintNodeSpawner* Spawner = UBlueprintNodeSpawner::Create(Action);
        ActionRegistrar.AddBlueprintAction(Action, Spawner);
    }
}

void UK2Node_StateTreeBlueprintPropertyRef::ReconstructNode()
{
	Super::ReconstructNode();
	UpdateOutputPin();
}

void UK2Node_StateTreeBlueprintPropertyRef::ClearCachedBlueprintData(UBlueprint* Blueprint)
{
	Super::ClearCachedBlueprintData(Blueprint);
	Modify(true);
	UpdateOutputPin();
}

void UK2Node_StateTreeBlueprintPropertyRef::ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	const UClass* BlueprintClass = GetBlueprintClassFromNode();
	if (!BlueprintClass->IsChildOf<UStateTreeNodeBlueprintBase>())
	{
		const FText ErrorText = LOCTEXT("InvalidSelfType", "This blueprint (self) is not a 'State Tree Blueprint Node'.");
		MessageLog.Error(*ErrorText.ToString(), this);
	}
}

void UK2Node_StateTreeBlueprintPropertyRef::UpdateOutputPin() const
{
	UEdGraphPin& OutValuePin = *FindPinChecked(UE::StateTree::BlueprintPropertyRef::OutReferencePinName);
	const UEdGraphPin& PropertyRefPin = *FindPinChecked(UE::StateTree::BlueprintPropertyRef::PropertyRefPinName);

	const UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(this);
	if (Blueprint && Blueprint->GeneratedClass && PropertyRefPin.LinkedTo.Num() == 1)
	{
		UK2Node_VariableGet* VariableGet = Cast<UK2Node_VariableGet>(PropertyRefPin.LinkedTo[0]->GetOwningNode());
		if (VariableGet)
		{
			UObject* CDO = Blueprint->GeneratedClass->GetDefaultObject<UObject>();
			FProperty* Property = Blueprint->GeneratedClass->FindPropertyByName(VariableGet->VariableReference.GetMemberName());
			FStructProperty* StructProperty = CastField<FStructProperty>(Property);
			if (StructProperty && StructProperty->Struct == FStateTreeBlueprintPropertyRef::StaticStruct())
			{
				FStateTreeBlueprintPropertyRef PropertyRef;
				Property->GetValue_InContainer(CDO, &PropertyRef);
				FEdGraphPinType PinType = UE::StateTree::PropertyRefHelpers::GetBlueprintPropertyRefInternalTypeAsPin(PropertyRef);

				PinType.bIsReference = true;
				OutValuePin.PinType = PinType;
				GetDefault<UE::StateTree::Editor::UEdGraphSchema_StateTree>()->SetPinAutogeneratedDefaultValueBasedOnType(&OutValuePin);
				return;
			}
		}
	}

	FCreatePinParams PinParams{};
	OutValuePin.PinType = FEdGraphPinType(UEdGraphSchema_K2::PC_Wildcard, NAME_None, nullptr, PinParams.ContainerType, PinParams.bIsReference, PinParams.ValueTerminalType);
}

void FKCHandler_StateTreeBlueprintPropertyRefGet::RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node)
{
	const UK2Node_StateTreeBlueprintPropertyRef* GetRefNode = CastChecked<UK2Node_StateTreeBlueprintPropertyRef>(Node);

	UEdGraphPin* OutPin = GetRefNode->FindPinChecked(UE::StateTree::BlueprintPropertyRef::OutReferencePinName);
	FBPTerminal* OutValueTerminal = new FBPTerminal();
	Context.InlineGeneratedValues.Add(OutValueTerminal);
	OutValueTerminal->CopyFromPin(OutPin, Context.NetNameMap->MakeValidName(OutPin));
	Context.NetMap.Add(OutPin, OutValueTerminal);

	FBPTerminal* BoolTerm = Context.CreateLocalTerminal();
	BoolTerm->Type.PinCategory = UEdGraphSchema_K2::PC_Boolean;
	BoolTerm->Source = Node;
	BoolTerm->Name = Context.NetNameMap->MakeValidName(Node, TEXT("ReferenceValid"));
	TemporaryBoolTerminals.Add(Node, BoolTerm);
}

void FKCHandler_StateTreeBlueprintPropertyRefGet::Compile(FKismetFunctionContext& Context, UEdGraphNode* Node)
{
	const UK2Node_StateTreeBlueprintPropertyRef* GetRefNode = CastChecked<UK2Node_StateTreeBlueprintPropertyRef>(Node);
   
    UEdGraphPin* PropertyRefPin = GetRefNode->FindPinChecked(UE::StateTree::BlueprintPropertyRef::PropertyRefPinName);
    FBPTerminal* PropertyRefTerminal = Context.NetMap.FindRef(FEdGraphUtilities::GetNetFromPin(PropertyRefPin));
	check(PropertyRefTerminal);
    
	const UEdGraphPin* OutPin = GetRefNode->FindPinChecked(UE::StateTree::BlueprintPropertyRef::OutReferencePinName);
    FBPTerminal* OutTerminal = Context.NetMap.FindRef(OutPin);
	check(OutTerminal);

	FBlueprintCompiledStatement* GetRef = new FBlueprintCompiledStatement();
	GetRef->Type = KCST_CallFunction;
	GetRef->FunctionToCall = UStateTreeNodeBlueprintBase::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UStateTreeNodeBlueprintBase, GetPropertyReference));
	GetRef->FunctionContext = nullptr;
	GetRef->RHS.Add(PropertyRefTerminal);

    OutTerminal->InlineGeneratedParameter = GetRef;
	OutTerminal->bPassedByReference = true;

	FBPTerminal* IsValidTerminal = TemporaryBoolTerminals.FindChecked( Node);
	FBlueprintCompiledStatement& HasPropertyFunction = Context.AppendStatementForNode(Node);
	HasPropertyFunction.Type = KCST_CallFunction;
	HasPropertyFunction.FunctionToCall = UStateTreeNodeBlueprintBase::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UStateTreeNodeBlueprintBase, IsPropertyRefValid));
	HasPropertyFunction.FunctionContext = nullptr;
	HasPropertyFunction.LHS = IsValidTerminal;
	HasPropertyFunction.RHS.Add(PropertyRefTerminal);

	FBlueprintCompiledStatement& OnInvalidRef = Context.AppendStatementForNode(Node);
	OnInvalidRef.Type = KCST_GotoIfNot;
	OnInvalidRef.LHS = IsValidTerminal;
	Context.GotoFixupRequestMap.Add(&OnInvalidRef, GetRefNode->FindPinChecked(UE::StateTree::BlueprintPropertyRef::FailedExecPinName));

	FBlueprintCompiledStatement& OnValidRef = Context.AppendStatementForNode(Node);
	OnValidRef.Type = KCST_UnconditionalGoto;
	OnValidRef.LHS = IsValidTerminal;
	Context.GotoFixupRequestMap.Add(&OnValidRef, GetRefNode->FindPinChecked(UE::StateTree::BlueprintPropertyRef::SucceededExecPinName));
}

#undef LOCTEXT_NAMESPACE
