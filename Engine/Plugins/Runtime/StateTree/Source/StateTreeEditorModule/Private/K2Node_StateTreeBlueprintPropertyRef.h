// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "K2Node.h"
#include "KismetCompilerMisc.h"
#include "K2Node_StateTreeBlueprintPropertyRef.generated.h"

class FKCHandler_StateTreeBlueprintPropertyRefGet : public FNodeHandlingFunctor
{
public:
	using FNodeHandlingFunctor::FNodeHandlingFunctor;
	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node) override;
	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override;

private:
	TMap<UEdGraphNode*, FBPTerminal*> TemporaryBoolTerminals;
};

UCLASS()
class UK2Node_StateTreeBlueprintPropertyRef : public UK2Node
{
	GENERATED_BODY()

public:
    FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
    FText GetTooltipText() const override;
    FText GetMenuCategory() const override;

	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual void ReconstructNode() override;
	virtual void ClearCachedBlueprintData(UBlueprint* Blueprint) override;

	virtual void ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const override;

	virtual FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override
	{
		return new FKCHandler_StateTreeBlueprintPropertyRefGet(CompilerContext);
	}

protected:
	void UpdateOutputPin() const;
	void AllocateDefaultPins() override;
};