// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_StateTreeNodeGetPropertyDescription.h"
#include "Blueprint/StateTreeNodeBlueprintBase.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "K2Node_CallFunction.h"
#include "KismetCompiler.h"
#include "StateTreeGraphSchema.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_StateTreeNodeGetPropertyDescription)

#define LOCTEXT_NAMESPACE "K2Node_StateTreeNodeGetPropertyDescription"


void UK2Node_StateTreeNodeGetPropertyDescription::AllocateDefaultPins()
{
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Text, UEdGraphSchema_K2::PN_ReturnValue);
	
	Super::AllocateDefaultPins();
}

FText UK2Node_StateTreeNodeGetPropertyDescription::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	UBlueprint* Blueprint = HasValidBlueprint() ? GetBlueprint() : nullptr;
	const FProperty* Property = const_cast<FMemberReference&>(Variable).ResolveMember<FProperty>(Blueprint);
	const FText SelectedPropertyName = Property ? Property->GetDisplayNameText() : LOCTEXT("None", "<None>");

	return FText::Format(LOCTEXT("NodeTitle", "Get Description for {0}"), SelectedPropertyName);
}

FText UK2Node_StateTreeNodeGetPropertyDescription::GetTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Returns text describing the specified member variable.");
}

FText UK2Node_StateTreeNodeGetPropertyDescription::GetMenuCategory() const
{
	return LOCTEXT("NodeCategory", "StateTree");
}

void UK2Node_StateTreeNodeGetPropertyDescription::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	Super::GetMenuActions(ActionRegistrar);
	UClass* Action = GetClass();
	if (ActionRegistrar.IsOpenForRegistration(Action))
    {
        UBlueprintNodeSpawner* Spawner = UBlueprintNodeSpawner::Create(Action);
        ActionRegistrar.AddBlueprintAction(Action, Spawner);
    }
}

void UK2Node_StateTreeNodeGetPropertyDescription::ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	const UClass* BlueprintClass = GetBlueprintClassFromNode();
	if (!BlueprintClass->IsChildOf<UStateTreeNodeBlueprintBase>())
	{
		const FText ErrorText = LOCTEXT("InvalidSelfType", "This blueprint (self) is not a 'State Tree Blueprint Node'.");
		MessageLog.Error(*ErrorText.ToString(), this);
	}

	UBlueprint* Blueprint = HasValidBlueprint() ? GetBlueprint() : nullptr;
	const FProperty* Property = const_cast<FMemberReference&>(Variable).ResolveMember<FProperty>(Blueprint);
	if (!Property)
	{
		const FText ErrorText = FText::Format(LOCTEXT("InvalidProperty", "Cannot find property '{0}'."), FText::FromName(Variable.GetMemberName()));
		MessageLog.Error(*ErrorText.ToString(), this);
	}
}

void UK2Node_StateTreeNodeGetPropertyDescription::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	UBlueprint* Blueprint = HasValidBlueprint() ? GetBlueprint() : nullptr;

	// Property name
	const FProperty* Property = Variable.ResolveMember<FProperty>(Blueprint);
	FString SelectedPropertyName = Property ? Property->GetName() : TEXT("");
	
	UK2Node_CallFunction* CallGetPropertyDescription = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	CallGetPropertyDescription->SetFromFunction(UStateTreeNodeBlueprintBase::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UStateTreeNodeBlueprintBase, GetPropertyDescriptionByPropertyName)));
	CallGetPropertyDescription->AllocateDefaultPins();

	UEdGraphPin* PropertyNamePin = CallGetPropertyDescription->FindPinChecked(TEXT("PropertyName"));
	check(PropertyNamePin);
	PropertyNamePin->DefaultValue = SelectedPropertyName;

	UEdGraphPin* OrgReturnPin = FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
	UEdGraphPin* NewReturnPin = CallGetPropertyDescription->GetReturnValuePin();
	check(NewReturnPin);
	CompilerContext.MovePinLinksToIntermediate(*OrgReturnPin, *NewReturnPin);
}

#undef LOCTEXT_NAMESPACE
