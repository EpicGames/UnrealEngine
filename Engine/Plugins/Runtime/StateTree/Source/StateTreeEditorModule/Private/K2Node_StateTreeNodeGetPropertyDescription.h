// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "K2Node.h"
#include "Engine/MemberReference.h"
#include "K2Node_StateTreeNodeGetPropertyDescription.generated.h"

class FKismetCompilerContext;

/**
 * Returns description for a specific property in the class.
 * If the property has a binding, the binding string will be returned.
 * Otherwise the current value of the property is returned.
 */
UCLASS()
class UK2Node_StateTreeNodeGetPropertyDescription : public UK2Node
{
	GENERATED_BODY()

public:
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetMenuCategory() const override;

	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual bool IsNodePure() const override { return true; }
	virtual bool ShouldShowNodeProperties() const override { return true; }
	virtual void ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual void ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const override;

protected:
	virtual void AllocateDefaultPins() override;

	/** Property of the class to describe. */
	UPROPERTY(EditAnywhere, Category = "Node", meta=(PropertyReference, AllowOnlyVisibleProperties))
	FMemberReference Variable;

	friend class FKCHandler_StateTreeNodeGetPropertyDescription;
};
