// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_StateTreeReference.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "EditorCategoryUtils.h"
#include "EdGraph/EdGraphPin.h"
#include "K2Node_Variable.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "StateTree.h"
#include "StateTreeDelegates.h"
#include "StateTreeFunctionLibrary.h"
#include "StateTreeGraphSchema.h"
#include "StateTreeReference.h"
#include "StructUtils/PropertyBag.h"

#include "KismetNodes/SGraphNodeK2Default.h"
#include "SGraphPin.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_StateTreeReference)

#define LOCTEXT_NAMESPACE "K2Node_StateTreeReference"

namespace UE::StateTreeEditor::Private
{
	static FLazyName StateTreePinName = "BA2CE32D97D46A3A524AC510A794C3C";

	bool IsPropertyPin(const UEdGraphPin* Pin)
	{
		if (Pin->ParentPin)
		{
			return IsPropertyPin(Pin->ParentPin);
		}

		return Pin->Direction == EEdGraphPinDirection::EGPD_Input
			&& Pin->PinType.PinCategory != UEdGraphSchema_K2::PC_Exec
			&& Pin->PinName != StateTreePinName;
	}

	bool CanUseProperty(const UEdGraphPin* Pin)
	{
		// A property needs to be linked to be considered.
		//The default value won't matches the value in the state tree asset.
		return !Pin->bOrphanedPin
			&& Pin->ParentPin == nullptr
			&& Pin->LinkedTo.Num() > 0;
	}

	bool DoRenamedPinsMatch(UStateTree* StateTree, const UEdGraphPin* NewPin, const UEdGraphPin* OldPin)
	{
		using namespace UE::StateTree::Editor;

		if (StateTree == nullptr
			|| NewPin == nullptr
			|| OldPin == nullptr
			|| OldPin->Direction != NewPin->Direction)
		{
			return false;
		}

		const FInstancedPropertyBag& Parameters = StateTree->GetDefaultParameters();
		if (!Parameters.IsValid())
		{
			return false;
		}

		const bool bCompatible = GetDefault<UEdGraphSchema_StateTree>()->ArePinTypesCompatible(NewPin->PinType, OldPin->PinType);
		if (!bCompatible)
		{
			return false;
		}

		UScriptStruct* Struct = const_cast<UScriptStruct*>(Parameters.GetValue().GetScriptStruct());
		return UK2Node_Variable::DoesRenamedVariableMatch(OldPin->PinName, NewPin->PinName, Struct);
	}
}

UK2Node_MakeStateTreeReference::UK2Node_MakeStateTreeReference()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		ParametersChangedHandle = UE::StateTree::Delegates::OnPostCompile.AddUObject(this, &UK2Node_MakeStateTreeReference::HandleStateTreeCompiled);
	}
}

void UK2Node_MakeStateTreeReference::BeginDestroy()
{
	UE::StateTree::Delegates::OnPostCompile.Remove(ParametersChangedHandle);
	Super::BeginDestroy();
}

FText UK2Node_MakeStateTreeReference::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	auto MakeNodeTitle = []()
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("StructName"), FStateTreeReference::StaticStruct()->GetDisplayNameText());
			return FText::Format(LOCTEXT("MakeNodeTitle", "Make {StructName}"), Args);
		};
	static FText LocalCachedNodeTitle = MakeNodeTitle();
	return LocalCachedNodeTitle;
}

FText UK2Node_MakeStateTreeReference::GetTooltipText() const
{
	auto MakeNodeTooltip = []()
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("StructName"), FStateTreeReference::StaticStruct()->GetDisplayNameText());
			return FText::Format(LOCTEXT("MakeNodeTooltip", "Adds a node that create a  {StructName} from its members"), Args);
		};
	static FText LocalCachedTooltip = MakeNodeTooltip();
	return LocalCachedTooltip;
}

FText UK2Node_MakeStateTreeReference::GetMenuCategory() const
{
	static FText LocalCachedCategory = FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Struct);
	return LocalCachedCategory;
}

FSlateIcon UK2Node_MakeStateTreeReference::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon LocalCachedIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.MakeStruct_16x");
	return LocalCachedIcon;
}

FLinearColor UK2Node_MakeStateTreeReference::GetNodeTitleColor() const
{
	auto MakeTitleColor = []() -> FLinearColor
		{
			FEdGraphPinType PinType;
			PinType.PinCategory = UEdGraphSchema_K2::PC_Struct;
			PinType.PinSubCategoryObject = FStateTreeReference::StaticStruct();
			return GetDefault<UE::StateTree::Editor::UEdGraphSchema_StateTree>()->GetPinTypeColor(PinType);
		};

	static FLinearColor LocalCachedTitleColor = MakeTitleColor();
	return LocalCachedTitleColor;
}

void UK2Node_MakeStateTreeReference::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);
	UEdGraphPin* ReturnValuePin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Struct, FStateTreeReference::StaticStruct(), UEdGraphSchema_K2::PN_ReturnValue);
	ReturnValuePin->PinFriendlyName = FStateTreeReference::StaticStruct()->GetDisplayNameText();

	UEdGraphPin* StateTreePin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, UStateTree::StaticClass(), UE::StateTreeEditor::Private::StateTreePinName);
	StateTreePin->bNotConnectable = true;
	StateTreePin->PinFriendlyName = LOCTEXT("StateTreePinName", "State Tree");

	CreatePropertyPins();
}

void UK2Node_MakeStateTreeReference::CreatePropertyPins()
{
	if (StateTree)
	{
		const FInstancedPropertyBag& Parameters = StateTree->GetDefaultParameters();
		if (Parameters.IsValid())
		{
			UScriptStruct* Struct = const_cast<UScriptStruct*>(Parameters.GetValue().GetScriptStruct());
			FOptionalPinManager OptionalPinManager;
			OptionalPinManager.RebuildPropertyList(ShowPinForProperties, Struct);
			OptionalPinManager.CreateVisiblePins(ShowPinForProperties, Struct, EGPD_Input, this);

			for (UEdGraphPin* Pin : Pins)
			{
				if (UE::StateTreeEditor::Private::IsPropertyPin(Pin))
				{
					// Force the property to be linked until we have the enabled/disabled on the default value.
					Pin->bDefaultValueIsIgnored = true;
				}
			}
		}
	}
}

UK2Node::ERedirectType UK2Node_MakeStateTreeReference::DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const
{
	ERedirectType Result = Super::DoPinsMatchForReconstruction(NewPin, NewPinIndex, OldPin, OldPinIndex);

	if (Result == ERedirectType_None
		&& UE::StateTreeEditor::Private::DoRenamedPinsMatch(StateTree, NewPin, OldPin))
	{
		Result = ERedirectType_Name;
	}
	return Result;
}

void UK2Node_MakeStateTreeReference::PinDefaultValueChanged(UEdGraphPin* Pin)
{
	Super::PinDefaultValueChanged(Pin);

	if (Pin == FindPinChecked(UE::StateTreeEditor::Private::StateTreePinName))
	{
		SetStateTree(GetStateTreeDefaultValue());
	}
}

namespace UE::StateTreeEditor::Private
{
	class SMakeStateTreeRefeerenceNode : public SGraphNodeK2Default
	{
		TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* Pin) const
		{
			TSharedPtr<SGraphPin> Result = SGraphNodeK2Default::CreatePinWidget(Pin);
			if (Pin->PinName == UE::StateTreeEditor::Private::StateTreePinName)
			{
				Result->GetPinImageWidget()->SetVisibility(EVisibility::Hidden);
			}
			return Result;
		}
	};
}

TSharedPtr<SGraphNode> UK2Node_MakeStateTreeReference::CreateVisualWidget()
{
	return SNew(UE::StateTreeEditor::Private::SMakeStateTreeRefeerenceNode, this);
}

void UK2Node_MakeStateTreeReference::PreloadRequiredAssets()
{
	PreloadObject(FStateTreeReference::StaticStruct());
	if (StateTree)
	{
		PreloadObject(StateTree);
	}

	Super::PreloadRequiredAssets();
}

void UK2Node_MakeStateTreeReference::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	bool bTestExistingProperties = true;
	if (StateTree)
	{
		// Tests if the property pins are valid
		const FInstancedPropertyBag& Parameters = StateTree->GetDefaultParameters();
		if (Parameters.IsValid())
		{
			bTestExistingProperties = false;
			const UScriptStruct* Struct = Parameters.GetValue().GetScriptStruct();
			for (UEdGraphPin* Pin : Pins)
			{
				if (UE::StateTreeEditor::Private::IsPropertyPin(Pin)
					&& UE::StateTreeEditor::Private::CanUseProperty(Pin))
				{
					const FProperty* Property = Struct->FindPropertyByName(Pin->PinName);
					const FPropertyBagPropertyDesc* PropertyDesc = Parameters.FindPropertyDescByName(Pin->PinName);
					if (Property == nullptr || PropertyDesc == nullptr)
					{
						FFormatNamedArguments Args;
						Args.Add(TEXT("PropertyName"), FText::FromName(Pin->PinName));
						const FText Error = FText::Format(LOCTEXT("CanNotFindProeprty_Error", "Can't find the property {PropertyName} in @@"), Args);
						MessageLog.Error(*Error.ToString(), this);
					}
				}
			}
		}

		// Tests if the cached value matches the value of the pin. It should match unless it was set manually by code.
		{
			UEdGraphPin* ThisStateTreePin = FindPinChecked(UE::StateTreeEditor::Private::StateTreePinName);
			if (StateTree != ThisStateTreePin->DefaultObject)
			{
				MessageLog.Error(*LOCTEXT("StateTreeMatchingError", "The State Tree asset does not match with the pin @@. Clear and set the State Tree pin.").ToString(), this);
			}
		}
	}
	
	// Tests if we expect a state tree (it is valid to construct an empty struct)
	if (bTestExistingProperties)
	{
		const bool bHasProperty = Pins.ContainsByPredicate([](const UEdGraphPin* Pin)
			{
				return UE::StateTreeEditor::Private::IsPropertyPin(Pin)
					&& UE::StateTreeEditor::Private::CanUseProperty(Pin);
			});
		if (bHasProperty)
		{
			MessageLog.Error(*LOCTEXT("NoStateTree_Error", "No State Tree in @@").ToString(), this);
		}
	}
}

void UK2Node_MakeStateTreeReference::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	UClass* Action = GetClass();
	if (ActionRegistrar.IsOpenForRegistration(Action))
	{
		UBlueprintNodeSpawner* MakeNodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(MakeNodeSpawner != nullptr);
		ActionRegistrar.AddBlueprintAction(Action, MakeNodeSpawner);
	}
}

void UK2Node_MakeStateTreeReference::SetStateTree(UStateTree* InStateTree)
{
	StateTree = InStateTree;
	FBlueprintEditorUtils::MarkBlueprintAsModified(GetBlueprint());
	ReconstructNode();
}

UStateTree* UK2Node_MakeStateTreeReference::GetStateTreeDefaultValue() const
{
	return Cast<UStateTree>(FindPinChecked(UE::StateTreeEditor::Private::StateTreePinName)->DefaultObject);
}

void UK2Node_MakeStateTreeReference::HandleStateTreeCompiled(const UStateTree& InStateTree)
{
	if (&InStateTree == StateTree)
	{
		SetStateTree(StateTree);
	}
}

void UK2Node_MakeStateTreeReference::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	if (CompilerContext.bIsFullCompile)
	{
		const UEdGraphSchema_K2* K2Schema = CompilerContext.GetSchema();

		// Convert to
		//local = MakeStateTreeReference(StateTree)
		//for each properties
		//  K2_SetParametersProperty(local, id, value)
		UEdGraphPin* LastThen = nullptr;
		UEdGraphPin* MakeStateTreeReferenceNodeResultPin = nullptr;
		{
			UK2Node_CallFunction* MakeStateTreeReferenceNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
			UFunction* Function = UStateTreeFunctionLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UStateTreeFunctionLibrary, MakeStateTreeReference));
			MakeStateTreeReferenceNode->SetFromFunction(Function);
			MakeStateTreeReferenceNode->AllocateDefaultPins();
			CompilerContext.MessageLog.NotifyIntermediateObjectCreation(MakeStateTreeReferenceNode, SourceGraph);
			{
				UEdGraphPin* ThisStateTreePin = FindPinChecked(UE::StateTreeEditor::Private::StateTreePinName);
				UEdGraphPin* NewStateTreePin = MakeStateTreeReferenceNode->FindPinChecked(FName("StateTree"));
				CompilerContext.MovePinLinksToIntermediate(*ThisStateTreePin, *NewStateTreePin);
			}
			{
				UEdGraphPin* ThisResultPin = FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
				MakeStateTreeReferenceNodeResultPin = MakeStateTreeReferenceNode->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
				CompilerContext.MovePinLinksToIntermediate(*ThisResultPin, *MakeStateTreeReferenceNodeResultPin);
			}
			{
				UEdGraphPin* ThisExecPin = GetExecPin();
				UEdGraphPin* NewExecPin = MakeStateTreeReferenceNode->GetExecPin();
				CompilerContext.MovePinLinksToIntermediate(*ThisExecPin, *NewExecPin);
			}
			{
				UEdGraphPin* ThisThenPin = GetThenPin();
				UEdGraphPin* NewThenPin = MakeStateTreeReferenceNode->GetThenPin();
				CompilerContext.MovePinLinksToIntermediate(*ThisThenPin, *NewThenPin);
				LastThen = NewThenPin;
			}
		}

		if (StateTree != nullptr)
		{
			//for each pin call K2_SetParametersProperty
			for (UEdGraphPin* Pin : Pins)
			{
				if (UE::StateTreeEditor::Private::IsPropertyPin(Pin)
					&& UE::StateTreeEditor::Private::CanUseProperty(Pin))
				{
						UK2Node_CallFunction* SetParametersPropertyNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
						UFunction* Function = UStateTreeFunctionLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UStateTreeFunctionLibrary, K2_SetParametersProperty));
						SetParametersPropertyNode->SetFromFunction(Function);
						SetParametersPropertyNode->AllocateDefaultPins();
						CompilerContext.MessageLog.NotifyIntermediateObjectCreation(SetParametersPropertyNode, SourceGraph);
						
						{
							UEdGraphPin* NewValuePin = SetParametersPropertyNode->FindPinChecked(FName("Reference"), EGPD_Input);
							ensure(K2Schema->TryCreateConnection(MakeStateTreeReferenceNodeResultPin, NewValuePin));
						}
						{
							UEdGraphPin* NewValuePin = SetParametersPropertyNode->FindPinChecked(FName("PropertyID"), EGPD_Input);
							
							const FPropertyBagPropertyDesc* PropertyDesc = StateTree->GetDefaultParameters().FindPropertyDescByName(Pin->PinName);
							check(PropertyDesc);

							FGuid Default;
							FGuid TempValue = PropertyDesc->ID;
							TBaseStructure<FGuid>::Get()->ExportText(NewValuePin->DefaultValue, &TempValue, &Default, nullptr, PPF_None, nullptr);
						}
						{
							UEdGraphPin* NewValuePin = SetParametersPropertyNode->FindPinChecked(FName("NewValue"), EGPD_Input);
							NewValuePin->PinType = Pin->PinType;
							CompilerContext.MovePinLinksToIntermediate(*Pin, *NewValuePin);
						}
						// move last Then to new Then and link the last Then to new exec
						{
							UEdGraphPin* NewThenPin = SetParametersPropertyNode->GetThenPin();
							CompilerContext.MovePinLinksToIntermediate(*LastThen, *NewThenPin);
						}
						{
							UEdGraphPin* NewExecPin = SetParametersPropertyNode->GetExecPin();
							ensure(K2Schema->TryCreateConnection(LastThen, NewExecPin));
						}
						LastThen = SetParametersPropertyNode->GetThenPin();
				}
			}
		}
	}
	BreakAllNodeLinks();
}

#undef LOCTEXT_NAMESPACE
