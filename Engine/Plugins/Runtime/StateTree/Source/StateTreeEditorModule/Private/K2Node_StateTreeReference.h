// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node_CallFunction.h"
#include "K2Node_StateTreeReference.generated.h"

class FKismetCompilerContext;
class SGraphNode;
class UStateTree;

UCLASS()
class UK2Node_MakeStateTreeReference : public UK2Node
{
	GENERATED_BODY()

public:
	UK2Node_MakeStateTreeReference();
	virtual void BeginDestroy() override;

	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetMenuCategory() const override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	virtual FLinearColor GetNodeTitleColor() const;

	virtual void AllocateDefaultPins() override;
	virtual ERedirectType DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const override;
	virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;

	virtual void PreloadRequiredAssets() override;
	virtual void ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;

	virtual void ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;

	virtual bool NodeCausesStructuralBlueprintChange() const override
	{
		return false;
	}

	virtual bool IsNodePure() const override
	{
		return false;
	}

	virtual bool DrawNodeAsVariable() const override
	{
		return false;
	}

	virtual bool CanSplitPin(const UEdGraphPin* Pin) const override
	{
		return false;
	}

private:
	void CreatePropertyPins();
	UStateTree* GetStateTreeDefaultValue() const;
	void SetStateTree(UStateTree* InStateTree);
	void HandleStateTreeCompiled(const UStateTree& StateTree);

private:
	/** Created pins from the state tree properties */
	UPROPERTY()
	TArray<FOptionalPinFromProperty> ShowPinForProperties;
	/** State tree asset set in the pin and saved here to rebuild the property pins.*/
	UPROPERTY()
	TObjectPtr<UStateTree> StateTree;
	
	FDelegateHandle ParametersChangedHandle;
};