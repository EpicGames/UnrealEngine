// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStateTreeDiff.h"
#include "Customizations/StateTreeBindingExtension.h"
#include "StateTreeEditorStyle.h"
#include "DetailsDiff.h"
#include "DetailTreeNode.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailsView.h"
#include "SDetailsDiff.h"
#include "SlateOptMacros.h"
#include "SStateTreeSplitter.h"
#include "StateTreeDiffControl.h"
#include "StateTreeDiffHelper.h"
#include "StateTreeState.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"

#define LOCTEXT_NAMESPACE "SStateTreeDif"

namespace UE::StateTree::Diff
{

SDiffWidget::SDiffWidget()
{
}

SDiffWidget::~SDiffWidget()
{
	if (AssetEditorCloseHandle.IsValid())
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetEditorRequestClose().Remove(AssetEditorCloseHandle);
	}
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SDiffWidget::Construct(const FArguments& InArgs)
{
	check(InArgs._OldAsset || InArgs._NewAsset);
	OldAssetPanel.StateTree = TStrongObjectPtr(InArgs._OldAsset);
	NewAssetPanel.StateTree = TStrongObjectPtr(InArgs._NewAsset);
	OldAssetPanel.RevisionInfo = InArgs._OldRevision;
	NewAssetPanel.RevisionInfo = InArgs._NewRevision;

	// sometimes we want to clearly identify the assets being diffed (when it's
	// not the same asset in each panel)
	OldAssetPanel.bShowAssetName = InArgs._ShowAssetNames;
	NewAssetPanel.bShowAssetName = InArgs._ShowAssetNames;

	if (InArgs._ParentWindow.IsValid())
	{
		WeakParentWindow = InArgs._ParentWindow;

		AssetEditorCloseHandle = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetEditorRequestClose().AddSP(this, &SDiffWidget::HandleAssetEditorRequestClose);
	}

	FToolBarBuilder NavToolBarBuilder(TSharedPtr<const FUICommandList>(), FMultiBoxCustomization::None);
	NavToolBarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP(this, &SDiffWidget::PrevDiff),
			FCanExecuteAction::CreateSP(this, &SDiffWidget::HasPrevDiff)),
		NAME_None, LOCTEXT("PrevDiffLabel", "Prev"), LOCTEXT("PrevDiffTooltip", "Go to previous difference"), FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintDif.PrevDiff"));
	NavToolBarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP(this, &SDiffWidget::NextDiff),
			FCanExecuteAction::CreateSP(this, &SDiffWidget::HasNextDiff)),
		NAME_None, LOCTEXT("NextDiffLabel", "Next"), LOCTEXT("NextDiffTooltip", "Go to next difference"), FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintDif.NextDiff"));

	DifferencesTreeView = DiffTreeView::CreateTreeView(&Differences);

	GenerateDifferencesList();

	const auto TextBlock = [](const FText Text) -> TSharedRef<SWidget>
	{
		return SNew(SBox)
			.Padding(FMargin(4.0f, 10.0f))
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.Visibility(EVisibility::HitTestInvisible)
				.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
				.Text(Text)
			];
	};

	TopRevisionInfoWidget =
		SNew(SSplitter)
		.Visibility(EVisibility::HitTestInvisible)
		+ SSplitter::Slot()
		.Value(.2f)
		[
			SNew(SBox)
		]
		+ SSplitter::Slot()
		.Value(.8f)
		[
			SNew(SSplitter)
			.PhysicalSplitterHandleSize(10.0f)
			+ SSplitter::Slot()
			.Value(.5f)
			[
				TextBlock(DiffViewUtils::GetPanelLabel(OldAssetPanel.StateTree.Get(), OldAssetPanel.RevisionInfo, FText()))
			]
			+ SSplitter::Slot()
			.Value(.5f)
			[
				TextBlock(DiffViewUtils::GetPanelLabel(NewAssetPanel.StateTree.Get(), NewAssetPanel.RevisionInfo, FText()))
			]
		];

	this->ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Docking.Tab", ".ContentAreaBrush"))
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				.VAlign(VAlign_Top)
				[
					TopRevisionInfoWidget.ToSharedRef()
				]
				+ SOverlay::Slot()
				[
					SNew(SSplitter)
					.Orientation(EOrientation::Orient_Vertical)
					+ SSplitter::Slot()
					.Value(.55f)
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 2.0f, 0.0f, 2.0f)
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.Padding(4.f)
							.AutoWidth()
							[
								NavToolBarBuilder.MakeWidget()
							]
							+ SHorizontalBox::Slot()
							[
								SNew(SSpacer)
							]
						]
						+ SVerticalBox::Slot()
						[
							SNew(SSplitter)
							+ SSplitter::Slot()
							.Value(.2f)
							[
								SNew(SBorder)
								.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
								[
									DifferencesTreeView.ToSharedRef()
								]
							]
							+ SSplitter::Slot()
							.Value(.8f)
							[
								StateTreePanel.Splitter.ToSharedRef()
							]
						]
					]
					+ SSplitter::Slot()
					.Value(.45f)
					[
						SAssignNew(DetailsViewContents, SBox)
					]
				]
			]
		];
	SetDetailsDiff();
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SDiffWidget::HandleAssetEditorRequestClose(UObject* Asset, const EAssetEditorCloseReason CloseReason)
{
	if (OldAssetPanel.StateTree.Get()== Asset || NewAssetPanel.StateTree.Get() == Asset || CloseReason == EAssetEditorCloseReason::CloseAllAssetEditors)
	{
		// Tell our window to close and set our selves to collapsed to try and stop it from ticking
		SetVisibility(EVisibility::Collapsed);

		if (AssetEditorCloseHandle.IsValid())
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetEditorRequestClose().Remove(AssetEditorCloseHandle);
		}

		if (const TSharedPtr<SWindow> ParentWindow = WeakParentWindow.Pin())
		{
			ParentWindow->RequestDestroyWindow();
		}
	}
}

TSharedRef<SDiffWidget> SDiffWidget::CreateDiffWindow(const FText WindowTitle, TNotNull<const UStateTree*> OldStateTree, TNotNull<const UStateTree*> NewStateTree, const FRevisionInfo& OldRevision, const FRevisionInfo& NewRevision)
{
	// sometimes we're comparing different revisions of one single asset (other
	// times we're comparing two completely separate assets altogether)
	const bool bIsSingleAsset = (NewStateTree->GetName() == OldStateTree->GetName());

	const TSharedPtr<SWindow> Window = SNew(SWindow)
		.Title(WindowTitle)
		.ClientSize(FVector2D(1000.f, 800.f));

	TSharedRef<SDiffWidget> StateTreeDiff = SNew(SDiffWidget)
		.OldAsset(OldStateTree)
		.NewAsset(NewStateTree)
		.OldRevision(OldRevision)
		.NewRevision(NewRevision)
		.ShowAssetNames(!bIsSingleAsset)
		.ParentWindow(Window);

	Window->SetContent(StateTreeDiff);

	// Make this window a child of the modal window if we've been spawned while one is active.
	const TSharedPtr<SWindow> ActiveModal = FSlateApplication::Get().GetActiveTopLevelWindow();
	if (ActiveModal.IsValid())
	{
		FSlateApplication::Get().AddWindowAsNativeChild(Window.ToSharedRef(), ActiveModal.ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(Window.ToSharedRef());
	}

	return StateTreeDiff;
}

TSharedRef<SDiffWidget> SDiffWidget::CreateDiffWindow(TNotNull<const UStateTree*> OldStateTree, TNotNull<const UStateTree*> NewStateTree,
	const FRevisionInfo& OldRevision, const FRevisionInfo& NewRevision, const UClass* StateTreeClass)
{
	// sometimes we're comparing different revisions of one single asset (other
	// times we're comparing two completely separate assets altogether)
	//@TODO use pathname instead of asset name.
	const bool bIsSingleAsset = NewStateTree->GetFName() == OldStateTree->GetFName();

	FText WindowTitle = FText::Format(LOCTEXT("NamelessStateTreeDiff", "{0} Diff (experimental)"), StateTreeClass->GetDisplayNameText());
	// if we're diffing one asset against itself
	if (bIsSingleAsset)
	{
		// identify the assumed single asset in the window's title
		const FString STName = NewStateTree->GetName();
		WindowTitle = FText::Format(LOCTEXT("NamedStateTreeDiff", "{0} - {1} Diff (experimental)"), FText::FromString(STName), StateTreeClass->GetDisplayNameText());
	}

	return CreateDiffWindow(WindowTitle, OldStateTree, NewStateTree, OldRevision, NewRevision);
}

void SDiffWidget::NextDiff() const
{
	DiffTreeView::HighlightNextDifference(DifferencesTreeView.ToSharedRef(), Differences, Differences);
}

void SDiffWidget::PrevDiff() const
{
	DiffTreeView::HighlightPrevDifference(DifferencesTreeView.ToSharedRef(), Differences, Differences);
}

bool SDiffWidget::HasNextDiff() const
{
	return DiffTreeView::HasNextDifference(DifferencesTreeView.ToSharedRef(), Differences);
}

bool SDiffWidget::HasPrevDiff() const
{
	return DiffTreeView::HasPrevDifference(DifferencesTreeView.ToSharedRef(), Differences);
}

void SDiffWidget::GenerateDifferencesList()
{
	Differences.Empty();

	GenerateDiffPanel();

	DifferencesTreeView->RebuildList();
}

void SDiffWidget::GenerateDiffPanel()
{
	const UStateTree* OldStateTree = OldAssetPanel.StateTree.Get();
	const UStateTree* NewStateTree = NewAssetPanel.StateTree.Get();
	StateTreePanel.DiffControl = MakeShared<FDiffControl>(
		OldStateTree,
		NewStateTree,
		FOnDiffEntryFocused{});
	StateTreePanel.DiffControl->GenerateTreeEntries(Differences);
	StateTreePanel.DiffControl->GetOnStateDiffEntryFocused().AddSP(this, &SDiffWidget::HandleStateDiffEntryFocused);

	TSharedPtr<SDiffSplitter> DiffSplitter = SNew(SDiffSplitter);
	if (OldAssetPanel.StateTree)
	{
		DiffSplitter->AddSlot(
			SDiffSplitter::Slot()
			.Value(0.5f)
			.StateTreeView(StateTreePanel.DiffControl->GetDetailsWidget(OldStateTree))
			.StateTree(OldStateTree));
	}
	if (NewAssetPanel.StateTree)
	{
		DiffSplitter->AddSlot(
			SDiffSplitter::Slot()
			.Value(0.5f)
			.StateTreeView(StateTreePanel.DiffControl->GetDetailsWidget(NewStateTree))
			.StateTree(NewStateTree));
	}
	StateTreePanel.Splitter = DiffSplitter;
}

void SDiffWidget::HandleStateDiffEntryFocused(const FSingleDiffEntry& StateDiff)
{
	const FStateSoftPath LeftStatePath = StateDiff.Identifier;
	const FStateSoftPath RightStatePath = StateDiff.SecondaryIdentifier ? StateDiff.SecondaryIdentifier : StateDiff.Identifier;
	StateTreePanel.Splitter->HandleSelectionChanged(LeftStatePath, RightStatePath);

	const UStateTree* OldStateTree = OldAssetPanel.StateTree.Get();
	const UStateTree* NewStateTree = NewAssetPanel.StateTree.Get();
	const UStateTreeState* OldState = OldStateTree != nullptr ? LeftStatePath.ResolvePath(OldStateTree) : nullptr;
	const UStateTreeState* NewState = NewStateTree != nullptr ? RightStatePath.ResolvePath(NewStateTree) : nullptr;

	// If comparing states that exist in both state trees display them in the details diff view
	if (OldState && NewState)
	{
		SetDetailsDiff(OldState, NewState);

	}
	// If we clear selection on both state trees we can display an empty details diff view
	else if (!OldState && !NewState)
	{
		SetDetailsDiff();
	}
	// If the state only exists in one of the state trees (either added or removed), details diff view will not work.
	else
	{
		// So the states are put into separate details views
		const TSharedPtr<SBox> LeftWidget = SNew(SBox);
		const TSharedPtr<SBox> RightWidget = SNew(SBox);
		if (OldState)
		{
			const FDetailsDiff DetailsDiff(OldState, true);
			LeftWidget->SetContent(DetailsDiff.DetailsWidget());
		}
		if (NewState)
		{
			const FDetailsDiff DetailsDiff(NewState, false);
			RightWidget->SetContent(DetailsDiff.DetailsWidget());
		}
		// And displayed in a way that resembles the details diff view
		DetailsViewContents->SetContent(
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Docking.Tab", ".ContentAreaBrush"))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 2.0f, 0.0f, 2.0f)
				+ SVerticalBox::Slot()
				[
					SNew(SSplitter)
					+ SSplitter::Slot()
					.Value(.2f)
					+ SSplitter::Slot()
					.Value(.8f)
					[
						SNew(SSplitter)
						.PhysicalSplitterHandleSize(5.f)
						+ SSplitter::Slot()
						.Value(.5f)
						[
							SNew(SBox)
							.Padding(15.f, 0.f, 15.f, 0.f)
							[
								LeftWidget.ToSharedRef()
							]
						]
						+ SSplitter::Slot()
						.Value(.5f)
						[
							SNew(SBox)
							.Padding(15.f, 0.f, 15.f, 0.f)
							[
								RightWidget.ToSharedRef()
							]
						]
					]
				]
			]

		);
	}
}

void SDiffWidget::SetDetailsDiff(const UStateTreeState* OldState, const UStateTreeState* NewState)
{
	const UObject* OldAsset = OldState ? OldState : OldAssetPanel.StateTree ? static_cast<const UObject*>(OldAssetPanel.StateTree->EditorData) : nullptr;
	const UObject* NewAsset = NewState ? NewState : NewAssetPanel.StateTree ? static_cast<const UObject*>(NewAssetPanel.StateTree->EditorData) : nullptr;

	if (const bool bIsState = OldState || NewState)
	{
		StateBindingDiffs.Reset(StateTreePanel.DiffControl->GetBindingDifferences().Num());
		for (const FSingleDiffEntry& BindingDiff : StateTreePanel.DiffControl->GetBindingDifferences())
		{
			const UStateTree* OldStateTree = OldAssetPanel.StateTree.Get();
			const UStateTree* NewStateTree = NewAssetPanel.StateTree.Get();
			if (OldStateTree != nullptr
				&& NewStateTree != nullptr
				&& BindingDiff.Identifier.ResolvePath(OldStateTree) == OldState
				&& BindingDiff.SecondaryIdentifier.ResolvePath(NewStateTree) == NewState)
			{
				StateBindingDiffs.Push(BindingDiff);
			}
		}
	}
	else
	{
		StateBindingDiffs.Reset();
	}

	TArray<FSingleObjectDiffEntry> Entries;
	const TSharedRef<SDetailsDiff> DetailsDiff = SNew(SDetailsDiff)
		.OldAsset(OldAsset)
		.NewAsset(NewAsset)
		.OldRevision(OldAssetPanel.RevisionInfo)
		.NewRevision(NewAssetPanel.RevisionInfo)
		.ShowAssetNames(false)
		.OnCustomizeDetailsWidget_Static(&SDiffWidget::AddStateTreeExtensionToDetailsView)
		.OnGenerateCustomDiffEntries(this, &SDiffWidget::AddBindingDiffToDiffEntries)
		.OnOrganizeDiffEntries_Static(&SDiffWidget::OrganizeDiffEntries, OldState, NewState)
		.OnGenerateCustomDiffEntryWidget_Static(&SDiffWidget::GenerateCustomDiffEntryWidget, OldState, NewState)
		.RowHighlightColor_Static(&SDiffWidget::GetRowHighlightColor)
		.ShouldHighlightRow(this, &SDiffWidget::ShouldHighlightRow);

	DetailsViewContents->SetContent(DetailsDiff);
}

void SDiffWidget::AddBindingDiffToDiffEntries(TArray<FSingleObjectDiffEntry>& OutEntries)
{
	OutEntries.Reserve(StateBindingDiffs.Num());

	for (const FSingleDiffEntry& BindingDiff : StateBindingDiffs)
	{
		EPropertyDiffType::Type DiffType = EPropertyDiffType::Invalid;
		switch (BindingDiff.DiffType)
		{
		case EStateDiffType::BindingAddedToA:
		case EStateDiffType::BindingAddedToB:
		case EStateDiffType::BindingChanged:
			DiffType = EPropertyDiffType::PropertyValueChanged;
			break;
		}

		if (DiffType != EPropertyDiffType::Invalid)
		{
			FSingleObjectDiffEntry Entry(BindingDiff.BindingPath, DiffType);
			OutEntries.Add(Entry);
		}
	}
}

void SDiffWidget::OrganizeDiffEntries(
	TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& OutDiffTreeEntries,
	const TArray<FSingleObjectDiffEntry>& DiffEntries,
	TFunctionRef<TSharedPtr<FBlueprintDifferenceTreeEntry>(const FSingleObjectDiffEntry&)> GenerateDiffTreeEntry,
	TFunctionRef<TSharedPtr<FBlueprintDifferenceTreeEntry>(FText&)> GenerateCategoryEntry, const UStateTreeState* OldState, const UStateTreeState* NewState)
{
	static FText RightRevision = LOCTEXT("NewRevisionIdentifier", "Right Revision");
	static FText StateText = LOCTEXT("StateText", "State");
	static FText ParameterText = LOCTEXT("ParametersText", "Parameters");
	static FText ConditionText = LOCTEXT("EnterConditionsText", "Enter Conditions");
	static FText TaskText = LOCTEXT("TasksText", "Tasks");
	static FText TransitionText = LOCTEXT("TransitionsText", "Transitions");
	static FText ConsiderationText = LOCTEXT("ConsiderationText", "Utility");

	TSet<int32> ConditionIndices;
	TSet<int32> TaskIndices;
	TSet<int32> TransitionIndices;
	TSet<int32> ConsiderationIndices;
	TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>> ParametersEntries;
	TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>> ConditionEntries;
	TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>> TaskEntries;
	TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>> TransitionEntries;
	TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>> ConsiderationEntries;
	TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>> StateEntries;

	TArray<FSingleObjectDiffEntry> SortedEntries = DiffEntries;
	SortedEntries.Sort([](const FSingleObjectDiffEntry& A, const FSingleObjectDiffEntry& B)
	{
		return A.Identifier.TryReadIndex(0) < B.Identifier.TryReadIndex(0);
	});

	for (const FSingleObjectDiffEntry& Difference : SortedEntries)
	{
		constexpr int32 PropertyCountFromRoot = 2; // 2 levels down from the root; first level being the category/parent collection and the second level the property that changed
		FSingleObjectDiffEntry SimplifiedEntry(Difference.Identifier.GetRootProperty(PropertyCountFromRoot), Difference.DiffType);
		TSharedPtr<FBlueprintDifferenceTreeEntry> Entry = GenerateDiffTreeEntry(SimplifiedEntry);
		if (Difference.Identifier.IsSubPropertyMatch(ConditionName))
		{
			const int32 Index = Difference.Identifier.TryReadIndex(0);
			if (!ConditionIndices.Contains(Index))
			{
				ConditionIndices.Add(Index);
				ConditionEntries.Add(Entry);
			}
		}
		else if (Difference.Identifier.IsSubPropertyMatch(TaskName))
		{
			const int32 Index = Difference.Identifier.TryReadIndex(0);
			if (!TaskIndices.Contains(Index))
			{
				TaskIndices.Add(Index);
				TaskEntries.Add(Entry);
			}
		}
		else if (Difference.Identifier.IsSubPropertyMatch(TransitionName))
		{
			const int32 Index = Difference.Identifier.TryReadIndex(0);
			if (!TransitionIndices.Contains(Index))
			{
				TransitionIndices.Add(Index);
				TransitionEntries.Add(Entry);
			}
		}
		else if (Difference.Identifier.IsSubPropertyMatch(ConsiderationName))
		{
			const int32 Index = Difference.Identifier.TryReadIndex(0);
			if (!ConsiderationIndices.Contains(Index))
			{
				ConsiderationIndices.Add(Index);
				ConsiderationEntries.Add(Entry);
			}
		}
		else if (Difference.Identifier.IsSubPropertyMatch(ParameterName))
		{
			// @todo investigate: currently unable to resolve full property path (FInstancedPropertyBag issue?)
			ParametersEntries.Add(GenerateDiffTreeEntry(Difference));
		}
		else
		{
			StateEntries.Add(Entry);
		}
	}

	OutDiffTreeEntries.Append(StateEntries);

	if (ParametersEntries.Num() > 0)
	{
		TSharedPtr<FBlueprintDifferenceTreeEntry> ParametersEntry = GenerateCategoryEntry(ParameterText);
		ParametersEntry->Children = ParametersEntries;
		OutDiffTreeEntries.Push(ParametersEntry);
	}

	if (ConditionEntries.Num() > 0)
	{
		TSharedPtr<FBlueprintDifferenceTreeEntry> ConditionEntry = GenerateCategoryEntry(ConditionText);
		ConditionEntry->Children = ConditionEntries;
		OutDiffTreeEntries.Push(ConditionEntry);
	}

	if (ConsiderationEntries.Num() > 0)
	{
		TSharedPtr<FBlueprintDifferenceTreeEntry> ConsiderationEntry = GenerateCategoryEntry(ConsiderationText);
		ConsiderationEntry->Children = ConsiderationEntries;
		OutDiffTreeEntries.Push(ConsiderationEntry);
	}

	if (TaskEntries.Num() > 0)
	{
		TSharedPtr<FBlueprintDifferenceTreeEntry> TaskEntry = GenerateCategoryEntry(TaskText);
		TaskEntry->Children = TaskEntries;
		OutDiffTreeEntries.Push(TaskEntry);
	}

	if (TransitionEntries.Num() > 0)
	{
		TSharedPtr<FBlueprintDifferenceTreeEntry> TransitionEntry = GenerateCategoryEntry(TransitionText);
		TransitionEntry->Children = TransitionEntries;
		OutDiffTreeEntries.Push(TransitionEntry);
	}
}

TSharedRef<SWidget> SDiffWidget::GenerateCustomDiffEntryWidget(const FSingleObjectDiffEntry& DiffEntry, FText&, const UStateTreeState* OldState, const UStateTreeState* NewState)
{
	const UStateTreeState* SourceState = DiffEntry.DiffType == EPropertyDiffType::PropertyAddedToB ? NewState : OldState;
	FText PropertyName = FText::FromString(DiffEntry.Identifier.ToDisplayName());
	if (DiffEntry.Identifier.IsSubPropertyMatch(ConditionName))
	{
		const int32 ConditionIndex = DiffEntry.Identifier.TryReadIndex(0);
		const FStateTreeEditorNode& ConditionEntry = SourceState->EnterConditions[ConditionIndex];
		PropertyName = FText::Format(FText::FromString(TEXT("[{0}]")), FText::FromName(ConditionEntry.GetName()));
	}
	else if (DiffEntry.Identifier.IsSubPropertyMatch(TaskName))
	{
		const int32 TaskIndex = DiffEntry.Identifier.TryReadIndex(0);
		const FStateTreeEditorNode& TaskEntry = SourceState->Tasks[TaskIndex];
		PropertyName = FText::Format(FText::FromString(TEXT("[{0}]")), FText::FromName(TaskEntry.GetName()));
	}
	else if (DiffEntry.Identifier.IsSubPropertyMatch(ConsiderationName))
	{
		const int32 ConsiderationIndex = DiffEntry.Identifier.TryReadIndex(0);
		const FStateTreeEditorNode& ConsiderationEntry = SourceState->Considerations[ConsiderationIndex];
		PropertyName = FText::Format(FText::FromString(TEXT("[{0}]")), FText::FromName(ConsiderationEntry.GetName()));
	}
	else if (DiffEntry.Identifier.IsSubPropertyMatch(ParameterName))
	{
		constexpr int32 NumberOfPathElements = 1;
		PropertyName = FText::Format(FText::FromString(TEXT("[{0}]")), FText::FromString(DiffEntry.Identifier.ToDisplayName(NumberOfPathElements)));
	}

	return SNew(STextBlock)
		.Text(GetStateDiffMessage(DiffEntry, PropertyName))
		.ToolTipText(GetStateDiffMessage(DiffEntry, PropertyName))
		.ColorAndOpacity(GetStateDiffMessageColor(DiffEntry));
}

bool SDiffWidget::ShouldHighlightRow(const TUniquePtr<FAsyncDetailViewDiff::DiffNodeType>& DiffNode)
{
	if (DiffNode->DiffResult != ETreeDiffResult::Identical)
	{
		return true;
	}

	TSharedPtr<FDetailTreeNode> DetailNode = DiffNode->ValueA.Pin();
	DetailNode = DetailNode ? DetailNode : DiffNode->ValueB.Pin();
	const FPropertySoftPath PropertySoftPath(DetailNode->GetPropertyPath());
	if (PropertySoftPath.ToDisplayName().Len() == 0)
	{
		return false;
	}

	if (StateBindingDiffs.Num() > 0)
	{
		const FSingleDiffEntry& BindingDiff = StateBindingDiffs[0];
		return BindingDiff.BindingPath.IsSubPropertyMatch(PropertySoftPath) || BindingDiff.BindingPath == PropertySoftPath;
	}

	return false;
}

FLinearColor SDiffWidget::GetRowHighlightColor(const TUniquePtr<FAsyncDetailViewDiff::DiffNodeType>& DiffNode)
{
	switch (DiffNode->DiffResult)
	{
	case ETreeDiffResult::MissingFromTree1:
		return FStateTreeEditorStyle::Get().GetSlateColor("Colors.Diff.MissingFromTree1").GetSpecifiedColor();
	case ETreeDiffResult::MissingFromTree2:
		return FStateTreeEditorStyle::Get().GetSlateColor("Colors.Diff.MissingFromTree2").GetSpecifiedColor();
	default:
		return FStateTreeEditorStyle::Get().GetSlateColor("Colors.Diff.Changed").GetSpecifiedColor();
	}
}

void SDiffWidget::AddStateTreeExtensionToDetailsView(const TSharedRef<IDetailsView>& DetailsView)
{
	DetailsView->SetExtensionHandler(MakeShared<FStateTreeBindingExtension>().ToSharedPtr());
}

} // UE::StateTree::Diff

#undef LOCTEXT_NAMESPACE