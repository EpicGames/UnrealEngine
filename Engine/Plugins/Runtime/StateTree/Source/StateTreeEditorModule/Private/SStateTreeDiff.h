// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "StateTree.h"
#include "AsyncDetailViewDiff.h"
#include "IAssetTypeActions.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API STATETREEEDITORMODULE_API

class FSpawnTabArgs;
class FTabManager;
class FUICommandList;
class UStateTreeState;

namespace UE::StateTree::Diff
{
	class SDiffSplitter;
	class FDiffControl;
	struct FSingleDiffEntry;
	struct FStateSoftPath;


/** Panel used to display the state tree */
struct FDiffPanel
{
	/** The asset that owns the state tree view we are showing */
	TStrongObjectPtr<const UStateTree> StateTree = nullptr;

	/** Revision information for this asset */
	FRevisionInfo RevisionInfo;

	/** True if we should show a name identifying which asset this panel is displaying */
	bool bShowAssetName = true;

	/** The widget that contains the revision info in graph mode */
	TSharedPtr<SWidget> OverlayRevisionInfo;

private:
	/** Command list for this diff panel */
	TSharedPtr<FUICommandList> GraphEditorCommands;
};

/** Visual Diff between two StateTree assets */
class SDiffWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDiffWidget) {}
		SLATE_ARGUMENT(const UStateTree*, OldAsset)
		SLATE_ARGUMENT(const UStateTree*, NewAsset)
		SLATE_ARGUMENT(struct FRevisionInfo, OldRevision)
		SLATE_ARGUMENT(struct FRevisionInfo, NewRevision)
		SLATE_ARGUMENT(bool, ShowAssetNames)
		SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)
	SLATE_END_ARGS()

	UE_API SDiffWidget();
	UE_API void Construct(const FArguments& InArgs);
	UE_API virtual ~SDiffWidget();

	/** Helper function to create a window that holds a diff widget */
	static UE_API TSharedRef<SDiffWidget> CreateDiffWindow(FText WindowTitle, TNotNull<const UStateTree*> OldStateTree, TNotNull<const UStateTree*> NewStateTree, const FRevisionInfo& OldRevision, const FRevisionInfo& NewRevision);

	/** Helper function to create a window that holds a diff widget (default window title) */
	static UE_API TSharedRef<SDiffWidget> CreateDiffWindow(TNotNull<const UStateTree*> OldStateTree, TNotNull<const UStateTree*> NewStateTree, const FRevisionInfo& OldRevision, const FRevisionInfo& NewRevision, const UClass* StateTreeClass);

protected:
	/** Called when user clicks button to go to next difference */
	UE_API void NextDiff() const;

	/** Called when user clicks button to go to prev difference */
	UE_API void PrevDiff() const;

	/** Called to determine whether we have a list of differences to cycle through */
	UE_API bool HasNextDiff() const;
	UE_API bool HasPrevDiff() const;

	/** Function used to generate the list of differences and the widgets needed to calculate that list */
	UE_API void GenerateDifferencesList();

	/** Called when editor may need to be closed */
	UE_API void HandleAssetEditorRequestClose(UObject* Asset, EAssetEditorCloseReason CloseReason);

	struct FStateTreePanel
	{
		TSharedPtr<SDiffSplitter> Splitter;
		TSharedPtr<FDiffControl> DiffControl;
	};

	UE_API void GenerateDiffPanel();

	UE_API void HandleStateDiffEntryFocused(const FSingleDiffEntry& StateDiff);
	
	UE_API void SetDetailsDiff(const UStateTreeState* OldState = nullptr, const UStateTreeState* NewState = nullptr);

	static UE_API void AddStateTreeExtensionToDetailsView(const TSharedRef<IDetailsView>& DetailsView);

	UE_API void AddBindingDiffToDiffEntries(TArray<FSingleObjectDiffEntry>& OutEntries);

	static UE_API TSharedRef<SWidget> GenerateCustomDiffEntryWidget(const FSingleObjectDiffEntry& DiffEntry, FText& ObjectName, const UStateTreeState* OldState, const UStateTreeState* NewState);

	static UE_API void OrganizeDiffEntries(TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& OutDiffTreeEntries, const TArray<FSingleObjectDiffEntry>& DiffEntries,
		TFunctionRef<TSharedPtr<FBlueprintDifferenceTreeEntry>(const FSingleObjectDiffEntry&)> GenerateDiffTreeEntry,
		TFunctionRef<TSharedPtr<FBlueprintDifferenceTreeEntry>(FText&)> GenerateCategoryEntry,
		const UStateTreeState* OldState, const UStateTreeState* NewState);

	UE_API bool ShouldHighlightRow(const TUniquePtr<FAsyncDetailViewDiff::DiffNodeType>& DiffNode);

	static UE_API FLinearColor GetRowHighlightColor(const TUniquePtr<FAsyncDetailViewDiff::DiffNodeType>& DiffNode);

	/** The panel used to show the old revision */
	FDiffPanel OldAssetPanel;

	/** The panel used to show the new revision */
	FDiffPanel NewAssetPanel;

	TSharedPtr<SBox> DetailsViewContents;

	TSharedPtr<SSplitter> TopRevisionInfoWidget;

	TArray<FSingleDiffEntry> StateBindingDiffs;

	/** List of all differences, cached so that we can iterate only the differences and not labels, etc. */
	TArray<TSharedPtr<class FBlueprintDifferenceTreeEntry>> Differences;

	/** Tree view that displays the differences, cached for the buttons that iterate the differences: */
	TSharedPtr<STreeView<TSharedPtr<FBlueprintDifferenceTreeEntry>>> DifferencesTreeView;

	/** Stored reference to widget used to display the StateTree */
	FStateTreePanel StateTreePanel;

	/** A pointer to the window holding this */
	TWeakPtr<SWindow> WeakParentWindow;

	FDelegateHandle AssetEditorCloseHandle;

private:
	static inline const FLazyName ConditionName = FLazyName("EnterConditions");
	static inline const FLazyName TaskName = FLazyName("Tasks");
	static inline const FLazyName TransitionName = FLazyName("Transitions");
	static inline const FLazyName ConsiderationName = FLazyName("Considerations");
	static inline const FLazyName ParameterName = FLazyName("Parameters");
};

} // UE::StateTree::Diff

#undef UE_API
