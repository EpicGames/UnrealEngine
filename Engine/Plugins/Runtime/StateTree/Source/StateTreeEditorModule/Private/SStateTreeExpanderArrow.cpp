// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStateTreeExpanderArrow.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/STableRow.h"

SStateTreeExpanderArrow::SStateTreeExpanderArrow()
{
}

void SStateTreeExpanderArrow::Construct( const FArguments& InArgs, const TSharedPtr<class ITableRow>& TableRow  )
{
	OwnerRowPtr = TableRow;
	IndentAmount = InArgs._IndentAmount;
	BaseIndentLevel = InArgs._BaseIndentLevel;
	StyleSet = InArgs._StyleSet;
	
	WireColor = InArgs._WireColorAndOpacity;
	ImageSize = InArgs._ImageSize;
	ImagePadding = InArgs._ImagePadding;

	this->ChildSlot
	[
		SAssignNew(ExpanderArrow, SButton)
		.ButtonStyle(FCoreStyle::Get(), "NoBorder")
		.Visibility(this, &SStateTreeExpanderArrow::GetExpanderVisibility)
		.ClickMethod(EButtonClickMethod::MouseDown )
		.OnClicked(this, &SStateTreeExpanderArrow::OnArrowClicked)
		.ForegroundColor(FLinearColor(1,1,1,0.75f))
		.IsFocusable(false)
		.ContentPadding(TAttribute<FMargin>( this, &SStateTreeExpanderArrow::GetExpanderPadding )) //FMargin(0))
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Top)
		[
			SNew(SImage)
			.DesiredSizeOverride(FVector2D(InArgs._ImageSize))
			.Image(this, &SStateTreeExpanderArrow::GetExpanderImage)
			.ColorAndOpacity(FLinearColor(1,1,1,0.5f))
		]
	];
}

int32 SStateTreeExpanderArrow::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	static constexpr float WireThickness = 2.0f;
	static constexpr float HalfWireThickness = WireThickness / 2.0f;

	// We want to support drawing wires for the tree
	//                 Needs Wire Array
	//   v-[A]         {}
	//   |-v[B]        {1}
	//   | '-v[B]      {1,1}
	//   |   |--[C]    {1,0,1}
	//   |   |--[D]    {1,0,1}
	//   |   '--[E]    {1,0,1} 
	//   |>-[F]        {}
	//   '--[G]        {}
	//   
	//

	const float Indent = IndentAmount;
	const FSlateBrush* VerticalBarBrush = FAppStyle::GetBrush("WhiteBrush");;

	const float OffsetX = ImageSize.Y * 0.5f + ImagePadding.Left;;
	const float VerticalWireLoc = ImageSize.Y * 0.5f + ImagePadding.Top;

	if (VerticalBarBrush != nullptr)
	{
		const TSharedPtr<ITableRow> OwnerRow = OwnerRowPtr.Pin();
		const FLinearColor WireTint = WireColor.GetSpecifiedColor();

		// Draw vertical wires to indicate paths to parent nodes.
		const TBitArray<>& NeedsWireByLevel = OwnerRow->GetWiresNeededByDepth();
		const int32 NumLevels = NeedsWireByLevel.Num();
		for (int32 Level = 1; Level < NumLevels; Level++)
		{
			const float CurrentIndent = Indent * static_cast<float>(Level - 1);

			if (NeedsWireByLevel[Level])
			{
				const FVector2f Offset(CurrentIndent + OffsetX, 0);
				const FVector2f Size(WireThickness, AllottedGeometry.Size.Y);
				FSlateDrawElement::MakeBox(
					OutDrawElements,
					LayerId,
					AllottedGeometry.ToPaintGeometry(Size, FSlateLayoutTransform(Offset)),
					VerticalBarBrush,
					ESlateDrawEffect::None,
					WireTint
				);
			}
		}
		
		// For items that are the last expanded child in a list, we need to draw a special angle connector wire.
		if (OwnerRow->IsLastChild())
		{
			const float CurrentIndent = Indent * static_cast<float>(NumLevels - 2);
			const FVector2f Offset(CurrentIndent + OffsetX, 0);
			const FVector2f Size(WireThickness, VerticalWireLoc + HalfWireThickness);
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(Size, FSlateLayoutTransform(Offset)),
				VerticalBarBrush,
				ESlateDrawEffect::None,
				WireTint
			);
		}

		// If this item is expanded, we need to draw a 1/2-height the line down to its first child cell.
		if (OwnerRow->IsItemExpanded() && OwnerRow->DoesItemHaveChildren())
		{
			const float CurrentIndent = Indent * static_cast<float>(NumLevels - 1);
			const FVector2f Offset(CurrentIndent + OffsetX, ImageSize.Y + ImagePadding.Top);
			const FVector2f Size(WireThickness,  (AllottedGeometry.Size.Y - (ImageSize.Y + ImagePadding.Top)));
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(Size, FSlateLayoutTransform(Offset)),
				VerticalBarBrush,
				ESlateDrawEffect::None,
				WireTint
			);
		}

		// Draw horizontal connector from parent wire to child.
		if (NumLevels > 1)
		{
			const float HorizontalWireStart = static_cast<float>(NumLevels - 2) * Indent + OffsetX;
			const float HorizontalWireEnd = static_cast<float>(NumLevels - 1) * Indent + ImagePadding.Left - WireThickness + (OwnerRow->DoesItemHaveChildren() ? 0.0f : ImageSize.X);
			const FVector2f Offset(HorizontalWireStart + WireThickness, VerticalWireLoc - WireThickness * 0.5f);
			const FVector2f Size(HorizontalWireEnd - HorizontalWireStart, WireThickness);
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(Size, FSlateLayoutTransform(Offset)),
				VerticalBarBrush,
				ESlateDrawEffect::None,
				WireTint
			);
		}	
	}

	return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId + 1, InWidgetStyle, bParentEnabled);
}

/** Invoked when the expanded button is clicked (toggle item expansion) */
FReply SStateTreeExpanderArrow::OnArrowClicked()
{
	// Recurse the expansion if "shift" is being pressed
	const FModifierKeysState ModKeyState = FSlateApplication::Get().GetModifierKeys();
	if(ModKeyState.IsShiftDown())
	{
		OwnerRowPtr.Pin()->Private_OnExpanderArrowShiftClicked();
	}
	else
	{
		OwnerRowPtr.Pin()->ToggleExpansion();
	}

	return FReply::Handled();
}

/** @return Visible when has children; invisible otherwise */
EVisibility SStateTreeExpanderArrow::GetExpanderVisibility() const
{
	return OwnerRowPtr.Pin()->DoesItemHaveChildren() ? EVisibility::Visible : EVisibility::Hidden;
}

/** @return the name of an image that should be shown as the expander arrow */
const FSlateBrush* SStateTreeExpanderArrow::GetExpanderImage() const
{
	const bool bIsItemExpanded = OwnerRowPtr.Pin()->IsItemExpanded();

	FName ResourceName;
	if (bIsItemExpanded)
	{
		if ( ExpanderArrow->IsHovered() )
		{
			static FName ExpandedHoveredName = "TreeArrow_Expanded_Hovered";
			ResourceName = ExpandedHoveredName;
		}
		else
		{
			static FName ExpandedName = "TreeArrow_Expanded";
			ResourceName = ExpandedName;
		}
	}
	else
	{
		if ( ExpanderArrow->IsHovered() )
		{
			static FName CollapsedHoveredName = "TreeArrow_Collapsed_Hovered";
			ResourceName = CollapsedHoveredName;
		}
		else
		{
			static FName CollapsedName = "TreeArrow_Collapsed";
			ResourceName = CollapsedName;
		}
	}

	return StyleSet->GetBrush(ResourceName);
}

/** @return the margin corresponding to how far this item is indented */
FMargin SStateTreeExpanderArrow::GetExpanderPadding() const
{
	const int32 NestingDepth = FMath::Max(0, OwnerRowPtr.Pin()->GetIndentLevel() - BaseIndentLevel);
	FMargin Padding = ImagePadding;
	Padding.Left += static_cast<float>(NestingDepth) * IndentAmount;
	return Padding;
}
