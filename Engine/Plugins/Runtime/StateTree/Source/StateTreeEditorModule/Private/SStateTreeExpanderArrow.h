// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Attribute.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Layout/Margin.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/CoreStyle.h"

class ITableRow;
class SButton;
class SImage;

/**
 * Bespoke implementation of expander arrow for State Tree.
 */
class SStateTreeExpanderArrow : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SStateTreeExpanderArrow )
		: _StyleSet(&FCoreStyle::Get())
		, _IndentAmount(10)
		, _BaseIndentLevel(0)
		, _Image(nullptr)
		, _ImageSize(16.f, 16.f)
		, _ImagePadding(FMargin())
	{ }
		SLATE_ARGUMENT(const ISlateStyle*, StyleSet)
		SLATE_ARGUMENT(float, IndentAmount)
		SLATE_ARGUMENT(int32, BaseIndentLevel)
		SLATE_ATTRIBUTE(const FSlateBrush*, Image)
		SLATE_ATTRIBUTE(FSlateColor, ColorAndOpacity)
		SLATE_ARGUMENT(FVector2f, ImageSize)
		SLATE_ARGUMENT(FSlateColor, WireColorAndOpacity)
		SLATE_ARGUMENT(FMargin, ImagePadding)

	SLATE_END_ARGS()

	SStateTreeExpanderArrow();
	void Construct( const FArguments& InArgs, const TSharedPtr<class ITableRow>& TableRow );

protected:

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	/** Invoked when the expanded button is clicked (toggle item expansion) */
	FReply OnArrowClicked();

	/** @return Visible when has children; invisible otherwise */
	EVisibility GetExpanderVisibility() const;

	/** @return the name of an image that should be shown as the expander arrow */
	const FSlateBrush* GetExpanderImage() const;

	/** @return the margin corresponding to how far this item is indented */
	FMargin GetExpanderPadding() const;

	/** Pointer to the owning row. */
	TWeakPtr<class ITableRow> OwnerRowPtr;

	/** The amount of space to indent at each level */
	float IndentAmount = 10.0f;

	/** The level in the tree that begins the indention amount */
	int32 BaseIndentLevel = 0;

	/** Color for the wires */
	FSlateColor WireColor;

	/** Size of the expander image. */
	FVector2f ImageSize = FVector2f(16, 16);

	/** Padding for the expander image. */
	FMargin ImagePadding;
	
	/** A reference to the expander button */
	TSharedPtr<SButton> ExpanderArrow;

	/** The slate style to use */
	const ISlateStyle* StyleSet = nullptr;
};
