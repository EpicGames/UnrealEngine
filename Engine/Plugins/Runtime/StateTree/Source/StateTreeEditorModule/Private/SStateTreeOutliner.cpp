// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStateTreeOutliner.h"

#include "StateTreeDelegates.h"
#include "StateTreeEditorCommands.h"
#include "StateTreeEditorData.h"
#include "StateTreeState.h"
#include "StateTreeViewModel.h"
#include "Customizations/Widgets/SCompactTreeEditorView.h"
#include "Debugger/StateTreeDebuggerTypes.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"

SStateTreeOutliner::SStateTreeOutliner()
{
}

SStateTreeOutliner::~SStateTreeOutliner()
{
	if (StateTreeViewModel)
	{
		StateTreeViewModel->GetOnAssetChanged().RemoveAll(this);
		StateTreeViewModel->GetOnStatesRemoved().RemoveAll(this);
		StateTreeViewModel->GetOnStatesMoved().RemoveAll(this);
		StateTreeViewModel->GetOnStateAdded().RemoveAll(this);
		StateTreeViewModel->GetOnStatesChanged().RemoveAll(this);
		StateTreeViewModel->GetOnSelectionChanged().RemoveAll(this);
	}
	UE::StateTree::Delegates::OnVisualThemeChanged.RemoveAll(this);
}

void SStateTreeOutliner::Construct(const FArguments& InArgs, TSharedRef<FStateTreeViewModel> InStateTreeViewModel, const TSharedRef<FUICommandList>& InCommandList)
{
	StateTreeViewModel = InStateTreeViewModel;

	StateTreeViewModel->GetOnAssetChanged().AddSP(this, &SStateTreeOutliner::HandleModelAssetChanged);
	StateTreeViewModel->GetOnStatesRemoved().AddSP(this, &SStateTreeOutliner::HandleModelStatesRemoved);
	StateTreeViewModel->GetOnStatesMoved().AddSP(this, &SStateTreeOutliner::HandleModelStatesMoved);
	StateTreeViewModel->GetOnStateAdded().AddSP(this, &SStateTreeOutliner::HandleModelStateAdded);
	StateTreeViewModel->GetOnStatesChanged().AddSP(this, &SStateTreeOutliner::HandleModelStatesChanged);
	StateTreeViewModel->GetOnSelectionChanged().AddSP(this, &SStateTreeOutliner::HandleModelSelectionChanged);

	UE::StateTree::Delegates::OnVisualThemeChanged.AddSP(this, &SStateTreeOutliner::HandleVisualThemeChanged);
	
	bUpdatingSelection = false;

	ChildSlot
	[
		SAssignNew(CompactTreeView, UE::StateTree::Editor::SCompactTreeEditorView, StateTreeViewModel)
		.SelectionMode(ESelectionMode::Multi)
		.StateTreeEditorData(StateTreeViewModel->GetStateTreeEditorData())
		.OnSelectionChanged(this, &SStateTreeOutliner::HandleTreeViewSelectionChanged)
		.OnContextMenuOpening(this, &SStateTreeOutliner::HandleContextMenuOpening)
		.ShowLinkedStates(true)
	];
	
	CommandList = InCommandList;
	BindCommands();
}

void SStateTreeOutliner::BindCommands()
{
	const FStateTreeEditorCommands& Commands = FStateTreeEditorCommands::Get();

	CommandList->MapAction(
		Commands.AddSiblingState,
		FExecuteAction::CreateSP(this, &SStateTreeOutliner::HandleAddSiblingState),
		FCanExecuteAction());

	CommandList->MapAction(
		Commands.AddChildState,
		FExecuteAction::CreateSP(this, &SStateTreeOutliner::HandleAddChildState),
		FCanExecuteAction::CreateSP(this, &SStateTreeOutliner::HasSelection));

	CommandList->MapAction(
		Commands.CutStates,
		FExecuteAction::CreateSP(this, &SStateTreeOutliner::HandleCutSelectedStates),
		FCanExecuteAction::CreateSP(this, &SStateTreeOutliner::HasSelection));

	CommandList->MapAction(
		Commands.CopyStates,
		FExecuteAction::CreateSP(this, &SStateTreeOutliner::HandleCopySelectedStates),
		FCanExecuteAction::CreateSP(this, &SStateTreeOutliner::HasSelection));

	CommandList->MapAction(
		Commands.DeleteStates,
		FExecuteAction::CreateSP(this, &SStateTreeOutliner::HandleDeleteStates),
		FCanExecuteAction::CreateSP(this, &SStateTreeOutliner::HasSelection));

	CommandList->MapAction(
		Commands.PasteStatesAsSiblings,
		FExecuteAction::CreateSP(this, &SStateTreeOutliner::HandlePasteStatesAsSiblings),
		FCanExecuteAction::CreateSP(this, &SStateTreeOutliner::CanPaste));

	CommandList->MapAction(
		Commands.PasteStatesAsChildren,
		FExecuteAction::CreateSP(this, &SStateTreeOutliner::HandlePasteStatesAsChildren),
		FCanExecuteAction::CreateSP(this, &SStateTreeOutliner::CanPaste));

	CommandList->MapAction(
		Commands.DuplicateStates,
		FExecuteAction::CreateSP(this, &SStateTreeOutliner::HandleDuplicateSelectedStates),
		FCanExecuteAction::CreateSP(this, &SStateTreeOutliner::HasSelection));

	CommandList->MapAction(
		Commands.EnableStates,
		FExecuteAction::CreateSP(this, &SStateTreeOutliner::HandleEnableSelectedStates),
		FCanExecuteAction(),
		FGetActionCheckState::CreateSPLambda(this, [this]
			{
				const bool bCanEnable = CanEnableStates();
				const bool bCanDisable = CanDisableStates();
				if (bCanEnable && bCanDisable)
				{
					return ECheckBoxState::Undetermined;
				}
				
				if (bCanDisable)
				{
					return ECheckBoxState::Checked;
				}

				if (bCanEnable)
				{
					return ECheckBoxState::Unchecked;
				}

				// Should not happen since action is not visible in this case
				return ECheckBoxState::Undetermined;
			}),
		FIsActionButtonVisible::CreateSPLambda(this, [this]
		{
			return CanEnableStates() || CanDisableStates();
		}));

#if WITH_STATETREE_TRACE_DEBUGGER
	CommandList->MapAction(
		Commands.EnableOnEnterStateBreakpoint,
		FExecuteAction::CreateSPLambda(this, [this]
		{
			if (StateTreeViewModel)
			{
				StateTreeViewModel->HandleEnableStateBreakpoint(EStateTreeBreakpointType::OnEnter);
			}
		}),
		FCanExecuteAction(),
		FGetActionCheckState::CreateSPLambda(this, [this]
		{
			return StateTreeViewModel ? StateTreeViewModel->GetStateBreakpointCheckState(EStateTreeBreakpointType::OnEnter) : ECheckBoxState::Unchecked;
		}),
		FIsActionButtonVisible::CreateSPLambda(this, [this]
		{
			return (StateTreeViewModel)
				&& (StateTreeViewModel->CanAddStateBreakpoint(EStateTreeBreakpointType::OnEnter)
					|| StateTreeViewModel->CanRemoveStateBreakpoint(EStateTreeBreakpointType::OnEnter));
		}));

	CommandList->MapAction(
		Commands.EnableOnExitStateBreakpoint,
		FExecuteAction::CreateSPLambda(this, [this]
		{
			if (StateTreeViewModel)
			{
				StateTreeViewModel->HandleEnableStateBreakpoint(EStateTreeBreakpointType::OnExit);
			}
		}),
		FCanExecuteAction(),
		FGetActionCheckState::CreateSPLambda(this, [this]
		{
			return StateTreeViewModel ? StateTreeViewModel->GetStateBreakpointCheckState(EStateTreeBreakpointType::OnExit) : ECheckBoxState::Unchecked;
		}),
		FIsActionButtonVisible::CreateSPLambda(this, [this]
		{
			return (StateTreeViewModel)
				&& (StateTreeViewModel->CanAddStateBreakpoint(EStateTreeBreakpointType::OnExit)
					|| StateTreeViewModel->CanRemoveStateBreakpoint(EStateTreeBreakpointType::OnExit));
		}));
#endif // WITH_STATETREE_TRACE_DEBUGGER
}


void SStateTreeOutliner::HandleModelAssetChanged()
{
	bItemsDirty = true;

	if (CompactTreeView && StateTreeViewModel)
	{
		CompactTreeView->Refresh(StateTreeViewModel->GetStateTreeEditorData());
	}
}

void SStateTreeOutliner::HandleModelStatesRemoved(const TSet<UStateTreeState*>& AffectedParents)
{
	bItemsDirty = true;
	
	if (CompactTreeView && StateTreeViewModel)
	{
		CompactTreeView->Refresh(StateTreeViewModel->GetStateTreeEditorData());
	}
}

void SStateTreeOutliner::HandleModelStatesMoved(const TSet<UStateTreeState*>& AffectedParents, const TSet<UStateTreeState*>& MovedStates)
{
	bItemsDirty = true;
	
	if (CompactTreeView && StateTreeViewModel)
	{
		CompactTreeView->Refresh(StateTreeViewModel->GetStateTreeEditorData());
	}
}

void SStateTreeOutliner::HandleModelStateAdded(UStateTreeState* ParentState, UStateTreeState* NewState)
{
	bItemsDirty = true;
	
	if (CompactTreeView && StateTreeViewModel)
	{
		CompactTreeView->Refresh(StateTreeViewModel->GetStateTreeEditorData());
	}
}

void SStateTreeOutliner::HandleModelStatesChanged(const TSet<UStateTreeState*>& AffectedStates, const FPropertyChangedEvent& PropertyChangedEvent)
{
	bool bArraysChanged = false;

	// The purpose of the rebuild below is to update the task visualization (number of widgets change).
	// This method is called when anything in a state changes, make sure to only rebuild when needed.
	if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UStateTreeState, Tasks))
	{
		bArraysChanged = true;
	}
		
	if (bArraysChanged)
	{
	}

	if (CompactTreeView && StateTreeViewModel)
	{
		CompactTreeView->Refresh(StateTreeViewModel->GetStateTreeEditorData());
	}
}

void SStateTreeOutliner::HandleModelSelectionChanged(const TArray<TWeakObjectPtr<UStateTreeState>>& SelectedStates)
{
	if (bUpdatingSelection)
	{
		return;
	}

	if (CompactTreeView && StateTreeViewModel)
	{
		TArray<FGuid> StateIDs;
		for (const TWeakObjectPtr<UStateTreeState>& WeakState : SelectedStates)
		{
			if (const UStateTreeState* State = WeakState.Get())
			{
				StateIDs.Add(State->ID);
			}
		}
	
		CompactTreeView->SetSelection(StateIDs);
	}
}

void SStateTreeOutliner::HandleTreeViewSelectionChanged(TConstArrayView<FGuid> SelectedStateIDs)
{
	if (StateTreeViewModel)
	{
		TArray<TWeakObjectPtr<UStateTreeState>> Selection;

		if (const UStateTreeEditorData* StateTreeEditorData = StateTreeViewModel->GetStateTreeEditorData())
		{
			for (const FGuid& StateID : SelectedStateIDs)
			{
				if (const UStateTreeState* State = StateTreeEditorData->GetStateByID(StateID))
				{
					Selection.Add(const_cast<UStateTreeState*>(State));
				}
			}
		}
		
		StateTreeViewModel->SetSelection(Selection);
	}
}

void SStateTreeOutliner::HandleVisualThemeChanged(const UStateTree& StateTree)
{
	if (StateTreeViewModel
		&& StateTreeViewModel->GetStateTree() == &StateTree)
	{
		CompactTreeView->Refresh(StateTreeViewModel->GetStateTreeEditorData());
	}
}

FReply SStateTreeOutliner::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if(CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	else
	{
		return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
	}
}

TSharedPtr<SWidget> SStateTreeOutliner::HandleContextMenuOpening()
{
	if (!StateTreeViewModel)
	{
		return nullptr;
	}

	FMenuBuilder MenuBuilder(true, CommandList);

	MenuBuilder.AddSubMenu(
		LOCTEXT("AddState", "Add State"),
		FText(),
		FNewMenuDelegate::CreateLambda([this](FMenuBuilder& MenuBuilder)
		{
			MenuBuilder.AddMenuEntry(FStateTreeEditorCommands::Get().AddSiblingState);
			MenuBuilder.AddMenuEntry(FStateTreeEditorCommands::Get().AddChildState);
		}),
		/*bInOpenSubMenuOnClick =*/false,
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Plus")
	);

	MenuBuilder.AddSeparator();

	MenuBuilder.AddMenuEntry(FStateTreeEditorCommands::Get().CutStates);
	MenuBuilder.AddMenuEntry(FStateTreeEditorCommands::Get().CopyStates);

	MenuBuilder.AddSubMenu(
		LOCTEXT("Paste", "Paste"),
		FText(),
		FNewMenuDelegate::CreateLambda([this](FMenuBuilder& MenuBuilder)
		{
			MenuBuilder.AddMenuEntry(FStateTreeEditorCommands::Get().PasteStatesAsSiblings);
			MenuBuilder.AddMenuEntry(FStateTreeEditorCommands::Get().PasteStatesAsChildren);
		}),
		/*bInOpenSubMenuOnClick =*/false,
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Paste")
	);
	
	MenuBuilder.AddMenuEntry(FStateTreeEditorCommands::Get().DuplicateStates);
	MenuBuilder.AddMenuEntry(FStateTreeEditorCommands::Get().DeleteStates);
	MenuBuilder.AddSeparator();
	MenuBuilder.AddMenuEntry(FStateTreeEditorCommands::Get().EnableStates);

#if WITH_STATETREE_TRACE_DEBUGGER
	MenuBuilder.AddSeparator();
	MenuBuilder.AddMenuEntry(FStateTreeEditorCommands::Get().EnableOnEnterStateBreakpoint);
	MenuBuilder.AddMenuEntry(FStateTreeEditorCommands::Get().EnableOnExitStateBreakpoint);
#endif // WITH_STATETREE_TRACE_DEBUGGER
	
	return MenuBuilder.MakeWidget();
}

UStateTreeState* SStateTreeOutliner::GetFirstSelectedState() const
{
	TArray<UStateTreeState*> SelectedStates;
	if (StateTreeViewModel)
	{
		StateTreeViewModel->GetSelectedStates(SelectedStates);
	}
	return SelectedStates.IsEmpty() ? nullptr : SelectedStates[0];
}

void SStateTreeOutliner::HandleAddSiblingState()
{
	if (StateTreeViewModel)
	{
		StateTreeViewModel->AddState(GetFirstSelectedState());
	}
}

void SStateTreeOutliner::HandleAddChildState()
{
	if (StateTreeViewModel)
	{
		UStateTreeState* ParentState = GetFirstSelectedState();
		if (ParentState)
		{
			StateTreeViewModel->AddChildState(ParentState);
		}
	}
}

void SStateTreeOutliner::HandleCutSelectedStates()
{
	if (StateTreeViewModel)
	{
		StateTreeViewModel->CopySelectedStates();
		StateTreeViewModel->RemoveSelectedStates();
	}
}

void SStateTreeOutliner::HandleCopySelectedStates()
{
	if (StateTreeViewModel)
	{
		StateTreeViewModel->CopySelectedStates();
	}
}

void SStateTreeOutliner::HandlePasteStatesAsSiblings()
{
	if (StateTreeViewModel)
	{
		StateTreeViewModel->PasteStatesFromClipboard(GetFirstSelectedState());
	}
}

void SStateTreeOutliner::HandlePasteStatesAsChildren()
{
	if (StateTreeViewModel)
	{
		StateTreeViewModel->PasteStatesAsChildrenFromClipboard(GetFirstSelectedState());
	}
}

void SStateTreeOutliner::HandleDuplicateSelectedStates()
{
	if (StateTreeViewModel)
	{
		StateTreeViewModel->DuplicateSelectedStates();
	}
}

void SStateTreeOutliner::HandleDeleteStates()
{
	if (StateTreeViewModel)
	{
		StateTreeViewModel->RemoveSelectedStates();
	}
}

void SStateTreeOutliner::HandleEnableSelectedStates()
{
	if (StateTreeViewModel)
	{
		// Process CanEnable first so in case of undetermined state (mixed selection) we Enable by default. 
		if (CanEnableStates())
		{
			StateTreeViewModel->SetSelectedStatesEnabled(true);	
		}
		else if (CanDisableStates())
		{
			StateTreeViewModel->SetSelectedStatesEnabled(false);
		}
	}
}

void SStateTreeOutliner::HandleDisableSelectedStates()
{
	if (StateTreeViewModel)
	{
		StateTreeViewModel->SetSelectedStatesEnabled(false);
	}
}

bool SStateTreeOutliner::HasSelection() const
{
	return StateTreeViewModel && StateTreeViewModel->HasSelection();
}

bool SStateTreeOutliner::CanPaste() const
{
	return StateTreeViewModel
			&& StateTreeViewModel->HasSelection()
			&& StateTreeViewModel->CanPasteStatesFromClipboard();
}

bool SStateTreeOutliner::CanEnableStates() const
{
	return StateTreeViewModel
			&& StateTreeViewModel->HasSelection()
			&& StateTreeViewModel->CanEnableStates();
}

bool SStateTreeOutliner::CanDisableStates() const
{
	return StateTreeViewModel
			&& StateTreeViewModel->HasSelection()
			&& StateTreeViewModel->CanDisableStates();
}


#undef LOCTEXT_NAMESPACE
