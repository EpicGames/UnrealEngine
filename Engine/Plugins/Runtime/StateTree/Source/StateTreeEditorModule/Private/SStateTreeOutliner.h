// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

class UStateTree;
class FStateTreeViewModel;
struct FPropertyChangedEvent;
class UStateTreeState;
class FUICommandList;

namespace UE::StateTree::Editor
{
class SCompactTreeEditorView;
}

class SStateTreeOutliner : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SStateTreeOutliner) {}
	SLATE_END_ARGS()

	SStateTreeOutliner();
	virtual ~SStateTreeOutliner() override;

	void Construct(const FArguments& InArgs, TSharedRef<FStateTreeViewModel> StateTreeViewModel, const TSharedRef<FUICommandList>& InCommandList);

private:
	// ViewModel handlers
	void HandleModelAssetChanged();
	void HandleModelStatesRemoved(const TSet<UStateTreeState*>& AffectedParents);
	void HandleModelStatesMoved(const TSet<UStateTreeState*>& AffectedParents, const TSet<UStateTreeState*>& MovedStates);
	void HandleModelStateAdded(UStateTreeState* ParentState, UStateTreeState* NewState);
	void HandleModelStatesChanged(const TSet<UStateTreeState*>& AffectedStates, const FPropertyChangedEvent& PropertyChangedEvent);
	void HandleModelSelectionChanged(const TArray<TWeakObjectPtr<UStateTreeState>>& SelectedStates);
	void HandleTreeViewSelectionChanged(TConstArrayView<FGuid> SelectedStateIDs);
	void HandleVisualThemeChanged(const UStateTree& StateTree);

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	TSharedPtr<SWidget> HandleContextMenuOpening();

	// Action handlers
	// @todo: these are also defined in the SStateTreeView, figure out how to share code.
	UStateTreeState* GetFirstSelectedState() const;
	void HandleAddSiblingState();
	void HandleAddChildState();
	void HandleCutSelectedStates();
	void HandleCopySelectedStates();
	void HandlePasteStatesAsSiblings();
	void HandlePasteStatesAsChildren();
	void HandleDuplicateSelectedStates();
	void HandleDeleteStates();
	void HandleEnableSelectedStates();
	void HandleDisableSelectedStates();

	bool HasSelection() const;
	bool CanPaste() const;
	bool CanEnableStates() const;
	bool CanDisableStates() const;

	void BindCommands();

	TSharedPtr<FStateTreeViewModel> StateTreeViewModel;

	TSharedPtr<UE::StateTree::Editor::SCompactTreeEditorView> CompactTreeView;

	TSharedPtr<FUICommandList> CommandList;

	bool bItemsDirty = false;
	bool bUpdatingSelection = false;
};
