// Copyright Epic Games, Inc. All Rights Reserved.
#include "SStateTreeSplitter.h"
#include "SStateTreeView.h"
#include "StateTree.h"
#include "StateTreeDiffHelper.h"
#include "StateTreeEditorData.h"
#include "StateTreeViewModel.h"

namespace UE::StateTree::Diff
{
void SDiffSplitter::Construct(const FArguments& InArgs)
{
	Splitter = SNew(SSplitter).PhysicalSplitterHandleSize(5.f).Orientation(EOrientation::Orient_Horizontal);

	for (const FSlot::FSlotArguments& SlotArgs : InArgs._Slots)
	{
		AddSlot(SlotArgs);
	}

	ChildSlot
		[
			Splitter.ToSharedRef()
		];
}

void SDiffSplitter::AddSlot(const FSlot::FSlotArguments& SlotArgs, int32 Index)
{
	if (Index == INDEX_NONE)
	{
		Index = Panels.Num();
	}

	Splitter->AddSlot(Index)
		.Value(SlotArgs._Value)
		[
			SNew(SBox).Padding(15.f, 0.f, 15.f, 0.f)
			[
				SlotArgs._StateTreeView.ToSharedRef()
			]
		];

	Panels.Insert(
		{
			SlotArgs._StateTreeView,
			SlotArgs._StateTree,
			SlotArgs._IsReadonly,
			SlotArgs._DifferencesWithRightPanel,
		},
		Index);
	if (SlotArgs._StateTreeView)
	{
		SlotArgs._StateTreeView->GetViewModel()->GetOnSelectionChanged().AddSP(this, &SDiffSplitter::HandleSelectionChanged);
	}
}

void SDiffSplitter::HandleSelectionChanged(const FStateSoftPath& StatePath, const FStateSoftPath& SecondaryStatePath)
{
	if (StatePath != SelectedState)
	{
		SelectedState = StatePath;

		for (const FPanel& Panel : Panels)
		{
			const FStateTreeViewModel* ViewModel = Panel.StateTreeView->GetViewModel().Get();
			const UStateTreeEditorData* EditorData = Cast<UStateTreeEditorData>(Panel.StateTree->EditorData);
			if (EditorData != nullptr && ViewModel != nullptr)
			{
				UStateTreeState* PanelState = SelectedState.ResolvePath(EditorData);
				if (!PanelState)
				{
					PanelState = SecondaryStatePath.ResolvePath(EditorData);
				}

				TArray<UStateTreeState*> CurSelectedStates;
				ViewModel->GetSelectedStates(CurSelectedStates);
				if (CurSelectedStates.Num() != 1 || CurSelectedStates[0] != PanelState)
				{
					Panel.StateTreeView->SetSelection({PanelState});
				}
			}
		}
	}
}

void SDiffSplitter::HandleSelectionChanged(const TArray<TWeakObjectPtr<UStateTreeState>>& SelectedStates)
{
	if (const UStateTreeState* State = SelectedStates.Num() == 1 ? SelectedStates[0].Get() : nullptr)
	{
		const FStateSoftPath StatePath = FStateSoftPath(State);
		HandleSelectionChanged(StatePath, FStateSoftPath());
	}
}

SDiffSplitter::FSlot::FSlotArguments SDiffSplitter::Slot()
{
	return FSlot::FSlotArguments(MakeUnique<FSlot>());
}

} // UE::StateTree::Diff