// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "StateTreeDiffHelper.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API STATETREEEDITORMODULE_API

class SStateTreeView;

namespace UE::StateTree::Diff
{
class FAsyncDiff;

/** Splitter that allows you to provide an FAsyncStateTreeViewDiff to connect like-properties between two or more state tree panels */
class SDiffSplitter : public SCompoundWidget
{
public:
	class FSlot : public TSlotBase<FSlot>
	{
	public:
		FSlot() = default;

		SLATE_SLOT_BEGIN_ARGS(FSlot, TSlotBase<FSlot>)
			/** When the RuleSize is set to FractionOfParent, the size of the slot is the Value percentage of its parent size. */
			SLATE_ATTRIBUTE(float, Value)
			SLATE_ARGUMENT(TSharedPtr<SStateTreeView>, StateTreeView)
			SLATE_ARGUMENT(const UStateTree*, StateTree)
			SLATE_ATTRIBUTE(bool, IsReadonly)
			SLATE_ATTRIBUTE(TSharedPtr<FAsyncDiff>, DifferencesWithRightPanel)
		SLATE_SLOT_END_ARGS()
	};
	static UE_API FSlot::FSlotArguments Slot();

	SLATE_BEGIN_ARGS(SDiffSplitter)
	{
	}
		SLATE_SLOT_ARGUMENT(FSlot, Slots)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

	UE_API void AddSlot(const FSlot::FSlotArguments& SlotArgs, int32 Index = INDEX_NONE);

	UE_API void HandleSelectionChanged(const FStateSoftPath& StatePath, const FStateSoftPath& SecondaryStatePath);

protected:
	UE_API void HandleSelectionChanged(const TArray<TWeakObjectPtr<UStateTreeState>>& SelectedStates);

private:
	struct FPanel
	{
		FPanel(const TSharedPtr<SStateTreeView>& StateTreeView, const UStateTree* StateTree, const TAttribute<bool>& bIsReadonly,
			const TAttribute<TSharedPtr<FAsyncDiff>>& DiffRight)
			: StateTreeView(StateTreeView)
			, StateTree(StateTree)
			, IsReadonly(bIsReadonly)
			, DiffRight(DiffRight)
		{
		}

		TSharedPtr<SStateTreeView> StateTreeView;
		const UStateTree* StateTree = nullptr;
		TAttribute<bool> IsReadonly;
		TAttribute<TSharedPtr<FAsyncDiff>> DiffRight;
	};

	TArray<FPanel> Panels;
	TSharedPtr<SSplitter> Splitter;

	FStateSoftPath SelectedState;
};
} // UE::StateTree::Diff

#undef UE_API
