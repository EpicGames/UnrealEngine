// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStateTreeView.h"
#include "Debugger/StateTreeDebuggerTypes.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Modules/ModuleManager.h"
#include "SEnumCombo.h"
#include "SPositiveActionButton.h"
#include "SStateTreeViewRow.h"
#include "StateTreeViewModel.h"
#include "StateTreeState.h"
#include "StateTreeEditorCommands.h"
#include "StateTreeEditorUserSettings.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSpacer.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailCustomization.h"
#include "IDetailsView.h"
#include "PropertyEditorModule.h"

// @TODO: Dependency onto state centric to enable view toggling. 
// Long term plan is to have this in a separate tab.
#include "StateCentricView/StateCentricViewSettings.h"
#include "StateCentricView/View/SStateCentricToolbar.h"
#include "StateCentricView/View/SStateCentricViewMainStateNode.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"

SStateTreeView::SStateTreeView()
	: RequestedRenameState(nullptr)
	, bItemsDirty(false)
	, bUpdatingSelection(false)
{
}

SStateTreeView::~SStateTreeView()
{
	if (UObjectInitialized())
	{
		GetMutableDefault<UStateTreeEditorUserSettings>()->OnSettingsChanged.Remove(SettingsChangedHandle);

		if (StateTreeViewModel)
		{
			StateTreeViewModel->GetOnAssetChanged().RemoveAll(this);
			StateTreeViewModel->GetOnStatesRemoved().RemoveAll(this);
			StateTreeViewModel->GetOnStatesMoved().RemoveAll(this);
			StateTreeViewModel->GetOnStateAdded().RemoveAll(this);
			StateTreeViewModel->GetOnStatesChanged().RemoveAll(this);
			StateTreeViewModel->GetOnSelectionChanged().RemoveAll(this);
			StateTreeViewModel->GetOnStateNodesChanged().RemoveAll(this);
		}
	}
}

void SStateTreeView::Construct(const FArguments& InArgs, TSharedRef<FStateTreeViewModel> InStateTreeViewModel, const TSharedRef<FUICommandList>& InCommandList)
{
	StateTreeViewModel = InStateTreeViewModel;

	StateTreeViewModel->GetOnAssetChanged().AddSP(this, &SStateTreeView::HandleModelAssetChanged);
	StateTreeViewModel->GetOnStatesRemoved().AddSP(this, &SStateTreeView::HandleModelStatesRemoved);
	StateTreeViewModel->GetOnStatesMoved().AddSP(this, &SStateTreeView::HandleModelStatesMoved);
	StateTreeViewModel->GetOnStateAdded().AddSP(this, &SStateTreeView::HandleModelStateAdded);
	StateTreeViewModel->GetOnStatesChanged().AddSP(this, &SStateTreeView::HandleModelStatesChanged);
	StateTreeViewModel->GetOnSelectionChanged().AddSP(this, &SStateTreeView::HandleModelSelectionChanged);
	StateTreeViewModel->GetOnStateNodesChanged().AddSP(this, &SStateTreeView::HandleModelStateNodesChanged);

	SettingsChangedHandle = GetMutableDefault<UStateTreeEditorUserSettings>()->OnSettingsChanged.AddSP(this, &SStateTreeView::HandleUserSettingsChanged);

	bUpdatingSelection = false;

	TSharedRef<SScrollBar> HorizontalScrollBar = SNew(SScrollBar)
		.Orientation(Orient_Horizontal)
		.Thickness(FVector2D(12.0f, 12.0f));

	TSharedRef<SScrollBar> VerticalScrollBar = SNew(SScrollBar)
		.Orientation(Orient_Vertical)
		.Thickness(FVector2D(12.0f, 12.0f));

	StateTreeViewModel->GetSubTrees(Subtrees);

	TreeView = SNew(STreeView<TWeakObjectPtr<UStateTreeState>>)
		.OnGenerateRow(this, &SStateTreeView::HandleGenerateRow)
		.OnGetChildren(this, &SStateTreeView::HandleGetChildren)
		.TreeItemsSource(&Subtrees)
		.OnSelectionChanged(this, &SStateTreeView::HandleTreeSelectionChanged)
		.OnExpansionChanged(this, &SStateTreeView::HandleTreeExpansionChanged)
		.OnContextMenuOpening(this, &SStateTreeView::HandleContextMenuOpening)
		.AllowOverscroll(EAllowOverscroll::Yes)
		.ExternalScrollbar(VerticalScrollBar);

	TSharedRef<SWidget> StateTreeView = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(2.0f)
			[
				SNew(SHorizontalBox)

				// New State
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(4.0f, 2.0f)
				.AutoWidth()
				[
					SNew(SPositiveActionButton)
					.ToolTipText(LOCTEXT("AddStateToolTip", "Add New State"))
					.Icon(FAppStyle::Get().GetBrush("Icons.Plus")) 
					.Text(LOCTEXT("AddState", "Add State"))
					.OnClicked(this, &SStateTreeView::HandleAddStateButton)
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNew(SSpacer)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				[
					SNew(SComboButton)
					.HasDownArrow(false)
					.ContentPadding(0.0f)
					.ForegroundColor(FSlateColor::UseForeground())
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.MenuContent()
					[
						HandleGenerateSettingsMenu()
					]
					.ButtonContent()
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("DetailsView.ViewOptions"))
					]
				]
			]
		]

		+SVerticalBox::Slot()
		.Padding(0.0f, 6.0f, 0.0f, 0.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(0.0f)
			[
				SAssignNew(ViewBox, SScrollBox)
				.Orientation(Orient_Horizontal)
				.ExternalScrollbar(HorizontalScrollBar)
				+SScrollBox::Slot()
				.FillSize(1.0f)
				[
					TreeView.ToSharedRef()
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				VerticalScrollBar
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			HorizontalScrollBar
		];

	if (!UStateCentricViewSettings::Get().IsStateCentricViewEnabled())
	{
		// No state centric view, only tree view
		ChildSlot
		[
			StateTreeView
		];
	}
	else
	{
		// @TODO: Wrap main state centric view with something that accepts a null state
		UStateTreeState* StateCentricViewState = StateTreeViewModel->GetViewState();
		ensure(StateCentricViewState);
		ChildSlot
		[
			SNew(SSplitter)
			+ SSplitter::Slot()
			.Value(0.15f)
			[
				StateTreeView
			]
			+SSplitter::Slot()
			.Value(0.85f)
			[
				SNew(SSplitter)
				.Orientation(EOrientation::Orient_Vertical)
				+ SSplitter::Slot()
				.SizeRule(SSplitter::ESizeRule::SizeToContent)
				[
					SNew(UE::StateTree::Editor::StateCentricView::SStateCentricToolbar, InStateTreeViewModel)
				]
				+ SSplitter::Slot()
				.Value(1.0f)
				[
					SNew(SScrollBox)
					.Orientation(Orient_Vertical)
					+ SScrollBox::Slot()
					.FillSize(1.0f)
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
						.Padding(FMargin(2.0, 4.0))
						[
							SNew(UE::StateTree::Editor::StateCentricView::SStateCentricViewMainStateNode, InStateTreeViewModel, StateCentricViewState)
						]
					]
				]
			]
		];
	}

	UpdateTree(true);

	CommandList = InCommandList;
	BindCommands();
}

void SStateTreeView::BindCommands()
{
	const FStateTreeEditorCommands& Commands = FStateTreeEditorCommands::Get();

	CommandList->MapAction(
		Commands.AddSiblingState,
		FExecuteAction::CreateSP(this, &SStateTreeView::HandleAddSiblingState),
		FCanExecuteAction());

	CommandList->MapAction(
		Commands.AddChildState,
		FExecuteAction::CreateSP(this, &SStateTreeView::HandleAddChildState),
		FCanExecuteAction::CreateSP(this, &SStateTreeView::HasSelection));

	CommandList->MapAction(
		Commands.CutStates,
		FExecuteAction::CreateSP(this, &SStateTreeView::HandleCutSelectedStates),
		FCanExecuteAction::CreateSP(this, &SStateTreeView::HasSelection));

	CommandList->MapAction(
		Commands.CopyStates,
		FExecuteAction::CreateSP(this, &SStateTreeView::HandleCopySelectedStates),
		FCanExecuteAction::CreateSP(this, &SStateTreeView::HasSelection));

	CommandList->MapAction(
		Commands.DeleteStates,
		FExecuteAction::CreateSP(this, &SStateTreeView::HandleDeleteStates),
		FCanExecuteAction::CreateSP(this, &SStateTreeView::HasSelection));

	CommandList->MapAction(
		Commands.PasteStatesAsSiblings,
		FExecuteAction::CreateSP(this, &SStateTreeView::HandlePasteStatesAsSiblings),
		FCanExecuteAction::CreateSP(this, &SStateTreeView::CanPasteStates));

	CommandList->MapAction(
		Commands.PasteStatesAsChildren,
		FExecuteAction::CreateSP(this, &SStateTreeView::HandlePasteStatesAsChildren),
		FCanExecuteAction::CreateSP(this, &SStateTreeView::CanPasteStates));

	CommandList->MapAction(
		Commands.PasteNodesToSelectedStates,
		FExecuteAction::CreateSP(this, &SStateTreeView::HandlePasteNodesToState),
		FCanExecuteAction::CreateSP(this, &SStateTreeView::CanPasteNodesToSelectedStates));

	CommandList->MapAction(
		Commands.DuplicateStates,
		FExecuteAction::CreateSP(this, &SStateTreeView::HandleDuplicateSelectedStates),
		FCanExecuteAction::CreateSP(this, &SStateTreeView::HasSelection));

	CommandList->MapAction(
		Commands.RenameState,
		FExecuteAction::CreateSP(this, &SStateTreeView::HandleRenameState),
		FCanExecuteAction::CreateSP(this, &SStateTreeView::HasSelection));

	CommandList->MapAction(
		Commands.EnableStates,
		FExecuteAction::CreateSP(this, &SStateTreeView::HandleEnableSelectedStates),
		FCanExecuteAction(),
		FGetActionCheckState::CreateSPLambda(this, [this]
			{
				const bool bCanEnable = CanEnableStates();
				const bool bCanDisable = CanDisableStates();
				if (bCanEnable && bCanDisable)
				{
					return ECheckBoxState::Undetermined;
				}
				
				if (bCanDisable)
				{
					return ECheckBoxState::Checked;
				}

				if (bCanEnable)
				{
					return ECheckBoxState::Unchecked;
				}

				// Should not happen since action is not visible in this case
				return ECheckBoxState::Undetermined;
			}),
		FIsActionButtonVisible::CreateSPLambda(this, [this]
		{
			return CanEnableStates() || CanDisableStates();
		}));

#if WITH_STATETREE_TRACE_DEBUGGER
	CommandList->MapAction(
		Commands.EnableOnEnterStateBreakpoint,
		FExecuteAction::CreateSPLambda(this, [this]
		{
			if (StateTreeViewModel)
			{
				StateTreeViewModel->HandleEnableStateBreakpoint(EStateTreeBreakpointType::OnEnter);
			}
		}),
		FCanExecuteAction::CreateSPLambda(this, [this]
		{
			return (StateTreeViewModel)
				&& (StateTreeViewModel->CanAddStateBreakpoint(EStateTreeBreakpointType::OnEnter)
					|| StateTreeViewModel->CanRemoveStateBreakpoint(EStateTreeBreakpointType::OnEnter));
		}),
		FGetActionCheckState::CreateSPLambda(this, [this]
		{
			return StateTreeViewModel ? StateTreeViewModel->GetStateBreakpointCheckState(EStateTreeBreakpointType::OnEnter) : ECheckBoxState::Unchecked;
		}),
		FIsActionButtonVisible::CreateSPLambda(this, [this]
		{
			return StateTreeViewModel.IsValid();
		}));

	CommandList->MapAction(
		Commands.EnableOnExitStateBreakpoint,
		FExecuteAction::CreateSPLambda(this, [this]
		{
			if (StateTreeViewModel)
			{
				StateTreeViewModel->HandleEnableStateBreakpoint(EStateTreeBreakpointType::OnExit);
			}
		}),
		FCanExecuteAction::CreateSPLambda(this, [this]
		{
			return (StateTreeViewModel)
				&& (StateTreeViewModel->CanAddStateBreakpoint(EStateTreeBreakpointType::OnExit)
					|| StateTreeViewModel->CanRemoveStateBreakpoint(EStateTreeBreakpointType::OnExit));
		}),
		FGetActionCheckState::CreateSPLambda(this, [this]
		{
			return StateTreeViewModel ? StateTreeViewModel->GetStateBreakpointCheckState(EStateTreeBreakpointType::OnExit) : ECheckBoxState::Unchecked;
		}),
		FIsActionButtonVisible::CreateSPLambda(this, [this]
		{
			return StateTreeViewModel.IsValid();
		}));
#endif // WITH_STATETREE_TRACE_DEBUGGER
}

bool SStateTreeView::HasSelection() const
{
	return StateTreeViewModel && StateTreeViewModel->HasSelection();
}

bool SStateTreeView::CanPasteStates() const
{
	return StateTreeViewModel
			&& StateTreeViewModel->HasSelection()
			&& StateTreeViewModel->CanPasteStatesFromClipboard();
}

bool SStateTreeView::CanEnableStates() const
{
	return StateTreeViewModel
			&& StateTreeViewModel->HasSelection()
			&& StateTreeViewModel->CanEnableStates();
}

bool SStateTreeView::CanDisableStates() const
{
	return StateTreeViewModel
			&& StateTreeViewModel->HasSelection()
			&& StateTreeViewModel->CanDisableStates();
}

bool SStateTreeView::CanPasteNodesToSelectedStates() const
{
	return StateTreeViewModel && StateTreeViewModel->HasSelection() && StateTreeViewModel->CanPasteNodesToSelectedStates();
}

FReply SStateTreeView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if(CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	else
	{
		return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
	}
}

void SStateTreeView::SavePersistentExpandedStates()
{
	if (!StateTreeViewModel)
	{
		return;
	}

	TSet<TWeakObjectPtr<UStateTreeState>> ExpandedStates;
	TreeView->GetExpandedItems(ExpandedStates);
	StateTreeViewModel->SetPersistentExpandedStates(ExpandedStates);
}

void SStateTreeView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bItemsDirty)
	{
		UpdateTree(/*bExpandPersistent*/true);
	}

	if (RequestedRenameState && !TreeView->IsPendingRefresh())
	{
		if (TSharedPtr<SStateTreeViewRow> Row = StaticCastSharedPtr<SStateTreeViewRow>(TreeView->WidgetFromItem(RequestedRenameState)))
		{
			Row->RequestRename();
		}
		RequestedRenameState = nullptr;
	}
}

void SStateTreeView::UpdateTree(bool bExpandPersistent)
{
	if (!StateTreeViewModel)
	{
		return;
	}

	TSet<TWeakObjectPtr<UStateTreeState>> ExpandedStates;
	if (bExpandPersistent)
	{
		// Get expanded state from the tree data.
		StateTreeViewModel->GetPersistentExpandedStates(ExpandedStates);
	}
	else
	{
		// Restore current expanded state.
		TreeView->GetExpandedItems(ExpandedStates);
	}

	// Remember selection
	TArray<TWeakObjectPtr<UStateTreeState>> SelectedStates;
	StateTreeViewModel->GetSelectedStates(SelectedStates);

	// Regenerate items
	StateTreeViewModel->GetSubTrees(Subtrees);
	TreeView->SetTreeItemsSource(&Subtrees);

	// Restore expanded state
	for (const TWeakObjectPtr<UStateTreeState>& State : ExpandedStates)
	{
		TreeView->SetItemExpansion(State, true);
	}

	// Restore selected state
	TreeView->ClearSelection();
	TreeView->SetItemSelection(SelectedStates, true);

	TreeView->RequestTreeRefresh();

	bItemsDirty = false;
}

void SStateTreeView::HandleUserSettingsChanged()
{
	TreeView->RebuildList();
}

void SStateTreeView::HandleModelAssetChanged()
{
	// this only refresh the list. i.e. each row widget will not be refreshed
	bItemsDirty = true;

	// we need to rebuild the list to update each row widget
	TreeView->RebuildList();
}

void SStateTreeView::HandleModelStatesRemoved(const TSet<UStateTreeState*>& AffectedParents)
{
	bItemsDirty = true;
}

void SStateTreeView::HandleModelStatesMoved(const TSet<UStateTreeState*>& AffectedParents, const TSet<UStateTreeState*>& MovedStates)
{
	bItemsDirty = true;
}

void SStateTreeView::HandleModelStateAdded(UStateTreeState* ParentState, UStateTreeState* NewState)
{
	bItemsDirty = true;

	// Request to rename the state immediately.
	RequestedRenameState = NewState;

	if (StateTreeViewModel.IsValid())
	{
		StateTreeViewModel->SetSelection(NewState);
	}
}

void SStateTreeView::HandleModelStatesChanged(const TSet<UStateTreeState*>& AffectedStates, const FPropertyChangedEvent& PropertyChangedEvent)
{
	// When the tasks or conditions array changed(this includes both normal array operations: Add, Remove. Clear, Move,
	// and Paste or Duplicate an element in the array), The TreeView needs to be rebuilt because new elements came in or old elements have gone or both.
	// This will not rebuild the list when we change an inner property in a condition or in a task node because of InstanceStruct wrapper
	// @todo: change it to cache and re-set the content of the widget instead of rebuilding the whole list for perf
	if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UStateTreeState, Tasks)
		|| PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UStateTreeState, EnterConditions)
		|| PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UStateTreeState, bHasRequiredEventToEnter)
		|| PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UStateTreeState, RequiredEventToEnter))
	{
		TreeView->RebuildList();
	}
}

void SStateTreeView::HandleModelStateNodesChanged(const UStateTreeState* AffectedState)
{
	TreeView->RebuildList();
}

void SStateTreeView::HandleModelSelectionChanged(const TArray<TWeakObjectPtr<UStateTreeState>>& SelectedStates)
{
	if (bUpdatingSelection)
	{
		return;
	}

	TreeView->ClearSelection();

	if (SelectedStates.Num() > 0)
	{
		TreeView->SetItemSelection(SelectedStates, /*bSelected*/true);

		if (SelectedStates.Num() == 1)
		{
			TreeView->RequestScrollIntoView(SelectedStates[0]);	
		}
	}
}


TSharedRef<ITableRow> SStateTreeView::HandleGenerateRow(TWeakObjectPtr<UStateTreeState> InState, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	return SNew(SStateTreeViewRow, InOwnerTableView, InState, ViewBox, StateTreeViewModel.ToSharedRef());
}

void SStateTreeView::HandleGetChildren(TWeakObjectPtr<UStateTreeState> InParent, TArray<TWeakObjectPtr<UStateTreeState>>& OutChildren)
{
	if (const UStateTreeState* Parent = InParent.Get())
	{
		OutChildren.Append(Parent->Children);
	}
}

void SStateTreeView::HandleTreeSelectionChanged(TWeakObjectPtr<UStateTreeState> InSelectedItem, ESelectInfo::Type SelectionType)
{
	if (!StateTreeViewModel)
	{
		return;
	}

	// Do not report code based selection changes.
	if (SelectionType == ESelectInfo::Direct)
	{
		return;
	}

	TArray<TWeakObjectPtr<UStateTreeState>> SelectedItems = TreeView->GetSelectedItems();

	bUpdatingSelection = true;
	StateTreeViewModel->SetSelection(SelectedItems);
	bUpdatingSelection = false;
}

void SStateTreeView::HandleTreeExpansionChanged(TWeakObjectPtr<UStateTreeState> InSelectedItem, bool bExpanded)
{
	// Not calling Modify() on the state as we don't want the expansion to dirty the asset.
	// @todo: this is temporary fix for a bug where adding a state will reset the expansion state. 
	if (UStateTreeState* State = InSelectedItem.Get())
	{
		State->bExpanded = bExpanded;
	}
}

TSharedRef<SWidget> SStateTreeView::HandleGenerateSettingsMenu()
{
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.ViewIdentifier = NAME_None;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	DetailsView->RegisterInstancedCustomPropertyLayout(UStateTreeEditorUserSettings::StaticClass(), FOnGetDetailCustomizationInstance::CreateLambda([]()
		{
			class FStateTreeEditorUserSettingsDetailsCustomication : public IDetailCustomization
			{
			public:
				virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override
				{
					DetailLayout.HideCategory("OtherStuff");
					{
						auto FindProperty = [](TArrayView<TSharedRef<IPropertyHandle>> AllProperties, FName PropertyToFind)
							{
								return AllProperties.FindByPredicate(
									[PropertyToFind](TSharedRef<IPropertyHandle>& Other)
									{
										return Other->GetProperty()->GetFName() == PropertyToFind;
									});
							};

						{
							IDetailCategoryBuilder& CategoryBuilder = DetailLayout.EditCategory("State View");
							TArray<TSharedRef<IPropertyHandle>> AllProperties;
							CategoryBuilder.GetDefaultProperties(AllProperties);

							TSharedRef<IPropertyHandle>* FoundProperty = FindProperty(AllProperties, "StatesViewDisplayNodeType");
							if (ensure(FoundProperty))
							{
								CategoryBuilder.AddProperty(*FoundProperty).CustomWidget()
									.NameContent()
									[
										(*FoundProperty)->CreatePropertyNameWidget()
									]
									.ValueContent()
									[
										SNew(SEnumComboBox, StaticEnum<EStateTreeEditorUserSettingsNodeType>())
											.OnEnumSelectionChanged(SEnumComboBox::FOnEnumSelectionChanged::CreateLambda([](int32 NewValue, ESelectInfo::Type)
												{
													GetMutableDefault<UStateTreeEditorUserSettings>()->SetStatesViewDisplayNodeType((EStateTreeEditorUserSettingsNodeType)NewValue);
												}))
											.CurrentValue(MakeAttributeLambda([]() { return (int32)GetDefault<UStateTreeEditorUserSettings>()->GetStatesViewDisplayNodeType(); }))
											.Font(IDetailLayoutBuilder::GetDetailFont())
									];
							}
						}
						{
							IDetailCategoryBuilder& CategoryBuilder = DetailLayout.EditCategory("Details View");
							TArray<TSharedRef<IPropertyHandle>> AllProperties;
							CategoryBuilder.GetDefaultProperties(AllProperties);

							TSharedRef<IPropertyHandle>* FoundProperty = FindProperty(AllProperties, "DetailsViewDisplayCount");
							if (ensure(FoundProperty))
							{
								CategoryBuilder.AddProperty(*FoundProperty).CustomWidget()
									.NameContent()
									[
										(*FoundProperty)->CreatePropertyNameWidget()
									]
									.ValueContent()
									[
										SNew(SEnumComboBox, StaticEnum<EStateTreeEditorUserSettingsNodeType>())
											.OnEnumSelectionChanged(SEnumComboBox::FOnEnumSelectionChanged::CreateLambda([](int32 NewValue, ESelectInfo::Type)
												{
													GetMutableDefault<UStateTreeEditorUserSettings>()->SetDetailsViewDisplayCount((EStateTreeEditorUserSettingsNodeType)NewValue);
												}))
											.CurrentValue(MakeAttributeLambda([]() { return (int32)GetDefault<UStateTreeEditorUserSettings>()->GetDetailsViewDisplayCount(); }))
											.Font(IDetailLayoutBuilder::GetDetailFont())
									];
							}
						}
					}
				}
			};
			return MakeShared<FStateTreeEditorUserSettingsDetailsCustomication>();
		}));

	DetailsView->SetObject(GetMutableDefault<UStateTreeEditorUserSettings>());
	return DetailsView;
}

TSharedPtr<SWidget> SStateTreeView::HandleContextMenuOpening()
{
	if (!StateTreeViewModel)
	{
		return nullptr;
	}

	FMenuBuilder MenuBuilder(true, CommandList);

	MenuBuilder.AddSubMenu(
		LOCTEXT("AddState", "Add State"),
		FText(),
		FNewMenuDelegate::CreateLambda([this](FMenuBuilder& MenuBuilder)
		{
			MenuBuilder.AddMenuEntry(FStateTreeEditorCommands::Get().AddSiblingState);
			MenuBuilder.AddMenuEntry(FStateTreeEditorCommands::Get().AddChildState);
		}),
		/*bInOpenSubMenuOnClick =*/false,
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Plus")
	);

	MenuBuilder.AddSeparator();

	MenuBuilder.AddMenuEntry(FStateTreeEditorCommands::Get().CutStates);
	MenuBuilder.AddMenuEntry(FStateTreeEditorCommands::Get().CopyStates);

	MenuBuilder.AddSubMenu(
		LOCTEXT("Paste", "Paste"),
		FText(),
		FNewMenuDelegate::CreateLambda([this](FMenuBuilder& MenuBuilder)
		{
			MenuBuilder.AddMenuEntry(FStateTreeEditorCommands::Get().PasteStatesAsSiblings);
			MenuBuilder.AddMenuEntry(FStateTreeEditorCommands::Get().PasteStatesAsChildren);
			MenuBuilder.AddMenuEntry(FStateTreeEditorCommands::Get().PasteNodesToSelectedStates);
		}),
		/*bInOpenSubMenuOnClick =*/false,
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Paste")
	);
	
	MenuBuilder.AddMenuEntry(FStateTreeEditorCommands::Get().DuplicateStates);
	MenuBuilder.AddMenuEntry(FStateTreeEditorCommands::Get().DeleteStates);
	MenuBuilder.AddMenuEntry(FStateTreeEditorCommands::Get().RenameState);
	MenuBuilder.AddSeparator();
	MenuBuilder.AddMenuEntry(FStateTreeEditorCommands::Get().EnableStates);

#if WITH_STATETREE_TRACE_DEBUGGER
	MenuBuilder.AddSeparator();
	MenuBuilder.AddMenuEntry(FStateTreeEditorCommands::Get().EnableOnEnterStateBreakpoint);
	MenuBuilder.AddMenuEntry(FStateTreeEditorCommands::Get().EnableOnExitStateBreakpoint);
#endif // WITH_STATETREE_TRACE_DEBUGGER
	
	return MenuBuilder.MakeWidget();
}


FReply SStateTreeView::HandleAddStateButton()
{
	if (StateTreeViewModel == nullptr)
	{
		return FReply::Handled();
	}
	
	TArray<UStateTreeState*> SelectedStates;
	StateTreeViewModel->GetSelectedStates(SelectedStates);
	UStateTreeState* FirstSelectedState = SelectedStates.Num() > 0 ? SelectedStates[0] : nullptr;

	if (FirstSelectedState != nullptr)
	{
		// If the state is root, add child state, else sibling.
		if (FirstSelectedState->Parent == nullptr)
		{
			StateTreeViewModel->AddChildState(FirstSelectedState);
			TreeView->SetItemExpansion(FirstSelectedState, true);
		}
		else
		{
			StateTreeViewModel->AddState(FirstSelectedState);
		}
	}
	else
	{
		// Add root state at the lowest level.
		StateTreeViewModel->AddState(nullptr);
	}

	return FReply::Handled();
}

UStateTreeState* SStateTreeView::GetFirstSelectedState() const
{
	TArray<UStateTreeState*> SelectedStates;
	if (StateTreeViewModel)
	{
		StateTreeViewModel->GetSelectedStates(SelectedStates);
	}
	return SelectedStates.IsEmpty() ? nullptr : SelectedStates[0];
}

void SStateTreeView::HandleAddSiblingState()
{
	if (StateTreeViewModel)
	{
		StateTreeViewModel->AddState(GetFirstSelectedState());
	}
}

void SStateTreeView::HandleAddChildState()
{
	if (StateTreeViewModel)
	{
		UStateTreeState* ParentState = GetFirstSelectedState();
		if (ParentState)
		{
			StateTreeViewModel->AddChildState(ParentState);
			TreeView->SetItemExpansion(ParentState, true);
		}
	}
}

void SStateTreeView::HandleCutSelectedStates()
{
	if (StateTreeViewModel)
	{
		StateTreeViewModel->CopySelectedStates();
		StateTreeViewModel->RemoveSelectedStates();
	}
}

void SStateTreeView::HandleCopySelectedStates()
{
	if (StateTreeViewModel)
	{
		StateTreeViewModel->CopySelectedStates();
	}
}

void SStateTreeView::HandlePasteStatesAsSiblings()
{
	if (StateTreeViewModel)
	{
		StateTreeViewModel->PasteStatesFromClipboard(GetFirstSelectedState());
	}
}

void SStateTreeView::HandlePasteStatesAsChildren()
{
	if (StateTreeViewModel)
	{
		StateTreeViewModel->PasteStatesAsChildrenFromClipboard(GetFirstSelectedState());
	}
}

void SStateTreeView::HandleDuplicateSelectedStates()
{
	if (StateTreeViewModel)
	{
		StateTreeViewModel->DuplicateSelectedStates();
	}
}

void SStateTreeView::HandlePasteNodesToState()
{
	if (StateTreeViewModel)
	{
		StateTreeViewModel->PasteNodesToSelectedStates();
	}
}

void SStateTreeView::HandleDeleteStates()
{
	if (StateTreeViewModel)
	{
		StateTreeViewModel->RemoveSelectedStates();
	}
}

void SStateTreeView::HandleRenameState()
{
	RequestedRenameState = GetFirstSelectedState();
}

void SStateTreeView::HandleEnableSelectedStates()
{
	if (StateTreeViewModel)
	{
		// Process CanEnable first so in case of undetermined state (mixed selection) we Enable by default. 
		if (CanEnableStates())
		{
			StateTreeViewModel->SetSelectedStatesEnabled(true);	
		}
		else if (CanDisableStates())
		{
			StateTreeViewModel->SetSelectedStatesEnabled(false);
		}
	}
}

void SStateTreeView::HandleDisableSelectedStates()
{
	if (StateTreeViewModel)
	{
		StateTreeViewModel->SetSelectedStatesEnabled(false);
	}
}

TSharedPtr<FStateTreeViewModel> SStateTreeView::GetViewModel() const
{
	return StateTreeViewModel;
}

void SStateTreeView::SetSelection(const TArray<TWeakObjectPtr<UStateTreeState>>& SelectedStates) const
{
	for (const TWeakObjectPtr<UStateTreeState>& WeakState : SelectedStates)
	{
		if (const UStateTreeState* SelectedState = WeakState.Get())
		{
			UStateTreeState* ParentState = SelectedState->Parent;
			while (ParentState)
			{
				constexpr bool bShouldExpandItem(true);
				TreeView->SetItemExpansion(ParentState, bShouldExpandItem);
				ParentState = ParentState->Parent;
			}
		}
	}
	StateTreeViewModel->SetSelection(SelectedStates);
}

#undef LOCTEXT_NAMESPACE
