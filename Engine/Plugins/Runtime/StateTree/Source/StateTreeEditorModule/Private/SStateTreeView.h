// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

class FStateTreeViewModel;
class ITableRow;
class SScrollBar;
class STableViewBase;
namespace ESelectInfo { enum Type : int; }
struct FPropertyChangedEvent;
class UStateTreeState;
class SScrollBox;
class FUICommandList;

class SStateTreeView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SStateTreeView) {}
	SLATE_END_ARGS()

	SStateTreeView();
	virtual ~SStateTreeView() override;

	void Construct(const FArguments& InArgs, TSharedRef<FStateTreeViewModel> StateTreeViewModel, const TSharedRef<FUICommandList>& InCommandList);

	void SavePersistentExpandedStates();

	TSharedPtr<FStateTreeViewModel> GetViewModel() const;

	void SetSelection(const TArray<TWeakObjectPtr<UStateTreeState>>& SelectedStates) const;

private:
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	void UpdateTree(bool bExpandPersistent = false);

	//~ Editor/User settings handlers
	void HandleUserSettingsChanged();

	//~ ViewModel handlers
	void HandleModelAssetChanged();
	void HandleModelStatesRemoved(const TSet<UStateTreeState*>& AffectedParents);
	void HandleModelStatesMoved(const TSet<UStateTreeState*>& AffectedParents, const TSet<UStateTreeState*>& MovedStates);
	void HandleModelStateAdded(UStateTreeState* ParentState, UStateTreeState* NewState);
	void HandleModelStatesChanged(const TSet<UStateTreeState*>& AffectedStates, const FPropertyChangedEvent& PropertyChangedEvent);
	void HandleModelStateNodesChanged(const UStateTreeState* AffectedState);
	void HandleModelSelectionChanged(const TArray<TWeakObjectPtr<UStateTreeState>>& SelectedStates);

	//~ Treeview handlers
	TSharedRef<ITableRow> HandleGenerateRow(TWeakObjectPtr<UStateTreeState> InState, const TSharedRef<STableViewBase>& InOwnerTableView);
	void HandleGetChildren(TWeakObjectPtr<UStateTreeState> InParent, TArray<TWeakObjectPtr<UStateTreeState>>& OutChildren);
	void HandleTreeSelectionChanged(TWeakObjectPtr<UStateTreeState> InSelectedItem, ESelectInfo::Type SelectionType);
	void HandleTreeExpansionChanged(TWeakObjectPtr<UStateTreeState> InSelectedItem, bool bExpanded);
	
	TSharedPtr<SWidget> HandleContextMenuOpening();
	TSharedRef<SWidget> HandleGenerateSettingsMenu();

	//~ Action handlers
	//~ @todo: these are also defined in the outliner, figure out how to share code.
	UStateTreeState* GetFirstSelectedState() const;
	FReply HandleAddStateButton();
	void HandleAddSiblingState();
	void HandleAddChildState();
	void HandleCutSelectedStates();
	void HandleCopySelectedStates();
	void HandlePasteStatesAsSiblings();
	void HandlePasteStatesAsChildren();
	void HandleDuplicateSelectedStates();
	void HandlePasteNodesToState();
	void HandleRenameState();
	void HandleDeleteStates();
	void HandleEnableSelectedStates();
	void HandleDisableSelectedStates();

	bool HasSelection() const;
	bool CanPasteStates() const;
	bool CanEnableStates() const;
	bool CanDisableStates() const;
	bool CanPasteNodesToSelectedStates() const;

	void BindCommands();

	TSharedPtr<FStateTreeViewModel> StateTreeViewModel;

	TSharedPtr<STreeView<TWeakObjectPtr<UStateTreeState>>> TreeView;
	TSharedPtr<SScrollBar> ExternalScrollbar;
	TSharedPtr<SScrollBox> ViewBox;
	TArray<TWeakObjectPtr<UStateTreeState>> Subtrees;

	TSharedPtr<FUICommandList> CommandList;

	UStateTreeState* RequestedRenameState;
	FDelegateHandle SettingsChangedHandle;
	bool bItemsDirty;
	bool bUpdatingSelection;
};
