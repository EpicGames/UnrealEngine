// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStateTreeViewRow.h"
#include "SStateTreeView.h"
#include "SStateTreeExpanderArrow.h"
#include "StateTreeEditor.h"
#include "StateTreeEditorData.h"
#include "StateTreeEditorStyle.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "StateTree.h"
#include "StateTreeConditionBase.h"
#include "StateTreeDescriptionHelpers.h"
#include "StateTreeDragDrop.h"
#include "StateTreeEditorModule.h"
#include "StateTreeEditorUserSettings.h"
#include "StateTreeState.h"
#include "StateTreeTaskBase.h"
#include "StateTreeTypes.h"
#include "StateTreeViewModel.h"
#include "Widgets/Views/SListView.h"
#include "TextStyleDecorator.h"
#include "Customizations/StateTreeEditorNodeUtils.h"
#include "Customizations/Widgets/SStateTreeContextMenuButton.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Input/SButton.h"

// @TODO: Dependency onto state centric to enable hierarchy driven colors. 
// Should instead be driven by view model.
#include "StateCentricView/StateCentricViewSettings.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"

namespace UE::StateTree::Editor
{
	FLinearColor LerpColorSRGB(const FLinearColor ColorA, FLinearColor ColorB, float T)
	{
		const FColor A = ColorA.ToFColorSRGB();
		const FColor B = ColorB.ToFColorSRGB();
		return FLinearColor(FColor(
			static_cast<uint8>(FMath::RoundToInt(static_cast<float>(A.R) * (1.f - T) + static_cast<float>(B.R) * T)),
			static_cast<uint8>(FMath::RoundToInt(static_cast<float>(A.G) * (1.f - T) + static_cast<float>(B.G) * T)),
			static_cast<uint8>(FMath::RoundToInt(static_cast<float>(A.B) * (1.f - T) + static_cast<float>(B.B) * T)),
			static_cast<uint8>(FMath::RoundToInt(static_cast<float>(A.A) * (1.f - T) + static_cast<float>(B.A) * T))));
	}

	static constexpr FLinearColor IconTint = FLinearColor(1, 1, 1, 0.5f);
} // UE:StateTree::Editor

void SStateTreeViewRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TWeakObjectPtr<UStateTreeState> InState, const TSharedPtr<SScrollBox>& ViewBox, TSharedPtr<FStateTreeViewModel> InStateTreeViewModel)
{
	StateTreeViewModel = InStateTreeViewModel;
	WeakState = InState;
	const UStateTreeState* State = InState.Get();
	WeakEditorData = State != nullptr ? State->GetTypedOuter<UStateTreeEditorData>() : nullptr;

	AssetChangedHandle = StateTreeViewModel->GetOnAssetChanged().AddSP(this, &SStateTreeViewRow::HandleAssetChanged);
	StatesChangedHandle = StateTreeViewModel->GetOnStatesChanged().AddSP(this, &SStateTreeViewRow::HandleStatesChanged);

	ConstructInternal(STableRow::FArguments()
		.OnDragDetected(this, &SStateTreeViewRow::HandleDragDetected)
		.OnDragLeave(this, &SStateTreeViewRow::HandleDragLeave)
		.OnCanAcceptDrop(this, &SStateTreeViewRow::HandleCanAcceptDrop)
		.OnAcceptDrop(this, &SStateTreeViewRow::HandleAcceptDrop)
		.Style(&FStateTreeEditorStyle::Get().GetWidgetStyle<FTableRowStyle>("StateTree.Selection"))
		, InOwnerTableView);

	TSharedPtr<SVerticalBox> StateAndTasksVerticalBox;
	TSharedPtr<SHorizontalBox> StateHorizontalBox;
	TSharedPtr<SBorder> FlagBorder;

	this->ChildSlot
	.HAlign(HAlign_Fill)
	[
		SNew(SBox)
		.MinDesiredWidth_Lambda([WeakOwnerViewBox = ViewBox.ToWeakPtr()]()
			{
				// Captured as weak ptr so we don't prevent our parent widget from being destroyed (circular pointer reference).
				if (const TSharedPtr<SScrollBox> OwnerViewBox = WeakOwnerViewBox.Pin())
				{
					// Make the row at least as wide as the view.
					// The -1 is needed or we'll see a scrollbar.
					return OwnerViewBox->GetTickSpaceGeometry().GetLocalSize().X - 1;
				}
				return 0.f;
			})
		.Padding(FMargin(0, 0, 0, 0))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SStateTreeExpanderArrow, SharedThis(this))
				.IndentAmount(24.f)
				.BaseIndentLevel(0)
				.ImageSize(FVector2f(16,16))
				.ImagePadding(FMargin(9,14,0,0))
				.Image(this, &SStateTreeViewRow::GetSelectorIcon)
				.ColorAndOpacity(FLinearColor(1, 1, 1, 0.2f))
				.WireColorAndOpacity(FLinearColor(1, 1, 1, 0.2f))
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Left)
			.AutoWidth()
			.Padding(FMargin(0, 6, 0, 6))
			[
				// State and tasks
				SAssignNew(StateAndTasksVerticalBox, SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					// State
					SNew(SBox)
					.HeightOverride(GetDefault<UStateTreeEditorUserSettings>()->GetStatesViewStateRowHeight())
					.HAlign(HAlign_Left)
					[
						SAssignNew(StateHorizontalBox, SHorizontalBox)

						// State Box
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						//.FillWidth(1.f)
						.AutoWidth()
						[
							SNew(SBox)
							.HeightOverride(GetDefault<UStateTreeEditorUserSettings>()->GetStatesViewStateRowHeight())
							.VAlign(VAlign_Fill)
							[
								SNew(SBorder)
								.BorderImage(FStateTreeEditorStyle::Get().GetBrush("StateTree.State.Border"))
								.BorderBackgroundColor(this, &SStateTreeViewRow::GetActiveStateColor)
								[
									SNew(SBorder)
									.BorderImage(FStateTreeEditorStyle::Get().GetBrush("StateTree.State"))
									.BorderBackgroundColor(this, &SStateTreeViewRow::GetTitleColor, 1.0f, 0.0f)
									.Padding(FMargin(0.f, 0.f, 12.f, 0.f))
									.IsEnabled_Lambda([InState]
									{
										const UStateTreeState* State = InState.Get();
										return State != nullptr && State->bEnabled;
									})
									[
										SNew(SOverlay)
										+ SOverlay::Slot()
										[
											SNew(SHorizontalBox)

											// Sub tree marker
											+ SHorizontalBox::Slot()
											.VAlign(VAlign_Center)
											.AutoWidth()
											.Padding(FMargin(0,0,0,0))
											[
												SNew(SBox)
												.WidthOverride(4.f)
												.HeightOverride(GetDefault<UStateTreeEditorUserSettings>()->GetStatesViewStateRowHeight())
												.Visibility(this, &SStateTreeViewRow::GetSubTreeVisibility)
												.VAlign(VAlign_Fill)
												.HAlign(HAlign_Fill)
												[
													SNew(SBorder)
													.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
													.BorderBackgroundColor(FLinearColor(1,1,1,0.25f))
												]
											]

											// Conditions icon
											+SHorizontalBox::Slot()
											.VAlign(VAlign_Center)
											.AutoWidth()
											[
												SNew(SBox)
												.Padding(FMargin(4.f, 0.f, -4.f, 0.f))
												.Visibility(this, &SStateTreeViewRow::GetConditionVisibility)
												[
													SNew(SImage)
													.ColorAndOpacity(UE::StateTree::Editor::IconTint)
													.Image(FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.StateConditions"))
													.ToolTipText(LOCTEXT("StateHasEnterConditions", "State selection is guarded with enter conditions."))
												]
											]

											// Selector icon
											+ SHorizontalBox::Slot()
											.VAlign(VAlign_Center)
											.AutoWidth()
											[
												SNew(SBox)
												.Padding(FMargin(4.f, 0.f, 0.f, 0.f))
												[
													SNew(SImage)
													.Image(this, &SStateTreeViewRow::GetSelectorIcon)
													.ColorAndOpacity(UE::StateTree::Editor::IconTint)
													.ToolTipText(this, &SStateTreeViewRow::GetSelectorTooltip)
												]
											]

											// Warnings
											+SHorizontalBox::Slot()
											.VAlign(VAlign_Center)
											.AutoWidth()
											[
												SNew(SBox)
												.Padding(FMargin(2.f, 0.f, 2.f, 1.f))
												.Visibility(this, &SStateTreeViewRow::GetWarningsVisibility)
												[
													SNew(SImage)
													.Image(FAppStyle::Get().GetBrush("Icons.Warning"))
													.ToolTipText(this, &SStateTreeViewRow::GetWarningsTooltipText)
												]
											]
											
											// State Name
											+ SHorizontalBox::Slot()
											.VAlign(VAlign_Center)
											.AutoWidth()
											[
												SAssignNew(NameTextBlock, SInlineEditableTextBlock)
												.Style(FStateTreeEditorStyle::Get(), "StateTree.State.TitleInlineEditableText")
												.OnTextCommitted(this, &SStateTreeViewRow::HandleNodeLabelTextCommitted)
												.OnVerifyTextChanged(this, &SStateTreeViewRow::HandleVerifyNodeLabelTextChanged)
												.Text(this, &SStateTreeViewRow::GetStateDesc)
												.ToolTipText(this, &SStateTreeViewRow::GetStateTypeTooltip)
												.Clipping(EWidgetClipping::ClipToBounds)
												.IsSelected(this, &SStateTreeViewRow::IsStateSelected)
											]

											// Description
											+ SHorizontalBox::Slot()
											.VAlign(VAlign_Center)
											.AutoWidth()
											[
												SNew(SBox)
												.Padding(FMargin(2.f, 0.f, 2.f, 1.f))
												.Visibility(this, &SStateTreeViewRow::GetStateDescriptionVisibility)
												[
													SNew(SImage)
													.Image(FAppStyle::Get().GetBrush(TEXT("Icons.Comment")))
													.ColorAndOpacity(FStyleColors::Foreground)
													.ColorAndOpacity(UE::StateTree::Editor::IconTint)
													.ToolTipText(this, &SStateTreeViewRow::GetStateDescription)
												]
											]

											// Flags icons
											+ SHorizontalBox::Slot()
											.VAlign(VAlign_Center)
											.AutoWidth()
											.Padding(FMargin(0.0f))
											[
												SAssignNew(FlagsContainer, SBorder)
												.BorderImage(FStyleDefaults::GetNoBrush())
											]

											// Linked State
											+ SHorizontalBox::Slot()
											.VAlign(VAlign_Center)
											.AutoWidth()
											[
												SNew(SBox)
												.HeightOverride(GetDefault<UStateTreeEditorUserSettings>()->GetStatesViewStateRowHeight())
												.VAlign(VAlign_Fill)
												.Visibility(this, &SStateTreeViewRow::GetLinkedStateVisibility)
												[
													// Link icon
													SNew(SHorizontalBox)
													+ SHorizontalBox::Slot()
													.VAlign(VAlign_Center)
													.AutoWidth()
													.Padding(FMargin(4.f, 0.f, 4.f, 0.f))
													[
														SNew(SImage)
														.ColorAndOpacity(UE::StateTree::Editor::IconTint)
														.Image(FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.StateLinked"))
													]

													// Linked State
													+ SHorizontalBox::Slot()
													.VAlign(VAlign_Center)
													.AutoWidth()
													[
														SNew(STextBlock)
														.Text(this, &SStateTreeViewRow::GetLinkedStateDesc)
														.TextStyle(FStateTreeEditorStyle::Get(), "StateTree.Details")
													]
												]
											]
											
											// State ID
											+ SHorizontalBox::Slot()
											.VAlign(VAlign_Center)
											.AutoWidth()
											[
												SNew(STextBlock)
												.Visibility_Lambda([]()
												{
													return UE::StateTree::Editor::GbDisplayItemIds ? EVisibility::Visible : EVisibility::Collapsed;
												})
												.Text(this, &SStateTreeViewRow::GetStateIDDesc)
												.TextStyle(FStateTreeEditorStyle::Get(), "StateTree.Details")
											]
										]
										+ SOverlay::Slot()
										[
											SNew(SHorizontalBox)

											// State breakpoint box
											+ SHorizontalBox::Slot()
											.VAlign(VAlign_Top)
											.HAlign(HAlign_Left)
											.AutoWidth()
											[
												SNew(SBox)
												.Padding(FMargin(-12.f, -6.f, 0.f, 0.f))
												[
													SNew(SImage)
													.DesiredSizeOverride(FVector2D(12.f, 12.f))
													.Image(FStateTreeEditorStyle::Get().GetBrush(TEXT("StateTreeEditor.Debugger.Breakpoint.EnabledAndValid")))
													.Visibility(this, &SStateTreeViewRow::GetStateBreakpointVisibility)
													.ToolTipText(this, &SStateTreeViewRow::GetStateBreakpointTooltipText)
												]
											]
										]
									]
								]
							]
						]
					]
				]
			]
		]
	];

	if (EnumHasAllFlags(GetDefault<UStateTreeEditorUserSettings>()->GetStatesViewDisplayNodeType(), EStateTreeEditorUserSettingsNodeType::Transition))
	{
		StateHorizontalBox->AddSlot()
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Left)
		[
			// Transitions
			SAssignNew(TransitionsContainer, SHorizontalBox)
		];
	}

	if (EnumHasAllFlags(GetDefault<UStateTreeEditorUserSettings>()->GetStatesViewDisplayNodeType(), EStateTreeEditorUserSettingsNodeType::Condition))
	{
		StateAndTasksVerticalBox->AddSlot()
		.AutoHeight()
		.Padding(FMargin(0, 2, 0, 0))
		[
			MakeConditionsWidget(ViewBox)
		];
	}

	if (EnumHasAllFlags(GetDefault<UStateTreeEditorUserSettings>()->GetStatesViewDisplayNodeType(), EStateTreeEditorUserSettingsNodeType::Task))
	{
		StateAndTasksVerticalBox->AddSlot()
		.AutoHeight()
		.Padding(FMargin(0, 2, 0, 0))
		[
			MakeTasksWidget(ViewBox)
		];
	}

	MakeTransitionsWidget();
	MakeFlagsWidget();
}


SStateTreeViewRow::~SStateTreeViewRow()
{
	if (StateTreeViewModel)
	{
		StateTreeViewModel->GetOnAssetChanged().Remove(AssetChangedHandle);
		StateTreeViewModel->GetOnStatesChanged().Remove(StatesChangedHandle);
	}
}

TSharedRef<SWidget> SStateTreeViewRow::MakeTasksWidget(const TSharedPtr<SScrollBox>& ViewBox)
{
	const UStateTreeEditorData* EditorData = WeakEditorData.Get();
	const UStateTreeState* State = WeakState.Get();
	if (!EditorData || !State)
	{
		return SNullWidget::NullWidget;
	}

	const TSharedRef<SWrapBox> TasksBox = SNew(SWrapBox)
	.PreferredSize_Lambda([WeakOwnerViewBox = ViewBox.ToWeakPtr()]()
	{
		// Captured as weak ptr so we don't prevent our parent widget from being destroyed (circular pointer reference).
		if (const TSharedPtr<SScrollBox> OwnerViewBox = WeakOwnerViewBox.Pin())
		{
			return FMath::Max(300, OwnerViewBox->GetTickSpaceGeometry().GetLocalSize().X - 200);
		}
		return 0.f;
	});

	if (State->Tasks.IsEmpty())
	{
		return SNullWidget::NullWidget;
	}

	const int32 NumTasks = State->Tasks.Num();

	// The task descriptions can get long. Make some effort to limit how long they can get.
	for (int32 TaskIndex = 0; TaskIndex < NumTasks; TaskIndex++)
	{
		const FStateTreeEditorNode& TaskNode = State->Tasks[TaskIndex];
		if (const FStateTreeTaskBase* Task = TaskNode.GetNode().GetPtr<FStateTreeTaskBase>())
		{
			const FGuid TaskId = State->Tasks[TaskIndex].ID;
			auto IsTaskEnabledFunc = [WeakState = WeakState, TaskIndex]
				{
					const UStateTreeState* State = WeakState.Get();
					if (State != nullptr && State->Tasks.IsValidIndex(TaskIndex))
					{
						if (const FStateTreeTaskBase* Task = State->Tasks[TaskIndex].GetNode().GetPtr<FStateTreeTaskBase>())
						{
							return (State->bEnabled && Task->bTaskEnabled);
						}
					}
					return true;
				};

			auto IsTaskBreakpointEnabledFunc = [WeakEditorData = WeakEditorData, TaskId]
				{
#if WITH_STATETREE_TRACE_DEBUGGER
					const UStateTreeEditorData* EditorData = WeakEditorData.Get();
					if (EditorData != nullptr && EditorData->HasAnyBreakpoint(TaskId))
					{
						return EVisibility::Visible;
					}
#endif // WITH_STATETREE_TRACE_DEBUGGER
					return EVisibility::Hidden;
				};
			
			auto GetTaskBreakpointTooltipFunc = [WeakEditorData = WeakEditorData, TaskId]
				{
#if WITH_STATETREE_TRACE_DEBUGGER
					if (const UStateTreeEditorData* EditorData = WeakEditorData.Get())
					{
						const bool bHasBreakpointOnEnter = EditorData->HasBreakpoint(TaskId, EStateTreeBreakpointType::OnEnter);
						const bool bHasBreakpointOnExit = EditorData->HasBreakpoint(TaskId, EStateTreeBreakpointType::OnExit);
						if (bHasBreakpointOnEnter && bHasBreakpointOnExit)
						{
							return LOCTEXT("StateTreeTaskBreakpointOnEnterAndOnExitTooltip","Break when entering or exiting task");
						}

						if (bHasBreakpointOnEnter)
						{
							return LOCTEXT("StateTreeTaskBreakpointOnEnterTooltip","Break when entering task");
						}

						if (bHasBreakpointOnExit)
						{
							return LOCTEXT("StateTreeTaskBreakpointOnExitTooltip","Break when exiting task");
						}
					}
#endif // WITH_STATETREE_TRACE_DEBUGGER
					return FText::GetEmpty();
				};

			TasksBox->AddSlot()
				.Padding(FMargin(0.0f))
				[
					SNew(SStateTreeContextMenuButton, StateTreeViewModel.ToSharedRef(), WeakState, TaskId)
					.bAllowDragAndDrop(true)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ContentPadding(FMargin(0, 0))
					[
						SNew(SBorder)
						.VAlign(VAlign_Center)
						.BorderImage(FAppStyle::GetNoBrush())
						.Padding(0)
						.IsEnabled_Lambda(IsTaskEnabledFunc)
						[
							SNew(SOverlay)
							+ SOverlay::Slot()
							[
								SNew(SBox)
								.HeightOverride(GetDefault<UStateTreeEditorUserSettings>()->GetStatesViewNodeRowHeight())
								.Padding(FMargin(0.f, 0.f))
								[
									SNew(SHorizontalBox)
									+ SHorizontalBox::Slot()
									.VAlign(VAlign_Center)
									.HAlign(HAlign_Left)
									.FillContentWidth(0.f, 0.f)
									[
										SNew(SBox)
										.Padding(FMargin(0.f, 0.f, 2.f, 0.f))
										.Visibility(this, &SStateTreeViewRow::GetTaskIconVisibility, TaskId)
										[
											SNew(SImage)
											.Image(this, &SStateTreeViewRow::GetTaskIcon, TaskId)
											.ColorAndOpacity(this, &SStateTreeViewRow::GetTaskIconColor, TaskId)
										]
									]

									+ SHorizontalBox::Slot()
									.VAlign(VAlign_Center)
									.HAlign(HAlign_Left)
									.FillContentWidth(0.f, 1.f)
									[
										SNew(SRichTextBlock)
										.Text(this, &SStateTreeViewRow::GetTaskDesc, TaskId, EStateTreeNodeFormatting::RichText)
										.ToolTipText(this, &SStateTreeViewRow::GetTaskDesc, TaskId, EStateTreeNodeFormatting::Text)
										.TextStyle(&FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("StateTree.Task.Title"))
										.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
										.Clipping(EWidgetClipping::OnDemand)
										+SRichTextBlock::Decorator(FTextStyleDecorator::Create(TEXT(""), FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("StateTree.Task.Title")))
										+SRichTextBlock::Decorator(FTextStyleDecorator::Create(TEXT("b"), FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("StateTree.Task.Title.Bold")))
										+SRichTextBlock::Decorator(FTextStyleDecorator::Create(TEXT("s"), FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("StateTree.Task.Title.Subdued")))
									]
								]
							]
							+ SOverlay::Slot()
							[
								// Task Breakpoint box
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.VAlign(VAlign_Top)
								.HAlign(HAlign_Left)
								.AutoWidth()
								[
									SNew(SBox)
									.Padding(FMargin(-2.0f, -2.0f, 0.0f, 0.0f))
									[
										SNew(SImage)
										.DesiredSizeOverride(FVector2D(10.f, 10.f))
										.Image(FStateTreeEditorStyle::Get().GetBrush(TEXT("StateTreeEditor.Debugger.Breakpoint.EnabledAndValid")))
										.Visibility_Lambda(IsTaskBreakpointEnabledFunc)
										.ToolTipText_Lambda(GetTaskBreakpointTooltipFunc)
									]
								]
							]
						]
					]
				];
		}
	}

	return TasksBox;
}

TSharedRef<SWidget> SStateTreeViewRow::MakeConditionsWidget(const TSharedPtr<SScrollBox>& ViewBox)
{
	const UStateTreeEditorData* EditorData = WeakEditorData.Get();
	const UStateTreeState* State = WeakState.Get();
	if (!EditorData || !State)
	{
		return SNullWidget::NullWidget;
	}

	if (!State->bHasRequiredEventToEnter && State->EnterConditions.IsEmpty())
	{
		return SNullWidget::NullWidget;
	}

	TSharedRef<SVerticalBox> VerticalBox = SNew(SVerticalBox);
	if (State->bHasRequiredEventToEnter)
	{
		auto IsConditionEnabledFunc = [WeakState = WeakState]
			{
				const UStateTreeState* State = WeakState.Get();
				return State && State->bEnabled;
			};

		const FName PayloadStructName = State->RequiredEventToEnter.PayloadStruct ? State->RequiredEventToEnter.PayloadStruct->GetFName() : FName();
		const FText Description = FText::Format(LOCTEXT("Condition", "<b>Tag(</>{0}<b>) Payload(</>{1}<b>)</>"), FText::FromName(State->RequiredEventToEnter.Tag.GetTagName()), FText::FromName(PayloadStructName));

		VerticalBox->AddSlot()
		[
			SNew(SBorder)
			.VAlign(VAlign_Center)
			.BorderImage(FAppStyle::GetNoBrush())
			.Padding(FMargin(4.0f, 2.0f, 4.0f, 0.0f))
			.IsEnabled_Lambda(IsConditionEnabledFunc)
			.Padding(0)
			[
				SNew(SBox)
				.HeightOverride(GetDefault<UStateTreeEditorUserSettings>()->GetStatesViewNodeRowHeight())
				[
					SNew(SHorizontalBox)
					// Icon
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					.Padding(FMargin(4.0f, 0.0f, 4.0f, 0.0f))
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FStateTreeEditorStyle::Get().GetBrush(FName("StateTreeEditor.Conditions")))
					]
					// Desc
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					.AutoWidth()
					[
						SNew(SRichTextBlock)
						.Text(Description)
						.TextStyle(&FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("StateTree.Task.Title"))
						.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
						.Clipping(EWidgetClipping::OnDemand)
						+ SRichTextBlock::Decorator(FTextStyleDecorator::Create(TEXT(""), FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("StateTree.Task.Title")))
						+ SRichTextBlock::Decorator(FTextStyleDecorator::Create(TEXT("b"), FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("StateTree.Task.Title.Bold")))
					]
				]
			]
		];
	}

	if (!State->EnterConditions.IsEmpty())
	{
		TSharedRef<SWrapBox> ConditionsBox = SNew(SWrapBox)
			.PreferredSize_Lambda([WeakOwnerViewBox = ViewBox.ToWeakPtr()]()
				{
					// Captured as weak ptr so we don't prevent our parent widget from being destroyed (circular pointer reference).
					if (const TSharedPtr<SScrollBox> OwnerViewBox = WeakOwnerViewBox.Pin())
					{
						return FMath::Max(300, OwnerViewBox->GetTickSpaceGeometry().GetLocalSize().X - 200);
					}
					return 0.f;
				});

		const int32 NumConditions = State->EnterConditions.Num();
		for (int32 ConditionIndex = 0; ConditionIndex < NumConditions; ConditionIndex++)
		{
			const FStateTreeEditorNode& ConditionNode = State->EnterConditions[ConditionIndex];
			if (const FStateTreeConditionBase* Condition = ConditionNode.GetNode().GetPtr<FStateTreeConditionBase>())
			{
				const FGuid ConditionId = ConditionNode.ID;

				auto IsConditionEnabledFunc = [WeakState = WeakState]
				{
					const UStateTreeState* State = WeakState.Get();
					return State && State->bEnabled;
				};

				auto IsForcedConditionVisibleFunc = [WeakState = WeakState, ConditionIndex]()
					{
						const UStateTreeState* State = WeakState.Get();
						if (State != nullptr && State->EnterConditions.IsValidIndex(ConditionIndex))
						{
							if (const FStateTreeConditionBase* Condition = State->EnterConditions[ConditionIndex].GetNode().GetPtr<FStateTreeConditionBase>())
							{
								return Condition->EvaluationMode != EStateTreeConditionEvaluationMode::Evaluated ? EVisibility::Visible : EVisibility::Hidden;
							}
						}
						return EVisibility::Hidden;
					};

				auto GetForcedConditionTooltipFunc = [WeakState = WeakState, ConditionIndex]()
					{
						const UStateTreeState* State = WeakState.Get();
						if (State != nullptr && State->EnterConditions.IsValidIndex(ConditionIndex))
						{
							if (const FStateTreeConditionBase* Condition = State->EnterConditions[ConditionIndex].GetNode().GetPtr<FStateTreeConditionBase>())
							{
								if (Condition->EvaluationMode == EStateTreeConditionEvaluationMode::ForcedTrue)
								{
									return LOCTEXT("ForcedTrueConditionTooltip", "This condition is not evaluated and result forced to 'true'.");
								}
								if (Condition->EvaluationMode == EStateTreeConditionEvaluationMode::ForcedFalse)
								{
									return LOCTEXT("ForcedFalseConditionTooltip", "This condition is not evaluated and result forced to 'false'.");
								}
							}
						}
						return FText::GetEmpty();
					};

				auto GetForcedConditionImageFunc = [WeakState = WeakState, ConditionIndex]() -> const FSlateBrush*
					{
						const UStateTreeState* State = WeakState.Get();
						if (State != nullptr && State->EnterConditions.IsValidIndex(ConditionIndex))
						{
							if (const FStateTreeConditionBase* Condition = State->EnterConditions[ConditionIndex].GetNode().GetPtr<FStateTreeConditionBase>())
							{
								if (Condition->EvaluationMode == EStateTreeConditionEvaluationMode::ForcedTrue)
								{
									return FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Debugger.Condition.Passed");
								}
								if (Condition->EvaluationMode == EStateTreeConditionEvaluationMode::ForcedFalse)
								{
									return FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Debugger.Condition.Failed");
								}
							}
						}
						return nullptr;
					};

				ConditionsBox->AddSlot()
					[
						SNew(SBorder)
						.VAlign(VAlign_Center)
						.BorderImage(FAppStyle::GetNoBrush())
						.IsEnabled_Lambda(IsConditionEnabledFunc)
						.Padding(0)
						[
							SNew(SBox)
							.HeightOverride(GetDefault<UStateTreeEditorUserSettings>()->GetStatesViewNodeRowHeight())
							[
								SNew(SHorizontalBox)

								// Operand
								+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								[
									SNew(SBox)
									.Padding(FMargin(4, 2, 4, 0))
									.VAlign(VAlign_Center)
									[
										SNew(STextBlock)
										.TextStyle(FStateTreeEditorStyle::Get(), "StateTree.Node.Operand")
										.Text(this, &SStateTreeViewRow::GetOperandText, ConditionIndex)
									]
								]
								// Open parens
								+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								[
									SNew(SBox)
									.Padding(FMargin(FMargin(0.0f, 1.0f, 0.0f, 0.0f)))
									[
										SNew(STextBlock)
										.TextStyle(FStateTreeEditorStyle::Get(), "StateTree.Task.Title")
										.Text(this, &SStateTreeViewRow::GetOpenParens, ConditionIndex)
									]
								]
								// Open parens
								+ SHorizontalBox::Slot() 
								.AutoWidth()
								.VAlign(VAlign_Center)
								[
									SNew(SOverlay)
									+ SOverlay::Slot()
									[

										SNew(SStateTreeContextMenuButton, StateTreeViewModel.ToSharedRef(), WeakState, ConditionId)
										.bAllowDragAndDrop(true)
										.ButtonStyle(FAppStyle::Get(), "SimpleButton")
										.ContentPadding(FMargin(0.0f))
										[
											SNew(SHorizontalBox)
										
											// Icon
											+ SHorizontalBox::Slot()
											.VAlign(VAlign_Center)
											.HAlign(HAlign_Left)
											.AutoWidth()
											[
												SNew(SBox)
												.Padding(FMargin(0.f, 0.f, 2.f, 0.f))
												.Visibility(this, &SStateTreeViewRow::GetConditionIconVisibility, ConditionId)
												[
													SNew(SImage)
													.Image(this, &SStateTreeViewRow::GetConditionIcon, ConditionId)
													.ColorAndOpacity(this, &SStateTreeViewRow::GetConditionIconColor, ConditionId)
												]
											]
											// Desc
											+ SHorizontalBox::Slot()
											.VAlign(VAlign_Center)
											.HAlign(HAlign_Left)
											.AutoWidth()
											[
												SNew(SRichTextBlock)
												.Text(this, &SStateTreeViewRow::GetConditionDesc, ConditionId, EStateTreeNodeFormatting::RichText)
												.ToolTipText(this, &SStateTreeViewRow::GetConditionDesc, ConditionId, EStateTreeNodeFormatting::Text)
												.TextStyle(&FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("StateTree.Task.Title"))
												.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
												.Clipping(EWidgetClipping::OnDemand)
												+SRichTextBlock::Decorator(FTextStyleDecorator::Create(TEXT(""), FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("StateTree.Task.Title")))
												+SRichTextBlock::Decorator(FTextStyleDecorator::Create(TEXT("b"), FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("StateTree.Task.Title.Bold")))
												+SRichTextBlock::Decorator(FTextStyleDecorator::Create(TEXT("s"), FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("StateTree.Task.Title.Subdued")))
											]
										]
									]

									+ SOverlay::Slot()
									[
										// Condition override box
										SNew(SHorizontalBox)
										+ SHorizontalBox::Slot()
										.VAlign(VAlign_Top)
										.HAlign(HAlign_Left)
										.AutoWidth()
										[
											SNew(SBox)
											.Padding(FMargin(-2.0f, -2.0f, 0.0f, 0.0f))
											[
												SNew(SImage)
												.DesiredSizeOverride(FVector2D(16.f, 16.f))
												.Image_Lambda(GetForcedConditionImageFunc)
												.Visibility_Lambda(IsForcedConditionVisibleFunc)
												.ToolTipText_Lambda(GetForcedConditionTooltipFunc)
											]
										]
									]
								]
							
								// Close parens
								+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								[
									SNew(SBox)
									.Padding(FMargin(0.0f, 1.0f, 0.0f, 0.0f))
									[
										SNew(STextBlock)
										.TextStyle(FStateTreeEditorStyle::Get(), "StateTree.Task.Title")
										.Text(this, &SStateTreeViewRow::GetCloseParens, ConditionIndex)
									]
								]
							]
						]
					];
			}
		}
		VerticalBox->AddSlot()
		[
			ConditionsBox
		];
	}

	return VerticalBox;
}

void SStateTreeViewRow::MakeTransitionsWidget()
{
	SHorizontalBox* TransitionsContainerPtr = TransitionsContainer.Get();
	if (TransitionsContainerPtr == nullptr)
	{
		return;
	}

	TransitionsContainerPtr->ClearChildren();

	TransitionsContainerPtr->AddSlot()
	.VAlign(VAlign_Top)
	.AutoWidth()
	[
		SNew(SBox)
		.HeightOverride(GetDefault<UStateTreeEditorUserSettings>()->GetStatesViewStateRowHeight())
		.Visibility(this, &SStateTreeViewRow::GetTransitionDashVisibility)
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Transition.Dash"))
			.ColorAndOpacity(UE::StateTree::Editor::IconTint)
		]
	];

	// On State Completed
	//We don't show any additional signs for On Completed transitions, just the dash
	constexpr FSlateBrush* OnCompletedSlateIcon = nullptr;
	TransitionsContainerPtr->AddSlot()
	.VAlign(VAlign_Top)
	.AutoWidth()
	[
		MakeTransitionWidget(EStateTreeTransitionTrigger::OnStateCompleted, OnCompletedSlateIcon)
	];

	// On State Succeeded
	TransitionsContainerPtr->AddSlot()
	.VAlign(VAlign_Top)
	.AutoWidth()
	[
		MakeTransitionWidget(EStateTreeTransitionTrigger::OnStateSucceeded, FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Transition.Succeeded"))
	];

	// On State Failed
	TransitionsContainerPtr->AddSlot()
	.VAlign(VAlign_Top)
	.AutoWidth()
	[
		MakeTransitionWidget(EStateTreeTransitionTrigger::OnStateFailed, FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Transition.Failed"))
	];

	// On Tick, Event, Delegate
	TransitionsContainerPtr->AddSlot()
	.VAlign(VAlign_Top)
	.AutoWidth()
	[
		MakeTransitionWidget(EStateTreeTransitionTrigger::OnTick | EStateTreeTransitionTrigger::OnEvent | EStateTreeTransitionTrigger::OnDelegate, FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Transition.Condition"))
	];
}

TSharedRef<SWidget> SStateTreeViewRow::MakeTransitionWidget(const EStateTreeTransitionTrigger Trigger, const FSlateBrush* Icon)
{
		FTransitionDescFilterOptions FilterOptions;
		FilterOptions.bUseMask = EnumHasAnyFlags(Trigger, EStateTreeTransitionTrigger::OnTick | EStateTreeTransitionTrigger::OnEvent | EStateTreeTransitionTrigger::OnDelegate);

		return SNew(SBox)
		.HeightOverride(GetDefault<UStateTreeEditorUserSettings>()->GetStatesViewStateRowHeight())
		.Visibility(this, &SStateTreeViewRow::GetTransitionsVisibility, Trigger)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(FMargin(0.f, 0.f, 0.f, 0.f))
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SImage)
					.Image(Icon)
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SImage)
					.Image(this, &SStateTreeViewRow::GetTransitionsIcon, Trigger)
					.ColorAndOpacity(UE::StateTree::Editor::IconTint)
				]
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(FMargin(4.f, 0.f, 12.f, 0.f))
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				[
					MakeTransitionWidgetInternal(Trigger, FilterOptions)
				]
				+ SOverlay::Slot()
				[
					// Breakpoint box
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Top)
					.HAlign(HAlign_Left)
					.AutoWidth()
					[
						SNew(SBox)
						.Padding(FMargin(-4.f, -4.f, 0.f, 0.f))
						[
							SNew(SImage)
							.DesiredSizeOverride(FVector2D(10.f, 10.f))
							.Image(FStateTreeEditorStyle::Get().GetBrush(TEXT("StateTreeEditor.Debugger.Breakpoint.EnabledAndValid")))
							.Visibility(this, &SStateTreeViewRow::GetTransitionsBreakpointVisibility, Trigger)
							.ToolTipText_Lambda([this, Trigger, InFilterOptions = FilterOptions]
							{
								FTransitionDescFilterOptions FilterOptions = InFilterOptions;
								FilterOptions.WithBreakpoint = ETransitionDescRequirement::RequiredTrue;

								return FText::Format(LOCTEXT("TransitionBreakpointTooltip", "Break when executing transition: {0}"),
													 GetTransitionsDesc(Trigger, FilterOptions));
							})
						]
					]
				]
			]
		];
};

TSharedRef<SWidget> SStateTreeViewRow::MakeTransitionWidgetInternal(const EStateTreeTransitionTrigger Trigger, const FTransitionDescFilterOptions FilterOptions)
{
	const UStateTreeEditorData* TreeEditorData = WeakEditorData.Get();
	const UStateTreeState* State = WeakState.Get();

	if (!TreeEditorData || !State)
	{
		return SNullWidget::NullWidget;
	}
	
	struct FItem
	{
		FItem() = default;
		FItem(const FStateTreeStateLink& InLink, const FGuid InNodeID)
			: Link(InLink)
			, NodeID(InNodeID)
		{
		}
		FItem(const FText& InDesc, const FText& InTooltip)
			: Desc(InDesc)
			, Tooltip(InTooltip)
		{
		}
		
		FText Desc;
		FText Tooltip;
		FStateTreeStateLink Link;
		FGuid NodeID;
		bool bSkipComma = false;
	};

	TArray<FItem> DescItems;

	for (const FStateTreeTransition& Transition : State->Transitions)
	{
		// Apply filter for enabled/disabled transitions
		if ((FilterOptions.Enabled == ETransitionDescRequirement::RequiredTrue && Transition.bTransitionEnabled == false)
			|| (FilterOptions.Enabled == ETransitionDescRequirement::RequiredFalse && Transition.bTransitionEnabled))
		{
			continue;
		}

#if WITH_STATETREE_TRACE_DEBUGGER
		// Apply filter for transitions with/without breakpoint
		const bool bHasBreakpoint = TreeEditorData != nullptr && TreeEditorData->HasBreakpoint(Transition.ID, EStateTreeBreakpointType::OnTransition);
		if ((FilterOptions.WithBreakpoint == ETransitionDescRequirement::RequiredTrue && bHasBreakpoint == false)
			|| (FilterOptions.WithBreakpoint == ETransitionDescRequirement::RequiredFalse && bHasBreakpoint))
		{
			continue;
		}
#endif // WITH_STATETREE_TRACE_DEBUGGER

		const bool bMatch = FilterOptions.bUseMask ? EnumHasAnyFlags(Transition.Trigger, Trigger) : Transition.Trigger == Trigger;
		if (bMatch)
		{
			DescItems.Emplace(Transition.State, Transition.ID);

			if (Transition.ReactivateTargetState == EStateTreeTransitionChangeTypeRules::ForceChanged)
			{
				FItem ReselectionNote;
				ReselectionNote.Desc = LOCTEXT("ReactivateDesc", "(Re-Activate)");
				ReselectionNote.Tooltip = LOCTEXT("ReactivateDescToolTip", "The target state and all the following states will be reactivated. Their instance data will be reinstantiated and will receive EnterState.");
				ReselectionNote.bSkipComma = true;
				DescItems.Emplace(ReselectionNote);
			}
		}
	}

	// Find states from transition tasks
	if (EnumHasAnyFlags(Trigger, EStateTreeTransitionTrigger::OnTick | EStateTreeTransitionTrigger::OnEvent | EStateTreeTransitionTrigger::OnDelegate))
	{
		auto AddLinksFromStruct = [&DescItems, TreeEditorData](FStateTreeDataView Struct, const FGuid NodeID)
		{
			if (!Struct.IsValid())
			{
				return;
			}
			for (TPropertyValueIterator<FStructProperty> It(Struct.GetStruct(), Struct.GetMemory()); It; ++It)
			{
				const UScriptStruct* StructType = It.Key()->Struct;
				if (StructType == TBaseStructure<FStateTreeStateLink>::Get())
				{
					const FStateTreeStateLink& Link = *static_cast<const FStateTreeStateLink*>(It.Value());
					if (Link.LinkType != EStateTreeTransitionType::None)
					{
						DescItems.Emplace(Link, NodeID);
					}
				}
			}
		};
		
		for (const FStateTreeEditorNode& Task : State->Tasks)
		{
			AddLinksFromStruct(FStateTreeDataView(Task.GetNode().GetScriptStruct(), const_cast<uint8*>(Task.GetNode().GetMemory())), Task.ID);
			AddLinksFromStruct(Task.GetInstance(), Task.ID);
		}

		AddLinksFromStruct(FStateTreeDataView(State->SingleTask.GetNode().GetScriptStruct(), const_cast<uint8*>(State->SingleTask.GetNode().GetMemory())), State->SingleTask.ID);
		AddLinksFromStruct(State->SingleTask.GetInstance(), State->SingleTask.ID);
	}

	if (IsLeafState()
		&& DescItems.Num() == 0
		&& EnumHasAnyFlags(Trigger, EStateTreeTransitionTrigger::OnStateCompleted))
	{
		if (HasParentTransitionForTrigger(*State, Trigger))
		{
			DescItems.Emplace(
				LOCTEXT("TransitionActionHandleInParentRich", "<i>Parent</>"),
				LOCTEXT("TransitionActionHandleInParent", "Handle transition in parent State")
				);
		}
		else
		{
			DescItems.Emplace(
				LOCTEXT("TransitionActionRootRich", "<i>Root</>"),
				LOCTEXT("TransitionActionRoot", "Transition to Root State.")
				);
		}
	}

	TSharedRef<SHorizontalBox> TransitionContainer = SNew(SHorizontalBox);

	auto IsTransitionEnabledFunc = [WeakState = WeakState]
	{
		const UStateTreeState* State = WeakState.Get();
		return State && State->bEnabled;
	};

	for (int32 Index = 0; Index < DescItems.Num(); Index++)
	{
		const FItem& Item = DescItems[Index];

		if (Index > 0 && !Item.bSkipComma)
		{
			TransitionContainer->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromString(TEXT(", ")))
				.TextStyle(&FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Transition.Subdued"))
			];
		}

		constexpr bool bIsTransition = true;
		TransitionContainer->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SStateTreeContextMenuButton, StateTreeViewModel.ToSharedRef(), WeakState, Item.NodeID, bIsTransition)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ContentPadding(FMargin(0, 0))
			[
				SNew(SBorder)
				.VAlign(VAlign_Center)
				.BorderImage(FAppStyle::GetNoBrush())
				.Padding(0)
				.IsEnabled_Lambda(IsTransitionEnabledFunc)
				[
					SNew(SRichTextBlock)
					.Text_Lambda([WeakEditorData = WeakEditorData, Item]()
					{
						if (!Item.Desc.IsEmpty())
						{
							return Item.Desc;
						}
						return UE::StateTree::Editor::GetStateLinkDesc(WeakEditorData.Get(), Item.Link, EStateTreeNodeFormatting::RichText);
					})
					.ToolTipText_Lambda([this, Item]()
					{
						if (!Item.Tooltip.IsEmpty())
						{
							return Item.Tooltip;
						}
						return GetLinkTooltip(Item.Link, Item.NodeID);
					})
					.TextStyle(&FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Transition.Normal"))
					+SRichTextBlock::Decorator(FTextStyleDecorator::Create(TEXT(""), FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Transition.Normal")))
					+SRichTextBlock::Decorator(FTextStyleDecorator::Create(TEXT("b"), FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Transition.Bold")))
					+SRichTextBlock::Decorator(FTextStyleDecorator::Create(TEXT("i"), FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Transition.Italic")))
					+SRichTextBlock::Decorator(FTextStyleDecorator::Create(TEXT("s"), FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Transition.Subdued")))
				]
			]
		];
	}
	
	return TransitionContainer;
}

void SStateTreeViewRow::MakeFlagsWidget()
{
	FlagsContainer->SetPadding(FMargin(0.0f));
	FlagsContainer->SetContent(SNullWidget::NullWidget);

	const UStateTree* StateTree = StateTreeViewModel ? StateTreeViewModel->GetStateTree() : nullptr;
	const UStateTreeState* State = WeakState.Get();
	const bool bDisplayFlags = EnumHasAllFlags(GetDefault<UStateTreeEditorUserSettings>()->GetStatesViewDisplayNodeType(), EStateTreeEditorUserSettingsNodeType::Flag);
	static constexpr FLinearColor IconTint = FLinearColor(1.0f, 1.0f, 1.0f, 0.5f);

	if (bDisplayFlags && State && StateTree)
	{
		if (const FCompactStateTreeState* RuntimeState = StateTree->GetStateFromHandle(StateTree->GetStateHandleFromId(State->ID)))
		{
			const bool bHasEvents = true;
			const bool bHasBroadcastedDelegates = true;
			if (RuntimeState->bHasCustomTickRate
				|| RuntimeState->DoesRequestTickTasks(bHasEvents)
				|| RuntimeState->ShouldTickTransitions(bHasEvents, bHasBroadcastedDelegates))
			{
				bool bShow = true;
				TSharedRef<SHorizontalBox> Box = SNew(SHorizontalBox);
				if (RuntimeState->bHasCustomTickRate)
				{
					Box->AddSlot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Flags.Tick"))
						.ColorAndOpacity(IconTint)
						.ToolTipText(LOCTEXT("StateCustomTick", "The state has a custom tick rate."))
					];
				}
				else if (RuntimeState->bCachedRequestTick)
				{
					Box->AddSlot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Flags.Tick"))
						.ColorAndOpacity(IconTint)
						.ToolTipText(LOCTEXT("StateNodeTick", "The state contains at least one task that ticks at runtime."))
					];
				}
				//The task can tick, but it may not affect the scheduled system (see DelayTask as an example).
				else if (RuntimeState->bCachedRequestTickOnlyOnEvents)
				{
					Box->AddSlot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Flags.TickOnEvent"))
						.ColorAndOpacity(IconTint)
						.ToolTipText(LOCTEXT("StateNodeTickEvent", "The state contains at least one task that ticks at runtime when there's an event."))
					];
				}
				else if (RuntimeState->bHasTransitionTasks)
				{
					Box->AddSlot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Flags.Tick"))
						.ColorAndOpacity(IconTint)
						.ToolTipText(LOCTEXT("StateNodeTickTransitionTask", "The state contains at least one task that ticks at runtime when evaluating transitions."))
					];
				}
				else if (RuntimeState->bHasTickTriggerTransitions)
				{
					Box->AddSlot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Flags.Tick"))
						.ColorAndOpacity(IconTint)
						.ToolTipText(LOCTEXT("StateNodeTickTransition", "The state contains at least one transition that ticks at runtime."))
					];
				}
				else if (RuntimeState->bHasEventTriggerTransitions)
				{
					Box->AddSlot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Flags.TickOnEvent"))
						.ColorAndOpacity(IconTint)
						.ToolTipText(LOCTEXT("StateNodeTickTransitionEvent", "The state contains at least one transition that ticks at runtime when there's an event."))
					];
				}
				else
				{
					// Don't show when bHasDelegateTriggerTransitions. It is not considered ticking.
					// Don't show when the task ticked but not considered for scheduling.
					bShow = false;
				}

				if (bShow)
				{
					FlagsContainer->SetPadding(FMargin(4.0f));
					FlagsContainer->SetContent(Box);
				}
			}
		}
	}
}

void SStateTreeViewRow::RequestRename() const
{
	if (NameTextBlock)
	{
		NameTextBlock->EnterEditingMode();
	}
}

FSlateColor SStateTreeViewRow::GetTitleColor(const float Alpha, const float Lighten) const
{
	const UStateTreeState* State = WeakState.Get();
	const UStateTreeEditorData* EditorData = WeakEditorData.Get();

	FLinearColor Color(FColor(31, 151, 167));

	if (State && StateTreeViewModel)
	{
		Color = StateTreeViewModel->GetStateColor(State);

		// @TODO: Dependency onto state centric to enable hierarchy driven colors. 
		// Should instead be driven by view model.
		if (UStateCentricViewSettings::IsStateCentricViewEnabled() && UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(StateTreeViewModel.Get()).bEnableHierarchyDrivenColors)
		{
			Color = StateTreeViewModel->GetHierarchyBasedBackgroundColor(State);
		}

		if (IsRootState() || State->Type == EStateTreeStateType::Subtree)
		{
			Color = UE::StateTree::Editor::LerpColorSRGB(Color, FColor::Black, 0.25f);
		}
	}

	if (Lighten > 0.0f)
	{
		Color = UE::StateTree::Editor::LerpColorSRGB(Color, FColor::White, Lighten);
	}
	
	return Color.CopyWithNewOpacity(Alpha);
}

FSlateColor SStateTreeViewRow::GetActiveStateColor() const
{
	if (const UStateTreeState* State = WeakState.Get())
	{
		if (StateTreeViewModel && StateTreeViewModel->IsStateActiveInDebugger(*State))
		{
			return FLinearColor::Yellow;
		}
		if (StateTreeViewModel && StateTreeViewModel->IsSelected(State))
		{
			return FStateTreeEditorStyle::Get().GetSlateColor("Colors.StateSelection");
		}
	}

	return FLinearColor::Transparent;
}

FSlateColor SStateTreeViewRow::GetSubTreeMarkerColor() const
{
	// Show color for subtree.
	if (const UStateTreeState* State = WeakState.Get())
	{
		if (IsRootState() || State->Type == EStateTreeStateType::Subtree)
		{
			const FSlateColor TitleColor = GetTitleColor();
			return UE::StateTree::Editor::LerpColorSRGB(TitleColor.GetSpecifiedColor(), FLinearColor::White, 0.2f);
		}
	}

	return GetTitleColor();
}

EVisibility SStateTreeViewRow::GetSubTreeVisibility() const
{
	if (const UStateTreeState* State = WeakState.Get())
	{
		if (IsRootState() || State->Type == EStateTreeStateType::Subtree)
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;	
}

FText SStateTreeViewRow::GetStateDesc() const
{
	if (const UStateTreeState* State = WeakState.Get())
	{
		return FText::FromName(State->Name);
	}
	return FText::FromName(FName());
}

FText SStateTreeViewRow::GetStateIDDesc() const
{
	if (const UStateTreeState* State = WeakState.Get())
	{
		return FText::FromString(*LexToString(State->ID));
	}
	return FText::FromName(FName());
}

EVisibility SStateTreeViewRow::GetConditionVisibility() const
{
	if (const UStateTreeState* State = WeakState.Get())
	{
		return State->EnterConditions.Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed;
	}
	return EVisibility::Collapsed;
}

EVisibility SStateTreeViewRow::GetStateBreakpointVisibility() const
{
#if WITH_STATETREE_TRACE_DEBUGGER
	const UStateTreeState* State = WeakState.Get();
	const UStateTreeEditorData* EditorData = WeakEditorData.Get();
	if (State != nullptr && EditorData != nullptr)
	{
		return (EditorData != nullptr && EditorData->HasAnyBreakpoint(State->ID)) ? EVisibility::Visible : EVisibility::Hidden;
	}
#endif // WITH_STATETREE_TRACE_DEBUGGER
	return EVisibility::Hidden;
}

FText SStateTreeViewRow::GetStateBreakpointTooltipText() const
{
#if WITH_STATETREE_TRACE_DEBUGGER
	const UStateTreeState* State = WeakState.Get();
	const UStateTreeEditorData* EditorData = WeakEditorData.Get();
	if (State != nullptr && EditorData != nullptr)
	{
		const bool bHasBreakpointOnEnter = EditorData->HasBreakpoint(State->ID, EStateTreeBreakpointType::OnEnter);
		const bool bHasBreakpointOnExit = EditorData->HasBreakpoint(State->ID, EStateTreeBreakpointType::OnExit);

		if (bHasBreakpointOnEnter && bHasBreakpointOnExit)
		{
			return LOCTEXT("StateTreeStateBreakpointOnEnterAndOnExitTooltip","Break when entering or exiting state");
		}

		if (bHasBreakpointOnEnter)
		{
			return LOCTEXT("StateTreeStateBreakpointOnEnterTooltip","Break when entering state");
		}

		if (bHasBreakpointOnExit)
		{
			return LOCTEXT("StateTreeStateBreakpointOnExitTooltip","Break when exiting state");
		}
	}
#endif // WITH_STATETREE_TRACE_DEBUGGER
	return FText::GetEmpty();
}

const FSlateBrush* SStateTreeViewRow::GetSelectorIcon() const
{
	if (const UStateTreeState* State = WeakState.Get())
	{
		return FStateTreeEditorStyle::GetBrushForSelectionBehaviorType(State->SelectionBehavior, !State->Children.IsEmpty(), State->Type);		
	}

	return nullptr;
}

FText SStateTreeViewRow::GetSelectorTooltip() const
{
	if (const UStateTreeState* State = WeakState.Get())
	{
		const UEnum* Enum = StaticEnum<EStateTreeStateSelectionBehavior>();
		check(Enum);
		const int32 Index = Enum->GetIndexByValue((int64)State->SelectionBehavior);
		
		switch (State->SelectionBehavior)
		{
			case EStateTreeStateSelectionBehavior::None:
			case EStateTreeStateSelectionBehavior::TryEnterState:
			case EStateTreeStateSelectionBehavior::TryFollowTransitions:
				return Enum->GetToolTipTextByIndex(Index);
			case EStateTreeStateSelectionBehavior::TrySelectChildrenInOrder:
			case EStateTreeStateSelectionBehavior::TrySelectChildrenAtRandom:
			case EStateTreeStateSelectionBehavior::TrySelectChildrenWithHighestUtility:
			case EStateTreeStateSelectionBehavior::TrySelectChildrenAtRandomWeightedByUtility:
				if (State->Children.IsEmpty()
					|| State->Type == EStateTreeStateType::Linked
					|| State->Type == EStateTreeStateType::LinkedAsset)
				{
					const int32 EnterStateIndex = Enum->GetIndexByValue((int64)EStateTreeStateSelectionBehavior::TryEnterState);
					return FText::Format(LOCTEXT("ConvertedToEnterState", "{0}\nAutomatically converted from '{1}' because the State has no child States."),
						Enum->GetToolTipTextByIndex(EnterStateIndex), UEnum::GetDisplayValueAsText(State->SelectionBehavior));
				}
				else
				{
					return Enum->GetToolTipTextByIndex(Index);
				}
			default:
				check(false);
		}
	}

	return FText::GetEmpty();
}

FText SStateTreeViewRow::GetStateTypeTooltip() const
{
	if (const UStateTreeState* State = WeakState.Get())
	{
		const UEnum* Enum = StaticEnum<EStateTreeStateType>();
		check(Enum);
		const int32 Index = Enum->GetIndexByValue((int64)State->Type);
		return Enum->GetToolTipTextByIndex(Index);
	}

	return FText::GetEmpty();
}

const FStateTreeEditorNode* SStateTreeViewRow::GetTaskNodeByID(FGuid TaskID) const
{
	const UStateTreeState* State = WeakState.Get();
	const UStateTreeEditorData* EditorData = WeakEditorData.Get();
	if (EditorData != nullptr
		&& State != nullptr)
	{
		return State->Tasks.FindByPredicate([&TaskID](const FStateTreeEditorNode& Node)
		{
			return Node.ID == TaskID;
		});
	}
	return nullptr;
}

EVisibility SStateTreeViewRow::GetTaskIconVisibility(FGuid TaskID) const
{
	bool bHasIcon = false;
	if (const FStateTreeEditorNode* TaskNode = GetTaskNodeByID(TaskID))
	{
		if (const FStateTreeNodeBase* BaseNode = TaskNode->GetNode().GetPtr<const FStateTreeNodeBase>())
		{
			bHasIcon = !BaseNode->GetIconName().IsNone();
		}
	}
	return bHasIcon ? EVisibility::Visible : EVisibility::Collapsed;  	
}

const FSlateBrush* SStateTreeViewRow::GetTaskIcon(FGuid TaskID) const
{
	if (const FStateTreeEditorNode* TaskNode = GetTaskNodeByID(TaskID))
	{
		if (const FStateTreeNodeBase* BaseNode = TaskNode->GetNode().GetPtr<const FStateTreeNodeBase>())
		{
			return UE::StateTreeEditor::EditorNodeUtils::ParseIcon(BaseNode->GetIconName()).GetIcon();
		}
	}
	return nullptr;	
}

FSlateColor SStateTreeViewRow::GetTaskIconColor(FGuid TaskID) const
{
	if (const FStateTreeEditorNode* TaskNode = GetTaskNodeByID(TaskID))
	{
		if (const FStateTreeNodeBase* BaseNode = TaskNode->GetNode().GetPtr<const FStateTreeNodeBase>())
		{
			return FLinearColor(BaseNode->GetIconColor());
		}
	}
	return FSlateColor::UseForeground();
}

FText SStateTreeViewRow::GetTaskDesc(FGuid TaskID, EStateTreeNodeFormatting Formatting) const
{
	FText TaskName;
	if (const UStateTreeEditorData* EditorData = WeakEditorData.Get())
	{
		if (const FStateTreeEditorNode* TaskNode = GetTaskNodeByID(TaskID))
		{
			if (UE::StateTree::Editor::GbDisplayItemIds)
			{
				TaskName = FText::Format(LOCTEXT("NodeNameWithID", "{0} ({1})"), EditorData->GetNodeDescription(*TaskNode, Formatting), FText::AsCultureInvariant(*LexToString(TaskID)));
			}
			else
			{
				TaskName = EditorData->GetNodeDescription(*TaskNode, Formatting);
			}
		}
	}
	return TaskName;
}

const FStateTreeEditorNode* SStateTreeViewRow::GetConditionNodeByID(FGuid ConditionID) const
{
	const UStateTreeState* State = WeakState.Get();
	const UStateTreeEditorData* EditorData = WeakEditorData.Get();
	if (EditorData != nullptr
		&& State != nullptr)
	{
		return State->EnterConditions.FindByPredicate([&ConditionID](const FStateTreeEditorNode& Node)
		{
			return Node.ID == ConditionID;
		});
	}
	return nullptr;
}

EVisibility SStateTreeViewRow::GetConditionIconVisibility(FGuid ConditionID) const
{
	bool bHasIcon = false;
	if (const FStateTreeEditorNode* Node = GetConditionNodeByID(ConditionID))
	{
		if (const FStateTreeNodeBase* BaseNode = Node->GetNode().GetPtr<const FStateTreeNodeBase>())
		{
			bHasIcon = !BaseNode->GetIconName().IsNone();
		}
	}
	return bHasIcon ? EVisibility::Visible : EVisibility::Collapsed;  	
}

const FSlateBrush* SStateTreeViewRow::GetConditionIcon(FGuid ConditionID) const
{
	if (const FStateTreeEditorNode* Node = GetConditionNodeByID(ConditionID))
	{
		if (const FStateTreeNodeBase* BaseNode = Node->GetNode().GetPtr<const FStateTreeNodeBase>())
		{
			return UE::StateTreeEditor::EditorNodeUtils::ParseIcon(BaseNode->GetIconName()).GetIcon();
		}
	}
	return nullptr;	
}

FSlateColor SStateTreeViewRow::GetConditionIconColor(FGuid ConditionID) const
{
	if (const FStateTreeEditorNode* Node = GetConditionNodeByID(ConditionID))
	{
		if (const FStateTreeNodeBase* BaseNode = Node->GetNode().GetPtr<const FStateTreeNodeBase>())
		{
			return FLinearColor(BaseNode->GetIconColor());
		}
	}
	return FSlateColor::UseForeground();
}

FText SStateTreeViewRow::GetConditionDesc(FGuid ConditionID, EStateTreeNodeFormatting Formatting) const
{
	FText Description;
	if (const UStateTreeEditorData* EditorData = WeakEditorData.Get())
	{
		if (const FStateTreeEditorNode* Node = GetConditionNodeByID(ConditionID))
		{
			if (UE::StateTree::Editor::GbDisplayItemIds)
			{
				Description = FText::Format(LOCTEXT("NodeNameWithID", "{0} ({1})"), EditorData->GetNodeDescription(*Node, Formatting), FText::AsCultureInvariant(*LexToString(ConditionID)));
			}
			else
			{
				Description = EditorData->GetNodeDescription(*Node, Formatting);
			}
		}
	}
	return Description;
}

FText SStateTreeViewRow::GetOperandText(const int32 ConditionIndex) const
{
	const UStateTreeState* State = WeakState.Get();
	if (!State
		|| !State->EnterConditions.IsValidIndex(ConditionIndex))
	{
		return FText::GetEmpty();
	}

	// First item does not relate to anything existing, it could be empty. 
	// return IF to indicate that we're building condition and IS for consideration.
	if (ConditionIndex == 0)
	{
		return LOCTEXT("IfOperand", "IF");
	}

	const EStateTreeExpressionOperand Operand = State->EnterConditions[ConditionIndex].ExpressionOperand;

	if (Operand == EStateTreeExpressionOperand::And)
	{
		return LOCTEXT("AndOperand", "AND");
	}
	else if (Operand == EStateTreeExpressionOperand::Or)
	{
		return LOCTEXT("OrOperand", "OR");
	}
	else
	{
		ensureMsgf(false, TEXT("Unhandled operand %s"), *UEnum::GetValueAsString(Operand));
	}

	return FText::GetEmpty();
}

FText SStateTreeViewRow::GetOpenParens(const int32 ConditionIndex) const
{
	const UStateTreeState* State = WeakState.Get();
	if (!State
		|| !State->EnterConditions.IsValidIndex(ConditionIndex))
	{
		return FText::GetEmpty();
	}

	const int32 NumConditions = State->EnterConditions.Num();
	const int32 CurrIndent = ConditionIndex == 0 ? 0 : (State->EnterConditions[ConditionIndex].ExpressionIndent + 1);
	const int32 NextIndent = (ConditionIndex + 1) >= NumConditions ? 0 : (State->EnterConditions[ConditionIndex + 1].ExpressionIndent + 1);
	const int32 DeltaIndent = NextIndent - CurrIndent;
	const int32 OpenParens = FMath::Max(0, DeltaIndent);

	static_assert(UE::StateTree::MaxExpressionIndent == 4);
	switch (OpenParens)
	{
	case 1: return FText::FromString(TEXT("("));
	case 2: return FText::FromString(TEXT("(("));
	case 3: return FText::FromString(TEXT("((("));
	case 4: return FText::FromString(TEXT("(((("));
	case 5: return FText::FromString(TEXT("((((("));
	}
	return FText::GetEmpty();
}

FText SStateTreeViewRow::GetCloseParens(const int32 ConditionIndex) const
{
	const UStateTreeState* State = WeakState.Get();
	if (!State
		|| !State->EnterConditions.IsValidIndex(ConditionIndex))
	{
		return FText::GetEmpty();
	}

	const int32 NumConditions = State->EnterConditions.Num();
	const int32 CurrIndent = ConditionIndex == 0 ? 0 : (State->EnterConditions[ConditionIndex].ExpressionIndent + 1);
	const int32 NextIndent = (ConditionIndex + 1) >= NumConditions ? 0 : (State->EnterConditions[ConditionIndex + 1].ExpressionIndent + 1);
	const int32 DeltaIndent = NextIndent - CurrIndent;
	const int32 CloseParens = FMath::Max(0, -DeltaIndent);

	static_assert(UE::StateTree::MaxExpressionIndent == 4);
	switch (CloseParens)
	{
	case 1: return FText::FromString(TEXT(")"));
	case 2: return FText::FromString(TEXT("))"));
	case 3: return FText::FromString(TEXT(")))"));
	case 4: return FText::FromString(TEXT("))))"));
	case 5: return FText::FromString(TEXT(")))))"));
	}
	return FText::GetEmpty();
}


EVisibility SStateTreeViewRow::GetLinkedStateVisibility() const
{
	if (const UStateTreeState* State = WeakState.Get())
	{
		return (State->Type == EStateTreeStateType::Linked || State->Type == EStateTreeStateType::LinkedAsset) ? EVisibility::Visible : EVisibility::Collapsed;
	}
	return EVisibility::Collapsed;
}

bool SStateTreeViewRow::GetStateWarnings(FText* OutText) const
{
	bool bHasWarnings = false;
	
	const UStateTreeState* State = WeakState.Get();
	if (!State)
	{
		return bHasWarnings;
	}

	// Linked States cannot have children.
	if ((State->Type == EStateTreeStateType::Linked
		|| State->Type == EStateTreeStateType::LinkedAsset)
		&& State->Children.Num() > 0)
	{
		if (OutText)
		{
			*OutText = LOCTEXT("LinkedStateChildWarning", "Linked State cannot have child states, because the state selection will enter to the linked state on activation.");
		}
		bHasWarnings = true;
	}

	// Child states should not have any considerations if their parent doesn't use utility
	if (State->Considerations.Num() != 0)
	{
		if (!State->Parent 
			|| (State->Parent->SelectionBehavior != EStateTreeStateSelectionBehavior::TrySelectChildrenWithHighestUtility
				&& State->Parent->SelectionBehavior != EStateTreeStateSelectionBehavior::TrySelectChildrenAtRandomWeightedByUtility))
		{
			if (OutText)
			{
				*OutText = LOCTEXT("ChildStateUtilityConsiderationWarning", 
					"State has Utility Considerations but they don't have effect."
					"The Utility Considerations are used only when parent State's Selection Behavior is:"
					"\"Try Select Children with Highest Utility\" or \"Try Select Children At Random Weighted By Utility.");
			}
			bHasWarnings = true;
		}
	}

	return bHasWarnings;
}

FText SStateTreeViewRow::GetLinkedStateDesc() const
{
	const UStateTreeState* State = WeakState.Get();
	if (!State)
	{
		return FText::GetEmpty();
	}

	if (State->Type == EStateTreeStateType::Linked)
	{
		return FText::FromName(State->LinkedSubtree.Name);
	}
	else if (State->Type == EStateTreeStateType::LinkedAsset)
	{
		return FText::FromString(GetNameSafe(State->LinkedAsset.Get()));
	}
	
	return FText::GetEmpty();
}

EVisibility SStateTreeViewRow::GetWarningsVisibility() const
{
	return GetStateWarnings(nullptr) ? EVisibility::Visible : EVisibility::Collapsed;
}

FText SStateTreeViewRow::GetWarningsTooltipText() const
{
	FText Warnings = FText::GetEmpty();
	GetStateWarnings(&Warnings);
	return Warnings;
}

bool SStateTreeViewRow::HasParentTransitionForTrigger(const UStateTreeState& State, const EStateTreeTransitionTrigger Trigger) const
{
	EStateTreeTransitionTrigger CombinedTrigger = EStateTreeTransitionTrigger::None;
	for (const UStateTreeState* ParentState = State.Parent; ParentState != nullptr; ParentState = ParentState->Parent)
	{
		for (const FStateTreeTransition& Transition : ParentState->Transitions)
		{
			CombinedTrigger |= Transition.Trigger;
		}
	}
	return EnumHasAllFlags(CombinedTrigger, Trigger);
}

FText SStateTreeViewRow::GetLinkTooltip(const FStateTreeStateLink& Link, const FGuid NodeID) const
{
	if (const UStateTreeState* State = WeakState.Get())
	{
		const int32 TaskIndex = State->Tasks.IndexOfByPredicate([&NodeID](const FStateTreeEditorNode& Node)
		{
			return Node.ID == NodeID;
		});
		if (TaskIndex != INDEX_NONE)
		{
			return FText::Format(LOCTEXT("TaskTransitionDesc", "Task {0} transitions to {1}"),
				FText::FromName(State->Tasks[TaskIndex].GetName()),
				UE::StateTree::Editor::GetStateLinkDesc(WeakEditorData.Get(), Link, EStateTreeNodeFormatting::Text, /*bShowStatePath*/true));
		}

		if (State->SingleTask.ID == NodeID)
		{
			return FText::Format(LOCTEXT("TaskTransitionDesc", "Task {0} transitions to {1}"),
				FText::FromName(State->SingleTask.GetName()),
				UE::StateTree::Editor::GetStateLinkDesc(WeakEditorData.Get(), Link, EStateTreeNodeFormatting::Text, /*bShowStatePath*/true));
		}

		const int32 TransitionIndex = State->Transitions.IndexOfByPredicate([&NodeID](const FStateTreeTransition& Transition)
		{
			return Transition.ID == NodeID;
		});
		if (TransitionIndex != INDEX_NONE)
		{
			return UE::StateTree::Editor::GetTransitionDesc(WeakEditorData.Get(), State->Transitions[TransitionIndex], EStateTreeNodeFormatting::Text, /*bShowStatePath*/true);
		}
	}
	
	return FText::GetEmpty();
};

bool SStateTreeViewRow::IsLeafState() const
{
	const UStateTreeState* State = WeakState.Get();
	return State
		&& State->Children.Num() == 0
		&& !IsRootState()
		&& (State->Type == EStateTreeStateType::State
			|| State->Type == EStateTreeStateType::Linked
			|| State->Type == EStateTreeStateType::LinkedAsset);
}

FText SStateTreeViewRow::GetTransitionsDesc(const EStateTreeTransitionTrigger Trigger, const FTransitionDescFilterOptions FilterOptions) const
{
	const UStateTreeState* State = WeakState.Get();
	const UStateTreeEditorData* EditorData = WeakEditorData.Get();
	if (!State || !EditorData)
	{
		return FText::GetEmpty();
	}

	TArray<FText> DescItems;

	for (const FStateTreeTransition& Transition : State->Transitions)
	{
		// Apply filter for enabled/disabled transitions
		if ((FilterOptions.Enabled == ETransitionDescRequirement::RequiredTrue && Transition.bTransitionEnabled == false)
			|| (FilterOptions.Enabled == ETransitionDescRequirement::RequiredFalse && Transition.bTransitionEnabled))
		{
			continue;
		}

#if WITH_STATETREE_TRACE_DEBUGGER
		// Apply filter for transitions with/without breakpoint
		const bool bHasBreakpoint = EditorData->HasBreakpoint(Transition.ID, EStateTreeBreakpointType::OnTransition);
		if ((FilterOptions.WithBreakpoint == ETransitionDescRequirement::RequiredTrue && bHasBreakpoint == false)
			|| (FilterOptions.WithBreakpoint == ETransitionDescRequirement::RequiredFalse && bHasBreakpoint))
		{
			continue;
		}
#endif // WITH_STATETREE_TRACE_DEBUGGER

		const bool bMatch = FilterOptions.bUseMask ? EnumHasAnyFlags(Transition.Trigger, Trigger) : Transition.Trigger == Trigger;
		if (bMatch)
		{
			DescItems.Add(UE::StateTree::Editor::GetStateLinkDesc(EditorData, Transition.State, EStateTreeNodeFormatting::RichText));
		}
	}

	// Find states from transition tasks
	if (EnumHasAnyFlags(Trigger, EStateTreeTransitionTrigger::OnTick | EStateTreeTransitionTrigger::OnEvent| EStateTreeTransitionTrigger::OnDelegate))
	{
		auto AddLinksFromStruct = [EditorData, &DescItems](FStateTreeDataView Struct)
		{
			if (!Struct.IsValid())
			{
				return;
			}
			for (TPropertyValueIterator<FStructProperty> It(Struct.GetStruct(), Struct.GetMemory()); It; ++It)
			{
				const UScriptStruct* StructType = It.Key()->Struct;
				if (StructType == TBaseStructure<FStateTreeStateLink>::Get())
				{
					const FStateTreeStateLink& Link = *static_cast<const FStateTreeStateLink*>(It.Value());
					if (Link.LinkType != EStateTreeTransitionType::None)
					{
						DescItems.Add(UE::StateTree::Editor::GetStateLinkDesc(EditorData, Link, EStateTreeNodeFormatting::RichText));
					}
				}
			}
		};
		
		for (const FStateTreeEditorNode& Task : State->Tasks)
		{
			AddLinksFromStruct(FStateTreeDataView(Task.GetNode().GetScriptStruct(), const_cast<uint8*>(Task.GetNode().GetMemory())));
			AddLinksFromStruct(Task.GetInstance());
		}

		AddLinksFromStruct(FStateTreeDataView(State->SingleTask.GetNode().GetScriptStruct(), const_cast<uint8*>(State->SingleTask.GetNode().GetMemory())));
		AddLinksFromStruct(State->SingleTask.GetInstance());
	}

	if (IsLeafState()
		&& DescItems.Num() == 0
		&& EnumHasAnyFlags(Trigger, EStateTreeTransitionTrigger::OnStateCompleted))
	{
		if (HasParentTransitionForTrigger(*State, Trigger))
		{
			DescItems.Add(LOCTEXT("TransitionActionHandleInParentRich", "<i>Parent</>"));
		}
		else
		{
			DescItems.Add(LOCTEXT("TransitionActionRootRich", "<i>Root</>"));
		}
	}
	
	return FText::Join(FText::FromString(TEXT(", ")), DescItems);
}

const FSlateBrush* SStateTreeViewRow::GetTransitionsIcon(const EStateTreeTransitionTrigger Trigger) const
{
	const UStateTreeState* State = WeakState.Get();
	if (!State)
	{
		return nullptr;
	}

	if (EnumHasAnyFlags(Trigger, EStateTreeTransitionTrigger::OnTick | EStateTreeTransitionTrigger::OnEvent| EStateTreeTransitionTrigger::OnDelegate))
	{
		return FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Transition.Goto");
	}

	enum EIconType
	{
		IconNone = 0,
		IconGoto = 1 << 0,
		IconNext = 1 << 1,
		IconParent = 1 << 2,
	};
	uint8 IconType = IconNone;
	
	for (const FStateTreeTransition& Transition : State->Transitions)
	{
		// Apply filter for enabled/disabled transitions
/*		if ((FilterOptions.Enabled == ETransitionDescRequirement::RequiredTrue && Transition.bTransitionEnabled == false)
			|| (FilterOptions.Enabled == ETransitionDescRequirement::RequiredFalse && Transition.bTransitionEnabled))
		{
			continue;
		}*/

/*		
#if WITH_STATETREE_TRACE_DEBUGGER
		// Apply filter for transitions with/without breakpoint
		const bool bHasBreakpoint = EditorData != nullptr && EditorData->HasBreakpoint(Transition.ID, EStateTreeBreakpointType::OnTransition);
		if ((FilterOptions.WithBreakpoint == ETransitionDescRequirement::RequiredTrue && bHasBreakpoint == false)
			|| (FilterOptions.WithBreakpoint == ETransitionDescRequirement::RequiredFalse && bHasBreakpoint))
		{
			continue;
		}
#endif // WITH_STATETREE_TRACE_DEBUGGER
*/
		
		// The icons here depict "transition direction", not the type specifically.
		const bool bMatch = /*FilterOptions.bUseMask ? EnumHasAnyFlags(Transition.Trigger, Trigger) :*/ Transition.Trigger == Trigger;
		if (bMatch)
		{
			switch (Transition.State.LinkType)
			{
			case EStateTreeTransitionType::None:
				IconType |= IconGoto;
				break;
			case EStateTreeTransitionType::Succeeded:
				IconType |= IconGoto;
				break;
			case EStateTreeTransitionType::Failed:
				IconType |= IconGoto;
				break;
			case EStateTreeTransitionType::NextState:
			case EStateTreeTransitionType::NextSelectableState:
				IconType |= IconNext;
				break;
			case EStateTreeTransitionType::Parent:
			case EStateTreeTransitionType::NextParent:
			case EStateTreeTransitionType::NextSelectableParent:
				IconType |= IconParent;
				break;
			case EStateTreeTransitionType::GotoState:
				IconType |= IconGoto;
				break;
			default:
				ensureMsgf(false, TEXT("Unhandled transition type."));
				break;
			}
		}
	}

	if (FMath::CountBits64(static_cast<uint64>(IconType)) > 1)
	{
		// Prune down to just one icon.
		IconType = IconGoto;
	}

	if (IsLeafState()
		&& IconType == IconNone
		&& EnumHasAnyFlags(Trigger, EStateTreeTransitionTrigger::OnStateCompleted))
	{
		// Transition is handled on parent state, or implicit Root.
		IconType = IconParent;
	}

	switch (IconType)
	{
		case IconGoto:
			return FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Transition.Goto");
		case IconNext:
			return FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Transition.Next");
		case IconParent:
			return FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Transition.Parent");
		default:
			break;
	}
	
	return nullptr;
}

EVisibility SStateTreeViewRow::GetTransitionsVisibility(const EStateTreeTransitionTrigger Trigger) const
{
	const UStateTreeState* State = WeakState.Get();
	if (!State)
	{
		return EVisibility::Collapsed;
	}
	
	// Handle completed, succeeded and failed transitions.
	if (EnumHasAnyFlags(Trigger, EStateTreeTransitionTrigger::OnStateCompleted))
	{
		EStateTreeTransitionTrigger HandledTriggers = EStateTreeTransitionTrigger::None;
		bool bExactMatch = false;

		for (const FStateTreeTransition& Transition : State->Transitions)
		{
			// Skip disabled transitions
			if (Transition.bTransitionEnabled == false)
			{
				continue;
			}

			HandledTriggers |= Transition.Trigger;
			bExactMatch |= (Transition.Trigger == Trigger);

			if (bExactMatch)
			{
				break;
			}
		}

		// Assume that leaf states should have completion transitions.
		if (!bExactMatch && IsLeafState())
		{
			// Find the missing transition type, note: Completed = Succeeded|Failed.
			const EStateTreeTransitionTrigger MissingTriggers = HandledTriggers ^ EStateTreeTransitionTrigger::OnStateCompleted;
			return MissingTriggers == Trigger ? EVisibility::Visible : EVisibility::Collapsed;
		}
		
		return bExactMatch ? EVisibility::Visible : EVisibility::Collapsed;
	}

	// Find states from transition tasks
	if (EnumHasAnyFlags(Trigger, EStateTreeTransitionTrigger::OnTick | EStateTreeTransitionTrigger::OnEvent| EStateTreeTransitionTrigger::OnDelegate))
	{
		auto HasAnyLinksInStruct = [](FStateTreeDataView Struct) -> bool
		{
			if (!Struct.IsValid())
			{
				return false;
			}
			for (TPropertyValueIterator<FStructProperty> It(Struct.GetStruct(), Struct.GetMemory()); It; ++It)
			{
				const UScriptStruct* StructType = It.Key()->Struct;
				if (StructType == TBaseStructure<FStateTreeStateLink>::Get())
				{
					const FStateTreeStateLink& Link = *static_cast<const FStateTreeStateLink*>(It.Value());
					if (Link.LinkType != EStateTreeTransitionType::None)
					{
						return true;
					}
				}
			}
			return false;
		};
		
		for (const FStateTreeEditorNode& Task : State->Tasks)
		{
			if (HasAnyLinksInStruct(Task.GetNode())
				|| HasAnyLinksInStruct(Task.GetInstance())
				|| HasAnyLinksInStruct(Task.GetExecutionRuntimeData()))
			{
				return EVisibility::Visible;
			}
		}

		if (HasAnyLinksInStruct(State->SingleTask.GetNode())
			|| HasAnyLinksInStruct(State->SingleTask.GetInstance())
			|| HasAnyLinksInStruct(State->SingleTask.GetExecutionRuntimeData()))
		{
			return EVisibility::Visible;
		}
	}
	
	// Handle the test
	for (const FStateTreeTransition& Transition : State->Transitions)
	{
		// Skip disabled transitions
		if (Transition.bTransitionEnabled == false)
		{
			continue;
		}

		if (EnumHasAnyFlags(Trigger, Transition.Trigger))
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

EVisibility SStateTreeViewRow::GetTransitionsBreakpointVisibility(const EStateTreeTransitionTrigger Trigger) const
{
	if (const UStateTreeState* State = WeakState.Get())
	{
#if WITH_STATETREE_TRACE_DEBUGGER
		if (const UStateTreeEditorData* EditorData = WeakEditorData.Get())
		{
			for (const FStateTreeTransition& Transition : State->Transitions)
			{
				if (Transition.bTransitionEnabled && EnumHasAnyFlags(Trigger, Transition.Trigger))
				{
					if (EditorData->HasBreakpoint(Transition.ID, EStateTreeBreakpointType::OnTransition))
					{
						return GetTransitionsVisibility(Trigger);
					}
				}
			}
		}
#endif // WITH_STATETREE_TRACE_DEBUGGER
	}
	
	return EVisibility::Collapsed;
}

EVisibility SStateTreeViewRow::GetStateDescriptionVisibility() const
{
	if (const UStateTreeState* State = WeakState.Get())
	{
		return State->Description.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
	}
	return EVisibility::Collapsed;
}

FText SStateTreeViewRow::GetStateDescription() const
{
	if (const UStateTreeState* State = WeakState.Get())
	{
		return FText::FromString(State->Description);
	}
	return FText::GetEmpty();
}

EVisibility SStateTreeViewRow::GetTransitionDashVisibility() const
{
	if (const UStateTreeState* State = WeakState.Get())
	{
		return State->Transitions.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
	}
	return EVisibility::Collapsed;	
}

bool SStateTreeViewRow::IsRootState() const
{
	// Routines can be identified by not having parent state.
	const UStateTreeState* State = WeakState.Get();
	return State ? State->Parent == nullptr : false;
}

bool SStateTreeViewRow::IsStateSelected() const
{
	if (const UStateTreeState* State = WeakState.Get())
	{
		if (StateTreeViewModel)
		{
			return StateTreeViewModel->IsSelected(State);
		}
	}
	return false;
}

bool SStateTreeViewRow::HandleVerifyNodeLabelTextChanged(const FText& InText, FText& OutErrorMessage) const
{
	if (StateTreeViewModel)
	{
		if (const UStateTreeState* State = WeakState.Get())
		{
			const FString NewName = FText::TrimPrecedingAndTrailing(InText).ToString();
			if (NewName.Len() >= NAME_SIZE)
			{
				OutErrorMessage = LOCTEXT("VerifyNodeLabelFailed_MaxLength", "Max length exceeded");
				return false;
			}
			return NewName.Len() > 0;
		}
	}
	OutErrorMessage = LOCTEXT("VerifyNodeLabelFailed", "Invalid State Tree");
	return false;
}

void SStateTreeViewRow::HandleNodeLabelTextCommitted(const FText& NewLabel, ETextCommit::Type CommitType) const
{
	if (StateTreeViewModel)
	{
		if (UStateTreeState* State = WeakState.Get())
		{
			const FString NewName = FText::TrimPrecedingAndTrailing(NewLabel).ToString();
			if (NewName.Len() > 0 && NewName.Len() < NAME_SIZE)
			{
				StateTreeViewModel->RenameState(State, FName(NewName));
			}
		}
	}
}

FReply SStateTreeViewRow::HandleDragDetected(const FGeometry&, const FPointerEvent&) const
{
	return FReply::Handled().BeginDragDrop(FStateTreeSelectedDragDrop::New(StateTreeViewModel));
}

void SStateTreeViewRow::HandleDragLeave(const FDragDropEvent& DragDropEvent) const
{
	const TSharedPtr<FStateTreeSelectedDragDrop> DragDropOperation = DragDropEvent.GetOperationAs<FStateTreeSelectedDragDrop>();
	if (DragDropOperation.IsValid())
	{
		DragDropOperation->SetCanDrop(false);
	}	
}

TOptional<EItemDropZone> SStateTreeViewRow::HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TWeakObjectPtr<UStateTreeState> TargetState) const
{
	const TSharedPtr<FStateTreeSelectedDragDrop> DragDropOperation = DragDropEvent.GetOperationAs<FStateTreeSelectedDragDrop>();
	if (DragDropOperation.IsValid())
	{
		DragDropOperation->SetCanDrop(true);
		
		if (DragDropOperation->IsDraggingNode())
		{
			const bool bIsSameState = DragDropOperation->OriginState == TargetState;
			DragDropOperation->SetCanDrop(!bIsSameState);
			if (bIsSameState || DropZone != EItemDropZone::OntoItem)
			{
				return TOptional<EItemDropZone>();
			}
			
			return DropZone;
		}

		// Cannot drop on selection or child of selection.
		if (StateTreeViewModel && StateTreeViewModel->IsChildOfSelection(TargetState.Get()))
		{
			DragDropOperation->SetCanDrop(false);
			return TOptional<EItemDropZone>();
		}

		return DropZone;
	}

	return TOptional<EItemDropZone>();
}

FReply SStateTreeViewRow::HandleAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TWeakObjectPtr<UStateTreeState> TargetState) const
{
	const TSharedPtr<FStateTreeSelectedDragDrop> DragDropOperation = DragDropEvent.GetOperationAs<FStateTreeSelectedDragDrop>();
	if (DragDropOperation.IsValid())
	{
		if (StateTreeViewModel)
		{
			if (DragDropOperation->IsDraggingNode())
			{
				if (DragDropOperation->bIsControlKeyDown)
				{
					StateTreeViewModel->CopyNode(DragDropOperation->OriginState.Get(), DragDropOperation->NodeID);
					StateTreeViewModel->PasteNodesToStates(MakeConstArrayView({TargetState.Get()}));
				}
				else
				{
					StateTreeViewModel->MoveNode(DragDropOperation->OriginState, TargetState, DragDropOperation->NodeID);
				}
			}
			else
			{
				if (DropZone == EItemDropZone::AboveItem)
				{
					StateTreeViewModel->MoveSelectedStatesBefore(TargetState.Get());
				}
				else if (DropZone == EItemDropZone::BelowItem)
				{
					StateTreeViewModel->MoveSelectedStatesAfter(TargetState.Get());
				}
				else
				{
					StateTreeViewModel->MoveSelectedStatesInto(TargetState.Get());
				}
			}

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

void SStateTreeViewRow::HandleAssetChanged()
{
	MakeFlagsWidget();
	MakeTransitionsWidget();
}

void SStateTreeViewRow::HandleStatesChanged(const TSet<UStateTreeState*>& ChangedStates, const FPropertyChangedEvent& PropertyChangedEvent)
{
	if (const UStateTreeState* OwnerState = WeakState.Get())
	{
		if (ChangedStates.Contains(OwnerState))
		{
			if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UStateTreeState, Transitions)
				|| PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FStateTreeStateLink, LinkType))
			{
				MakeTransitionsWidget();
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
