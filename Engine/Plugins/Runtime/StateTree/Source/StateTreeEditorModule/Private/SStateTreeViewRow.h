// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"

class STableViewBase;
class UStateTreeState;
enum class EStateTreeTransitionTrigger : uint8;
enum class EStateTreeNodeFormatting : uint8;
struct FStateTreeStateLink;
struct FStateTreeEditorNode;

class UStateTreeEditorData;
class SStateTreeView;
class SBorder;
class SInlineEditableTextBlock;
class SScrollBox;
class FStateTreeViewModel;

class SStateTreeViewRow : public STableRow<TWeakObjectPtr<UStateTreeState>>
{
public:

	SLATE_BEGIN_ARGS(SStateTreeViewRow)
	{
	}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TWeakObjectPtr<UStateTreeState> InState, const TSharedPtr<SScrollBox>& ViewBox, TSharedPtr<FStateTreeViewModel> InStateTreeViewModel);
	~SStateTreeViewRow();

	void RequestRename() const;

private:
	enum class ETransitionDescRequirement : uint8
	{
		Any,
		RequiredTrue,
		RequiredFalse,
	};

	/**
	 * Filtering options used to build the description of the transitions.
	 * The default setup includes only enabled transition,
	 * with or without breakpoints and requires exact trigger match (no partial mask)
	 */
	struct FTransitionDescFilterOptions
	{
		FTransitionDescFilterOptions() {};
		ETransitionDescRequirement Enabled = ETransitionDescRequirement::RequiredTrue;
		ETransitionDescRequirement WithBreakpoint = ETransitionDescRequirement::Any;
		bool bUseMask = false;
	};

	TSharedRef<SWidget> MakeTasksWidget(const TSharedPtr<SScrollBox>& ViewBox);
	TSharedRef<SWidget> MakeConditionsWidget(const TSharedPtr<SScrollBox>& ViewBox);
	void MakeTransitionsWidget();
	TSharedRef<SWidget> MakeTransitionWidget(const EStateTreeTransitionTrigger Trigger, const FSlateBrush* Icon);
	TSharedRef<SWidget> MakeTransitionWidgetInternal(const EStateTreeTransitionTrigger Trigger, const FTransitionDescFilterOptions FilterOptions = {});
	void MakeFlagsWidget();

	FSlateColor GetTitleColor(const float Alpha = 1.0f, const float Lighten = 0.0f) const;
	FSlateColor GetActiveStateColor() const;
	FText GetStateDesc() const;
	FText GetStateIDDesc() const;

	FSlateColor GetSubTreeMarkerColor() const;
	EVisibility GetSubTreeVisibility() const;
	
	EVisibility GetConditionVisibility() const;
	EVisibility GetStateBreakpointVisibility() const;
	FText GetStateBreakpointTooltipText() const;
	
	const FSlateBrush* GetSelectorIcon() const;
	FText GetSelectorTooltip() const;
	FText GetStateTypeTooltip() const;

	const FStateTreeEditorNode* GetTaskNodeByID(FGuid TaskID) const;
	EVisibility GetTaskIconVisibility(FGuid TaskID) const;
	const FSlateBrush* GetTaskIcon(FGuid TaskID) const;
	FSlateColor GetTaskIconColor(FGuid TaskID) const;
	FText GetTaskDesc(FGuid TaskID, EStateTreeNodeFormatting Formatting) const;

	const FStateTreeEditorNode* GetConditionNodeByID(FGuid ConditionID) const;
	EVisibility GetConditionIconVisibility(FGuid ConditionID) const;
	const FSlateBrush* GetConditionIcon(FGuid ConditionID) const;
	FSlateColor GetConditionIconColor(FGuid ConditionID) const;
	FText GetConditionDesc(FGuid ConditionID, EStateTreeNodeFormatting Formatting) const;

	FText GetOperandText(const int32 ConditionIndex) const;
	FText GetOpenParens(const int32 ConditionIndex) const;
	FText GetCloseParens(const int32 ConditionIndex) const;
	
	EVisibility GetLinkedStateVisibility() const;
	FText GetLinkedStateDesc() const;

	bool GetStateWarnings(FText* OutText) const;
	EVisibility GetWarningsVisibility() const;
	FText GetWarningsTooltipText() const;

	EVisibility GetStateDescriptionVisibility() const;
	FText GetStateDescription() const;

	EVisibility GetTransitionDashVisibility() const;

	FText GetLinkTooltip(const FStateTreeStateLink& Link, const FGuid NodeID) const;
	FText GetTransitionsDesc(const EStateTreeTransitionTrigger Trigger, const FTransitionDescFilterOptions FilterOptions = {}) const;
	const FSlateBrush* GetTransitionsIcon(const EStateTreeTransitionTrigger Trigger) const;
	EVisibility GetTransitionsVisibility(const EStateTreeTransitionTrigger Trigger) const;
	EVisibility GetTransitionsBreakpointVisibility(const EStateTreeTransitionTrigger Trigger) const;

	bool HasParentTransitionForTrigger(const UStateTreeState& State, const EStateTreeTransitionTrigger Trigger) const;

	bool IsRootState() const;
	bool IsLeafState() const;
	bool IsStateSelected() const;

	bool HandleVerifyNodeLabelTextChanged(const FText& InText, FText& OutErrorMessage) const;
	void HandleNodeLabelTextCommitted(const FText& NewLabel, ETextCommit::Type CommitType) const;

	FReply HandleDragDetected(const FGeometry&, const FPointerEvent&) const;
	void HandleDragLeave(const FDragDropEvent& DragDropEvent) const;
	TOptional<EItemDropZone> HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TWeakObjectPtr<UStateTreeState> TargetState) const;
	FReply HandleAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TWeakObjectPtr<UStateTreeState> TargetState) const;

	void HandleAssetChanged();
	void HandleStatesChanged(const TSet<UStateTreeState*>& ChangedStates, const FPropertyChangedEvent& PropertyChangedEvent);
	TSharedPtr<FStateTreeViewModel> StateTreeViewModel;
	TWeakObjectPtr<UStateTreeState> WeakState;
	TWeakObjectPtr<UStateTreeEditorData> WeakEditorData;
	TSharedPtr<SInlineEditableTextBlock> NameTextBlock;
	TSharedPtr<SBorder> FlagsContainer;
	TSharedPtr<SHorizontalBox> TransitionsContainer;

	FDelegateHandle AssetChangedHandle;
	FDelegateHandle StatesChangedHandle;
};
