// Copyright Epic Games, Inc. All Rights Reserved.

#include "StandaloneStateTreeEditorHost.h"
#include "StateTreeEditor.h"
#include "StateTreeEditorWorkspaceTabHost.h"

void FStandaloneStateTreeEditorHost::Init(const TWeakPtr<FStateTreeEditor>& InWeakStateTreeEditor)
{
	WeakStateTreeEditor = InWeakStateTreeEditor;
	TabHost = MakeShared<UE::StateTreeEditor::FWorkspaceTabHost>();
}

UStateTree* FStandaloneStateTreeEditorHost::GetStateTree() const
{
	if (TSharedPtr<FStateTreeEditor> SharedEditor = WeakStateTreeEditor.Pin())
	{
		return SharedEditor->StateTree;
	}
	return nullptr;
}

FName FStandaloneStateTreeEditorHost::GetCompilerLogName() const
{
	return FStateTreeEditor::CompilerLogListingName;
}

FName FStandaloneStateTreeEditorHost::GetCompilerTabName() const
{
	return FStateTreeEditor::CompilerResultsTabId;
}

bool FStandaloneStateTreeEditorHost::ShouldShowCompileButton() const
{
	return true;
}

bool FStandaloneStateTreeEditorHost::CanToolkitSpawnWorkspaceTab() const
{
	return false;
}

FSimpleMulticastDelegate& FStandaloneStateTreeEditorHost::OnStateTreeChanged()
{
	return OnStateTreeChangedDelegate;
}

TSharedPtr<IDetailsView> FStandaloneStateTreeEditorHost::GetAssetDetailsView()
{
	if (TSharedPtr<FStateTreeEditor> SharedEditor = WeakStateTreeEditor.Pin())
	{
		return SharedEditor->AssetDetailsView;
	}
	
	return nullptr;	
}

TSharedPtr<IDetailsView> FStandaloneStateTreeEditorHost::GetDetailsView()
{
	if (TSharedPtr<FStateTreeEditor> SharedEditor = WeakStateTreeEditor.Pin())
	{
		return SharedEditor->SelectionDetailsView;
	}
	
	return nullptr;	
}

TSharedPtr<UE::StateTreeEditor::FWorkspaceTabHost> FStandaloneStateTreeEditorHost::GetTabHost() const
{
	return TabHost;
}