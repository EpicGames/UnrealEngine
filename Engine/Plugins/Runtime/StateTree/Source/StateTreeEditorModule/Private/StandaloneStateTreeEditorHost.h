// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IStateTreeEditorHost.h"

class FStateTreeEditor;

class FStandaloneStateTreeEditorHost : public IStateTreeEditorHost
{	
public:
	void Init(const TWeakPtr<FStateTreeEditor>& InWeakStateTreeEditor);
	
	//~ IStateTreeEditorHost overrides
	virtual UStateTree* GetStateTree() const override;
	virtual FSimpleMulticastDelegate& OnStateTreeChanged() override;
	virtual TSharedPtr<IDetailsView> GetAssetDetailsView() override;
	virtual TSharedPtr<IDetailsView> GetDetailsView() override;
	virtual TSharedPtr<UE::StateTreeEditor::FWorkspaceTabHost> GetTabHost() const override;
	virtual bool ShouldShowCompileButton() const override;
	virtual bool CanToolkitSpawnWorkspaceTab() const override;
	virtual FName GetCompilerLogName() const override;
	virtual FName GetCompilerTabName() const override;

protected:
	TWeakPtr<FStateTreeEditor> WeakStateTreeEditor;
	FSimpleMulticastDelegate OnStateTreeChangedDelegate;
	TSharedPtr<UE::StateTreeEditor::FWorkspaceTabHost> TabHost;
};