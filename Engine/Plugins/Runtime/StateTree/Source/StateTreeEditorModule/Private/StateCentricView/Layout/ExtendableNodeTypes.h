// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Types/SlateStructs.h"
#include "ExtendableNodeTypes.generated.h"

/**
 * Experimental. LOD / displayed info that an extendable node may support.
 */
UENUM(Experimental)
enum class EExtendableNodeLOD : uint8
{
	Hidden,					// View provides no context, if appropriate should not even give a placeholder
	Collapsed,				// View provides no context, but visible as placeholder for when LOD changes
	Minimal,				// View provides minimal context deemed needed
	Medium,					// View is filtered, giving some but not all detail
	Full,					// View would not reasonably show more info
	Unused UMETA(Hidden),	// Indicate node itself does not change based on LOD. Node may still accept LOD to pass onto children.
	Default UMETA(Hidden),	// Indicate no preference at API level, exact behavior implementation specific
};
