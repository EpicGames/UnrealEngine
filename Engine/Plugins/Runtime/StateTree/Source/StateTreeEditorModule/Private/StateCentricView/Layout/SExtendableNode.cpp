// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateCentricView/Layout/SExtendableNode.h"

#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"

#define LOCTEXT_NAMESPACE "SExtendableNode"

namespace UE::StateTree::Editor::StateCentricView
{


namespace Private
{

	/** Dummy widget to make widget reflector path clearer */
	class SHorizontalExtensionBox : public SStackBox
	{
	};

	/** Dummy widget to make widget reflector path clearer */
	class SVerticalExtensionBox : public SStackBox
	{
	};

	/** Dummy widget to make widget reflector path clearer */
	class SHorizontalExtensionSplitter : public SSplitter
	{
	};

	/** Dummy widget to make widget reflector path clearer */
	class SVerticalExtensionSplitter : public SSplitter
	{
	};

	/** Dummy widget to make widget reflector path clearer */
	class SMainNodeStackBox : public SStackBox
	{
	};

	/** 
	 * Support ability to offset layer ID. Needed to handle overlapping UI that is laid out flat correctly
	 * IE: Two slots next to each other in a box, but they need to have a defined overlap order
	 */
	class SOverlayExtensionBox : public SOverlay
	{
	public:
		virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
		{
			int32 OffsetLayerId = LayerId + Offset;
			return SOverlay::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, OffsetLayerId, InWidgetStyle, bParentEnabled);
		}

	public:

		int32 Offset = 0;
	};

} // namespace Private


void SExtendableNode::Construct(const FArguments& InArgs)
{
	Config = InArgs._Config;

	if (Config.InitialLOD == EExtendableNodeLOD::Default)
	{
		// Store whatever default was in constructor
		Config.InitialLOD = CurrentLOD;
	}
	else
	{
		CurrentLOD = Config.InitialLOD;
	}

	Construct_GatherExtensionsFromExternalSubsystems();
	Construct_InitializeNodePersistentPreLOD();
	RegenerateWidgetForLOD(CurrentLOD);
}

void SExtendableNode::AddExtension(TSharedRef<SWidget> ExtensionWidget, const FExtensionAddNodeParams& ExtensionParams)
{
	const FExtensionAddLayoutInfo& LayoutInfo = ExtensionParams.LayoutInfo;

	enum class EMainNodeExtensionDirection
	{
		After,	// Bottom, Right, & Foreground, add after main node in the layout
		Before,	// Top, Left, & Background, add before main node in the layout
	};

	// Adding in a cardinal direction is only unique w.r.t. orientation & direction relative to main node
	auto AddCardinalExtension = [&](const EOrientation InOrientation, const EMainNodeExtensionDirection InAddDirection)
	{
		// We know we want to insert, but not where yet. Forward declare so we can keep insert logic consolidated.
		TSharedPtr<SStackBox> LayoutToInsert = nullptr;
		int8* MainNodeInsertIndex = nullptr;

		auto WrapCurrentLayout = [&]()
		{
			if (InOrientation == Orient_Horizontal)
			{
				SAssignNew(CurrentLayoutBox, StateCentricView::Private::SHorizontalExtensionBox)
					.Orientation(InOrientation);
			}
			else
			{
				SAssignNew(CurrentLayoutBox, StateCentricView::Private::SVerticalExtensionBox)
					.Orientation(InOrientation);
			}

			if (!InnerLayoutBox)
			{
				TSharedRef<SWidget> MainNodeWidget = MainNodeOverlay 
					? StaticCastSharedRef<SWidget>(MainNodeOverlay.ToSharedRef())
					: StaticCastSharedRef<SWidget>(MainNodeBox.ToSharedRef());

				// @TODO: Remove. We don't support cardinal layouts after adding a splitter.
				// Plan is to add splitter support as a feature toggle to base layout. See: 48250292.
				if (CurrentSplitterBox)
				{
					MainNodeWidget = CurrentSplitterBox.ToSharedRef();
				}

				CurrentLayoutBox->AddSlot()
				.Expose(MainNodeLayoutSlot)
				.SizeParam(MainNodeLayoutInfo.SizeParam ? MainNodeLayoutInfo.SizeParam : LayoutInfo.SizeParam)
				.MinSize(MainNodeLayoutInfo.MinSize.IsSet() ? MainNodeLayoutInfo.MinSize : LayoutInfo.MinSize)
				.MaxSize(MainNodeLayoutInfo.MaxSize.IsSet() ? MainNodeLayoutInfo.MaxSize : LayoutInfo.MaxSize)
				.Padding(MainNodeLayoutInfo.Padding.IsSet() ? MainNodeLayoutInfo.Padding : LayoutInfo.Padding)
				.HAlign(MainNodeLayoutInfo.HAlign.Get({ LayoutInfo.HAlign.Get(HAlign_Fill) }))
				.VAlign(MainNodeLayoutInfo.VAlign.Get({ LayoutInfo.VAlign.Get(VAlign_Fill) }))
				[
					MainNodeWidget
				];
			}
			else
			{
				CurrentLayoutBox->AddSlot()
				.SizeParam(MainNodeLayoutInfo.SizeParam ? MainNodeLayoutInfo.SizeParam : LayoutInfo.SizeParam)
				.MinSize(MainNodeLayoutInfo.MinSize.IsSet() ? MainNodeLayoutInfo.MinSize : LayoutInfo.MinSize)
				.MaxSize(MainNodeLayoutInfo.MaxSize.IsSet() ? MainNodeLayoutInfo.MaxSize : LayoutInfo.MaxSize)
				.Padding(MainNodeLayoutInfo.Padding.IsSet() ? MainNodeLayoutInfo.Padding : LayoutInfo.Padding)
				.HAlign(MainNodeLayoutInfo.HAlign.Get({ LayoutInfo.HAlign.Get(HAlign_Fill) }))
				.VAlign(MainNodeLayoutInfo.VAlign.Get({ LayoutInfo.VAlign.Get(VAlign_Fill) }))
				[
					InnerLayoutBox.ToSharedRef()
				];
			}
			MainNodeCurrentLayoutIndex = 0;

			// Replace the actual widget shown by this composite widget.
			if (EntireNodeOverlaySlot)
			{
				(*EntireNodeOverlaySlot)
				[
					CurrentLayoutBox.ToSharedRef()
				];
			}
			else
			{
				ChildSlot
				[
					CurrentLayoutBox.ToSharedRef()
				];
			}

			// Technically not a part of wrapping, but if we just created a layout we'll always insert to it. So assign here.
			LayoutToInsert = CurrentLayoutBox;
			MainNodeInsertIndex = &MainNodeCurrentLayoutIndex;
		};

		if (!CurrentLayoutBox)
		{
			WrapCurrentLayout();
		}
		else if (CurrentLayoutBox->GetOrientation() != InOrientation)
		{
			if (ExtensionParams.AddLocationBehavior == EExtensionAddLocationBehavior::Default
				|| ExtensionParams.AddLocationBehavior == EExtensionAddLocationBehavior::Wrap)
			{
				InnerLayoutBox = CurrentLayoutBox;
				MainNodeInnerLayoutIndex = MainNodeCurrentLayoutIndex;

				WrapCurrentLayout();
			}
			else if (ensure(ExtensionParams.AddLocationBehavior == EExtensionAddLocationBehavior::UseInnerLayout))
			{
				if (!InnerLayoutBox)
				{
					if (InOrientation == Orient_Horizontal)
					{
						SAssignNew(InnerLayoutBox, StateCentricView::Private::SHorizontalExtensionBox)
							.Orientation(InOrientation);
					}
					else
					{
						SAssignNew(InnerLayoutBox, StateCentricView::Private::SVerticalExtensionBox)
							.Orientation(InOrientation);
					}

					TSharedRef<SWidget> MainNodeWidget = MainNodeOverlay 
						? StaticCastSharedRef<SWidget>(MainNodeOverlay.ToSharedRef())
						: StaticCastSharedRef<SWidget>(MainNodeBox.ToSharedRef());
					
					(*MainNodeLayoutSlot)
					[
						InnerLayoutBox.ToSharedRef()
					];

					InnerLayoutBox->AddSlot()
					.Expose(MainNodeLayoutSlot)
					.SizeParam(MainNodeLayoutInfo.SizeParam ? MainNodeLayoutInfo.SizeParam : LayoutInfo.SizeParam)
					.MinSize(MainNodeLayoutInfo.MinSize.IsSet() ? MainNodeLayoutInfo.MinSize : LayoutInfo.MinSize)
					.MaxSize(MainNodeLayoutInfo.MaxSize.IsSet() ? MainNodeLayoutInfo.MaxSize : LayoutInfo.MaxSize)
					.Padding(MainNodeLayoutInfo.Padding.IsSet() ? MainNodeLayoutInfo.Padding : LayoutInfo.Padding)
					.HAlign(MainNodeLayoutInfo.HAlign.Get({ LayoutInfo.HAlign.Get(HAlign_Fill) }))
					.VAlign(MainNodeLayoutInfo.VAlign.Get({ LayoutInfo.VAlign.Get(VAlign_Fill) }))
					[
						MainNodeWidget
					];
				}

				LayoutToInsert = InnerLayoutBox;
				MainNodeInsertIndex = &MainNodeInnerLayoutIndex;
			}
		}
		else
		{
			LayoutToInsert = CurrentLayoutBox;
			MainNodeInsertIndex = &MainNodeCurrentLayoutIndex;
		}

		int8 InsertIndex = INDEX_NONE;
		if (InAddDirection == EMainNodeExtensionDirection::After)
		{
			InsertIndex = ExtensionParams.AddIndex == INDEX_NONE
				? IntCastChecked<int8>(LayoutToInsert->NumSlots())
				: *MainNodeInsertIndex + ExtensionParams.AddIndex + 1;
		}
		else if (ensure(InAddDirection == EMainNodeExtensionDirection::Before))
		{
			InsertIndex = ExtensionParams.AddIndex == INDEX_NONE
				? 0
				: *MainNodeInsertIndex - ExtensionParams.AddIndex;

			(*MainNodeInsertIndex)++;
		}

		LayoutToInsert->InsertSlot(InsertIndex)
		.SizeParam(LayoutInfo.SizeParam)
		.MinSize(LayoutInfo.MinSize)
		.MaxSize(LayoutInfo.MaxSize)
		.Padding(LayoutInfo.Padding)
		.HAlign(LayoutInfo.HAlign.Get(HAlign_Fill))
		.VAlign(LayoutInfo.VAlign.Get(VAlign_Fill))
		[
			ExtensionWidget
		];
	};

	// Same in overlays are only unique per direction relative to main node
	auto AddOverlayExtension = [&](const EMainNodeExtensionDirection InAddDirection)
	{
		// This is possible to support but not important at the moment
		checkf(ExtensionParams.AddIndex == INDEX_NONE, TEXT("Unimplemented. SExtendableNode does not currently support specified overlay indexes: %s"), *GetNodeName().ToString());

		// We know we want to insert, but not where yet. Forward declare so we can keep insert logic consolidated.
		TSharedPtr<SOverlay> OverlayToInsert = nullptr;
		int8* OverlayToInsertLowestZOrder = nullptr;

		if (ExtensionParams.AddLocationBehavior == EExtensionAddLocationBehavior::Default
			|| ExtensionParams.AddLocationBehavior == EExtensionAddLocationBehavior::OverlayMainNode)
		{
			if (!MainNodeOverlay)
			{
				TSharedPtr<StateCentricView::Private::SOverlayExtensionBox> OverlayExtensionBox = SNew(StateCentricView::Private::SOverlayExtensionBox);
				OverlayExtensionBox->Offset = MainNodeLayoutInfo.OverlayOffset;
				MainNodeOverlay = OverlayExtensionBox;
				
				MainNodeOverlay->AddSlot()
				.Padding(MainNodeLayoutInfo.Padding)
				.HAlign(MainNodeLayoutInfo.HAlign.Get(HAlign_Fill))
				.VAlign(MainNodeLayoutInfo.VAlign.Get(VAlign_Fill))
				[
					MainNodeBox.ToSharedRef()
				];

				// Update correct slot. Starting with main node which 
				// will be inside any overlays / layout widgets

				// @TODO: Remove. Splitter aspect should be a feature toggle on layout. Not a dedicated incompitable layout. See: 48250292.
				if (MainNodeSplitterSlot)
				{
					(*MainNodeSplitterSlot)
					[
						MainNodeOverlay.ToSharedRef()
					];
				}

				else if (MainNodeLayoutSlot)
				{
					(*MainNodeLayoutSlot)
					[
						MainNodeOverlay.ToSharedRef()
					];
				}
				else if (EntireNodeOverlaySlot)
				{
					(*EntireNodeOverlaySlot)
					[
						MainNodeOverlay.ToSharedRef()
					];
				}
				else
				{
					ChildSlot
					[
						MainNodeOverlay.ToSharedRef()
					];
				}

				// Default to -1 so when we decrement it's -2, which overlay supports for inserting behind.
				MainNodeLowestZOrder = INDEX_NONE;
			}

			OverlayToInsert = MainNodeOverlay;
			OverlayToInsertLowestZOrder = &MainNodeLowestZOrder;
		}
		else if (ensure(ExtensionParams.AddLocationBehavior == EExtensionAddLocationBehavior::OverlayEntireNode))
		{
			// We don't support entire node overlays after adding a splitter.
			// Plan is to add splitter support as a feature toggle to base layout. See: 48250292.
			if (!ensure(!CurrentSplitterBox))
			{
				return;
			}

			if (!EntireNodeOverlay)
			{
				TSharedPtr<StateCentricView::Private::SOverlayExtensionBox> OverlayExtensionBox = SNew(StateCentricView::Private::SOverlayExtensionBox);
				OverlayExtensionBox->Offset = MainNodeLayoutInfo.OverlayOffset;
				EntireNodeOverlay = OverlayExtensionBox;

				TSharedRef<SWidget> MainNodeWidget = MainNodeOverlay
					? StaticCastSharedRef<SWidget>(MainNodeOverlay.ToSharedRef())
					: StaticCastSharedRef<SWidget>(MainNodeBox.ToSharedRef());

				EntireNodeOverlay->AddSlot()
				.Padding(MainNodeLayoutInfo.Padding)
				.HAlign(MainNodeLayoutInfo.HAlign.Get(HAlign_Fill))
				.VAlign(MainNodeLayoutInfo.VAlign.Get(VAlign_Fill))
				.Expose(EntireNodeOverlaySlot)
				[
					CurrentLayoutBox ? CurrentLayoutBox.ToSharedRef() : MainNodeWidget
				];

				ChildSlot
				[
					EntireNodeOverlay.ToSharedRef()
				];

				// Default to -1 so when we decrement it's -2, which overlay supports for inserting behind.
				EntireNodeLowestZOrder = INDEX_NONE;
			}

			OverlayToInsert = EntireNodeOverlay;
			OverlayToInsertLowestZOrder = &EntireNodeLowestZOrder;
		}

		if (InAddDirection == EMainNodeExtensionDirection::After)
		{
			// Just add at end and let overlay automatically increment Z order based on last child
			OverlayToInsert->AddSlot()
			.Padding(LayoutInfo.Padding)
			.HAlign(LayoutInfo.HAlign.Get(HAlign_Fill))
			.VAlign(LayoutInfo.VAlign.Get(VAlign_Fill))
			[
				ExtensionWidget
			];
		}
		else if (ensure(InAddDirection == EMainNodeExtensionDirection::Before))
		{
			// @TODO: Logic for inserts, currently not supported by this API. Possibly should belong in SOverlay
			//TPanelChildren<SOverlay::FOverlaySlot>* OverlayChildren = static_cast<TPanelChildren<SOverlay::FOverlaySlot>*>(Overlay->GetChildren());
			//for (int32 ChildIdx = 0; ChildIdx < OverlayChildren->Num(); ++ChildIdx)
			//{
			//	SOverlay::FOverlaySlot& OverlaySlot = (*OverlayChildren)[ChildIdx];
			//	OverlaySlot.SetZOrder(OverlaySlot.GetZOrder() + 1);
			//}
			
			// Use hack of inserting at -2 or lower to insert before widgets at Z Order 0.
			if (OverlayToInsertLowestZOrder)
			{
				(*OverlayToInsertLowestZOrder)--;
				OverlayToInsert->AddSlot(*OverlayToInsertLowestZOrder)
				.Padding(LayoutInfo.Padding)
				.HAlign(LayoutInfo.HAlign.Get(HAlign_Fill))
				.VAlign(LayoutInfo.VAlign.Get(VAlign_Fill))
				[
					ExtensionWidget
				];
			}
		}

	};

	// Add in a cardinal direction with a splitter
	// @TODO: This should be removed and just a feature toggle on add cardinal direction.
	auto AddSplitExtension = [&](const EOrientation InOrientation, const EMainNodeExtensionDirection InAddDirection)
	{
		if (!CurrentSplitterBox)
		{
			if (InOrientation == Orient_Horizontal)
			{
				SAssignNew(CurrentSplitterBox, StateCentricView::Private::SHorizontalExtensionSplitter)
					.PhysicalSplitterHandleSize(0.0f)
					.Orientation(InOrientation);
			}
			else
			{
				SAssignNew(CurrentSplitterBox, StateCentricView::Private::SVerticalExtensionSplitter)
					.PhysicalSplitterHandleSize(0.0f)
					.Orientation(InOrientation);
			}
				
			CurrentSplitterBox->AddSlot()
			.Expose(MainNodeSplitterSlot)
			.SizeRule(MainNodeLayoutInfo.SplitterSizeRule)
			.Value(MainNodeLayoutInfo.SplitterValue)
			.OnSlotResized(MainNodeLayoutInfo.SplitterOnSlotResized.Get({}))
			[
				MainNodeBox.ToSharedRef()
			];

			// We don't support arbitrary nesting order with current splitter impl
			if (ensure(!MainNodeOverlay)
				&& ensure(!MainNodeLayoutSlot)
				&& ensure(!EntireNodeOverlaySlot))
			{
				ChildSlot
				[
					CurrentSplitterBox.ToSharedRef()
				];
			}
		}

		// We don't support nesting or ordered insert for splitters since splitter container support is temp
		if (ensure(InOrientation == CurrentSplitterBox->GetOrientation())
			&& ensure(ExtensionParams.AddIndex == INDEX_NONE))
		{
			int8 InsertIndex = INDEX_NONE;
			if (InAddDirection == EMainNodeExtensionDirection::After)
			{
				InsertIndex = IntCastChecked<int8>(CurrentSplitterBox->NumSlots());
			}
			else if (ensure(InAddDirection == EMainNodeExtensionDirection::Before))
			{
				InsertIndex = 0;
			}

			CurrentSplitterBox->AddSlot(InsertIndex)
			.SizeRule(LayoutInfo.SplitterSizeRule)
			.Value(LayoutInfo.SplitterValue)
			.OnSlotResized(LayoutInfo.SplitterOnSlotResized.Get({}))
			[
				ExtensionWidget
			];
		}

	};

	switch (ExtensionParams.Location)
	{
		case EExtensionLocation::Bottom:
		{
			AddCardinalExtension(EOrientation::Orient_Vertical, EMainNodeExtensionDirection::After);
			break;
		}
		case EExtensionLocation::Left:
		{
			AddCardinalExtension(EOrientation::Orient_Horizontal, EMainNodeExtensionDirection::Before);
			break;
		}
		case EExtensionLocation::Right:
		{
			AddCardinalExtension(EOrientation::Orient_Horizontal, EMainNodeExtensionDirection::After);
			break;
		}
		case EExtensionLocation::Top:
		{
			AddCardinalExtension(EOrientation::Orient_Vertical, EMainNodeExtensionDirection::Before);
			break;
		}
		case EExtensionLocation::SubHeader:
		{
			if (!SubHeaderBox)
			{
				SAssignNew(SubHeaderBox, SStackBox)
				.Orientation(EOrientation::Orient_Vertical);

				int8 AfterHeaderIndex = NodeHeaderWidget ? 1 : 0;

				MainNodeBox->InsertSlot(AfterHeaderIndex)
				.AutoSize()
				[
					SubHeaderBox.ToSharedRef()
				];
			}

			int8 InsertIndex = ExtensionParams.AddIndex == INDEX_NONE
				? IntCastChecked<int8>(SubHeaderBox->NumSlots())
				: ExtensionParams.AddIndex;

			SubHeaderBox->InsertSlot(InsertIndex)
			.SizeParam(LayoutInfo.SizeParam)
			.MinSize(LayoutInfo.MinSize)
			.MaxSize(LayoutInfo.MaxSize)
			.Padding(LayoutInfo.Padding)
			.HAlign(LayoutInfo.HAlign.Get(HAlign_Fill))
			.VAlign(LayoutInfo.VAlign.Get(VAlign_Fill))
			[
				ExtensionWidget
			];

			break;
		}
		case EExtensionLocation::Footnote:
		{
			if (!FootnoteBox)
			{
				SAssignNew(FootnoteBox, SStackBox)
					.Orientation(EOrientation::Orient_Vertical);

				// Footnote is always last, so just add
				MainNodeBox->AddSlot()
				.AutoSize()
				[
					FootnoteBox.ToSharedRef()
				];
			}

			int8 InsertIndex = ExtensionParams.AddIndex == INDEX_NONE
				? IntCastChecked<int8>(FootnoteBox->NumSlots())
				: ExtensionParams.AddIndex;

			FootnoteBox->InsertSlot(InsertIndex)
			.SizeParam(LayoutInfo.SizeParam)
			.MinSize(LayoutInfo.MinSize)
			.MaxSize(LayoutInfo.MaxSize)
			.Padding(LayoutInfo.Padding)
			.HAlign(LayoutInfo.HAlign.Get(HAlign_Fill))
			.VAlign(LayoutInfo.VAlign.Get(VAlign_Fill))
			[
				ExtensionWidget
			];

			break;
		}
		case EExtensionLocation::Foreground:
		{
			AddOverlayExtension(EMainNodeExtensionDirection::After);
			break;
		}
		case EExtensionLocation::Background:
		{
			AddOverlayExtension(EMainNodeExtensionDirection::Before);
			break;
		}
		case EExtensionLocation::SplitBottom:
		{
			AddSplitExtension(EOrientation::Orient_Vertical, EMainNodeExtensionDirection::After);
			break;
		}
		case EExtensionLocation::SplitLeft:
		{
			AddSplitExtension(EOrientation::Orient_Horizontal, EMainNodeExtensionDirection::Before);
			break;
		}
		case EExtensionLocation::SplitRight:
		{
			AddSplitExtension(EOrientation::Orient_Horizontal, EMainNodeExtensionDirection::After);
			break;
		}
		case EExtensionLocation::SplitTop:
		{
			AddSplitExtension(EOrientation::Orient_Vertical, EMainNodeExtensionDirection::Before);
			break;
		}
		default:
			ensureMsgf(false, TEXT("Invalid extension location: %s - %u"), *GetNodeName().ToString(), EnumToUnderlyingType(ExtensionParams.Location));
			break;
	}

#if UE_EXTENDABLE_NODE_DEBUG
	AddedExtensions.Add({ ExtensionParams, ExtensionWidget });
#endif // UE_EXTENDABLE_NODE_DEBUG
}

FScopedExtensionAddNodeParams SExtendableNode::AddExtension(TSharedRef<SWidget> ExtensionWidget)
{
	return FScopedExtensionAddNodeParams(StaticCastSharedRef<SExtendableNode>(AsShared()), ExtensionWidget);
}

void SExtendableNode::SetMainNodeLayoutInfo(const FExtensionAddLayoutInfo& InLayoutInfo)
{
	MainNodeLayoutInfo = InLayoutInfo;
}

FScopedMainNodeLayoutInfo SExtendableNode::SetScopedMainNodeLayoutInfo()
{
	return FScopedMainNodeLayoutInfo(StaticCastSharedRef<SExtendableNode>(AsShared()));
}

void SExtendableNode::RegenerateWidgetForLOD(const EExtendableNodeLOD InLOD)
{
	CurrentLOD = InLOD;

	ResetCachedNodeAndLayout();

	Construct_GenerateMainNode(CurrentLOD);
	Construct_GenerateBuiltInExtensions(CurrentLOD);
	Construct_GenerateExternalExtensions(CurrentLOD);
}

EExtendableNodeLOD SExtendableNode::GetCurrentLOD() const
{
	return CurrentLOD;
}

void SExtendableNode::Construct_GatherExtensionsFromExternalSubsystems()
{
	// @TODO: Gather from a default ExtendableNode subsytem using our unique name ID & add to config
}

void SExtendableNode::Construct_InitializeNodePersistentPreLOD()
{
	// No-op
}

void SExtendableNode::Construct_GenerateMainNode(const EExtendableNodeLOD InLOD)
{
	TSharedRef<SWidget> Header = Construct_GenerateNodeHeader(InLOD);
	TSharedRef<SWidget> Body = Construct_GenerateNodeBody(InLOD);

	SAssignNew(MainNodeBox, StateCentricView::Private::SMainNodeStackBox)
	.Orientation(EOrientation::Orient_Vertical);

	if (Header != SNullWidget::NullWidget)
	{
		NodeHeaderWidget = Header;
		MainNodeBox->AddSlot()
		.Padding(NodeHeaderPadding)
		.SizeParam(NodeHeaderSizeParam.Get(FAuto()))
		[
			NodeHeaderWidget.ToSharedRef()
		];
	}

	if (Body != SNullWidget::NullWidget)
	{
		NodeBodyWidget = Body;
		MainNodeBox->AddSlot()
		.Padding(NodeBodyPadding)
		.SizeParam(NodeBodySizeParam.Get(FAuto()))
		[
			NodeBodyWidget.ToSharedRef()
		];
	}

	ChildSlot
	[
		MainNodeBox.ToSharedRef()
	];
}

void SExtendableNode::Construct_GenerateBuiltInExtensions(const EExtendableNodeLOD InLOD)
{
	// No-op
}

void SExtendableNode::Construct_GenerateExternalExtensions(const EExtendableNodeLOD InLOD)
{
	for (FAddExtensionDelegate& ExternalExtension : Config.ExternalExtensions)
	{
		ExternalExtension.ExecuteIfBound(StaticCastSharedRef<SExtendableNode>(AsShared()), InLOD);
	}
}

TSharedRef<SWidget> SExtendableNode::Construct_GenerateNodeHeader(const EExtendableNodeLOD InLOD)
{
	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SExtendableNode::Construct_GenerateNodeBody(const EExtendableNodeLOD InLOD)
{
	return SNullWidget::NullWidget;
}

void SExtendableNode::ResetCachedNodeAndLayout()
{
	NodeHeaderWidget = nullptr;
	NodeBodyWidget = nullptr;
	MainNodeBox = nullptr;
	SubHeaderBox = nullptr;
	FootnoteBox = nullptr;

	MainNodeLayoutInfo = {};

	CurrentLayoutBox = nullptr;
	CurrentSplitterBox = nullptr;
	InnerLayoutBox = nullptr;
	EntireNodeOverlay = nullptr;
	MainNodeOverlay = nullptr;
	MainNodeLayoutSlot = nullptr;
	MainNodeSplitterSlot = nullptr;
	EntireNodeOverlaySlot = nullptr;

	MainNodeCurrentLayoutIndex = INDEX_NONE;
	MainNodeInnerLayoutIndex = INDEX_NONE;
	EntireNodeLowestZOrder = INDEX_NONE;
	MainNodeLowestZOrder = INDEX_NONE;

#if UE_EXTENDABLE_NODE_DEBUG
	AddedExtensions.Reset();
#endif // UE_EXTENDABLE_NODE_DEBUG
}

} // namespace UE::StateTree::Editor::StateCentricView

#undef LOCTEXT_NAMESPACE
