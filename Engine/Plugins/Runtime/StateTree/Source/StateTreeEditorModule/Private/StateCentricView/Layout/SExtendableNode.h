// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "StateCentricView/Layout/ExtendableNodeTypes.h"
#include "Types/SlateEnums.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"


#ifndef UE_EXTENDABLE_NODE_DEBUG
	#define UE_EXTENDABLE_NODE_DEBUG 1
#endif

namespace UE::StateTree::Editor::StateCentricView
{
	class SExtendableNode;
	struct FScopedExtensionAddNodeParams;
	struct FScopedMainNodeLayoutInfo;
}

namespace UE::StateTree::Editor::StateCentricView
{


/** Experimental. Location for a node extension. */
enum class EExtensionLocation : uint8
{
	Bottom,			// Below the node
	Left,			// Left of node
	Right,			// Right of node
	Top,			// Above the node 

	SubHeader,		// Between main node header and body
	Footnote,		// Below the main node body.

	Foreground,		// Overlay in front of node. 
	Background,		// Overlay behind node. 

	// @TODO: Remove. Drag to resize should be a feature toggle not a container switch. See: 48250292.
	// Right now splitter / stackbox have disjoint functionality sets for no reason so we need these.
	// For now these have intentionally strict, known golden path only, support.

	SplitBottom,	// Below the node
	SplitLeft,		// Left of node
	SplitRight,		// Right of node
	SplitTop,		// Above the node 
};


/** Experimental. How a newly added extension should behave if there are any special considerations. */
enum class EExtensionAddLocationBehavior : uint8
{
	Default,			// Dynamic default depending on what the given location is

	Wrap,				// (Default) If current layout doesn't support location, wrap with a layout that does
	UseInnerLayout,		// If current layout doesn't support location, use first inner layout that does or create one

	OverlayMainNode,	// (Default) Add this overlay just for the main node, excluding extensions
	OverlayEntireNode,	// Add this overlay for the entire node including extensions (IE: Entire widget).
};


/**
 * Experimental. Callback node will fire when it is ready for generate external extensions
 * If you need domain specific info beyond LOD to determine if an extension is needed. Please subclass & cast ExtendableNode with that info.
 *
 * @param ExtendableNode	- Node to extend by programatically calling `SExtendableNode::AddExtension` based on provided LOD,
 * @param LOD				- Convenience param. LOD for the current node. Same as ExtendableNode's LOD.
 */
using FAddExtensionDelegate = TDelegate<void(TSharedRef<SExtendableNode> /*ExtendableNode*/, const EExtendableNodeLOD /*LOD*/)>;


/** Experimental. Config / params struct for extendable nodes. Persists as long as the widget exists. Delegates may re-execute deferred as LOD changes. */
struct FExtendableNodeConfig
{
	/** External extensions for this node. Provided at construction and also gathered in SExtendableNode::Construct_GatherExtensionsFromExternalSubsystems. */
	TArray<FAddExtensionDelegate> ExternalExtensions;

	/** Initial LOD for this widget. Only used if non-default. If default wil get over-written to Construct time initial value. */
	EExtendableNodeLOD InitialLOD = EExtendableNodeLOD::Default;
};


/** Experimental. Struct holding layout info used when adding extensions */
struct FExtensionAddLayoutInfo
{
	/** Sizing rules, auto = desired size / stretch = fill / strench content = desired size + fill */
	TOptional<FSizeParam> SizeParam;

	/** Min size of this widget */
	TAttribute<float> MinSize;

	/** Max size of this widget */
	TAttribute<float> MaxSize;

	/** Padding of extension */
	TAttribute<FMargin> Padding;

	/** Horizontal Alignment */
	TOptional<EHorizontalAlignment> HAlign;

	/** Vertical Alignment */
	TOptional<EVerticalAlignment> VAlign;

	/** 
	 * Layer ID Offset applied to overlay. Used to get consistent widget overlap behavior across contiguous entries in stack boxes.
	 * Only applies to main node layout at the moment.
	 */
	int32 OverlayOffset = 0;

	/** Splitter Sizing Rule */
	TAttribute<SSplitter::ESizeRule> SplitterSizeRule;

	/** Splitter Ratio Value */
	TAttribute<float> SplitterValue;

	/** Splitter On Resize Callback */
	TOptional<SSplitter::FOnSlotResized> SplitterOnSlotResized;
};


/** Experimental. Struct holding params for adding a node extension widget. Supports argument chaining via `FScopedExtensionAddNodeParams`. */
struct FExtensionAddNodeParams
{
	/** Location to add extension */
	EExtensionLocation Location = EExtensionLocation::SubHeader;

	/** Any specific behavior to consider when adding to vien location */
	EExtensionAddLocationBehavior AddLocationBehavior = EExtensionAddLocationBehavior::Default;

	/** If positive, index to add widget for given location (Ex: Left 0 will be 1st left widget) */
	int8 AddIndex = INDEX_NONE; 

	/** Size info when adding extension */
	FExtensionAddLayoutInfo LayoutInfo = {};
};


/** 
 * Experimental. Widget with a central Main Node (Header / Body), that can be extended horizontally / vertically / etc.
 * Also provides conditional node & extension re-generation based on LOD.
 * 
 * Simplified Node <Extension> Diagram:
 * 
 *    *---------------------------------*        *---------*-------------*---------*
 *    |           <Top>                 |        |         | <Top>       |         |
 *    *---------*-------------*---------*        |         *-------------*         |
 *    |         | Header      |         |        |         | Header      |         |
 *    |         *-------------*         |        |         *-------------*         |
 *    | <Left>  | <SubHeader> | <Right> |   or   | <Left>  | <SubHeader> | <Right> |
 *    |         *-------------*         |        |         *-------------*         |
 *    |         | Body        |         |        |         | Body        |         |
 *    *---------*-------------*---------*        |         *-------------*         |
 *    |           <Bottom>              |        |         | <Bottom>    |         |
 *    *---------------------------------*        *---------*-------------*---------*
 * 
 * The corners are order dependant. See `EExtensionAddLocationBehavior` for details.
 */
class SExtendableNode : public SCompoundWidget
{

public:

	SLATE_BEGIN_ARGS(SExtendableNode)
		: _Config(FExtendableNodeConfig())
		{
		}

		/** Initial set of item categories that this view should show - may be adjusted further by things like CanShowClasses or legacy delegate bindings */
		SLATE_ARGUMENT(FExtendableNodeConfig, Config)

	SLATE_END_ARGS()

public:

	friend struct FScopedMainNodeLayoutInfo;

public:

	void Construct(const FArguments& InArgs);

	/** Adds the given extension widget, constructing layout widgets as needed to support location */
	void AddExtension(TSharedRef<SWidget> ExtensionWidget, const FExtensionAddNodeParams& ExtensionParams);

	/** Add extension, supports argument chaining syntax to build params. Includes slate-style methods like AutoSize / FillSize / etc. */
	FScopedExtensionAddNodeParams AddExtension(TSharedRef<SWidget> ExtensionWidget);

	/** 
	 * Adding Layout incompatible extensions requires 2 slots. Main Node & Extension. This can be used to layout the main node slot. 
	 * If not set for cardinal extensions will re-use the extension layout for the main node. Ex: Fill next to fill, desired size next to desired size.
	 * @TODO: Get rid of this reuse behavior. Or make it an option that is disabled by default / can be disabled. The automatic change something else is confusing.
	 */
	void SetMainNodeLayoutInfo(const FExtensionAddLayoutInfo& InLayoutInfo);

	/** Same as above, but resets the value once return value goes out of scope */
	[[nodiscard]] FScopedMainNodeLayoutInfo SetScopedMainNodeLayoutInfo();

	/** Regenerates this widget for the given LOD, also updates current LOD. */
	virtual void RegenerateWidgetForLOD(const EExtendableNodeLOD InLOD);

	/** Get current LOD of this widget */
	EExtendableNodeLOD GetCurrentLOD() const;

	/** 
	 * Get the unique name for this node, used as a key so that external systems can extend this node.
	 * If you need access to this key to extend then we suggest just hardcoding a copy locally.
	 */
	virtual FName GetNodeName() const = 0;

protected:

	/** One time gather of subsystem externally registered extensions for this node, cached into node config. */
	virtual void Construct_GatherExtensionsFromExternalSubsystems();

	/** One time init for anything persistent for the node across LODs, pre LOD generation. */
	virtual void Construct_InitializeNodePersistentPreLOD();

	/** 
	 * Generation widget header & body based on LOD. Also initally populates ChildSlot
	 * Should not consider extensions as those are transient between LODs & will be re-added if needed. 
	 */
	virtual void Construct_GenerateMainNode(const EExtendableNodeLOD InLOD);

	/** Generation of built in extensions based on LOD. */
	virtual void Construct_GenerateBuiltInExtensions(const EExtendableNodeLOD InLOD);

	/** Generation of external extensions based on LOD. */
	virtual void Construct_GenerateExternalExtensions(const EExtendableNodeLOD InLOD);

	/** Builds header for this node based on LOD */
	virtual TSharedRef<SWidget> Construct_GenerateNodeHeader(const EExtendableNodeLOD InLOD);

	/** Builds body for this node based on LOD */
	virtual TSharedRef<SWidget> Construct_GenerateNodeBody(const EExtendableNodeLOD InLOD);

	/** Resets values stored during node construction, used to ensure a blank state when LOD changes */
	void ResetCachedNodeAndLayout();

protected:

	/** Persistent config for this node, such as extension delegates. */
	FExtendableNodeConfig Config = {};

	/** Current LOD for this node. */
	EExtendableNodeLOD CurrentLOD = EExtendableNodeLOD::Unused;

	/** Widget used for header, if any. Will not use SNullWidget. */
	TSharedPtr<SWidget> NodeHeaderWidget = nullptr;

	/** Widget used for body, if any. Will not use SNullWidget. */
	TSharedPtr<SWidget> NodeBodyWidget = nullptr;

	/** Container for header, subheader, and body. */
	TSharedPtr<SStackBox> MainNodeBox = nullptr;

	/** Widget for SubHeader extensions, nullptr till a single subheader extension exists. */
	TSharedPtr<SStackBox> SubHeaderBox = nullptr;

	/** Widget for Footnote extensions, nullptr till a single subheader extension exists. */
	TSharedPtr<SStackBox> FootnoteBox = nullptr;

	/** Padding for header */
	TAttribute<FMargin> NodeHeaderPadding;

	/** Padding for body */
	TAttribute<FMargin> NodeBodyPadding;

	/** Sizing for header */
	TOptional<FSizeParam> NodeHeaderSizeParam;

	/** Sizing for body */
	TOptional<FSizeParam> NodeBodySizeParam;

private:

	/** This supports custom layout for the main node slot, used for main node on for first time new layouts. */
	FExtensionAddLayoutInfo MainNodeLayoutInfo = {};

	/** Widget for current layout, nullptr till a horizontal / vertical extension is added */
	TSharedPtr<SStackBox> CurrentLayoutBox = nullptr;

	/** @TODO: Remove. Box holding current top level splitter. */
	TSharedPtr<SSplitter> CurrentSplitterBox = nullptr;

	/** Widget for first inner layout, nullptr till an extension with an incomptible location is added */
	TSharedPtr<SStackBox> InnerLayoutBox = nullptr;

	/** Widget for entire node overlays (includes extensions), nullptr till a foreground / background extension is added */
	TSharedPtr<SOverlay> EntireNodeOverlay = nullptr;

	/** Widget for main node overlays, nullptr till a foreground / background extension is added */
	TSharedPtr<SOverlay> MainNodeOverlay = nullptr;

	/** Innermost layout slot holding main node or widget containing it. Remains constant once created no matter future extensions. */
	SStackBox::FSlot* MainNodeLayoutSlot = nullptr;

	/** @TODO: Remove. Innermost splitter slot holding main node or widget containing it. */
	SSplitter::FSlot* MainNodeSplitterSlot = nullptr;

	/** Innermost overlay slot holding entire node Remains constant once created no matter future extensions. */
	SOverlay::FOverlaySlot* EntireNodeOverlaySlot = nullptr;

	/** Index to main node or widget containing it in the current layout */
	int8 MainNodeCurrentLayoutIndex = INDEX_NONE;

	/** Index to main node or widget containing it in the first inner layout */
	int8 MainNodeInnerLayoutIndex = INDEX_NONE;

	/** ZOrder for the furthest entire node background overlay (includes extensions). Used to continously insert behind the current background. */
	int8 EntireNodeLowestZOrder = INDEX_NONE;

	/** ZOrder for the furthest main node background overlay. Used to continously insert behind the current background */
	int8 MainNodeLowestZOrder = INDEX_NONE;

#if UE_EXTENDABLE_NODE_DEBUG
	/** Trace of all extensions added in order */
	TArray<TPair<FExtensionAddNodeParams, TSharedPtr<SWidget>>> AddedExtensions;
#endif // UE_EXTENDABLE_NODE_DEBUG

};


/** Experimental. Scoped struct to hold add param methods & support function chaining syntax.  */
struct FScopedExtensionAddNodeParams : public FExtensionAddNodeParams
{
public:
	FScopedExtensionAddNodeParams(TSharedRef<SExtendableNode> InAddNode, TSharedPtr<SWidget> InExtensionWidget)
		: AddNode(InAddNode)
		, ExtensionWidget(InExtensionWidget)
	{
	}

	FScopedExtensionAddNodeParams() = delete;
	FScopedExtensionAddNodeParams(const FScopedExtensionAddNodeParams&) = delete;
	FScopedExtensionAddNodeParams& operator=(const FScopedExtensionAddNodeParams&) = delete;
	FScopedExtensionAddNodeParams(FScopedExtensionAddNodeParams&&) = default;
	FScopedExtensionAddNodeParams& operator=(FScopedExtensionAddNodeParams&&) = default;

	virtual ~FScopedExtensionAddNodeParams()
	{
		if (ensure(AddNode))	
		{
			AddNode->AddExtension(ExtensionWidget.ToSharedRef(), *this);
		}
	}

public:

	FScopedExtensionAddNodeParams& SetLocation(const EExtensionLocation InLocation)
	{
		Location = InLocation;
		return *this;
	}

	FScopedExtensionAddNodeParams& SetAddLocationBehavior(const EExtensionAddLocationBehavior InAddLocationBehavior)
	{
		AddLocationBehavior = InAddLocationBehavior;
		return *this;
	}

	FScopedExtensionAddNodeParams& SetAddIndex(const int8 InAddIndex)
	{
		AddIndex = InAddIndex;
		return *this;
	}

	FScopedExtensionAddNodeParams& SetAutoSize()
	{
		LayoutInfo.SizeParam = FAuto();
		return *this;
	}

	FScopedExtensionAddNodeParams& SetFillSize(TAttribute<float> InStretchCoefficient)
	{
		LayoutInfo.SizeParam = FStretch(MoveTemp(InStretchCoefficient));
		return *this;
	}

	FScopedExtensionAddNodeParams& SetFillContentSize(TAttribute<float> InStretchCoefficient, TAttribute<float> InShrinkStretchCoefficient = TAttribute<float>())
	{
		LayoutInfo.SizeParam = FStretchContent(MoveTemp(InStretchCoefficient), MoveTemp(InShrinkStretchCoefficient));
		return *this;
	}

	FScopedExtensionAddNodeParams& SetMinSize(TAttribute<float> InMinSize)
	{
		LayoutInfo.MinSize = MoveTemp(InMinSize);
		return *this;
	}

	FScopedExtensionAddNodeParams& SetMaxSize(TAttribute<float> InMaxSize)
	{
		LayoutInfo.MaxSize = MoveTemp(InMaxSize);
		return *this;
	}

	FScopedExtensionAddNodeParams& SetPadding(TAttribute<FMargin> InPadding)
	{
		LayoutInfo.Padding = MoveTemp(InPadding);
		return *this;
	}

	FScopedExtensionAddNodeParams& SetHAlign(TOptional<EHorizontalAlignment> InHAlign)
	{
		LayoutInfo.HAlign = MoveTemp(InHAlign);
		return *this;
	}

	FScopedExtensionAddNodeParams& SetVAlign(TOptional<EVerticalAlignment> InVAlign)
	{
		LayoutInfo.VAlign = MoveTemp(InVAlign);
		return *this;
	}

	FScopedExtensionAddNodeParams& SetSplitterSizeRule(TAttribute<SSplitter::ESizeRule> InSplitterSizeRule)
	{
		LayoutInfo.SplitterSizeRule = MoveTemp(InSplitterSizeRule);
		return *this;
	}

	FScopedExtensionAddNodeParams& SetSplitterValue(TAttribute<float> InSplitterValue)
	{
		LayoutInfo.SplitterValue = MoveTemp(InSplitterValue);
		return *this;
	}

	FScopedExtensionAddNodeParams& SetSplitterOnSlotResized(TOptional<SSplitter::FOnSlotResized> InSplitterOnSlotResized)
	{
		LayoutInfo.SplitterOnSlotResized = MoveTemp(InSplitterOnSlotResized);
		return *this;
	}

private:
	TSharedPtr<SExtendableNode> AddNode = nullptr;
	TSharedPtr<SWidget> ExtensionWidget = nullptr;
};


/** Experimental. Scoped struct to toggle main node size info & support function chaining syntax. */
struct FScopedMainNodeLayoutInfo
{

protected:

	friend class SExtendableNode;

	FScopedMainNodeLayoutInfo(TSharedRef<SExtendableNode> InAddNode)
		: AddNode(InAddNode)
		, bCanRestore(false)
	{
		PrevAddSizeInfo = AddNode->MainNodeLayoutInfo;
	}

public:

	FScopedMainNodeLayoutInfo() = delete;
	FScopedMainNodeLayoutInfo(const FScopedMainNodeLayoutInfo& Other)
	{
		*this = Other;
	}

	FScopedMainNodeLayoutInfo& operator=(const FScopedMainNodeLayoutInfo& Other)
	{
		// Only restore in dtor if we are a copy, first instance is used to enable single-line argument chaning syntax
		bCanRestore = true;
		AddNode = Other.AddNode;
		PrevAddSizeInfo = Other.PrevAddSizeInfo;

		return *this;
	}

	FScopedMainNodeLayoutInfo(FScopedMainNodeLayoutInfo&& Other) = default;
	FScopedMainNodeLayoutInfo& operator=(FScopedMainNodeLayoutInfo&& Other) = default;

	virtual ~FScopedMainNodeLayoutInfo()
	{
		if (ensure(AddNode) && bCanRestore)
		{
			AddNode->MainNodeLayoutInfo = PrevAddSizeInfo;
		}
	}

public:

	FScopedMainNodeLayoutInfo& SetAutoSize()
	{
		AddNode->MainNodeLayoutInfo.SizeParam = FAuto();
		return *this;
	}

	FScopedMainNodeLayoutInfo& SetFillSize(TAttribute<float> InStretchCoefficient)
	{
		AddNode->MainNodeLayoutInfo.SizeParam = FStretch(MoveTemp(InStretchCoefficient));
		return *this;
	}

	FScopedMainNodeLayoutInfo& SetFillContentSize(TAttribute<float> InStretchCoefficient, TAttribute<float> InShrinkStretchCoefficient = TAttribute<float>())
	{
		AddNode->MainNodeLayoutInfo.SizeParam = FStretchContent(MoveTemp(InStretchCoefficient), MoveTemp(InShrinkStretchCoefficient));
		return *this;
	}

	FScopedMainNodeLayoutInfo& SetMinSize(TAttribute<float> InMinSize)
	{
		AddNode->MainNodeLayoutInfo.MinSize = MoveTemp(InMinSize);
		return *this;
	}

	FScopedMainNodeLayoutInfo& SetMaxSize(TAttribute<float> InMaxSize)
	{
		AddNode->MainNodeLayoutInfo.MaxSize = MoveTemp(InMaxSize);
		return *this;
	}

	FScopedMainNodeLayoutInfo& SetPadding(TAttribute<FMargin> InPadding)
	{
		AddNode->MainNodeLayoutInfo.Padding = MoveTemp(InPadding);
		return *this;
	}

	FScopedMainNodeLayoutInfo& SetHAlign(TOptional<EHorizontalAlignment> InHAlign)
	{
		AddNode->MainNodeLayoutInfo.HAlign = MoveTemp(InHAlign);
		return *this;
	}

	FScopedMainNodeLayoutInfo& SetVAlign(TOptional<EVerticalAlignment> InVAlign)
	{
		AddNode->MainNodeLayoutInfo.VAlign = MoveTemp(InVAlign);
		return *this;
	}

	FScopedMainNodeLayoutInfo& SetOverlayOffset(int32 InOverlayOffset)
	{
		AddNode->MainNodeLayoutInfo.OverlayOffset = InOverlayOffset;
		return *this;
	}

	FScopedMainNodeLayoutInfo& SetSplitterSizeRule(TAttribute<SSplitter::ESizeRule> InSplitterSizeRule)
	{
		AddNode->MainNodeLayoutInfo.SplitterSizeRule = MoveTemp(InSplitterSizeRule);
		return *this;
	}

	FScopedMainNodeLayoutInfo& SetSplitterValue(TAttribute<float> InSplitterValue)
	{
		AddNode->MainNodeLayoutInfo.SplitterValue = MoveTemp(InSplitterValue);
		return *this;
	}

	FScopedMainNodeLayoutInfo& SetSplitterOnSlotResized(TOptional<SSplitter::FOnSlotResized> InSplitterOnSlotResized)
	{
		AddNode->MainNodeLayoutInfo.SplitterOnSlotResized = MoveTemp(InSplitterOnSlotResized);
		return *this;
	}

private:
	TSharedPtr<SExtendableNode> AddNode = nullptr;
	FExtensionAddLayoutInfo PrevAddSizeInfo = FExtensionAddLayoutInfo();
	bool bCanRestore = false;
};

} // namespace UE::StateTree::Editor::StateCentricView