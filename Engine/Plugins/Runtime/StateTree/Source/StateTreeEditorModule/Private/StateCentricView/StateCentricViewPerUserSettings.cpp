// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateCentricView/StateCentricViewPerUserSettings.h"
#include "StateTreeEditorData.h"
#include "StateTreeViewModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateCentricViewPerUserSettings)

UStateCentricViewPerUserSettings::UStateCentricViewPerUserSettings()
{
	FStateCentricViewPerUserSetting DefaultPerUserSettings;
	DefaultPerUserSettings.ParentHireachyLOD = EExtendableNodeLOD::Minimal;
	DefaultPerUserSettings.InTransitionViewLOD = EExtendableNodeLOD::Minimal;
	DefaultPerUserSettings.OutTransitionLOD = EExtendableNodeLOD::Full;
	// @TODO: Feedback here was 2 in passing, but that seems super low.
	DefaultPerUserSettings.NumParentsShown = 2;
	DefaultPerUserSettings.bHideParentTransitions = false;
	PerSchemaSettings.Emplace(UStateTreeSchema::StaticClass(), DefaultPerUserSettings);
}

const FStateCentricViewPerUserSetting& UStateCentricViewPerUserSettings::GetSchemaSettingsDefaultFallback(const FStateTreeViewModel* ViewModel)
{
	if (ViewModel)
	{
		if (const UStateTree* StateTree = ViewModel->GetStateTree())
		{
			if (const UStateTreeSchema* Schema = StateTree->GetSchema())
			{
				const TSubclassOf<UStateTreeSchema> SchemaClass = Schema->GetClass();
				return UStateCentricViewPerUserSettings::GetSchemaSettingsDefaultFallback(SchemaClass);
			}
		}
	}

	return UStateCentricViewPerUserSettings::Get().PerSchemaSettings[UStateTreeSchema::StaticClass()];
}

#if WITH_EDITOR

FText UStateCentricViewPerUserSettings::GetSectionText() const
{
	return NSLOCTEXT("StateTreeEditor", "StateCentricViewPerUserSettingsName", "State Centric View Per User");
}

FText UStateCentricViewPerUserSettings::GetSectionDescription() const
{
	return NSLOCTEXT("StateTreeEditor", "StateCentricViewPerUserSettingsDescription", "Experimental. Configure per user options for state centric view.");
}

#endif // WITH_EDITOR

FName UStateCentricViewPerUserSettings::GetCategoryName() const
{
	// @TODO: Change me
	return FName(TEXT("Plugins"));
}
