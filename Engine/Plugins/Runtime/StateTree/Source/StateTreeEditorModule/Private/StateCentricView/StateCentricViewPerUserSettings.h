// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "Layout/ExtendableNodeTypes.h"
#include "StateCentricView/StateCentricViewSettings.h"
#include "StateTreeSchema.h"

#include "StateCentricViewPerUserSettings.generated.h"

#define UE_API STATETREEEDITORMODULE_API

DECLARE_MULTICAST_DELEGATE_OneParam(FOnStateCentricViewLOD_Changed, TOptional<EExtendableNodeLOD> /*InLOD*/);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnStateCentricViewIntValue_Changed, TOptional<uint32> /*InValue*/);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnStateCentricViewBoolValue_Changed, TOptional<bool> /*InValue*/);

/**
 * Experimental. Settings per user. This exists locally per user per schema. But also per state tree.
 * Per state tree this is used to maintain consistent immediate behavior when clicking across states.
 * Per user this is used to configure a default behavior when no per-state-tree setting has been set.
 */
USTRUCT(Experimental, MinimalAPI)
struct FStateCentricViewPerUserSetting
{
	GENERATED_BODY()

public:

	friend UStateCentricViewPerUserSettings;

private:

	/** Amount of info to show for every parent state in the hierachy. */
	UPROPERTY(EditAnywhere, Category="Layout", Config, meta = (ValidEnumValues = "Collapsed, Minimal, Full"))
	TOptional<EExtendableNodeLOD> ParentHireachyLOD = {};

	/** Amount of info to show for incoming transitions to this state. */
	UPROPERTY(EditAnywhere, Category = "Layout", Config, meta = (ValidEnumValues = "Minimal, Full"))
	TOptional<EExtendableNodeLOD> InTransitionViewLOD = {};

	/** Amount of info to show for outgoing transitions from this state. */
	UPROPERTY(EditAnywhere, Category = "Layout", Config, meta = (ValidEnumValues = "Minimal, Full"))
	TOptional<EExtendableNodeLOD> OutTransitionLOD = {};

	/** Number of parents to show if they exist. */
	UPROPERTY(EditAnywhere, Category="Layout", Config)
	TOptional<uint32> NumParentsShown = {};

	/** True if we should hide parents in transition view. */
	UPROPERTY(EditAnywhere, Category = "Layout", Config)
	TOptional<bool> bHideParentTransitions = {};

public:

	UE_API TOptional<EExtendableNodeLOD> GetParentHireachyLOD() const
	{
		return ParentHireachyLOD;
	}

	UE_API TOptional<EExtendableNodeLOD> GetInTransitionLOD() const
	{
		return InTransitionViewLOD;
	}

	UE_API TOptional<EExtendableNodeLOD> GetOutTransitionLOD() const
	{
		return OutTransitionLOD;
	}

	UE_API TOptional<uint32> GetNumParentsShown() const
	{
		return NumParentsShown;
	}

	UE_API TOptional<bool> GetHideParentTransitions() const
	{
		return bHideParentTransitions;
	}

	UE_API void SetParentHireachyLOD(TOptional<EExtendableNodeLOD> InLOD)
	{
		ensure(!InLOD
			|| InLOD.GetValue() == EExtendableNodeLOD::Collapsed
			|| InLOD.GetValue() == EExtendableNodeLOD::Minimal
			|| InLOD.GetValue() == EExtendableNodeLOD::Full);

		ParentHireachyLOD = InLOD;
		OnParentHireachyLOD_Changed.Broadcast(ParentHireachyLOD);
	}

	UE_API void SetInTransitionLOD(TOptional<EExtendableNodeLOD> InLOD)
	{
		ensure(!InLOD
			|| InLOD.GetValue() == EExtendableNodeLOD::Hidden
			|| InLOD.GetValue() == EExtendableNodeLOD::Minimal
			|| InLOD.GetValue() == EExtendableNodeLOD::Full);

		InTransitionViewLOD = InLOD;
		OnInTransitionViewLOD_Changed.Broadcast(InTransitionViewLOD);
	}

	UE_API void SetOutTransitionLOD(TOptional<EExtendableNodeLOD> InLOD)
	{
		ensure(!InLOD
			|| InLOD.GetValue() == EExtendableNodeLOD::Hidden
			|| InLOD.GetValue() == EExtendableNodeLOD::Minimal
			|| InLOD.GetValue() == EExtendableNodeLOD::Full);

		OutTransitionLOD = InLOD;
		OnOutTransitionViewLOD_Changed.Broadcast(OutTransitionLOD);
	}

	UE_API void SetNumParentsShown(TOptional<uint32> InNumParentShown)
	{
		if (NumParentsShown != InNumParentShown)
		{
			NumParentsShown = InNumParentShown;
			OnNumParentsShownChanged.Broadcast(NumParentsShown);
		}
	}

	UE_API void SetHideParentTransitions(TOptional<bool> bInHideParentTransitions)
	{
		if (bHideParentTransitions != bInHideParentTransitions)
		{
			bHideParentTransitions = bInHideParentTransitions;
			OnHideParentTransitionsChanged.Broadcast(bHideParentTransitions);
		}
	}

public:

	/** Callback whenever default parent hierarchy LOD changed. */
	FOnStateCentricViewLOD_Changed OnParentHireachyLOD_Changed;

	/** Callback whenever the LOD for 'In' transition view has changed. */
	FOnStateCentricViewLOD_Changed OnInTransitionViewLOD_Changed;

	/** Callback whenever the LOD for 'Out' transition view has changed. */
	FOnStateCentricViewLOD_Changed OnOutTransitionViewLOD_Changed;

	/** Callback whenever num parents shown has changed. */
	FOnStateCentricViewIntValue_Changed OnNumParentsShownChanged;

	/** Callback whenever hide parent transitions has changed. */
	FOnStateCentricViewBoolValue_Changed OnHideParentTransitionsChanged;
};


/**
 * Experimental. Settings for the state centric view per schema per user per schema.
 * These settings provide defaults to reset to along with defaults for newly opened statetrees.
 */
UCLASS(Experimental, MinimalAPI, config = EditorPerProjectUserSettings)
class UStateCentricViewPerUserSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:

	UStateCentricViewPerUserSettings();

	UE_API static const UStateCentricViewPerUserSettings& Get()
	{
		return *GetDefault<UStateCentricViewPerUserSettings>();
	}

	UE_API static UStateCentricViewPerUserSettings& GetMutable()
	{
		return *GetMutableDefault<UStateCentricViewPerUserSettings>();
	}

	UE_API static const FStateCentricViewPerUserSetting& GetSchemaSettingsDefaultFallback(const FStateTreeViewModel* ViewModel);

	UE_API static const FStateCentricViewPerUserSetting& GetSchemaSettingsDefaultFallback(const TSubclassOf<UStateTreeSchema> InSchema)
	{
		if (const FStateCentricViewPerUserSetting* SchemaSettings = UStateCentricViewPerUserSettings::Get().PerSchemaSettings.Find(TSoftClassPtr<UStateTreeSchema>(InSchema)))
		{
			return *SchemaSettings;
		}

		return UStateCentricViewPerUserSettings::Get().PerSchemaSettings[UStateTreeSchema::StaticClass()];
	}

public:

	/** Per schema settings. Defined per schema since we have no need for an editor schema to be defined. */
	UPROPERTY(EditAnywhere, Category="Appearance", Config, meta = (AllowAbstract = "true", EditCondition = "StateTreeEditorModule.StateCentricViewSettings.IsStateCentricViewEnabled", EditConditionHides))
	TMap<TSoftClassPtr<UStateTreeSchema>, FStateCentricViewPerUserSetting> PerSchemaSettings;

protected:

#if WITH_EDITOR
	//~ Begin UDeveloperSettings Interface
	UE_API virtual FText GetSectionText() const override;
	UE_API virtual FText GetSectionDescription() const override;
	//~ End UDeveloperSettings Interface
#endif

	//~ Begin UDeveloperSettings Interface
	UE_API virtual FName GetCategoryName() const override;
	//~ End UDeveloperSettings Interface
};

#undef UE_API
