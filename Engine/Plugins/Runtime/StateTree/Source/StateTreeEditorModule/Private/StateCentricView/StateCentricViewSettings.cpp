// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateCentricView/StateCentricViewSettings.h"
#include "StateTreeEditorData.h"
#include "StateTreeViewModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateCentricViewSettings)

bool bEnableStateCentricView = false;
FAutoConsoleVariableRef CVarEnableStateCentricView(
	TEXT("StateTree.Editor.Experimental.EnableStateCentricView"),
	bEnableStateCentricView,
	TEXT("Set true to enable state centric view")
);

UStateCentricViewSettings::UStateCentricViewSettings()
{
	PerSchemaSettings.Emplace(UStateTreeSchema::StaticClass(), FStateCentricViewPerSchemaSetting());
}

bool UStateCentricViewSettings::IsStateCentricViewEnabled()
{
	return bEnableStateCentricView;
}

const FStateCentricViewPerSchemaSetting& UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(const FStateTreeViewModel* ViewModel)
{
	if (ViewModel)
	{
		if (const UStateTree* StateTree = ViewModel->GetStateTree())
		{
			if (const UStateTreeSchema* Schema = StateTree->GetSchema())
			{
				const TSubclassOf<UStateTreeSchema> SchemaClass = Schema->GetClass();
				return UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(SchemaClass);
			}
		}
	}

	return UStateCentricViewSettings::Get().PerSchemaSettings[UStateTreeSchema::StaticClass()];
}

#if WITH_EDITOR

FText UStateCentricViewSettings::GetSectionText() const
{
	return NSLOCTEXT("StateTreeEditor", "StateCentricViewSettingsName", "State Centric View");
}

FText UStateCentricViewSettings::GetSectionDescription() const
{
	return NSLOCTEXT("StateTreeEditor", "StateCentricViewSettingsDescription", "Experimental. Configure options for state centric view.");
}

#endif // WITH_EDITOR

FName UStateCentricViewSettings::GetCategoryName() const
{
	// @TODO: Change me
	return FName(TEXT("Plugins"));
}
