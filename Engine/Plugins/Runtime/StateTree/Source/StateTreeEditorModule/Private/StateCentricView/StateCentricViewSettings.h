// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeSchema.h"

#include "Engine/DeveloperSettings.h"
#include "Templates/SubclassOf.h"

#include "StateCentricViewSettings.generated.h"

#define UE_API STATETREEEDITORMODULE_API

class FStateTreeViewModel;

/**
 * Experimental. Settings per schema. Mainly focused on styling & layout.
 */
USTRUCT(Experimental, MinimalAPI)
struct FStateCentricViewPerSchemaSetting
{
	GENERATED_BODY()

public:

	/** 
	 * True if state centric view colors based on hireachy rather than User Defined colors 
	 * @TODO: Also this should be part of base ST.
	 */
	UPROPERTY(EditAnywhere, Category="Style")
	bool bEnableHierarchyDrivenColors = false;

	/** 
	 * True if state centric view shapes nodes based on hireachy rather than default shapes 
	 * @TODO: Commented out as not yet implemented. Also this should be part of base ST VM.
	 */
	//UPROPERTY(EditAnywhere, Category="Style")
	//bool bEnableHireachyDrivenShapes = false;

	/** True if central / leafmost view should have transitions justified instead of top aligned */
	UPROPERTY(EditAnywhere, Category = "Layout")
	bool bJustifyCentralViewTransitions = false;

	/** True if states in transitions should have word wrap on */
	UPROPERTY(EditAnywhere, Category = "Layout")
	bool bWordWrapTransitionStates = true;

	/** True if child states should have word wrap on */
	UPROPERTY(EditAnywhere, Category = "Layout")
	bool bWordWrapChildStates = false;

	/** True if state centric view allows for full detail expansion of parents, else max expansion is entry conditions */
	UPROPERTY(EditAnywhere, Category="Layout")
	bool bEnableFullParentDetailExpansion = true;

	/** At the moment we only allow state centric to be toggled left or right of the outliner. */
	UPROPERTY(EditAnywhere, Category="Layout")
	bool bEnableOutlinerRightOfStateCentric = false;

	/** True if transition nodes have fill based width. Enabled by default to avoid confusing amounts of random negative space. */
	UPROPERTY(EditAnywhere, Category="Layout")
	bool bEnableFillOnTransitionNodes = true;

	/** Width for bridge connecting transitions nodes to center nodes*/
	UPROPERTY(EditAnywhere, Category = "Layout")
	float TransitionNodeBridgeWidth = 48.0f;

	/** 
	 * Max number of parents we allow users to show. At the moment we do not encourage changing this as showing too many parents 
	 * can detract from the intended workflow of focusing on the central state and jumping around if needed.
	 */
	UPROPERTY(EditAnywhere, Category="Layout", meta=(UIMin="1", UIMax="50"))
	uint32 MaxNumParentsToShow = 4;
};


/**
 * Experimental. May be removed at any time and API may change at any time.
 * Toggle via: StateTree.Editor.Experimental.EnableStateCentricView
 * 
 * Settings for the state centric view per schema. As state centric is early in dev, there are many toggles.
 */
UCLASS(Experimental, MinimalAPI, config = EditorSettings, defaultconfig)
class UStateCentricViewSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:

	UStateCentricViewSettings();

	UE_API static const UStateCentricViewSettings& Get()
	{
		return *GetDefault<UStateCentricViewSettings>();
	}

	UE_API static UStateCentricViewSettings& GetMutable()
	{
		return *GetMutableDefault<UStateCentricViewSettings>();
	}

	UE_API static const FStateCentricViewPerSchemaSetting& GetSchemaSettingsDefaultFallback(const FStateTreeViewModel* ViewModel);

	UE_API static const FStateCentricViewPerSchemaSetting& GetSchemaSettingsDefaultFallback(const TSubclassOf<UStateTreeSchema> InSchema)
	{
		if (const FStateCentricViewPerSchemaSetting* SchemaSettings = UStateCentricViewSettings::Get().PerSchemaSettings.Find(TSoftClassPtr<UStateTreeSchema>(InSchema)))
		{
			return *SchemaSettings;
		}

		return UStateCentricViewSettings::Get().PerSchemaSettings[UStateTreeSchema::StaticClass()];
	}

public:

	/** Per schema settings. Defined per schema since we have no need for an editor schema to be defined. */
	UPROPERTY(EditAnywhere, Category="Appearance", Config, meta = (AllowAbstract = "true", EditCondition = "StateTreeEditorModule.StateCentricViewSettings.IsStateCentricViewEnabled", EditConditionHides))
	TMap<TSoftClassPtr<UStateTreeSchema>, FStateCentricViewPerSchemaSetting> PerSchemaSettings;

public:

	/** Static version of state centric enabled check, exists to hide based on edit condition */
	UFUNCTION()
	static UE_API bool IsStateCentricViewEnabled(); 

protected:

#if WITH_EDITOR
	//~ Begin UDeveloperSettings Interface
	UE_API virtual FText GetSectionText() const override;
	UE_API virtual FText GetSectionDescription() const override;
	//~ End UDeveloperSettings Interface
#endif

	//~ Begin UDeveloperSettings Interface
	UE_API virtual FName GetCategoryName() const override;
	//~ End UDeveloperSettings Interface
};

#undef UE_API
