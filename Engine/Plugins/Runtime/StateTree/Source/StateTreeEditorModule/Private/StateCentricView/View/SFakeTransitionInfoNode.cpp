// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateCentricView/View/SFakeTransitionInfoNode.h"

#include "Customizations/StateTreeBindingExtension.h"
#include "StateCentricView/StateCentricViewSettings.h"
#include "StateCentricView/View/StateCentricViewUtils.h"
#include "StateTreeEditorStyle.h"

#include "Layout/Visibility.h"
#include "Modules/ModuleManager.h"
#include "ObjectEditorUtils.h"
#include "PropertyEditorModule.h"
#include "PropertyPathHelpers.h"
#include "Styling/SlateBrush.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SFakeTransitionInfoNode"

namespace UE::StateTree::Editor::StateCentricView
{


//////////////////////////////////////////////////////////////////////////
// SFakeTransitionInfoNode

void SFakeTransitionInfoNode::Construct_GenerateBuiltInExtensions(const EExtendableNodeLOD InLOD)
{
	// @TODO: Rather than handle the ParentSelection arrow via padding. Maybe just add an inner node for the ParentSelection & make the 
	// ParentSelection arrow not part of this node

	if (ViewState.IsValid())
	{
		// Parent state name as top extension
		if (const UStateTreeState* FakeTranstionInfoState = GetFakeTranstionInfoState())
		{
			StateBrush = FStateCentricViewUtils::MakeStateSubnodeBackgroundOutlined(ViewModel.ToSharedRef(), FakeTranstionInfoState);

			FScopedMainNodeLayoutInfo ScopedMainLayoutInfo = SetScopedMainNodeLayoutInfo()
				.SetPadding(FMargin(0.0));

			TSharedRef<SWidget> FakeTranstionInfoStateTopExtension = SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SImage)
					.Image(StateBrush->GetSlateBrush())
				]
				+ SOverlay::Slot()
				[
					FStateCentricViewUtils::GenerateGotoStateButton(InLOD, ViewModel.ToSharedRef(), FakeTranstionInfoState)
				]
				+ SOverlay::Slot()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(FMargin(12.0, 0.0, 2.0, 0.0))
					.AutoWidth()
					[
						SNew(SImage)
						.Image(ViewModel->GetSelectorIcon(FakeTranstionInfoState))
						.ColorAndOpacity(FStyleColors::Foreground)
						.Visibility(EVisibility::HitTestInvisible)
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(FText::FromName(FakeTranstionInfoState->Name))
						.TextStyle(&FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Normal.Normal"))
						.ColorAndOpacity(FStyleColors::White)
						.Visibility(EVisibility::HitTestInvisible)
						.AutoWrapText(UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(ViewModel.Get()).bWordWrapTransitionStates)
					]
				];

			AddExtension(FakeTranstionInfoStateTopExtension)
				.SetLocation(EExtensionLocation::Top)
				.SetAutoSize()
				.SetPadding(FMargin(0.0, 0.0, 0.0, 4.0));
		}

		// Small arrow used to indicate transition on left / right depending on direction
		const float TransitionNodeBridgeWidth = UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(ViewModel.Get()).TransitionNodeBridgeWidth;
		const float TransitionNodeBridgeHalfWidth = TransitionNodeBridgeWidth / 2.0f;
		{
			FScopedMainNodeLayoutInfo ScopedMainLayoutInfo = SetScopedMainNodeLayoutInfo()
				.SetFillSize(1.0f)
				.SetPadding(FMargin(0.0));

			TSharedRef<SWidget> TransitionArrowExtension =
				SNew(SButton)
				.NormalPaddingOverride(FMargin(2.0, 4.0))
				.PressedPaddingOverride(FMargin(2.0, 4.0))
				.ContentPadding(FMargin(TransitionNodeBridgeHalfWidth - 6.0f, 0.0f, TransitionNodeBridgeHalfWidth - 6.0f, 0))
				[
					SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						[
							SNew(SImage)
								.Image(FAppStyle::Get().GetBrush("Icons.ChevronRight"))
								.DesiredSizeOverride(FVector2D(10, 24))
								.ColorAndOpacity(FStyleColors::Foreground)
						]
						+ SHorizontalBox::Slot()
						.Padding(FMargin(-8.0, 0.0, 0.0, 0.0))
						[
							SNew(SImage)
								.Image(FAppStyle::Get().GetBrush("Icons.ChevronRight"))
								.DesiredSizeOverride(FVector2D(10, 24))
								.ColorAndOpacity(FStyleColors::Foreground)
						]
				];

			EExtensionLocation TransitionArrowLocation = GetTransitionDirection() == EStateTransitionDirection::In
				? EExtensionLocation::Right
				: EExtensionLocation::Left;

			FMargin TransitionArrowPadding = GetTransitionDirection() == EStateTransitionDirection::In
				? FMargin(6.0, 0.0, -6.0, 0.0)
				: FMargin(-6.0, 0.0, 6.0, 0.0);
				
			AddExtension(TransitionArrowExtension)
				.SetLocation(TransitionArrowLocation)
				.SetVAlign(VAlign_Center)
				.SetAutoSize()
				.SetPadding(TransitionArrowPadding);
		}

		// Generic background extension, padded to not overlap with the small transition arrow
		{
			FScopedMainNodeLayoutInfo ScopedMainLayoutInfo = SetScopedMainNodeLayoutInfo()
				.SetPadding(FMargin(4.0));

			FMargin TransitionArrowPadding = GetTransitionDirection() == EStateTransitionDirection::In
				? FMargin(0.0, 0.0, 4.0 + TransitionNodeBridgeWidth, 0.0)
				: FMargin(4.0 + TransitionNodeBridgeWidth, 0.0, 0.0, 0.0);

			const FSlateBrush* SubNodeBackgroundBrush = FStateCentricViewUtils::GetSubNodeBackground();

			TSharedPtr<SWidget> SubNodeBackground = SNew(SBorder)
				.BorderImage(SubNodeBackgroundBrush)
				.Visibility(EVisibility::HitTestInvisible);

			AddExtension(SubNodeBackground.ToSharedRef())
				.SetLocation(EExtensionLocation::Background)
				.SetAddLocationBehavior(EExtensionAddLocationBehavior::OverlayEntireNode)
				.SetPadding(TransitionArrowPadding);
		}
	}
}

TSharedRef<SWidget> SFakeTransitionInfoNode::Construct_GenerateNodeHeader(const EExtendableNodeLOD InLOD)
{
	NodeBodyPadding = FMargin(0.0, 0.0, 0.0, 0.0);

	if (ViewState.IsValid())
	{
		NodeBodyPadding = FMargin(0.0, 2.0, 0.0, 0.0);
			
		return SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(FMargin(12.0, 0.0, 2.0, 0.0))
				.AutoWidth()
				[
					SNew(SImage)
					.Image(GetFakeTranstionInfoBrush())
					.ColorAndOpacity(FStyleColors::Foreground)
					.Visibility(EVisibility::HitTestInvisible)
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(GetFakeTranstionInfoText())
					.ToolTipText(GetFakeTranstionInfoToolTipText())
					.TextStyle(&FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Normal.Normal"))
					.ColorAndOpacity(FStyleColors::White)
					.Visibility(EVisibility::Visible)
					.AutoWrapText(true)
				]
			];
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SFakeTransitionInfoNode::Construct_GenerateNodeBody(const EExtendableNodeLOD InLOD)
{
	return SNullWidget::NullWidget;
}


//////////////////////////////////////////////////////////////////////////

} // namespace UE::StateTree::Editor::StateCentricView

#undef LOCTEXT_NAMESPACE

