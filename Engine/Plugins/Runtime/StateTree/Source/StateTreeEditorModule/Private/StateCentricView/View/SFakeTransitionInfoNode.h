// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "StateCentricView/View/SStateTreeExtendableNode.h"
#include "StateTreeState.h"

struct FSlateBrush;

class IStructureDetailsView;
class FStateTreeViewModel;
class SButton;
class UStateTreeState;

namespace UE::StateTree::Editor::StateCentricView
{


/**
 * Experimental. Node used to show some info in transition area that isn't a real transition.
 */
class SFakeTransitionInfoNode : public SStateTreeExtendableNode
{

protected:

	/** Get if we are intended to be shown at in or out transtions. */
	virtual EStateTransitionDirection GetTransitionDirection() const = 0;

	/** Get State to show for this node. */
	virtual const UStateTreeState* GetFakeTranstionInfoState() const = 0;

	/** Get Icon to use in header for this node. */
	virtual const FSlateBrush* GetFakeTranstionInfoBrush() const = 0;

	/** Get Text to show for this node */
	virtual FText GetFakeTranstionInfoText() const = 0;

	/** Get Text to show for this node */
	virtual FText GetFakeTranstionInfoToolTipText() const 
	{ 
		return FText::GetEmpty();
	};

protected:

	//~ Begin SExtendableNode Interface
	virtual void Construct_GenerateBuiltInExtensions(const EExtendableNodeLOD InLOD) override;
	virtual TSharedRef<SWidget> Construct_GenerateNodeHeader(const EExtendableNodeLOD InLOD) override;
	virtual TSharedRef<SWidget> Construct_GenerateNodeBody(const EExtendableNodeLOD InLOD) override;
	//~ End SExtendableNode Interface

private:

	TSharedPtr<FDeferredCleanupSlateBrush> StateBrush;
};


} // namespace UE::StateTree::Editor::StateCentricView