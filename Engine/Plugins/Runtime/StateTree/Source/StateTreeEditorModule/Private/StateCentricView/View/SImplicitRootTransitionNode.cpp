// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateCentricView/View/SImplicitRootTransitionNode.h"

#include "Customizations/StateTreeBindingExtension.h"
#include "StateCentricView/View/StateCentricViewUtils.h"
#include "StateTreeEditorStyle.h"

#include "Layout/Visibility.h"
#include "Modules/ModuleManager.h"
#include "ObjectEditorUtils.h"
#include "PropertyEditorModule.h"
#include "PropertyPathHelpers.h"
#include "Styling/SlateBrush.h"
#include "Styling/StyleColors.h"

#define LOCTEXT_NAMESPACE "SImplicitRootTransitionNode"

namespace UE::StateTree::Editor::StateCentricView
{


//////////////////////////////////////////////////////////////////////////
// SImplicitRootTransitionNode

const FName SImplicitRootTransitionNode::NodeID(TEXT("StateCentricView.ImplicitRootTranstion.Node"));

FName SImplicitRootTransitionNode::GetNodeName() const
{
	return NodeID;
}

EStateTransitionDirection SImplicitRootTransitionNode::GetTransitionDirection() const
{
	return EStateTransitionDirection::Out;
}

const UStateTreeState* SImplicitRootTransitionNode::GetFakeTranstionInfoState() const
{
	const UStateTreeState* RootState = nullptr;
	if (ViewState.IsValid())
	{
		auto SetRootState = [&RootState](UStateTreeState& ParentState)
		{
			RootState = &ParentState;
			return EStateTreeVisitor::Continue;
		};

		ViewModel->ForEachParent(ViewState.Get(), SetRootState);
	}

	return RootState;
}

const FSlateBrush* SImplicitRootTransitionNode::GetFakeTranstionInfoBrush() const
{
	return FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Transition.Parent");
}

FText SImplicitRootTransitionNode::GetFakeTranstionInfoText() const
{
	return LOCTEXT("ImplicitRootTransitionNodeHeader", "Implict Transition To Root");
}

FText SImplicitRootTransitionNode::GetFakeTranstionInfoToolTipText() const
{
	return LOCTEXT("ImplicitRootTransitionNodeToolTip"
		, "When no transitions exist. StateTree will automatically add an internal OnComplete transition to 'Root' to avoid getting stuck.");
}


//////////////////////////////////////////////////////////////////////////

} // namespace UE::StateTree::Editor::StateCentricView

#undef LOCTEXT_NAMESPACE

