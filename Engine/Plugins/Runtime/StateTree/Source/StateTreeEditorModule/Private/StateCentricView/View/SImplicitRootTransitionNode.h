// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "StateCentricView/View/SFakeTransitionInfoNode.h"

namespace UE::StateTree::Editor::StateCentricView
{


/**
 * Experimental. Node used to show the implicit transtion to root on complete.
 */
class SImplicitRootTransitionNode : public SFakeTransitionInfoNode
{

public:

	static const FName NodeID;

public:

	//~ Begin SExtendableNode Interface
	virtual FName GetNodeName() const override;
	//~ End SExtendableNode Interface

protected:

	//~ Begin SFakeTransitionInfoNode Interface
	virtual EStateTransitionDirection GetTransitionDirection() const override;
	virtual const UStateTreeState* GetFakeTranstionInfoState() const override;
	virtual const FSlateBrush* GetFakeTranstionInfoBrush() const override;
	virtual FText GetFakeTranstionInfoText() const override;
	virtual FText GetFakeTranstionInfoToolTipText() const override;
	//~ End SFakeTransitionInfoNode Interface

};


} // namespace UE::StateTree::Editor::StateCentricView