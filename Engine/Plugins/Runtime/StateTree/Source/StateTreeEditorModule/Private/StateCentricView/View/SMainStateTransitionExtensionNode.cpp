// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateCentricView/View/SMainStateTransitionExtensionNode.h"

#include "StateCentricView/ViewModel/StateCentricViewEditorDataExtension.h"
#include "StateCentricView/StateCentricViewSettings.h"
#include "StateCentricView/View/StateCentricViewUtils.h"
#include "StateCentricView/View/STransitionNode.h"
#include "Customizations/Widgets/SAddTransitionMenu.h"
#include "StateTreeEditorStyle.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "InputCoreTypes.h"
#include "Styling/SlateBrush.h"
#include "Styling/StyleColors.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SMainStateTransitionExtensionNode"

namespace UE::StateTree::Editor::StateCentricView
{

/**
 * Thin wrapper widget that intercepts right-mouse-button-down events in the transition body.
 * Unhandled right-clicks from deeper widgets bubble up and are captured here so the parent
 * can show the 'Add Transition' context menu without interfering with any left-click interactions
 * on existing transition rows.
 */
class STransitionBodyMouseCapture : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STransitionBodyMouseCapture) {}
		SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_EVENT(FPointerEventHandler, OnRightMouseButtonDown)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		OnRightMouseButtonDownHandler = InArgs._OnRightMouseButtonDown;
		ChildSlot
		[ 
			InArgs._Content.Widget 
		];
	}

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton
			&& OnRightMouseButtonDownHandler.IsBound()
			&& IsDirectlyHovered())
		{
			return OnRightMouseButtonDownHandler.Execute(MyGeometry, MouseEvent);
		}
		return FReply::Unhandled();
	}

private:
	FPointerEventHandler OnRightMouseButtonDownHandler;
};


//////////////////////////////////////////////////////////////////////////
// SMainStateTransitionExtensionNode

SMainStateTransitionExtensionNode::~SMainStateTransitionExtensionNode()
{
	if (ViewModel)
	{
		if (UStateTreeEditorData* EditorData = const_cast<UStateTreeEditorData*>(ViewModel->GetStateTreeEditorData()))
		{
			if (UStateCentricViewEditorDataExtension* StateCentricEditorData = EditorData->GetExtension<UStateCentricViewEditorDataExtension>())
			{
				StateCentricEditorData->PerStateTreeUserSettings.OnInTransitionViewLOD_Changed.RemoveAll(this);
				StateCentricEditorData->PerStateTreeUserSettings.OnOutTransitionViewLOD_Changed.RemoveAll(this);
			}
		}

		ViewModel->GetTransitionViewModel().GetOnTransitionsAdded().RemoveAll(this);
		ViewModel->GetTransitionViewModel().GetOnTransitionsRemoved().RemoveAll(this);
		ViewModel->GetTransitionViewModel().GetOnTransitionsMoved().RemoveAll(this);
	}
}

void SMainStateTransitionExtensionNode::Construct_InitializeNodePersistentPreLOD()
{
	UStateTreeEditorData* EditorData = const_cast<UStateTreeEditorData*>(ViewModel->GetStateTreeEditorData());
	UStateCentricViewEditorDataExtension* StateCentricEditorData = EditorData->GetOrCreateExtension<UStateCentricViewEditorDataExtension>();

	// Remove any prior bindings in case of issues with lifetime
	StateCentricEditorData->PerStateTreeUserSettings.OnInTransitionViewLOD_Changed.RemoveAll(this);
	StateCentricEditorData->PerStateTreeUserSettings.OnOutTransitionViewLOD_Changed.RemoveAll(this);
	ViewModel->GetTransitionViewModel().GetOnTransitionsAdded().RemoveAll(this);
	ViewModel->GetTransitionViewModel().GetOnTransitionsRemoved().RemoveAll(this);
	ViewModel->GetTransitionViewModel().GetOnTransitionsMoved().RemoveAll(this);

	TOptional<EExtendableNodeLOD> TransitionViewUserSetLOD;
	if (TransitionDirection == EStateTransitionDirection::In)
	{
		StateCentricEditorData->PerStateTreeUserSettings.OnInTransitionViewLOD_Changed.AddSP(this, &SMainStateTransitionExtensionNode::HandleOnTransitionViewLOD_SettingsChanged);
		TransitionViewUserSetLOD = StateCentricEditorData->PerStateTreeUserSettings.GetInTransitionLOD();
	}
	else if (ensure(TransitionDirection == EStateTransitionDirection::Out))
	{
		StateCentricEditorData->PerStateTreeUserSettings.OnOutTransitionViewLOD_Changed.AddSP(this, &SMainStateTransitionExtensionNode::HandleOnTransitionViewLOD_SettingsChanged);
		TransitionViewUserSetLOD = StateCentricEditorData->PerStateTreeUserSettings.GetOutTransitionLOD();
	}

	// We were given a LOD to use, but based on user config we may want to use a different LOD for now
	if (TransitionViewUserSetLOD)
	{
		CurrentLOD = TransitionViewUserSetLOD.GetValue();
	}

	ViewModel->GetTransitionViewModel().GetOnTransitionsAdded().AddSP(this, &SMainStateTransitionExtensionNode::HandleOnTransitionsNumChanged);
	ViewModel->GetTransitionViewModel().GetOnTransitionsRemoved().AddSP(this, &SMainStateTransitionExtensionNode::HandleOnTransitionsNumChanged);
	ViewModel->GetTransitionViewModel().GetOnTransitionsMoved().AddSP(this, &SMainStateTransitionExtensionNode::HandleOnTransitionsNumChanged);
}

void SMainStateTransitionExtensionNode::Construct_GenerateBuiltInExtensions(const EExtendableNodeLOD InLOD)
{
	// Entire widget state color outline in overlay extension depending on LOD
	if (InLOD != EExtendableNodeLOD::Hidden && InLOD != EExtendableNodeLOD::Collapsed)
	{
		StateOutlineBrush = FStateCentricViewUtils::MakeStateOutline(ViewModel.ToSharedRef(), ViewState.Get());

		TSharedPtr<SWidget> StateColorOutline = SNew(SBorder)
			.BorderImage(StateOutlineBrush->GetSlateBrush())
			.Visibility(EVisibility::HitTestInvisible);

		AddExtension(StateColorOutline.ToSharedRef())
			.SetLocation(EExtensionLocation::Foreground)
			.SetPadding(FMargin(0.0, 0.0, 0.0, 0.0));
	}

	// Muted background if hidden
	if (InLOD == EExtendableNodeLOD::Hidden)
	{
		StateBackgroundBrush = FStateCentricViewUtils::MakeStateCollapsedBackground(ViewModel.ToSharedRef(), ViewState.Get());

		TSharedPtr<SWidget> StateColorBackground = SNew(SBorder)
			.BorderImage(StateBackgroundBrush->GetSlateBrush())
			.Visibility(EVisibility::HitTestInvisible);

		double PaddingBackgroundSize = 16.0;

		FMargin StateBackgroundPadding;
		if (TransitionDirection == EStateTransitionDirection::In)
		{
			StateBackgroundPadding = FMargin(-2.0 + PaddingBackgroundSize, 4.0, 2.0, 4.0);
		}
		else if (ensure(TransitionDirection == EStateTransitionDirection::Out))
		{
			StateBackgroundPadding = FMargin(2.0, 4.0, -2.0 + PaddingBackgroundSize, 4.0);
		}

		AddExtension(StateColorBackground.ToSharedRef())
			.SetLocation(EExtensionLocation::Background)
			.SetPadding(StateBackgroundPadding);
	}
}

TSharedRef<SWidget> SMainStateTransitionExtensionNode::Construct_GenerateNodeHeader(const EExtendableNodeLOD InLOD)
{
	if (ViewState.IsValid())
	{
		UStateTreeEditorData* EditorData = const_cast<UStateTreeEditorData*>(ViewModel->GetStateTreeEditorData());
		UStateCentricViewEditorDataExtension* StateCentricEditorData = EditorData->GetOrCreateExtension<UStateCentricViewEditorDataExtension>();

		UpdateHideButtonAttributes();

		if (TransitionDirection == EStateTransitionDirection::In)
		{
			StateCentricEditorData->InTransitionSplitterSizeRule = StateCentricEditorData->PerStateTreeUserSettings.GetInTransitionLOD() == EExtendableNodeLOD::Hidden
				? SSplitter::ESizeRule::SizeToContent
				: SSplitter::ESizeRule::FractionOfParent;
		}
		else if (ensure(TransitionDirection == EStateTransitionDirection::Out))
		{
			StateCentricEditorData->OutTransitionSplitterSizeRule = StateCentricEditorData->PerStateTreeUserSettings.GetOutTransitionLOD() == EExtendableNodeLOD::Hidden
				? SSplitter::ESizeRule::SizeToContent
				: SSplitter::ESizeRule::FractionOfParent;
		}

		SAssignNew(HideButton, SButton)
			.OnClicked(this, &SMainStateTransitionExtensionNode::HandleOnCollapseClicked)
			.ButtonStyle(&FAppStyle::GetWidgetStyle<FButtonStyle>("NoBorder"))
			.ContentPadding(FMargin())
			[
				SNew(SImage)
				.Image(&FAppStyle::Get().GetWidgetStyle<FExpandableAreaStyle>("ExpandableArea").CollapsedImage)
				.RenderTransform(this, &SMainStateTransitionExtensionNode::GetHideButtonRenderTransform)
				.ColorAndOpacity(this, &SMainStateTransitionExtensionNode::GetHideButtonColor)
			];

		// Header is just a button when transitions hidden
		if (InLOD == EExtendableNodeLOD::Hidden)
		{
			return SNew(SOverlay)
				+ SOverlay::Slot()
				[
					// Setting background brush 32 px as it must be able to contain the child button + padding
					// Otherwise splitter layout calculation will be uneven. We rely on fixed 32px size to be in sync with parents when hidden.
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Brushes.Recessed"))
					.ColorAndOpacity(FStyleColors::Transparent)
					.DesiredSizeOverride(FVector2D(32.0f, 28.0f))
				]
				+ SOverlay::Slot()
				.HAlign(HideButtonAlignment)
				.VAlign(VAlign_Center)
				.Padding(MakeAttributeSP(this, &SMainStateTransitionExtensionNode::GetHideButtonPadding))
				[
					HideButton.ToSharedRef()
				];
		}

		TArray<FStateTreeTransition*> StateTransitions;
		EHorizontalAlignment HeaderAlignment = HAlign_Left;
		FMargin HeaderPadding;
		FText HeaderText;
		if (TransitionDirection == EStateTransitionDirection::In)
		{
			StateTransitions = ViewModel->GetStateInTransitions(ViewState.Get());
			HeaderAlignment = HAlign_Left;
			HeaderPadding = FMargin(12.0, 0.0, 2.0, 0.0);
			HeaderText = LOCTEXT("TransitionHeaderIn", "IN ({0})");
		}
		else if (ensure(TransitionDirection == EStateTransitionDirection::Out))
		{
			StateTransitions = ViewModel->GetStateOutTransitions(ViewState.Get());
			HeaderAlignment = HAlign_Right;
			HeaderPadding = FMargin(2.0, 0.0, 12.0, 0.0);
			HeaderText = LOCTEXT("TransitionHeaderOut", "OUT ({0})");
		}

		HeaderText = FText::Format(HeaderText, StateTransitions.Num());

		StateHeaderBackgroundBrush = FStateCentricViewUtils::MakeStateHeaderBackground(ViewModel.ToSharedRef(), ViewState.Get(), FVector2f(32.0f, 28.0f));

		TSharedPtr<SHorizontalBox> HeaderContentsBox;

		TSharedRef<SOverlay> HeaderWidget = SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image(StateHeaderBackgroundBrush->GetSlateBrush())
			]
			+ SOverlay::Slot()
			[
				SAssignNew(HeaderContentsBox, SHorizontalBox)
				.Clipping(EWidgetClipping::OnDemand)
				+ SHorizontalBox::Slot()
				.HAlign(HeaderAlignment)
				.VAlign(VAlign_Center)
				.FillWidth(1.0f)
				.Padding(HeaderPadding)
				[
					SNew(STextBlock)
					.Text(HeaderText)
					.TextStyle(&FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Normal.Normal"))
					.ColorAndOpacity(FStyleColors::White)
				]
			];

		// In full LOD, we add a button to allow for new transitions to be added
		if (InLOD == EExtendableNodeLOD::Full)
		{
			HeaderContentsBox->AddSlot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(FMargin(-8.0, 0.0, 8.0, 0.0))
			[
				SAssignNew(AddTransitionButton, SButton)
				.OnClicked(this, &SMainStateTransitionExtensionNode::HandleOnAddTransitionClicked)
				.ButtonStyle(&FAppStyle::GetWidgetStyle<FButtonStyle>("NoBorder"))
				.ContentPadding(FMargin())
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
					.ColorAndOpacity(this, &SMainStateTransitionExtensionNode::GetAddTransitionColor)
				]
			];
		}

		HeaderWidget->AddSlot()
			.HAlign(HideButtonAlignment)
			.VAlign(VAlign_Center)
			.Padding(MakeAttributeSP(this, &SMainStateTransitionExtensionNode::GetHideButtonPadding))
			[
				HideButton.ToSharedRef()
			];

		return HeaderWidget;
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SMainStateTransitionExtensionNode::Construct_GenerateNodeBody(const EExtendableNodeLOD InLOD)
{
	if (ViewState.IsValid())
	{
		TSubclassOf<UStateTreeSchema> SchemaClass = nullptr;

		if (const UStateTreeSchema* Schema = ViewModel->GetStateTreeSchema())
		{
			SchemaClass = Schema->GetClass();
		}

		if (UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(SchemaClass).bJustifyCentralViewTransitions)
		{
			NodeBodySizeParam = FStretchContent(1.0f);
		}

		TSharedRef<SWidget> TransitionsView =
			FStateCentricViewUtils::GenerateTransitionsViewWidget(InLOD, ViewModel.ToSharedRef(), ViewState.Get(), TransitionDirection);

		// At Full LOD, wrap the body in a mouse-capture widget so right-clicking anywhere
		// in the area (including behind existing transition rows) opens the Add Transition menu.
		if (InLOD == EExtendableNodeLOD::Full)
		{
			return SNew(STransitionBodyMouseCapture)
				.OnRightMouseButtonDown(this, &SMainStateTransitionExtensionNode::HandleTransitionAreaRightClick)
				[ 
					TransitionsView 
				];
		}

		return TransitionsView;
	}

	return SNullWidget::NullWidget;
}

FReply SMainStateTransitionExtensionNode::HandleOnCollapseClicked()
{
	if (ViewState.IsValid())
	{
		UStateTreeEditorData* EditorData = const_cast<UStateTreeEditorData*>(ViewModel->GetStateTreeEditorData());
		UStateCentricViewEditorDataExtension* StateCentricEditorData = EditorData->GetOrCreateExtension<UStateCentricViewEditorDataExtension>();

		if (TransitionDirection == EStateTransitionDirection::In)
		{
			bool bHidden = StateCentricEditorData->PerStateTreeUserSettings.GetInTransitionLOD().Get(EExtendableNodeLOD::Default) == EExtendableNodeLOD::Hidden;

			if (bHidden)
			{
				StateCentricEditorData->PerStateTreeUserSettings.SetInTransitionLOD({});
			}
			else
			{
				StateCentricEditorData->PerStateTreeUserSettings.SetInTransitionLOD(EExtendableNodeLOD::Hidden);
			}
		}
		else if (ensure(TransitionDirection == EStateTransitionDirection::Out))
		{
			bool bHidden = StateCentricEditorData->PerStateTreeUserSettings.GetOutTransitionLOD().Get(EExtendableNodeLOD::Default) == EExtendableNodeLOD::Hidden;

			if (bHidden)
			{
				StateCentricEditorData->PerStateTreeUserSettings.SetOutTransitionLOD({});
			}
			else
			{
				StateCentricEditorData->PerStateTreeUserSettings.SetOutTransitionLOD(EExtendableNodeLOD::Hidden);
			}
		}
	}

	return FReply::Handled();
}

FReply SMainStateTransitionExtensionNode::HandleOnAddTransitionClicked()
{
	if (ViewState.IsValid())
	{
		FStateTreeTransition DefaultTransition = {};
		ViewModel->AddTransition(ViewState.Get(), DefaultTransition.Trigger, EStateTreeTransitionType::None);
	}
	return FReply::Handled();
}

FReply SMainStateTransitionExtensionNode::HandleTransitionAreaRightClick(
	const FGeometry& /*MyGeometry*/, const FPointerEvent& MouseEvent)
{
	if (!ViewState.IsValid())
	{
		return FReply::Unhandled();
	}

	FMenuBuilder MenuBuilder(/*bCloseAfterSelection=*/ true, /*InCommandList=*/ nullptr);
	MenuBuilder.AddWidget(
		SNew(SBox)
			.MinDesiredWidth(300.f)
			.MaxDesiredHeight(400.f)
			[ 
				BuildTransitionStatePicker() 
			],
		FText::GetEmpty(),
		/*bNoIndent=*/ true);

	const FWidgetPath WidgetPath = MouseEvent.GetEventPath() ? *MouseEvent.GetEventPath() : FWidgetPath();
	FSlateApplication::Get().PushMenu(
		AsShared(),
		WidgetPath,
		MenuBuilder.MakeWidget(),
		MouseEvent.GetScreenSpacePosition(),
		FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));

	return FReply::Handled();
}

TSharedRef<SWidget> SMainStateTransitionExtensionNode::BuildTransitionStatePicker()
{
	return SNew(UE::StateTree::Editor::SAddTransitionMenu)
		.OwnerState(ViewState.Get())
		.ViewModel(ViewModel);
}

void SMainStateTransitionExtensionNode::HandleOnTransitionViewLOD_SettingsChanged(TOptional<EExtendableNodeLOD> InLOD)
{
	UpdateHideButtonAttributes();

	if (ViewState.IsValid())
	{
		UStateTreeEditorData* EditorData = const_cast<UStateTreeEditorData*>(ViewModel->GetStateTreeEditorData());
		UStateCentricViewEditorDataExtension* StateCentricEditorData = EditorData->GetOrCreateExtension<UStateCentricViewEditorDataExtension>();

		TOptional<EExtendableNodeLOD> TransitionViewLOD;
		if (TransitionDirection == EStateTransitionDirection::In)
		{
			TransitionViewLOD = StateCentricEditorData->PerStateTreeUserSettings.GetInTransitionLOD();
		}
		else if (ensure(TransitionDirection == EStateTransitionDirection::Out))
		{
			TransitionViewLOD = StateCentricEditorData->PerStateTreeUserSettings.GetOutTransitionLOD();
		}

		if (TransitionViewLOD)
		{
			RegenerateWidgetForLOD(TransitionViewLOD.GetValue());
		}
		else
		{
			// @TODO: May be better to use a PrevLOD. But this can get complicated so for later
			RegenerateWidgetForLOD(Config.InitialLOD);
		}
	}
}

void SMainStateTransitionExtensionNode::HandleOnTransitionsNumChanged(const UStateTreeState* /*OwningState*/, const TSet<FGuid>& /*TransitionIDs*/)
{
	RegenerateWidgetForLOD(CurrentLOD);
}

FSlateColor SMainStateTransitionExtensionNode::GetAddTransitionColor() const
{
	return AddTransitionButton->IsHovered() ? FStyleColors::White : FStyleColors::Foreground;
}

FSlateColor SMainStateTransitionExtensionNode::GetHideButtonColor() const
{
	return HideButton->IsHovered() ? FStyleColors::White : FStyleColors::Foreground;
}

void SMainStateTransitionExtensionNode::UpdateHideButtonAttributes()
{
	if (ViewState.IsValid())
	{
		UStateTreeEditorData* EditorData = const_cast<UStateTreeEditorData*>(ViewModel->GetStateTreeEditorData());
		UStateCentricViewEditorDataExtension* StateCentricEditorData = EditorData->GetOrCreateExtension<UStateCentricViewEditorDataExtension>();

		double IconWidth = 16.0;
		if (TransitionDirection == EStateTransitionDirection::In)
		{
			bool bHidden = StateCentricEditorData->PerStateTreeUserSettings.GetInTransitionLOD().Get(EExtendableNodeLOD::Default) == EExtendableNodeLOD::Hidden;
			HideButtonAlignment = HAlign_Right;
			HideButtonPadding = bHidden
				? FMargin(0.0, 0.0, 2.0 - IconWidth, 0.0)
				: FMargin(0.0, 0.0, 2.0, 0.0);
			HideButtonRenderTransform = bHidden
				? FSlateRenderTransform(FScale2D(-1, 1))
				: FSlateRenderTransform();
		}
		else if (ensure(TransitionDirection == EStateTransitionDirection::Out))
		{
			bool bHidden = StateCentricEditorData->PerStateTreeUserSettings.GetOutTransitionLOD().Get(EExtendableNodeLOD::Default) == EExtendableNodeLOD::Hidden;
			HideButtonAlignment = HAlign_Left;
			HideButtonPadding = bHidden
				? FMargin(2.0, 0.0, 0.0, 0.0)
				: FMargin(2.0 + IconWidth, 0.0, 0.0, 0.0);
			HideButtonRenderTransform = bHidden
				? FSlateRenderTransform()
				: FSlateRenderTransform(FScale2D(-1, 1));
		}
	}
}

TOptional<FSlateRenderTransform> SMainStateTransitionExtensionNode::GetHideButtonRenderTransform() const
{
	return HideButtonRenderTransform;
}

FMargin SMainStateTransitionExtensionNode::GetHideButtonPadding() const
{
	return HideButtonPadding;
}

EHorizontalAlignment SMainStateTransitionExtensionNode::GetHideButtonAlignment() const
{
	return HideButtonAlignment;
}


//////////////////////////////////////////////////////////////////////////
// SMainStateInTransitionExtensionNode

const FName SMainStateInTransitionExtensionNode::NodeID(TEXT("StateCentricView.MainState.InTransitions"));

FName SMainStateInTransitionExtensionNode::GetNodeName() const
{
	return NodeID;
}

void SMainStateInTransitionExtensionNode::Construct_InitializeNodePersistentPreLOD()
{
	TransitionDirection = EStateTransitionDirection::In;
	SMainStateTransitionExtensionNode::Construct_InitializeNodePersistentPreLOD();
}


//////////////////////////////////////////////////////////////////////////
// SMainStateOutTransitionExtensionNode

const FName SMainStateOutTransitionExtensionNode::NodeID(TEXT("StateCentricView.MainState.OutTransitions"));

FName SMainStateOutTransitionExtensionNode::GetNodeName() const
{
	return NodeID;
}

void SMainStateOutTransitionExtensionNode::Construct_InitializeNodePersistentPreLOD()
{
	TransitionDirection = EStateTransitionDirection::Out;
	SMainStateTransitionExtensionNode::Construct_InitializeNodePersistentPreLOD();
}


//////////////////////////////////////////////////////////////////////////

} // namespace UE::StateTree::Editor::StateCentricView

#undef LOCTEXT_NAMESPACE

