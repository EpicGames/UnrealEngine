// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "StateCentricView/View/SStateTreeExtendableNode.h"

struct FSlateBrush;

class SButton;
class UStateTreeEditorData;

namespace UE::StateTree::Editor::StateCentricView
{


/**
 * Experimental. Node used as an extension for In / Out transitions on the main state.
 */
class SMainStateTransitionExtensionNode : public SStateTreeExtendableNode
{
public:

	virtual ~SMainStateTransitionExtensionNode() override;

protected:

	//~ Begin SExtendableNode Interface
	virtual void Construct_InitializeNodePersistentPreLOD() override;
	virtual void Construct_GenerateBuiltInExtensions(const EExtendableNodeLOD InLOD) override;
	virtual TSharedRef<SWidget> Construct_GenerateNodeHeader(const EExtendableNodeLOD InLOD) override;
	virtual TSharedRef<SWidget> Construct_GenerateNodeBody(const EExtendableNodeLOD InLOD) override;
	//~ End SExtendableNode Interface

protected:

	/** If we are showing in / out transitions. Init in child classes. */
	EStateTransitionDirection TransitionDirection = EStateTransitionDirection::Out;

private:

	/** Callback for collapse button*/
	FReply HandleOnCollapseClicked();

	/** Callback for top right button */
	FReply HandleOnAddTransitionClicked();

	/** Callback for right-click anywhere in the body - opens an 'Add Transition' context menu. */
	FReply HandleTransitionAreaRightClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/** Builds the state-picker widget embedded in the 'Add Transition' context menu. */
	TSharedRef<SWidget> BuildTransitionStatePicker();

	/** Callback for when LOD settings change */
	void HandleOnTransitionViewLOD_SettingsChanged(TOptional<EExtendableNodeLOD> InLOD);

	/** Callback for transtions added / removed */
	void HandleOnTransitionsNumChanged(const UStateTreeState* /*OwningState*/, const TSet<FGuid>& /*TransitionIDs*/);

	/** Minor UI / UX callbacks */
	FSlateColor GetAddTransitionColor() const;
	FSlateColor GetHideButtonColor() const;
	
	void UpdateHideButtonAttributes();
	FMargin GetHideButtonPadding() const;
	EHorizontalAlignment GetHideButtonAlignment() const;
	TOptional<FSlateRenderTransform> GetHideButtonRenderTransform() const;

private:

	TSharedPtr<FDeferredCleanupSlateBrush> StateOutlineBrush;
	TSharedPtr<FDeferredCleanupSlateBrush> StateBackgroundBrush;
	TSharedPtr<FDeferredCleanupSlateBrush> StateHeaderBackgroundBrush;

	TSharedPtr<SButton> AddTransitionButton;
	TSharedPtr<SButton> HideButton;

	FMargin HideButtonPadding = FMargin();
	EHorizontalAlignment HideButtonAlignment = HAlign_Fill;
	FSlateRenderTransform HideButtonRenderTransform = FSlateRenderTransform();
};


/**
 * Experimental. Specialization of SMainStateTransitionExtensionNode for 'In' transitions
 */
class SMainStateInTransitionExtensionNode : public SMainStateTransitionExtensionNode
{

public:

	static const FName NodeID;

public:

	//~ Begin SExtendableNode Interface
	virtual FName GetNodeName() const override;
	//~ End SExtendableNode Interface

protected:

	//~ Begin SExtendableNode Interface
	virtual void Construct_InitializeNodePersistentPreLOD() override;
	//~ End SExtendableNode Interface

};


/**
 * Experimental. Specialization of SMainStateTransitionExtensionNode for 'Out' transitions
 */
class SMainStateOutTransitionExtensionNode : public SMainStateTransitionExtensionNode
{

public:

	static const FName NodeID;

public:

	//~ Begin SExtendableNode Interface
	virtual FName GetNodeName() const override;
	//~ End SExtendableNode Interface

protected:

	//~ Begin SExtendableNode Interface
	virtual void Construct_InitializeNodePersistentPreLOD() override;
	//~ End SExtendableNode Interface

};


} // namespace UE::StateTree::Editor::StateCentricView