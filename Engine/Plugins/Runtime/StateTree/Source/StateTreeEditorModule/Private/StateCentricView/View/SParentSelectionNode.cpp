// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateCentricView/View/SParentSelectionNode.h"

#include "Customizations/StateTreeBindingExtension.h"
#include "StateCentricView/View/StateCentricViewUtils.h"
#include "StateTreeEditorStyle.h"

#include "Layout/Visibility.h"
#include "Modules/ModuleManager.h"
#include "ObjectEditorUtils.h"
#include "PropertyEditorModule.h"
#include "PropertyPathHelpers.h"
#include "Styling/SlateBrush.h"
#include "Styling/StyleColors.h"

#define LOCTEXT_NAMESPACE "SParentSelectionNode"

namespace UE::StateTree::Editor::StateCentricView
{


//////////////////////////////////////////////////////////////////////////
// SParentSelectionNode

const FName SParentSelectionNode::NodeID(TEXT("StateCentricView.ParentSelection.Node"));

FName SParentSelectionNode::GetNodeName() const
{
	return NodeID;
}

EStateTransitionDirection SParentSelectionNode::GetTransitionDirection() const
{
	return EStateTransitionDirection::In;
}

const UStateTreeState* SParentSelectionNode::GetFakeTranstionInfoState() const
{
	if (ViewState.IsValid())
	{
		return ViewState->Parent;
	}

	return nullptr;
}

const FSlateBrush* SParentSelectionNode::GetFakeTranstionInfoBrush() const
{
	return FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.TrySelectChildrenInOrder");
}

FText SParentSelectionNode::GetFakeTranstionInfoText() const
{
	if (ViewState.IsValid())
	{
		// Show Parent Selection method
		if (const UStateTreeState* ParentState = ViewState->Parent)
		{
			return StaticEnum<EStateTreeStateSelectionBehavior>()->GetDisplayNameTextByValue((int64)ParentState->SelectionBehavior);
		}
	}

	return FText::GetEmpty();
}

FText SParentSelectionNode::GetFakeTranstionInfoToolTipText() const
{
	return LOCTEXT("ParentSelectionNodeToolTip"
		, "Parent Selection Mode. This state be entered via parent section.");
}


//////////////////////////////////////////////////////////////////////////

} // namespace UE::StateTree::Editor::StateCentricView

#undef LOCTEXT_NAMESPACE

