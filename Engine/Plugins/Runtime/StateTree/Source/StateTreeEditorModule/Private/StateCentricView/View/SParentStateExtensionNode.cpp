// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateCentricView/View/SParentStateExtensionNode.h"

#include "Customizations/StateTreeBindingExtension.h"
#include "StateCentricView/ViewModel/StateCentricViewEditorDataExtension.h"
#include "StateCentricView/View/SParentStateTransitionExtensionNode.h"
#include "StateCentricView/View/StateCentricViewUtils.h"
#include "StateTreeEditorStyle.h"

#include "IDetailsView.h"
#include "DetailsViewArgs.h"
#include "Layout/Visibility.h"
#include "Modules/ModuleManager.h"
#include "ObjectEditorUtils.h"
#include "PropertyEditorModule.h"
#include "Styling/SlateBrush.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SStateTreeExtendableNode"

namespace UE::StateTree::Editor::StateCentricView
{

const FName SParentStateExtensionNode::NodeID(TEXT("StateCentricView.ParentState.Node"));

FName SParentStateExtensionNode::GetNodeName() const
{
	return NodeID;
}

void SParentStateExtensionNode::Construct_GenerateBuiltInExtensions(const EExtendableNodeLOD InLOD)
{
	ChildStateBrushes.Reset();

	if (ViewState.IsValid())
	{
		// @TODO: Remove splitter references here once those become a feature toggle on cardinal extensions
		UStateTreeEditorData* EditorData = const_cast<UStateTreeEditorData*>(ViewModel->GetStateTreeEditorData());
		UStateCentricViewEditorDataExtension* StateCentricEditorData = EditorData->GetOrCreateExtension<UStateCentricViewEditorDataExtension>();
		
		// In Transition extension on left
		{
			FScopedMainNodeLayoutInfo ScopedMainLayoutInfo = SetScopedMainNodeLayoutInfo()
				.SetSplitterValue(MakeAttributeUObject(StateCentricEditorData, &UStateCentricViewEditorDataExtension::HandleGetMainNodeSplitterValue))
				.SetSplitterOnSlotResized(SSplitter::FOnSlotResized::CreateUObject(StateCentricEditorData, &UStateCentricViewEditorDataExtension::HandleOnMainNodeSplitterResized));

			// @TODO: Read Init LOD from project settings for in transitions.
			EExtendableNodeLOD TransitionsInLOD = InLOD == EExtendableNodeLOD::Collapsed ? EExtendableNodeLOD::Collapsed : EExtendableNodeLOD::Minimal;
			TSharedRef<SWidget> InTransitionLeftExtension = SNew(SParentStateInTransitionExtensionNode, ViewModel.ToSharedRef(), ViewState.Get())
				.Config({ .InitialLOD = TransitionsInLOD });

			AddExtension(InTransitionLeftExtension)
				.SetLocation(EExtensionLocation::SplitLeft)
				.SetSplitterSizeRule(MakeAttributeUObject(StateCentricEditorData, &UStateCentricViewEditorDataExtension::GetInTransitionSplitterSizeRule))
				.SetSplitterValue(MakeAttributeUObject(StateCentricEditorData, &UStateCentricViewEditorDataExtension::HandleGetInTransitionSplitterValue))
				.SetSplitterOnSlotResized(SSplitter::FOnSlotResized::CreateUObject(StateCentricEditorData, &UStateCentricViewEditorDataExtension::HandleOnInTransitionSplitterResized));

			// Add spacer should we hit max size caps, Needed since fill + max size = left align (Slate bug).
			AddExtension(
				SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Brushes.Recessed"))
					.DesiredSizeOverride(FVector2D(32.0, 8.0)))
				.SetLocation(EExtensionLocation::SplitLeft)
				.SetSplitterValue(MakeAttributeUObject(StateCentricEditorData, &UStateCentricViewEditorDataExtension::HandleGetLeftSpacerSplitterValue))
				.SetSplitterOnSlotResized(SSplitter::FOnSlotResized::CreateUObject(StateCentricEditorData, &UStateCentricViewEditorDataExtension::HandleOnLeftSpacerSplitterResized));
		}

		// Out Transition extension on right
		{
			// @TODO: Read Init LOD from project settings for out transitions.
			EExtendableNodeLOD TransitionsOutLOD = InLOD;
			TSharedRef<SWidget> OutTransitionRightExtension = SNew(SParentStateOutTransitionExtensionNode, ViewModel.ToSharedRef(), ViewState.Get())
				.Config({ .InitialLOD = TransitionsOutLOD });
			
			AddExtension(OutTransitionRightExtension)
				.SetLocation(EExtensionLocation::SplitRight)
				.SetSplitterSizeRule(MakeAttributeUObject(StateCentricEditorData, &UStateCentricViewEditorDataExtension::GetOutTransitionSplitterSizeRule))
				.SetSplitterValue(MakeAttributeUObject(StateCentricEditorData, &UStateCentricViewEditorDataExtension::HandleGetOutTransitionSplitterValue))
				.SetSplitterOnSlotResized(SSplitter::FOnSlotResized::CreateUObject(StateCentricEditorData, &UStateCentricViewEditorDataExtension::HandleOnOutTransitionSplitterResized));

			// Add spacer should we hit max size caps, Needed since fill + max size = left align (Slate bug).
			AddExtension(
				SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Brushes.Recessed"))
					.DesiredSizeOverride(FVector2D(32.0, 8.0)))
				.SetLocation(EExtensionLocation::SplitRight)
				.SetSplitterValue(MakeAttributeUObject(StateCentricEditorData, &UStateCentricViewEditorDataExtension::HandleGetRightSpacerSplitterValue))
				.SetSplitterOnSlotResized(SSplitter::FOnSlotResized::CreateUObject(StateCentricEditorData, &UStateCentricViewEditorDataExtension::HandleOnRightSpacerSplitterResized));
		}

		// Child states in footnote depending on LOD
		FStateCentricViewUtils::AddChildStateFootnoteExtensions(InLOD, this, OUT ChildStateBrushes);

		// Also small bottom rounded rectangle if full LOD
		if (InLOD == EExtendableNodeLOD::Full)
		{
			StateFootnoteBackgroundBrush = FStateCentricViewUtils::MakeStateFootnoteBackground(ViewModel.ToSharedRef(), ViewState.Get());

			AddExtension(
				SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Brushes.Recessed"))
					.DesiredSizeOverride(FVector2D(32.0, 8.0)))
				.SetLocation(EExtensionLocation::Footnote)
				.SetPadding(FMargin(0, 0, 0, 0));
		}

		// Entire widget state color outline in foreground extension, half circle if not full LOD
		{
			// Find what depth of a parent we are to find an overlay layer that will overlap correctly
			int32 ParentDepth = 0;
			TArray<UStateTreeState*> CentralStateParents = ViewModel->GetStateParents(ViewModel->GetViewState());
			for (UStateTreeState* Parent : CentralStateParents)
			{
				ParentDepth++;
				if (Parent->ID == ViewState->ID)
				{
					break;
				}
			}

			int32 OverlayOffset = (CentralStateParents.Num() - ParentDepth) * 5000;

			FScopedMainNodeLayoutInfo ScopedMainLayoutInfo = SetScopedMainNodeLayoutInfo()
				.SetOverlayOffset(OverlayOffset);

			if (InLOD == EExtendableNodeLOD::Full)
			{
				StateOutlineBrush = FStateCentricViewUtils::MakeStateOutlineMuted(ViewModel.ToSharedRef(), ViewState.Get());
			}
			else if (InLOD == EExtendableNodeLOD::Minimal)
			{
				StateOutlineBrush = FStateCentricViewUtils::MakeStateHalfTopOutlineMuted(ViewModel.ToSharedRef(), ViewState.Get());
			}
			else if (ensure(InLOD == EExtendableNodeLOD::Collapsed))
			{
				StateOutlineBrush = FStateCentricViewUtils::MakeStateHalfTopOutlineCollapsed(ViewModel.ToSharedRef(), ViewState.Get());
			}

			TSharedPtr<SWidget> StateColorOutline = SNew(SBorder)
				.BorderImage(StateOutlineBrush->GetSlateBrush())
				.Visibility(EVisibility::HitTestInvisible);

			if (InLOD == EExtendableNodeLOD::Full)
			{
				AddExtension(StateColorOutline.ToSharedRef())
					.SetLocation(EExtensionLocation::Foreground);
			}
			else if (ensure(InLOD == EExtendableNodeLOD::Collapsed || InLOD == EExtendableNodeLOD::Minimal))
			{
				// Negative pad based on size guess so outline edges reach to bottom of widget / beind next widget even with corner culling.
				EExtendableNodeLOD TransitionsInLOD = InLOD == EExtendableNodeLOD::Collapsed ? EExtendableNodeLOD::Collapsed : EExtendableNodeLOD::Minimal;
				EExtendableNodeLOD TransitionsOutLOD = InLOD;

				TArray<FStateTreeTransition*> TransitionsIn = ViewModel->GetStateInTransitions(ViewState.Get());
				TArray<FStateTreeTransition*> TransitionsOut = ViewModel->GetStateOutTransitions(ViewState.Get());

				int32 TransitionsInSizeEstimate = TransitionsInLOD == EExtendableNodeLOD::Collapsed ? -100 : TransitionsIn.Num() * -150;
				int32 TransitionsOutSizeEstimate = TransitionsOutLOD == EExtendableNodeLOD::Collapsed ? -100 : TransitionsOut.Num() * -150;

				AddExtension(StateColorOutline.ToSharedRef())
					.SetLocation(EExtensionLocation::Foreground)
					.SetPadding(FMargin(0.0, 0.0, 0.0, -200.0 + FMath::Min(TransitionsInSizeEstimate, TransitionsOutSizeEstimate)));
			}
		}

	}
}

TSharedRef<SWidget> SParentStateExtensionNode::Construct_GenerateNodeHeader(const EExtendableNodeLOD InLOD)
{
	if (ViewState.IsValid())
	{
		// @TODO: It should also write to project settings for LOD / only do that & we respond in this widget

		StateHeaderBackgroundBrush = FStateCentricViewUtils::MakeStateHeaderBackground(ViewModel.ToSharedRef(), ViewState.Get());

		const FSlateBrush* ExpandButtonBrush = nullptr;
		if (InLOD == EExtendableNodeLOD::Collapsed)
		{
			ExpandButtonBrush = &FAppStyle::Get().GetWidgetStyle<FExpandableAreaStyle>("ExpandableArea").CollapsedImage;
			StateHeaderBackgroundBrush = FStateCentricViewUtils::MakeStateHeaderBackgroundCollapsed(ViewModel.ToSharedRef(), ViewState.Get());
		}
		else if (InLOD == EExtendableNodeLOD::Minimal)
		{
			ExpandButtonBrush = &FAppStyle::Get().GetWidgetStyle<FExpandableAreaStyle>("ExpandableArea").ExpandedImage;
		}
		else
		{
			ensure(InLOD == EExtendableNodeLOD::Full);
			ExpandButtonBrush = FAppStyle::Get().GetBrush("Icons.Details");
		}
		
		return SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image(StateHeaderBackgroundBrush->GetSlateBrush())
			]
			+ SOverlay::Slot()
			.Padding(FMargin(8.0, 2.0, 32.0, 2.0))
			[
				FStateCentricViewUtils::GenerateGotoStateButton(InLOD, ViewModel.ToSharedRef(), ViewState.Get())
			]
			+ SOverlay::Slot()
			[
				SNew(SHorizontalBox)
				.Clipping(EWidgetClipping::OnDemand)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(FMargin(12.0, 0.0, 2.0, 0.0))
				.AutoWidth()
				[
					SNew(SImage)
					.Image(ViewModel->GetSelectorIcon(ViewState.Get()))
					.ColorAndOpacity(FStyleColors::Foreground)
					.Visibility(EVisibility::HitTestInvisible)
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(FText::FromName(ViewState->Name))
					.TextStyle(&FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("StateTree.CentricView.StateTitle.Regular"))
					.ColorAndOpacity(FStyleColors::Foreground)
					.Visibility(EVisibility::HitTestInvisible)
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.FillWidth(1.0f)
				.Padding(FMargin(0.0, 0.0, 10.0, 0.0))
				[
					SAssignNew(ChangeLOD_Button, SButton)
					.OnClicked(this, &SParentStateExtensionNode::HandleOnChangeLOD_Clicked)
					.ButtonStyle(&FAppStyle::GetWidgetStyle<FButtonStyle>("NoBorder"))
					.DesiredSizeScale(FVector2D(1.2f, 1.2f))
					.ContentPadding(FMargin())
					[
						SNew(SImage)
						.Image(ExpandButtonBrush)
						.ColorAndOpacity(this, &SParentStateExtensionNode::GetExpandButtonColor)
					]
				]
			];
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SParentStateExtensionNode::Construct_GenerateNodeBody(const EExtendableNodeLOD InLOD)
{
	if (ViewState.IsValid())
	{
		if (InLOD == EExtendableNodeLOD::Full || InLOD == EExtendableNodeLOD::Minimal)
		{
			NodeBodyPadding = FMargin(3.0, 0.0, 3.0, 0.0);

			FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
			FDetailsViewArgs DetailsViewArgs;
			DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
			DetailsViewArgs.InitialCategories = InLOD == EExtendableNodeLOD::Full
				? TSet<FName>{ "Tasks", "Enter Conditions" }
				: TSet<FName>{ "Enter Conditions" };
			DetailsViewArgs.bShowScrollBar = false;
			DetailsViewArgs.bShowOptions = false;
			DetailsViewArgs.bAllowSearch = false;
			DetailsViewArgs.ShouldForceHideProperty.BindLambda([ViewModel = ViewModel](const TSharedRef<IPropertyHandle>& InPropertyHandle)
			{
				// Hide properties other than enter conditions in the enter condition area
				if (InPropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UStateTreeState, bCheckPrerequisitesWhenActivatingChildDirectly)
					|| InPropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UStateTreeState, bHasRequiredEventToEnter)
					|| InPropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UStateTreeState, RequiredEventToEnter))
				{
					return true;
				}

				return false;
			});
			DetailsViewArgs.ViewIdentifier = FName(ViewState->Name.ToString() + "_SParentStateExtensionNode");
			DetailsViewArgs.ExpansionPersistenceMethod = FDetailsViewArgs::EExpansionPersistenceMethod::TransientPerViewIdentifier;

			TSharedRef<IDetailsView> StateDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
			StateDetailsView->SetExtensionHandler(MakeShared<FStateTreeBindingExtension>());
			StateDetailsView->SetChildrenCustomizationHandler(MakeShared<FStateTreeBindingsChildrenCustomization>());
			StateDetailsView->SetObject(ViewState.Get());

			return StateDetailsView;
		}
		else
		{
			// Negative offset so our background is behind the next node
			NodeBodyPadding = FMargin(0.0, 0.0, 0.0, -8.0);

			return SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Brushes.AccentBlack"))
				.DesiredSizeOverride(FVector2D(32.0, 12.0));
		}
	}

	return SNullWidget::NullWidget;
}

FReply SParentStateExtensionNode::HandleOnChangeLOD_Clicked()
{
	if (ViewState.IsValid())
	{
		// @TODO: Remove splitter references here once those become a feature toggle on cardinal extensions
		UStateTreeEditorData* EditorData = const_cast<UStateTreeEditorData*>(ViewModel->GetStateTreeEditorData());
		UStateCentricViewEditorDataExtension* StateCentricEditorData = EditorData->GetOrCreateExtension<UStateCentricViewEditorDataExtension>();

		if (CurrentLOD == EExtendableNodeLOD::Collapsed)
		{
			CurrentLOD = EExtendableNodeLOD::Minimal;
		}
		else if (CurrentLOD == EExtendableNodeLOD::Minimal)
		{
			CurrentLOD = EExtendableNodeLOD::Full;
		}
		else if (ensure(CurrentLOD == EExtendableNodeLOD::Full))
		{
			CurrentLOD = EExtendableNodeLOD::Collapsed;
		}

		if (StateCentricEditorData->bShouldStoreParentExpansion)
		{
			StateCentricEditorData->LastUserSetParentStateLOD.FindOrAdd(ViewState->ID) = CurrentLOD;
		}

		RegenerateWidgetForLOD(CurrentLOD);
	}

	return FReply::Handled();
}

FSlateColor SParentStateExtensionNode::GetExpandButtonColor() const
{
	return ChangeLOD_Button->IsHovered() ? FStyleColors::White : FStyleColors::Foreground;
}

} // namespace UE::StateTree::Editor::StateCentricView

#undef LOCTEXT_NAMESPACE

