// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "StateCentricView/View/SStateTreeExtendableNode.h"

struct FSlateBrush;

class SButton;

namespace UE::StateTree::Editor::StateCentricView
{


/**
 * Experimental. Node used an an extension to show parent states
 */
class SParentStateExtensionNode : public SStateTreeExtendableNode
{

public:

	static const FName NodeID;

public:

	//~ Begin SExtendableNode Interface
	virtual FName GetNodeName() const override;
	//~ End SExtendableNode Interface

protected:

	//~ Begin SExtendableNode Interface
	virtual void Construct_GenerateBuiltInExtensions(const EExtendableNodeLOD InLOD) override;
	virtual TSharedRef<SWidget> Construct_GenerateNodeHeader(const EExtendableNodeLOD InLOD) override;
	virtual TSharedRef<SWidget> Construct_GenerateNodeBody(const EExtendableNodeLOD InLOD) override;
	//~ End SExtendableNode Interface

private:

	/** Callback for top right button */
	FReply HandleOnChangeLOD_Clicked();

	/** Minor UI / UX callbacks */
	FSlateColor GetExpandButtonColor() const;

private:

	TArray<TSharedPtr<FDeferredCleanupSlateBrush>> ChildStateBrushes;
	TSharedPtr<FDeferredCleanupSlateBrush> StateOutlineBrush;
	TSharedPtr<FDeferredCleanupSlateBrush> StateHeaderBackgroundBrush;
	TSharedPtr<FDeferredCleanupSlateBrush> StateFootnoteBackgroundBrush;

	TSharedPtr<SButton> ChangeLOD_Button;
};


} // namespace UE::StateTree::Editor::StateCentricView