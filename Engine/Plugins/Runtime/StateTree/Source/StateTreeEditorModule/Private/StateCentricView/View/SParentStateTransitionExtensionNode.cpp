// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateCentricView/View/SParentStateTransitionExtensionNode.h"

#include "StateCentricView/ViewModel/StateCentricViewEditorDataExtension.h"
#include "StateCentricView/View/StateCentricViewUtils.h"
#include "StateCentricView/View/STransitionNode.h"
#include "StateTreeEditorStyle.h"

#include "Styling/SlateBrush.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SParentStateTransitionExtensionNode"

namespace UE::StateTree::Editor::StateCentricView
{


//////////////////////////////////////////////////////////////////////////
// SParentStateTransitionExtensionNode


SParentStateTransitionExtensionNode::~SParentStateTransitionExtensionNode()
{
	if (ViewModel)
	{
		if (UStateTreeEditorData* EditorData = const_cast<UStateTreeEditorData*>(ViewModel->GetStateTreeEditorData()))
		{
			if (UStateCentricViewEditorDataExtension* StateCentricEditorData = EditorData->GetExtension<UStateCentricViewEditorDataExtension>())
			{
				StateCentricEditorData->PerStateTreeUserSettings.OnInTransitionViewLOD_Changed.RemoveAll(this);
				StateCentricEditorData->PerStateTreeUserSettings.OnOutTransitionViewLOD_Changed.RemoveAll(this);
			}
		}

		ViewModel->GetTransitionViewModel().GetOnTransitionsAdded().RemoveAll(this);
		ViewModel->GetTransitionViewModel().GetOnTransitionsRemoved().RemoveAll(this);
		ViewModel->GetTransitionViewModel().GetOnTransitionsMoved().RemoveAll(this);
	}
}

void SParentStateTransitionExtensionNode::Construct_InitializeNodePersistentPreLOD()
{
	UStateTreeEditorData* EditorData = const_cast<UStateTreeEditorData*>(ViewModel->GetStateTreeEditorData());
	UStateCentricViewEditorDataExtension* StateCentricEditorData = EditorData->GetOrCreateExtension<UStateCentricViewEditorDataExtension>();

	// Remove any prior bindings in case of issues with lifetime
	StateCentricEditorData->PerStateTreeUserSettings.OnInTransitionViewLOD_Changed.RemoveAll(this);
	StateCentricEditorData->PerStateTreeUserSettings.OnOutTransitionViewLOD_Changed.RemoveAll(this);
	ViewModel->GetTransitionViewModel().GetOnTransitionsAdded().RemoveAll(this);
	ViewModel->GetTransitionViewModel().GetOnTransitionsRemoved().RemoveAll(this);
	ViewModel->GetTransitionViewModel().GetOnTransitionsMoved().RemoveAll(this);

	TOptional<EExtendableNodeLOD> TransitionViewUserSetLOD;
	if (TransitionDirection == EStateTransitionDirection::In)
	{
		StateCentricEditorData->PerStateTreeUserSettings.OnInTransitionViewLOD_Changed.AddSP(this, &SParentStateTransitionExtensionNode::HandleOnTransitionViewLOD_SettingsChanged);
		TransitionViewUserSetLOD = StateCentricEditorData->PerStateTreeUserSettings.GetInTransitionLOD();
	}
	else if (ensure(TransitionDirection == EStateTransitionDirection::Out))
	{
		StateCentricEditorData->PerStateTreeUserSettings.OnOutTransitionViewLOD_Changed.AddSP(this, &SParentStateTransitionExtensionNode::HandleOnTransitionViewLOD_SettingsChanged);
		TransitionViewUserSetLOD = StateCentricEditorData->PerStateTreeUserSettings.GetOutTransitionLOD();
	}

	// We were given a LOD to use, but based on user config we may want to use a different LOD for now
	if (TransitionViewUserSetLOD)
	{
		CurrentLOD = TransitionViewUserSetLOD.GetValue();
	}

	ViewModel->GetTransitionViewModel().GetOnTransitionsAdded().AddSP(this, &SParentStateTransitionExtensionNode::HandleOnTransitionsNumChanged);
	ViewModel->GetTransitionViewModel().GetOnTransitionsRemoved().AddSP(this, &SParentStateTransitionExtensionNode::HandleOnTransitionsNumChanged);
	ViewModel->GetTransitionViewModel().GetOnTransitionsMoved().AddSP(this, &SParentStateTransitionExtensionNode::HandleOnTransitionsNumChanged);
}

void SParentStateTransitionExtensionNode::Construct_GenerateBuiltInExtensions(const EExtendableNodeLOD InLOD)
{
	// Entire widget state color outline in overlay extension depending on LOD
	if (InLOD != EExtendableNodeLOD::Hidden && InLOD != EExtendableNodeLOD::Collapsed)
	{
		StateOutlineBrush = FStateCentricViewUtils::MakeStateOutlineMuted(ViewModel.ToSharedRef(), ViewState.Get());

		TSharedPtr<SWidget> StateColorOutline = SNew(SBorder)
			.BorderImage(StateOutlineBrush->GetSlateBrush())
			.Visibility(EVisibility::HitTestInvisible);

		AddExtension(StateColorOutline.ToSharedRef())
			.SetLocation(EExtensionLocation::Foreground)
			.SetPadding(FMargin(0.0, 0.0, 0.0, 0.0));
	}
}

TSharedRef<SWidget> SParentStateTransitionExtensionNode::Construct_GenerateNodeHeader(const EExtendableNodeLOD InLOD)
{
	// @TODO: Don't show anything if no transitions for parent, maybe

	if (ViewState.IsValid())
	{
		if (InLOD == EExtendableNodeLOD::Hidden)
		{
			return SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Brushes.Recessed"))
				.DesiredSizeOverride(FVector2D(32.0, 8.0));
		}

		TArray<FStateTreeTransition*> StateTransitions;
		EHorizontalAlignment HeaderAlignment = HAlign_Left;
		FMargin HeaderPadding;
		FText HeaderText;
		if (TransitionDirection == EStateTransitionDirection::In)
		{
			StateTransitions = ViewModel->GetStateInTransitions(ViewState.Get());
			HeaderAlignment = HAlign_Left;
			HeaderPadding = FMargin(12.0, 0.0, 2.0, 0.0);
			HeaderText = LOCTEXT("TransitionHeaderIn", "IN ({0})");
		}
		else if (ensure(TransitionDirection == EStateTransitionDirection::Out))
		{
			StateTransitions = ViewModel->GetStateOutTransitions(ViewState.Get());
			HeaderAlignment = HAlign_Right;
			HeaderPadding = FMargin(2.0, 0.0, 12.0, 0.0);
			HeaderText = LOCTEXT("TransitionHeaderOut", "OUT ({0})");
		}

		HeaderText = FText::Format(HeaderText, StateTransitions.Num());

		StateHeaderBackgroundMutedBrush = FStateCentricViewUtils::MakeStateHeaderBackgroundMuted(ViewModel.ToSharedRef(), ViewState.Get());

		// At Lowest LOD we actually use the header background for the outline
		if (InLOD == EExtendableNodeLOD::Collapsed)
		{
			StateHeaderBackgroundMutedBrush = FStateCentricViewUtils::MakeStateBackgroundCollapsedOutlined(ViewModel.ToSharedRef(), ViewState.Get());
		}

		TSharedPtr<SHorizontalBox> HeaderContentsBox;

		TSharedRef<SWidget> HeaderWidget = SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image(StateHeaderBackgroundMutedBrush->GetSlateBrush())
			]
			+ SOverlay::Slot()
			[
				SAssignNew(HeaderContentsBox, SHorizontalBox)
				.Clipping(EWidgetClipping::OnDemand)
				+ SHorizontalBox::Slot()
				.HAlign(HeaderAlignment)
				.VAlign(VAlign_Center)
				.FillWidth(1.0f)
				.Padding(HeaderPadding)
				[
					SNew(STextBlock)
					.Text(HeaderText)
					.TextStyle(&FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Normal.Normal"))
				]
			];

		// In full LOD, we add a button to allow for new transitions to be added
		if (InLOD == EExtendableNodeLOD::Full)
		{
			HeaderContentsBox->AddSlot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(FMargin(-8.0, 0.0, 8.0, 0.0))
			[
				SAssignNew(AddTransitionButton, SButton)
				.OnClicked(this, &SParentStateTransitionExtensionNode::HandleOnAddTransitionClicked)
				.ButtonStyle(&FAppStyle::GetWidgetStyle<FButtonStyle>("NoBorder"))
				.ContentPadding(FMargin())
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
					.ColorAndOpacity(this, &SParentStateTransitionExtensionNode::GetAddTransitionColor)
				]
			];
		}

		return HeaderWidget;
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SParentStateTransitionExtensionNode::Construct_GenerateNodeBody(const EExtendableNodeLOD InLOD)
{
	if (ViewState.IsValid())
	{
		return FStateCentricViewUtils::GenerateTransitionsViewWidget(InLOD, ViewModel.ToSharedRef(), ViewState.Get(), TransitionDirection);
	}

	return SNullWidget::NullWidget;
}

FReply SParentStateTransitionExtensionNode::HandleOnAddTransitionClicked()
{
	if (ViewState.IsValid())
	{
		FStateTreeTransition DefaultTransition = {};
		ViewModel->AddTransition(ViewState.Get(), DefaultTransition.Trigger, EStateTreeTransitionType::None);
	}
	return FReply::Handled();
}

void SParentStateTransitionExtensionNode::HandleOnTransitionViewLOD_SettingsChanged(TOptional<EExtendableNodeLOD> InLOD)
{
	if (ViewState.IsValid())
	{
		UStateTreeEditorData* EditorData = const_cast<UStateTreeEditorData*>(ViewModel->GetStateTreeEditorData());
		UStateCentricViewEditorDataExtension* StateCentricEditorData = EditorData->GetOrCreateExtension<UStateCentricViewEditorDataExtension>();

		TOptional<EExtendableNodeLOD> TransitionViewLOD;
		if (TransitionDirection == EStateTransitionDirection::In)
		{
			TransitionViewLOD = StateCentricEditorData->PerStateTreeUserSettings.GetInTransitionLOD();
		}
		else if (ensure(TransitionDirection == EStateTransitionDirection::Out))
		{
			TransitionViewLOD = StateCentricEditorData->PerStateTreeUserSettings.GetOutTransitionLOD();
		}

		if (TransitionViewLOD)
		{
			RegenerateWidgetForLOD(TransitionViewLOD.GetValue());
		}
		else
		{
			RegenerateWidgetForLOD(Config.InitialLOD);
		}
	}
}

void SParentStateTransitionExtensionNode::HandleOnTransitionsNumChanged(const UStateTreeState* /*OwningState*/, const TSet<FGuid>& /*TransitionIDs*/)
{
	RegenerateWidgetForLOD(CurrentLOD);
}

FSlateColor SParentStateTransitionExtensionNode::GetAddTransitionColor() const
{
	return AddTransitionButton->IsHovered() ? FStyleColors::White : FStyleColors::Foreground;
}


//////////////////////////////////////////////////////////////////////////
// SParentStateInTransitionExtensionNode

const FName SParentStateInTransitionExtensionNode::NodeID(TEXT("StateCentricView.ParentState.InTransitions"));

FName SParentStateInTransitionExtensionNode::GetNodeName() const
{
	return NodeID;
}

void SParentStateInTransitionExtensionNode::Construct_InitializeNodePersistentPreLOD()
{
	TransitionDirection = EStateTransitionDirection::In;
	SParentStateTransitionExtensionNode::Construct_InitializeNodePersistentPreLOD();
}


//////////////////////////////////////////////////////////////////////////
// SParentStateOutTransitionExtensionNode

const FName SParentStateOutTransitionExtensionNode::NodeID(TEXT("StateCentricView.ParentState.OutTransitions"));

FName SParentStateOutTransitionExtensionNode::GetNodeName() const
{
	return NodeID;
}

void SParentStateOutTransitionExtensionNode::Construct_InitializeNodePersistentPreLOD()
{
	TransitionDirection = EStateTransitionDirection::Out;
	SParentStateTransitionExtensionNode::Construct_InitializeNodePersistentPreLOD();
}


//////////////////////////////////////////////////////////////////////////

} // namespace UE::StateTree::Editor::StateCentricView

#undef LOCTEXT_NAMESPACE

