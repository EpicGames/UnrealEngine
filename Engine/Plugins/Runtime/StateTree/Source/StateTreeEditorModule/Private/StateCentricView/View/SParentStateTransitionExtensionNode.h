// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "StateCentricView/View/SStateTreeExtendableNode.h"

struct FSlateBrush;
struct FSlateColor;

class SButton;

namespace UE::StateTree::Editor::StateCentricView
{


/**
 * Experimental. Node used as an extension for In / Out transitions on the main state.
 */
class SParentStateTransitionExtensionNode : public SStateTreeExtendableNode
{

public:

	virtual ~SParentStateTransitionExtensionNode() override;

protected:

	//~ Begin SExtendableNode Interface
	virtual void Construct_InitializeNodePersistentPreLOD() override;
	virtual void Construct_GenerateBuiltInExtensions(const EExtendableNodeLOD InLOD) override;
	virtual TSharedRef<SWidget> Construct_GenerateNodeHeader(const EExtendableNodeLOD InLOD) override;
	virtual TSharedRef<SWidget> Construct_GenerateNodeBody(const EExtendableNodeLOD InLOD) override;
	//~ End SExtendableNode Interface

protected:

	/** If we are showing in / out transitions. Init in child classes. */
	EStateTransitionDirection TransitionDirection = EStateTransitionDirection::Out;

private:

	/** Callback for top right button */
	FReply HandleOnAddTransitionClicked();

	/** Callback for when LOD settings change */
	void HandleOnTransitionViewLOD_SettingsChanged(TOptional<EExtendableNodeLOD> InLOD);

	/** Callback for transtions added / removed */
	void HandleOnTransitionsNumChanged(const UStateTreeState* /*OwningState*/, const TSet<FGuid>& /*TransitionIDs*/);

	/** Minor UI / UX callbacks */
	FSlateColor GetAddTransitionColor() const;

private:

	TSharedPtr<FDeferredCleanupSlateBrush> StateOutlineBrush;
	TSharedPtr<FDeferredCleanupSlateBrush> StateHeaderBackgroundMutedBrush;

	TSharedPtr<SButton> AddTransitionButton;
};


/**
 * Experimental. Specialization of SParentStateTransitionExtensionNode for 'In' transitions
 */
class SParentStateInTransitionExtensionNode : public SParentStateTransitionExtensionNode
{

public:

	static const FName NodeID;

public:

	//~ Begin SExtendableNode Interface
	virtual FName GetNodeName() const override;
	//~ End SExtendableNode Interface

protected:

	//~ Begin SExtendableNode Interface
	virtual void Construct_InitializeNodePersistentPreLOD() override;
	//~ End SExtendableNode Interface

};


/**
 * Experimental. Specialization of SParentStateTransitionExtensionNode for 'Out' transitions
 */
class SParentStateOutTransitionExtensionNode : public SParentStateTransitionExtensionNode
{

public:

	static const FName NodeID;

public:

	//~ Begin SExtendableNode Interface
	virtual FName GetNodeName() const override;
	//~ End SExtendableNode Interface

protected:

	//~ Begin SExtendableNode Interface
	virtual void Construct_InitializeNodePersistentPreLOD() override;
	//~ End SExtendableNode Interface

};


} // namespace UE::StateTree::Editor::StateCentricView