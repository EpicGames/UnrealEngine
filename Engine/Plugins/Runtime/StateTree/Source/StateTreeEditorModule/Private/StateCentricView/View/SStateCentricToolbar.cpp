// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateCentricView/View/SStateCentricToolbar.h"

#include "StateCentricView/StateCentricViewSettings.h"
#include "StateCentricView/ViewModel/StateCentricViewEditorDataExtension.h"
#include "StateCentricView/View/SStateTreeExtendableNode.h"
#include "StateCentricView/View/StateCentricViewUtils.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/IConsoleManager.h"
#include "SlateGlobals.h"
#include "Styling/CoreStyle.h"
#include "Styling/ToolBarStyle.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SStateCentricToolbar"

namespace UE::StateTree::Editor::StateCentricView
{

void SStateCentricToolbar::Construct( const FArguments& InArgs, TSharedRef<FStateTreeViewModel> InViewModel)
{
	ViewModel = InViewModel;

	FSlimHorizontalToolBarBuilder ToolbarBuilder(TSharedPtr<const FUICommandList>(), FMultiBoxCustomization::None);

	CachedNumParentsShown = FStateCentricViewUtils::GetNumParentsToShow(ViewModel.ToSharedRef());

	TWeakPtr<FStateTreeViewModel> WeakViewModel = ViewModel;

	auto HandleOnCollapseParentsClicked = [WeakViewModel]()
	{
		if (TSharedPtr<FStateTreeViewModel> ViewModelPinned = WeakViewModel.Pin())
		{
			UStateTreeEditorData* EditorData = const_cast<UStateTreeEditorData*>(ViewModelPinned->GetStateTreeEditorData());
			UStateCentricViewEditorDataExtension* StateCentricEditorData = EditorData->GetOrCreateExtension<UStateCentricViewEditorDataExtension>();

			StateCentricEditorData->LastUserSetParentStateLOD.Reset();
			StateCentricEditorData->PerStateTreeUserSettings.SetParentHireachyLOD(EExtendableNodeLOD::Collapsed);
		}
	};
	
	auto HandleOnPreviewParentsClicked = [WeakViewModel]()
	{
		if (TSharedPtr<FStateTreeViewModel> ViewModelPinned = WeakViewModel.Pin())
		{
			UStateTreeEditorData* EditorData = const_cast<UStateTreeEditorData*>(ViewModelPinned->GetStateTreeEditorData());
			UStateCentricViewEditorDataExtension* StateCentricEditorData = EditorData->GetOrCreateExtension<UStateCentricViewEditorDataExtension>();

			StateCentricEditorData->LastUserSetParentStateLOD.Reset();
			StateCentricEditorData->PerStateTreeUserSettings.SetParentHireachyLOD(EExtendableNodeLOD::Minimal);
		}
	};
	
	auto HandleOnExpandParentsClicked = [WeakViewModel]()
	{
		if (TSharedPtr<FStateTreeViewModel> ViewModelPinned = WeakViewModel.Pin())
		{
			UStateTreeEditorData* EditorData = const_cast<UStateTreeEditorData*>(ViewModelPinned->GetStateTreeEditorData());
			UStateCentricViewEditorDataExtension* StateCentricEditorData = EditorData->GetOrCreateExtension<UStateCentricViewEditorDataExtension>();

			StateCentricEditorData->LastUserSetParentStateLOD.Reset();
			StateCentricEditorData->PerStateTreeUserSettings.SetParentHireachyLOD(EExtendableNodeLOD::Full);
		}
	};
	
	auto HandleOnHideParentTransitionsClicked = [WeakViewModel]()
	{
		if (TSharedPtr<FStateTreeViewModel> ViewModelPinned = WeakViewModel.Pin())
		{
			UStateTreeEditorData* EditorData = const_cast<UStateTreeEditorData*>(ViewModelPinned->GetStateTreeEditorData());
			UStateCentricViewEditorDataExtension* StateCentricEditorData = EditorData->GetOrCreateExtension<UStateCentricViewEditorDataExtension>();

			StateCentricEditorData->PerStateTreeUserSettings.SetHideParentTransitions(!StateCentricEditorData->PerStateTreeUserSettings.GetHideParentTransitions().Get(false));
		}
	};
	
	auto HandleOnClearPerStateExpansions = [WeakViewModel]()
	{
		if (TSharedPtr<FStateTreeViewModel> ViewModelPinned = WeakViewModel.Pin())
		{
			UStateTreeEditorData* EditorData = const_cast<UStateTreeEditorData*>(ViewModelPinned->GetStateTreeEditorData());
			UStateCentricViewEditorDataExtension* StateCentricEditorData = EditorData->GetOrCreateExtension<UStateCentricViewEditorDataExtension>();

			StateCentricEditorData->LastUserSetParentStateLOD.Reset();
			StateCentricEditorData->PerStateTreeUserSettings.SetParentHireachyLOD(StateCentricEditorData->PerStateTreeUserSettings.GetParentHireachyLOD());
		}
	};

	// @TODO: Toolbar builder doesn't work well if we don't have dedicated icons. Which we don't.

	ToolbarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateLambda(HandleOnCollapseParentsClicked)),
		NAME_None,
		LOCTEXT("CollapseParents", "Collapse Parents"),
		LOCTEXT("CollapseParentsTip", "Clears all saves per-state expansions, then collapses parents."),
		FSlateIcon());

	ToolbarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateLambda(HandleOnPreviewParentsClicked)),
		NAME_None,
		LOCTEXT("PreviewParents", "Preview Parents"),
		LOCTEXT("PreviewParentsTip", "Clears all per-state expansions, then previews parents."),
		FSlateIcon());

	ToolbarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateLambda(HandleOnExpandParentsClicked)),
		NAME_None,
		LOCTEXT("ExpandParents", "Expand Parents"),
		LOCTEXT("ExpandParentsTip", "Clears all per-state expansions, then expands parents."),
		FSlateIcon());

	ToolbarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateLambda(HandleOnHideParentTransitionsClicked)),
		NAME_None,
		LOCTEXT("HideParentTransitions", "Hide Parent Transitions"),
		LOCTEXT("HideParentTransitionsTip", "Hides parents from transitions."),
		FSlateIcon());

	ToolbarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateLambda(HandleOnClearPerStateExpansions)),
		NAME_None,
		LOCTEXT("ClearExpansions", "Clear Per-State Expansions"),
		LOCTEXT("ClearExpansionsTip", "Clears all per-state expansions."),
		FSlateIcon());

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(12.f, -2.0f, 4.f, -2.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NumParentsShown", "Num Parents Shown: "))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(100.f)
				.MaxDesiredWidth(250.f)
				[
					SNew(SSpinBox<uint32>)
					.Value(this, &SStateCentricToolbar::HandleNumParentsSliderValue)
					.MinValue(0)
					.MaxValue(10)
					.Delta(1)
					.OnValueChanged(this, &SStateCentricToolbar::HandleNumParentsSliderChanged)
				]
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1)
				.Padding(5.f, -2.0f, 5.f, -2.0f)
			[
				ToolbarBuilder.MakeWidget()
			]
		]
	];
}

void SStateCentricToolbar::HandleNumParentsSliderChanged(uint32 NewValue)
{
	if (ViewModel)
	{
		UStateTreeEditorData* EditorData = const_cast<UStateTreeEditorData*>(ViewModel->GetStateTreeEditorData());
		UStateCentricViewEditorDataExtension* StateCentricEditorData = EditorData->GetOrCreateExtension<UStateCentricViewEditorDataExtension>();

		StateCentricEditorData->PerStateTreeUserSettings.SetNumParentsShown(NewValue);
		CachedNumParentsShown = NewValue;
	}
}

uint32 SStateCentricToolbar::HandleNumParentsSliderValue() const
{
	return CachedNumParentsShown;
}

} // namespace UE::StateTree::Editor::StateCentricView

#undef LOCTEXT_NAMESPACE
