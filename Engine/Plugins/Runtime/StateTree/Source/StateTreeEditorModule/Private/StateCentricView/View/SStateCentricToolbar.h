// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Styling/SlateTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FStateTreeViewModel;

namespace UE::StateTree::Editor::StateCentricView
{

/**
 * Experimental. Toolbar to set options for state centric 
 * 
 * Current plan is placeholder to test out workflows. 
 * Long term state centric will need to inject into different toolbars.
 */
class SStateCentricToolbar : public SCompoundWidget
{
public:
	SLATE_USER_ARGS(SStateCentricToolbar) { }
	SLATE_END_ARGS()

	virtual void Construct(const FArguments& InArgs, TSharedRef<FStateTreeViewModel> InViewModel);

private:

	/** ViewModel for this ST */
	TSharedPtr<FStateTreeViewModel> ViewModel = nullptr;

	/** Cached number of parents set to be shown for this state tree */
	uint32 CachedNumParentsShown = 0;

private:

	void HandleNumParentsSliderChanged(uint32 NewValue);
	uint32 HandleNumParentsSliderValue() const;
};


} // namespace UE::StateTree::Editor::StateCentricView

