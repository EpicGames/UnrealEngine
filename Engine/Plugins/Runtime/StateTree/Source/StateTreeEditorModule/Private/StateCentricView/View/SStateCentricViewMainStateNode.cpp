// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateCentricView/View/SStateCentricViewMainStateNode.h"

#include "Customizations/StateTreeBindingExtension.h"
#include "StateCentricView/ViewModel/StateCentricViewEditorDataExtension.h"
#include "StateCentricView/View/SMainStateTransitionExtensionNode.h"
#include "StateCentricView/View/SParentStateExtensionNode.h"
#include "StateCentricView/View/StateCentricViewUtils.h"
#include "StateTreeEditorStyle.h"

#include "IDetailsView.h"
#include "DetailsViewArgs.h"
#include "Layout/Visibility.h"
#include "Modules/ModuleManager.h"
#include "ObjectEditorUtils.h"
#include "PropertyEditorModule.h"
#include "Styling/SlateBrush.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SStateCentricViewMainStateNode"

namespace UE::StateTree::Editor::StateCentricView
{

const FName SStateCentricViewMainStateNode::NodeID(TEXT("StateCentricView.MainState.Node"));

SStateCentricViewMainStateNode::~SStateCentricViewMainStateNode()
{
	if (ViewModel)
	{
		ViewModel->GetOnSelectionChanged().RemoveAll(this);
		ViewModel->GetOnStateAdded().RemoveAll(this);
		ViewModel->GetOnStatesRemoved().RemoveAll(this);
		ViewModel->GetOnStatesMoved().RemoveAll(this);

		if (UStateTreeEditorData* EditorData = const_cast<UStateTreeEditorData*>(ViewModel->GetStateTreeEditorData()))
		{
			if (UStateCentricViewEditorDataExtension* StateCentricEditorData = EditorData->GetExtension<UStateCentricViewEditorDataExtension>())
			{
				StateCentricEditorData->PerStateTreeUserSettings.OnNumParentsShownChanged.RemoveAll(this);
				StateCentricEditorData->PerStateTreeUserSettings.OnParentHireachyLOD_Changed.RemoveAll(this);
			}
		}

	}
}

FName SStateCentricViewMainStateNode::GetNodeName() const
{
	return NodeID;
}

void SStateCentricViewMainStateNode::Construct_InitializeNodePersistentPreLOD()
{
	UStateTreeEditorData* EditorData = const_cast<UStateTreeEditorData*>(ViewModel->GetStateTreeEditorData());
	UStateCentricViewEditorDataExtension* StateCentricEditorData = EditorData->GetOrCreateExtension<UStateCentricViewEditorDataExtension>();

	// Remove any prior bindings in case of issues with lifetime
	{
		ViewModel->GetOnSelectionChanged().RemoveAll(this);
		ViewModel->GetOnStateAdded().RemoveAll(this);
		ViewModel->GetOnStatesRemoved().RemoveAll(this);
		ViewModel->GetOnStatesMoved().RemoveAll(this);

		StateCentricEditorData->PerStateTreeUserSettings.OnNumParentsShownChanged.RemoveAll(this);
		StateCentricEditorData->PerStateTreeUserSettings.OnParentHireachyLOD_Changed.RemoveAll(this);
	}

	ViewModel->GetOnSelectionChanged().AddSP(this, &SStateCentricViewMainStateNode::HandleOnSelectionChanged);
	ViewModel->GetOnStateAdded().AddSP(this, &SStateCentricViewMainStateNode::HandleOnStateAdded);
	ViewModel->GetOnStatesRemoved().AddSP(this, &SStateCentricViewMainStateNode::HandleOnStateRemoved);
	ViewModel->GetOnStatesMoved().AddSP(this, &SStateCentricViewMainStateNode::HandleOnStateMoved);

	StateCentricEditorData->PerStateTreeUserSettings.OnNumParentsShownChanged.AddSP(this, &SStateCentricViewMainStateNode::HandleOnNumParentsShownChanged);
	StateCentricEditorData->PerStateTreeUserSettings.OnParentHireachyLOD_Changed.AddSP(this, &SStateCentricViewMainStateNode::HandleOnParentHireachyLOD_Changed);
}

void SStateCentricViewMainStateNode::Construct_GenerateBuiltInExtensions(const EExtendableNodeLOD InLOD)
{
	// We don't change per LOD, so we shouldn't be getting LOD changes
	ensure(InLOD == EExtendableNodeLOD::Unused);

	ChildStateBrushes.Reset();

	if (ViewState.IsValid())
	{
		// @TODO: Remove splitter references here once those become a feature toggle on cardinal extensions
		UStateTreeEditorData* EditorData = const_cast<UStateTreeEditorData*>(ViewModel->GetStateTreeEditorData());
		UStateCentricViewEditorDataExtension* StateCentricEditorData = EditorData->GetOrCreateExtension<UStateCentricViewEditorDataExtension>();
		
		// In Transition extension on left
		{
			FScopedMainNodeLayoutInfo ScopedMainLayoutInfo = SetScopedMainNodeLayoutInfo()
				.SetSplitterValue(MakeAttributeUObject(StateCentricEditorData, &UStateCentricViewEditorDataExtension::HandleGetMainNodeSplitterValue))
				.SetSplitterOnSlotResized(SSplitter::FOnSlotResized::CreateUObject(StateCentricEditorData, &UStateCentricViewEditorDataExtension::HandleOnMainNodeSplitterResized));

			// @TODO: Read Init LOD from project settings for in transitions.
			TSharedRef<SWidget> InTransitionLeftExtension = SNew(SMainStateInTransitionExtensionNode, ViewModel.ToSharedRef(), ViewState.Get())
				.Config({ .InitialLOD = EExtendableNodeLOD::Minimal });

			AddExtension(InTransitionLeftExtension)
				.SetLocation(EExtensionLocation::SplitLeft)
				.SetSplitterSizeRule(MakeAttributeUObject(StateCentricEditorData, &UStateCentricViewEditorDataExtension::GetInTransitionSplitterSizeRule))
				.SetSplitterValue(MakeAttributeUObject(StateCentricEditorData, &UStateCentricViewEditorDataExtension::HandleGetInTransitionSplitterValue))
				.SetSplitterOnSlotResized(SSplitter::FOnSlotResized::CreateUObject(StateCentricEditorData, &UStateCentricViewEditorDataExtension::HandleOnInTransitionSplitterResized));

			// Add spacer should we hit max size caps, Needed since fill + max size = left align (Slate bug).
			AddExtension(
				SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Brushes.Recessed"))
					.DesiredSizeOverride(FVector2D(32.0, 8.0)))
				.SetLocation(EExtensionLocation::SplitLeft)
				.SetSplitterValue(MakeAttributeUObject(StateCentricEditorData, &UStateCentricViewEditorDataExtension::HandleGetLeftSpacerSplitterValue))
				.SetSplitterOnSlotResized(SSplitter::FOnSlotResized::CreateUObject(StateCentricEditorData, &UStateCentricViewEditorDataExtension::HandleOnLeftSpacerSplitterResized));
		}

		// Out Transition extension on right
		{
			// @TODO: Read Init LOD from project settings for out transitions.
			TSharedRef<SWidget> OutTransitionRightExtension = SNew(SMainStateOutTransitionExtensionNode, ViewModel.ToSharedRef(), ViewState.Get())
				.Config({ .InitialLOD = EExtendableNodeLOD::Full });

			AddExtension(OutTransitionRightExtension)
				.SetLocation(EExtensionLocation::SplitRight)
				.SetSplitterSizeRule(MakeAttributeUObject(StateCentricEditorData, &UStateCentricViewEditorDataExtension::GetOutTransitionSplitterSizeRule))
				.SetSplitterValue(MakeAttributeUObject(StateCentricEditorData, &UStateCentricViewEditorDataExtension::HandleGetOutTransitionSplitterValue))
				.SetSplitterOnSlotResized(SSplitter::FOnSlotResized::CreateUObject(StateCentricEditorData, &UStateCentricViewEditorDataExtension::HandleOnOutTransitionSplitterResized));

			// Add spacer should we hit max size caps, Needed since fill + max size = left align (Slate bug).
			AddExtension(
				SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Brushes.Recessed"))
					.DesiredSizeOverride(FVector2D(32.0, 8.0)))
				.SetLocation(EExtensionLocation::SplitRight)
				.SetSplitterValue(MakeAttributeUObject(StateCentricEditorData, &UStateCentricViewEditorDataExtension::HandleGetRightSpacerSplitterValue))
				.SetSplitterOnSlotResized(SSplitter::FOnSlotResized::CreateUObject(StateCentricEditorData, &UStateCentricViewEditorDataExtension::HandleOnRightSpacerSplitterResized));
		}

		// Parent state extensions at top
		{
			uint32 NumParentStateShown = 0;
			uint32 NumParentsToShow = FStateCentricViewUtils::GetNumParentsToShow(ViewModel.ToSharedRef());

			FScopedMainNodeLayoutInfo ScopedMainLayoutInfo = SetScopedMainNodeLayoutInfo()
				.SetHAlign(HAlign_Fill)
				.SetFillContentSize(1.0f, 0.0f);

			TArray<UStateTreeState*> ParentStates = ViewModel->GetStateParents(ViewState.Get());

			for (UStateTreeState* ParentState : ParentStates)
			{
				if (NumParentsToShow == NumParentStateShown)
				{
					break;
				}

				TOptional<EExtendableNodeLOD> DesiredParentLOD = FStateCentricViewUtils::GetDesiredParentLOD(ViewModel.ToSharedRef(), ParentState);

				TSharedRef<SWidget> ParentStateTopExtension = SNew(SParentStateExtensionNode, ViewModel.ToSharedRef(), ParentState)
					.Config({ .InitialLOD = DesiredParentLOD.Get(EExtendableNodeLOD::Minimal) });

				AddExtension(ParentStateTopExtension)
					.SetLocation(EExtensionLocation::Top)
					.SetAutoSize()
					.SetPadding(FMargin(0.0, 0.0, 0.0, 2.0));

				NumParentStateShown++;
			}
		}

		// Child states in footnote
		FStateCentricViewUtils::AddChildStateFootnoteExtensions(EExtendableNodeLOD::Full, this, OUT ChildStateBrushes);

		// Entire widget state color outline in overlay extension, half circle if not full LOD
		{
			TArray<UStateTreeState*> CentralStateParents = ViewModel->GetStateParents(ViewModel->GetViewState());
			int32 OverlayOffset = CentralStateParents.Num() * 5000;

			FScopedMainNodeLayoutInfo ScopedMainLayoutInfo = SetScopedMainNodeLayoutInfo()
				.SetOverlayOffset(OverlayOffset);

			StateOutlineBrush = FStateCentricViewUtils::MakeStateOutline(ViewModel.ToSharedRef(), ViewState.Get());

			TSharedPtr<SWidget> StateColorOutline = SNew(SBorder)
				.BorderImage(StateOutlineBrush->GetSlateBrush())
				.Visibility(EVisibility::HitTestInvisible);

			AddExtension(StateColorOutline.ToSharedRef())
				.SetLocation(EExtensionLocation::Foreground);
		}
	}
}

TSharedRef<SWidget> SStateCentricViewMainStateNode::Construct_GenerateNodeHeader(const EExtendableNodeLOD InLOD)
{
	if (ViewState.IsValid())
	{
		UStateTreeEditorData* EditorData = const_cast<UStateTreeEditorData*>(ViewModel->GetStateTreeEditorData());
		UStateCentricViewEditorDataExtension* StateCentricEditorData = EditorData->GetOrCreateExtension<UStateCentricViewEditorDataExtension>();

		StateHeaderBackgroundBrush = FStateCentricViewUtils::MakeStateHeaderBackground(ViewModel.ToSharedRef(), ViewState.Get(), FVector2f(32.0f, 28.0f));

		// @TODO: Temp for now have double clicking the header reset layout. Discuss what we want to do. Feels weird to have it do nothing.
		TWeakObjectPtr<UStateCentricViewEditorDataExtension> WeakStateCentricEditorData = StateCentricEditorData;
		auto ResetLayout = [WeakStateCentricEditorData](const FGeometry&, const FPointerEvent& )
		{
			if (UStateCentricViewEditorDataExtension* StateCentricEditorDataPinned = WeakStateCentricEditorData.Get())
			{
				StateCentricEditorDataPinned->ResetSplitters();
			}

			return FReply::Handled();
		};

		// Generate via utils to match style
		TSharedRef<SButton> ResetLayoutButton = FStateCentricViewUtils::GenerateGotoStateButton(InLOD, ViewModel.ToSharedRef(), ViewState.Get());
		ResetLayoutButton->SetOnMouseDoubleClick(FPointerEventHandler::CreateLambda(ResetLayout));

		return SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SNew(SImage)
					.Image(StateHeaderBackgroundBrush->GetSlateBrush())
			]
			+ SOverlay::Slot()
			.Padding(FMargin(8.0, 2.0, 8.0, 2.0))
			[
				ResetLayoutButton
			]
			+ SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SHorizontalBox)
				.Clipping(EWidgetClipping::OnDemand)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(FMargin(-16.0, 0.0, 2.0, 0.0))
				.AutoWidth()
				[
					SNew(SImage)
					.Image(ViewModel->GetSelectorIcon(ViewState.Get()))
					.Visibility(EVisibility::HitTestInvisible)
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(FText::FromName(ViewState->Name))
					.TextStyle(&FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("StateTree.CentricView.StateTitle.Bold"))
					.Visibility(EVisibility::HitTestInvisible)
				]
			];
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SStateCentricViewMainStateNode::Construct_GenerateNodeBody(const EExtendableNodeLOD InLOD)
{
	if (ViewState.IsValid())
	{
		NodeBodyPadding = FMargin(3.0, 0.0, 3.0, 0.0);

		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.InitialCategories = TSet<FName>{ "Tasks", "Enter Conditions" };
		DetailsViewArgs.bShowScrollBar = false;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.ShouldForceHideProperty.BindLambda([ViewModel = ViewModel](const TSharedRef<IPropertyHandle>& InPropertyHandle)
		{
			// Hide properties other than enter conditions in the enter condition area
			if (InPropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UStateTreeState, bCheckPrerequisitesWhenActivatingChildDirectly)
				|| InPropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UStateTreeState, bHasRequiredEventToEnter)
				|| InPropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UStateTreeState, RequiredEventToEnter))
			{
				return true;
			}

			return false;
		});
		DetailsViewArgs.ViewIdentifier = FName(ViewState->Name.ToString() + "_SStateCentricViewMainStateNode");
		DetailsViewArgs.ExpansionPersistenceMethod = FDetailsViewArgs::EExpansionPersistenceMethod::TransientPerViewIdentifier;

		TSharedRef<IDetailsView> StateDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
		StateDetailsView->SetExtensionHandler(MakeShared<FStateTreeBindingExtension>());
		StateDetailsView->SetChildrenCustomizationHandler(MakeShared<FStateTreeBindingsChildrenCustomization>());
		StateDetailsView->SetObject(ViewState.Get());

		return StateDetailsView;
	}

	return SNullWidget::NullWidget;
}

void SStateCentricViewMainStateNode::HandleOnSelectionChanged(const TArray<TWeakObjectPtr<UStateTreeState>>& SelectedStates)
{
	UStateTreeState* LastSelectedState = ViewModel->GetLastSelectedState();
	if (ViewState != LastSelectedState)
	{
		ViewState = LastSelectedState;
		RegenerateWidgetForLOD(CurrentLOD);
	}
}

void SStateCentricViewMainStateNode::HandleOnParentHireachyLOD_Changed(TOptional<EExtendableNodeLOD> InLOD)
{
	RegenerateWidgetForLOD(CurrentLOD);
}

void SStateCentricViewMainStateNode::HandleOnNumParentsShownChanged(TOptional<uint32> InNumParentShown)
{
	RegenerateWidgetForLOD(CurrentLOD);
}

void SStateCentricViewMainStateNode::HandleOnStateAdded(UStateTreeState*, UStateTreeState*)
{
	RegenerateWidgetForLOD(CurrentLOD);
}

void SStateCentricViewMainStateNode::HandleOnStateRemoved(const TSet<UStateTreeState*>&)
{
	RegenerateWidgetForLOD(CurrentLOD);
}

void SStateCentricViewMainStateNode::HandleOnStateMoved(const TSet<UStateTreeState*>&, const TSet<UStateTreeState*>&)
{
	RegenerateWidgetForLOD(CurrentLOD);
}

} // namespace UE::StateTree::Editor::StateCentricView

#undef LOCTEXT_NAMESPACE

