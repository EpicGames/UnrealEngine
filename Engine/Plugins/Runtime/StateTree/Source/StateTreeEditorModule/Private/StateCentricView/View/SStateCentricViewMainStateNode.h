// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "StateCentricView/View/SStateTreeExtendableNode.h"

struct FSlateBrush;

namespace UE::StateTree::Editor::StateCentricView
{


/**
 * Experimental. Node for the main selected state in state centric view
 */
class SStateCentricViewMainStateNode : public SStateTreeExtendableNode
{

public:

	static const FName NodeID;
	virtual ~SStateCentricViewMainStateNode() override;

public:

	//~ Begin SExtendableNode Interface
	virtual FName GetNodeName() const override;
	//~ End SExtendableNode Interface

protected:

	//~ Begin SExtendableNode Interface
	virtual void Construct_InitializeNodePersistentPreLOD() override;
	virtual void Construct_GenerateBuiltInExtensions(const EExtendableNodeLOD InLOD) override;
	virtual TSharedRef<SWidget> Construct_GenerateNodeHeader(const EExtendableNodeLOD InLOD) override;
	virtual TSharedRef<SWidget> Construct_GenerateNodeBody(const EExtendableNodeLOD InLOD) override;
	//~ End SExtendableNode Interface

protected:

	/** Callback whenever selected state changes. */
	void HandleOnSelectionChanged(const TArray<TWeakObjectPtr<UStateTreeState>>& SelectedStates);

	/** Callback whenever parent state exapnsion changes */
	void HandleOnParentHireachyLOD_Changed(TOptional<EExtendableNodeLOD> InLOD);

	/** Callback whenever num parents shown changes */
	void HandleOnNumParentsShownChanged(TOptional<uint32> InNumParentShown);

	/** Callback whenever states added */
	void HandleOnStateAdded(UStateTreeState* /*ParentState*/, UStateTreeState* /*NewState*/);

	/** Callback whenever states removed */
	void HandleOnStateRemoved(const TSet<UStateTreeState*>& /*AffectedParents*/);

	/** Callback whenever states moved */
	void HandleOnStateMoved(const TSet<UStateTreeState*>& /*AffectedParents*/, const TSet<UStateTreeState*>& /*MovedStates*/);

private:

	TArray<TSharedPtr<FDeferredCleanupSlateBrush>> ChildStateBrushes;
	TSharedPtr<FDeferredCleanupSlateBrush> StateOutlineBrush;
	TSharedPtr<FDeferredCleanupSlateBrush> StateHeaderBackgroundBrush;
};


} // namespace UE::StateTree::Editor::StateCentricView