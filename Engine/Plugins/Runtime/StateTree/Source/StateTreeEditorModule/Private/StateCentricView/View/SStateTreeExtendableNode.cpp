// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateCentricView/View/SStateTreeExtendableNode.h"

#define LOCTEXT_NAMESPACE "SStateTreeExtendableNode"

namespace UE::StateTree::Editor::StateCentricView
{

void SStateTreeExtendableNode::Construct(const SExtendableNode::FArguments& InArgs
	, TSharedRef<FStateTreeViewModel> InViewModel
	, TNotNull<UStateTreeState*> InViewState)
{
	ViewModel = InViewModel;
	ViewState = InViewState;

	SExtendableNode::Construct(InArgs);
}

void SStateTreeExtendableNode::Construct_GatherExtensionsFromExternalSubsystems()
{
	// @TODO: 
	// 
	// 1. Create a StateTree ExtendableNode subsystem that allows extension registration given schema & node name
	// 2. Using Schema from ViewModel, gather all extensions that for our schema / node name combination
	// 3. Those extensions will be of the form `FAddStateTreeExtensionDelegate`, which is not compatible w/ `FAddExtensionDelegate`
	// 4. To address above, we can wrap that ST delegate a `FAddExtensionDelegate` made here, and provide (VM) to the inner ST delegate.

	// Register any schema-agnostic extensions
	SExtendableNode::Construct_GatherExtensionsFromExternalSubsystems();
}

} // namespace UE::StateTree::Editor::StateCentricView

#undef LOCTEXT_NAMESPACE

