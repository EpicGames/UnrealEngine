// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Debugger/StateTreeDebugger.h"
#include "StateTree.h"
#include "StateTreeEditorData.h"
#include "StateTreeDelegates.h"
#include "StateTreeViewModel.h"
#include "StateCentricView/Layout/SExtendableNode.h"

#include "Editor.h"
#include "Slate/DeferredCleanupSlateBrush.h"

namespace UE::StateTree::Editor::StateCentricView
{
	struct FStateCentricViewUtils;
}

namespace UE::StateTree::Editor::StateCentricView
{


/**
 * Experimental. If a state transition is in / out relative to main central state
 */
enum class EStateTransitionDirection : uint8
{
	In,			// Transition into central state
	Out,		// Transition out of central state
};


/**
 * Experimental. ST specific callback for extensions. 
 * These extensions will only get added if the schema they are registered against matches the schema for our current ViewModel. 
 * 
 * Note: Impl detail. We will internally wrap the extension delegate given into one that conforms to the default extension interface.
 *
 * @param ExtendableNode	- Node to extend by programatically calling `SExtendableNode::AddExtension` based on provided LOD,
 * @param LOD				- Convenience param. LOD for the current node. Same as ExtendableNode's LOD.
 * @param ViewModel			- ViewModel for the current state tree
 * @param ViewState			- State that this extension should specialize view around, may be different than the central state (IE: If showing a parent state).
 * 
 * @TODO: Currently not used.
 */
using FAddStateTreeExtensionDelegate = TDelegate<void(TSharedRef<SExtendableNode> /*ExtendableNode*/
	, const EExtendableNodeLOD /*LOD*/
	, TSharedRef<FStateTreeViewModel> /*ViewModel*/
	, TNotNull<UStateTreeState*> /*ViewState*/)>;


/** 
 * Experimental. Extendable node StateTree specialization. 
 * Guarantees access to a ST ViewModel & gathers external extensions on a per-schema basis.
 * Also gets relevant view state during construction;
 */
class SStateTreeExtendableNode : public SExtendableNode
{

protected:

	friend struct FStateCentricViewUtils;

public:

	/**
	 * @param InViewModel	ViewModel for the StateTree
	 * @param InViewState	State this node should try to provide a view for. 
	 */
	void Construct(const SExtendableNode::FArguments& InArgs
		, TSharedRef<FStateTreeViewModel> InViewModel
		, TNotNull<UStateTreeState*> InViewState);

protected:

	//~ Begin SExtendableNode Interface
	virtual void Construct_GatherExtensionsFromExternalSubsystems() override;
	//~ End SExtendableNode Interface

protected:

	/** ViewModel for the ST this node interacts with */
	TSharedPtr<FStateTreeViewModel> ViewModel = nullptr;

	/** View State. May be a different state at different parts of state centric view. Ex: Parent state for parent widgets. */
	TWeakObjectPtr<UStateTreeState> ViewState = nullptr;
};


} // namespace UE::StateTree::Editor::StateCentricView