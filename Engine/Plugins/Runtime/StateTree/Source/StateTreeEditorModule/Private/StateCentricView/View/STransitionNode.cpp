// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateCentricView/View/STransitionNode.h"

#include "Customizations/StateTreeBindingExtension.h"
#include "StateCentricView/StateCentricViewSettings.h"
#include "StateCentricView/View/StateCentricViewUtils.h"
#include "StateCentricView/ViewModel/StateCentricViewEditorDataExtension.h"
#include "StateTreeEditorStyle.h"

#include "Containers/Ticker.h"
#include "Customizations/StateTreeTransitionDetails.h"
#include "DetailsViewArgs.h"
#include "IDetailsView.h"
#include "IStructureDetailsView.h"
#include "Layout/Visibility.h"
#include "Modules/ModuleManager.h"
#include "ObjectEditorUtils.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "PropertyPathHelpers.h"
#include "Styling/SlateBrush.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "STransitionNode"

namespace UE::StateTree::Editor::StateCentricView
{


//////////////////////////////////////////////////////////////////////////
// STransitionNode

STransitionNode::~STransitionNode()
{
	if (ViewModel)
	{
		ViewModel->GetTransitionViewModel().GetOnTransitionModified().RemoveAll(this);

		if (UStateTreeEditorData* EditorData = const_cast<UStateTreeEditorData*>(ViewModel->GetStateTreeEditorData()))
		{
			if (UStateCentricViewEditorDataExtension* StateCentricEditorData = EditorData->GetExtension<UStateCentricViewEditorDataExtension>())
			{
				StateCentricEditorData->PerStateTreeUserSettings.OnHideParentTransitionsChanged.RemoveAll(this);
			}
		}
	}
}

void STransitionNode::Construct(const SExtendableNode::FArguments& InArgs
	, TSharedRef<FStateTreeViewModel> InViewModel
	, TNotNull<UStateTreeState*> InViewState
	, TNotNull<FStateTreeTransition*> InTransition)
{
	TransitionID = InTransition->ID;

	SStateTreeExtendableNode::Construct(InArgs, InViewModel, InViewState);
}


void STransitionNode::Construct_InitializeNodePersistentPreLOD()
{
	if (ViewState.IsValid())
	{
		UStateTreeEditorData* EditorData = const_cast<UStateTreeEditorData*>(ViewModel->GetStateTreeEditorData());
		UStateCentricViewEditorDataExtension* StateCentricEditorData = EditorData->GetOrCreateExtension<UStateCentricViewEditorDataExtension>();

		// Remove any prior bindings in case of issues with lifetime
		{
			ViewModel->GetTransitionViewModel().GetOnTransitionModified().RemoveAll(this);

			StateCentricEditorData->PerStateTreeUserSettings.OnHideParentTransitionsChanged.RemoveAll(this);
		}

		ViewModel->GetTransitionViewModel().GetOnTransitionModified().AddSP(this, &STransitionNode::HandleOnTransitionModified);

		StateCentricEditorData->PerStateTreeUserSettings.OnHideParentTransitionsChanged.AddSP(this, &STransitionNode::HandleOnHideParentTransitionsChanged);
	}
}

void STransitionNode::Construct_GenerateBuiltInExtensions(const EExtendableNodeLOD InLOD)
{
	// @TODO: Rather than handle the transition arrow via padding. Maybe just add an inner node for the transition & make the 
	// transition arrow not part of this node

	FStateTreeTransition* Transition = ViewModel->GetMutableTransitionByID(TransitionID);
	if (Transition && ViewState.IsValid())
	{
		UStateTreeEditorData* EditorData = const_cast<UStateTreeEditorData*>(ViewModel->GetStateTreeEditorData());
		UStateCentricViewEditorDataExtension* StateCentricEditorData = EditorData->GetOrCreateExtension<UStateCentricViewEditorDataExtension>();

		// Parent state name as top extension
		const UStateTreeState* ParentState = TransitionDirection == EStateTransitionDirection::In
			? ViewModel->GetSourceParentStateFromTransition(Transition)
			: ViewModel->GetTargetParentStateFromTransition(Transition);

		if (ParentState && !StateCentricEditorData->PerStateTreeUserSettings.GetHideParentTransitions().Get(false))
		{
			TransitionParentStateBrush = FStateCentricViewUtils::MakeStateSubnodeBackgroundOutlined(ViewModel.ToSharedRef(), ParentState);

			FScopedMainNodeLayoutInfo ScopedMainLayoutInfo = SetScopedMainNodeLayoutInfo()
				.SetPadding(FMargin(0.0));

			// @TODO: Make sure to add a warning icon if one of the parents in the nested hierachy has a entry condition
			TSharedRef<SWidget> ParentStateNameTopExtension = SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SImage)
					.Image(TransitionParentStateBrush->GetSlateBrush())
				]
				+ SOverlay::Slot()
				[
					FStateCentricViewUtils::GenerateGotoStateButton(InLOD, ViewModel.ToSharedRef(), ParentState)
				]
				+ SOverlay::Slot()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(FMargin(12.0, 0.0, 2.0, 0.0))
					.AutoWidth()
					[
						SNew(SImage)
						.Image(ViewModel->GetSelectorIcon(ParentState))
						.ColorAndOpacity(FStyleColors::Foreground)
						.Visibility(EVisibility::HitTestInvisible)
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(FText::FromName(ParentState->Name))
						.TextStyle(&FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Normal.Normal"))
						.ColorAndOpacity(FStyleColors::White)
						.Visibility(EVisibility::HitTestInvisible)
						.AutoWrapText(UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(ViewModel.Get()).bWordWrapTransitionStates)
					]
				];

			AddExtension(ParentStateNameTopExtension)
				.SetLocation(EExtensionLocation::Top)
				.SetAutoSize()
				.SetPadding(FMargin(0.0, 0.0, 0.0, 4.0));
		}

		// Target state entry conditions in subheader depending on LOD
		if (InLOD == EExtendableNodeLOD::Full)
		{
			if (const UStateTreeState* State = TransitionDirection == EStateTransitionDirection::In
				? ViewModel->GetSourceStateFromTransition(Transition)
				: ViewModel->GetTargetStateFromTransition(Transition))
			{
				TransitionStateBackgroundBrush = FStateCentricViewUtils::MakeStateSubnodeBackground(ViewModel.ToSharedRef(), State);

				TSharedPtr<SVerticalBox> EnterConditionsBox;

				TSharedRef<SWidget> EnterConditionsExtension = SNew(SOverlay)
				+SOverlay::Slot()
				[
					SNew(SImage)
					.Image(TransitionStateBackgroundBrush->GetSlateBrush())
				]
				+ SOverlay::Slot()
				[
					SAssignNew(EnterConditionsBox, SVerticalBox)
				];

				for (const FStateTreeEditorNode& EnterCondition : State->EnterConditions)
				{
					FText EnterConditionDesc = ViewModel->GetNodeDescription(EnterCondition, EStateTreeNodeFormatting::Text);

					EnterConditionsBox->AddSlot()
					.Padding(FMargin(8.0, 2.0, 8.0, 2.0))
					.AutoHeight()
					[
						SNew(STextBlock)
						.AutoWrapText(true)
						.Text(EnterConditionDesc)
					];
				}

				if (State->EnterConditions.Num() > 0)
				{
					SVerticalBox::FSlot& FirstSlot = EnterConditionsBox->GetSlot(0);
					FirstSlot.SetPadding(FirstSlot.GetPadding() + FMargin(0.0, 12.0, 0.0, 0.0));

					AddExtension(EnterConditionsExtension)
						.SetLocation(EExtensionLocation::SubHeader)
						.SetPadding(FMargin(0.0, -10.0, 0.0, 0.0));
				}
			}
		}

		// Small arrow used to indicate transition on left / right depending on direction
		const float TransitionNodeBridgeWidth = UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(ViewModel.Get()).TransitionNodeBridgeWidth;
		const float TransitionNodeBridgeHalfWidth = TransitionNodeBridgeWidth / 2.0f;
		{
			FScopedMainNodeLayoutInfo ScopedMainLayoutInfo = SetScopedMainNodeLayoutInfo()
				.SetFillSize(1.0f)
				.SetPadding(FMargin(0.0));

			TSharedRef<SWidget> TransitionArrowExtension =
				SNew(SButton)
				.NormalPaddingOverride(FMargin(2.0, 4.0))
				.PressedPaddingOverride(FMargin(2.0, 4.0))
				.ContentPadding(FMargin(TransitionNodeBridgeHalfWidth - 6.0f, 0.0f, TransitionNodeBridgeHalfWidth - 6.0f, 0))
				[
					SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						[
							SNew(SImage)
								.Image(FAppStyle::Get().GetBrush("Icons.ChevronRight"))
								.DesiredSizeOverride(FVector2D(10, 24))
								.ColorAndOpacity(FStyleColors::Foreground)
						]
						+ SHorizontalBox::Slot()
						.Padding(FMargin(-8.0, 0.0, 0.0, 0.0))
						[
							SNew(SImage)
								.Image(FAppStyle::Get().GetBrush("Icons.ChevronRight"))
								.DesiredSizeOverride(FVector2D(10, 24))
								.ColorAndOpacity(FStyleColors::Foreground)
						]
				];

			EExtensionLocation TransitionArrowLocation = TransitionDirection == EStateTransitionDirection::In
				? EExtensionLocation::Right
				: EExtensionLocation::Left;

			FMargin TransitionArrowPadding = TransitionDirection == EStateTransitionDirection::In
				? FMargin(6.0, 0.0, -6.0, 0.0)
				: FMargin(-6.0, 0.0, 6.0, 0.0);
				
			AddExtension(TransitionArrowExtension)
				.SetLocation(TransitionArrowLocation)
				.SetVAlign(VAlign_Center)
				.SetAutoSize()
				.SetPadding(TransitionArrowPadding);
		}

		// Generic background extension, padded to not overlap with the small transition arrow
		{
			FScopedMainNodeLayoutInfo ScopedMainLayoutInfo = SetScopedMainNodeLayoutInfo()
				.SetPadding(FMargin(4.0));

			FMargin TransitionArrowPadding = TransitionDirection == EStateTransitionDirection::In
				? FMargin(0.0, 0.0, 4.0 + TransitionNodeBridgeWidth, 0.0)
				: FMargin(4.0 + TransitionNodeBridgeWidth, 0.0, 0.0, 0.0);

			const FSlateBrush* SubNodeBackgroundBrush = FStateCentricViewUtils::GetSubNodeBackground();

			TSharedPtr<SWidget> SubNodeBackground = SNew(SBorder)
				.BorderImage(SubNodeBackgroundBrush)
				.Visibility(EVisibility::HitTestInvisible);

			AddExtension(SubNodeBackground.ToSharedRef())
				.SetLocation(EExtensionLocation::Background)
				.SetAddLocationBehavior(EExtensionAddLocationBehavior::OverlayEntireNode)
				.SetPadding(TransitionArrowPadding);
		}
	}
}

TSharedRef<SWidget> STransitionNode::Construct_GenerateNodeHeader(const EExtendableNodeLOD InLOD)
{
	NodeBodyPadding = FMargin(0.0, 0.0, 0.0, 0.0);

	FStateTreeTransition* Transition = ViewModel->GetMutableTransitionByID(TransitionID);
	if (Transition && ViewState.IsValid())
	{
		NodeBodyPadding = FMargin(0.0, 2.0, 0.0, 0.0);

		if (const UStateTreeState* State = TransitionDirection == EStateTransitionDirection::In
			? ViewModel->GetSourceStateFromTransition(Transition)
			: ViewModel->GetTargetStateFromTransition(Transition))
		{
			TransitionStateBrush = FStateCentricViewUtils::MakeStateSubnodeBackgroundOutlined(ViewModel.ToSharedRef(), State);

			// @TODO move this logic below into FStateCentricViewUtils
			TSharedPtr<SHorizontalBox> NodeHeaderMainHorizontalBox = nullptr;

			TSharedPtr<SOverlay> NodeHeaderOverlay = SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SImage)
					.Image(TransitionStateBrush->GetSlateBrush())
				]
				+ SOverlay::Slot()
				[
					// Draw twice to force our background on top of subheader 
					SNew(SImage)
					.Image(TransitionStateBrush->GetSlateBrush())
				]
				+ SOverlay::Slot()
				.Padding(FMargin(0.0, 0.0, InLOD == EExtendableNodeLOD::Full ? 26.0 : 0, 0.0))
				[
					FStateCentricViewUtils::GenerateGotoStateButton(InLOD, ViewModel.ToSharedRef(), State)
				]
				+ SOverlay::Slot()
				[
					SAssignNew(NodeHeaderMainHorizontalBox, SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(FMargin(12.0, 0.0, 2.0, 0.0))
					.AutoWidth()
					[
						SNew(SImage)
						.Image(ViewModel->GetSelectorIcon(State))
						.ColorAndOpacity(FStyleColors::Foreground)
						.Visibility(EVisibility::HitTestInvisible)
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(FText::FromName(State->Name))
						.TextStyle(&FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Normal.Normal"))
						.ColorAndOpacity(FStyleColors::White)
						.Visibility(EVisibility::HitTestInvisible)
						.AutoWrapText(UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(ViewModel.Get()).bWordWrapTransitionStates)
					]
				];

			if (InLOD == EExtendableNodeLOD::Full)
			{
				NodeHeaderMainHorizontalBox->AddSlot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(FMargin(0.0, 0.0, 6.0, 0.0))
				[
					SAssignNew(DeleteTransitionButton, SButton)
					.OnClicked(this, &STransitionNode::HandleOnDeleteTransitionClicked)
					.ButtonStyle(&FAppStyle::GetWidgetStyle<FButtonStyle>("NoBorder"))
					.ContentPadding(FMargin())
					.ToolTipText(LOCTEXT("TransitionNodeDelete", "Delete this Transition"))
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Icons.X"))
						.ColorAndOpacity(this, &STransitionNode::GetDeleteTransitionColor)
					]
				];
			}

			return NodeHeaderOverlay.ToSharedRef();
		}
		else
		{
			// State Invalid. Indicate so
			return SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(FMargin(12.0, 0.0, 2.0, 0.0))
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Transition.None"))
						.ColorAndOpacity(FStyleColors::Foreground)
						.Visibility(EVisibility::HitTestInvisible)
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TransitionNodeMissingState", "Invalid State"))
						.TextStyle(&FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Normal.Normal"))
						.ColorAndOpacity(FStyleColors::White)
						.Visibility(EVisibility::HitTestInvisible)
						.AutoWrapText(UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(ViewModel.Get()).bWordWrapTransitionStates)
					]
				];
		}
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> STransitionNode::Construct_GenerateNodeBody(const EExtendableNodeLOD InLOD)
{
	if (InLOD == EExtendableNodeLOD::Full)
	{
		FStateTreeTransition* Transition = ViewModel->GetMutableTransitionByID(TransitionID);
		if (UStateTreeState* StateOwningTransition = TransitionDirection == EStateTransitionDirection::In
			? const_cast<UStateTreeState*>(ViewModel->GetSourceStateFromTransition(Transition))
			: ViewState.Get())
		{
			// Struct view doesn't work since the customization is only on the object.
			// So show the object and filter to the transition

			// @TODO: This seems slow, revist & profile

			FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
			FDetailsViewArgs DetailsViewArgs;
			DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
			DetailsViewArgs.InitialCategories = TSet<FName>{ "Transitions" };
			DetailsViewArgs.bShowScrollBar = false;
			DetailsViewArgs.bShowOptions = false;
			DetailsViewArgs.bAllowSearch = false;
			DetailsViewArgs.bShowObjectLabel = false;
			DetailsViewArgs.bShowHeaders = false;
			DetailsViewArgs.ShouldForceHideProperty.BindLambda([ViewModel = ViewModel, TransitionID = TransitionID, StateOwningTransition = StateOwningTransition](const TSharedRef<IPropertyHandle>& InPropertyHandle)
			{
				const FStructProperty* TransitionPropertyStruct = nullptr;

				if (const FStructProperty* PropertyStruct = CastField<FStructProperty>(InPropertyHandle->GetProperty()))
				{
					if (PropertyStruct->Struct == FStateTreeTransition::StaticStruct())
					{
						TransitionPropertyStruct = PropertyStruct;
					}
				}

				TSharedPtr<IPropertyHandle> TransitionPropertyHandle = TransitionPropertyStruct ? InPropertyHandle : InPropertyHandle->GetParentHandle();

				// Search parents for transition property
				while (TransitionPropertyHandle && !TransitionPropertyStruct)
				{
					if (const FProperty* ParentProperty = TransitionPropertyHandle->GetProperty())
					{
						if (const FStructProperty* ParentPropertyStruct = CastField<FStructProperty>(ParentProperty))
						{
							if (ParentPropertyStruct->Struct == FStateTreeTransition::StaticStruct())
							{
								TransitionPropertyStruct = ParentPropertyStruct;
							}
						}
					}

					if (!TransitionPropertyStruct)
					{
						TransitionPropertyHandle = TransitionPropertyHandle->GetParentHandle();
					}
				}

				if (TransitionPropertyStruct && TransitionPropertyHandle)
				{
					FStateTreeTransition TransitionValue; 
					FString TransitionPropertyPath(TransitionPropertyHandle->GetPropertyPath());
					if (PropertyPathHelpers::GetPropertyValue(StateOwningTransition, TransitionPropertyPath, OUT TransitionValue))
					{
						return TransitionValue.ID != TransitionID;
					}
				}

				return true;
			});
			DetailsViewArgs.ViewIdentifier = FName(StateOwningTransition->Name.ToString() + "_Transition_" + TransitionID.ToString() + "_STransitionNode");
			DetailsViewArgs.ExpansionPersistenceMethod = FDetailsViewArgs::EExpansionPersistenceMethod::TransientPerViewIdentifier;
			
			TSharedRef<IDetailsView> StateDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
			StateDetailsView->SetExtensionHandler(MakeShared<FStateTreeBindingExtension>());
			StateDetailsView->SetChildrenCustomizationHandler(MakeShared<FStateTreeBindingsChildrenCustomization>());
			StateDetailsView->SetObject(StateOwningTransition);

			return StateDetailsView;
		}
	}

	return SNullWidget::NullWidget;
}

void STransitionNode::RefreshNodeDelayed()
{
	TWeakPtr<STransitionNode> ThisWeak = StaticCastSharedRef<STransitionNode>(AsShared());
	auto RefreshTransitionNode = [ThisWeak](float)
	{
		if (TSharedPtr<STransitionNode> ThisPinned = ThisWeak.Pin())
		{
			ThisPinned->RegenerateWidgetForLOD(ThisPinned->CurrentLOD);
		}

		// Run just once
		return false;
	};

	// Refresh on a delay. We might have been modified from our own details panel
	// If we instantly refresh we can kill that details mid-update which can remove property handles & cause update issues
	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(RefreshTransitionNode));
}

void STransitionNode::HandleOnTransitionModified(const UStateTreeState* OwningState, const FGuid& InTransitionID)
{
	if (TransitionID == InTransitionID)
	{
		RefreshNodeDelayed();
	}
}

void STransitionNode::HandleOnHideParentTransitionsChanged(TOptional<bool> /* Unused */)
{
	RefreshNodeDelayed();
}

FReply STransitionNode::HandleOnDeleteTransitionClicked()
{
	FStateTreeTransition* Transition = ViewModel->GetMutableTransitionByID(TransitionID);
	if (Transition && ViewState.IsValid())
	{
		if (const UStateTreeState* State = ViewModel->GetSourceStateFromTransition(Transition))
		{
			ViewModel->RemoveTransition(const_cast<UStateTreeState*>(State), TransitionID);
		}
	}

	return FReply::Handled();
}

FSlateColor STransitionNode::GetDeleteTransitionColor() const
{
	return DeleteTransitionButton->IsHovered() ? FStyleColors::White : FStyleColors::Foreground;
}


//////////////////////////////////////////////////////////////////////////
// SInTransitionNode

const FName SInTransitionNode::NodeID(TEXT("StateCentricView.Transition.InNode"));

FName SInTransitionNode::GetNodeName() const
{
	return NodeID;
}

void SInTransitionNode::Construct_InitializeNodePersistentPreLOD()
{
	TransitionDirection = EStateTransitionDirection::In;
	STransitionNode::Construct_InitializeNodePersistentPreLOD();
}


//////////////////////////////////////////////////////////////////////////
// SMainStateOutTransitionExtensionNode

const FName SOutTransitionNode::NodeID(TEXT("StateCentricView.Transition.OutNode"));

FName SOutTransitionNode::GetNodeName() const
{
	return NodeID;
}

void SOutTransitionNode::Construct_InitializeNodePersistentPreLOD()
{
	TransitionDirection = EStateTransitionDirection::Out;
	STransitionNode::Construct_InitializeNodePersistentPreLOD();
}


//////////////////////////////////////////////////////////////////////////

} // namespace UE::StateTree::Editor::StateCentricView

#undef LOCTEXT_NAMESPACE

