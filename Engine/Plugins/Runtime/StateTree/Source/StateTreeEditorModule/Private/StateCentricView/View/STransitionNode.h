// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "StateCentricView/View/SStateTreeExtendableNode.h"
#include "StateTreeState.h"

struct FSlateBrush;
struct FStateTreeTransition;

class IStructureDetailsView;
class FStateTreeViewModel;
class SButton;
class UStateTreeState;

enum class EStateTransitionDirection : uint8;

namespace UE::StateTree::Editor::StateCentricView
{


/**
 * Experimental. Node used to show a single state transition
 */
class STransitionNode : public SStateTreeExtendableNode
{

public:

	virtual ~STransitionNode() override;

	void Construct(const SExtendableNode::FArguments& InArgs
		, TSharedRef<FStateTreeViewModel> InViewModel
		, TNotNull<UStateTreeState*> InViewState
		, TNotNull<FStateTreeTransition*> InTransition);

protected:

	//~ Begin SExtendableNode Interface
	virtual void Construct_InitializeNodePersistentPreLOD() override;
	virtual void Construct_GenerateBuiltInExtensions(const EExtendableNodeLOD InLOD) override;
	virtual TSharedRef<SWidget> Construct_GenerateNodeHeader(const EExtendableNodeLOD InLOD) override;
	virtual TSharedRef<SWidget> Construct_GenerateNodeBody(const EExtendableNodeLOD InLOD) override;
	//~ End SExtendableNode Interface

protected:

	/** Method to refresh this node delayed */
	void RefreshNodeDelayed();

	/** Callback for transtion modified */
	void HandleOnTransitionModified(const UStateTreeState* OwningState, const FGuid& InTransitionID);

	/** Callback for hide parents modified */
	void HandleOnHideParentTransitionsChanged(TOptional<bool> /* Unused */);

	/** Callback for delete transition button */
	FReply HandleOnDeleteTransitionClicked();

	/** Minor UI / UX callbacks */
	FSlateColor GetDeleteTransitionColor() const;

protected:

	/** 
	 * Transition we represent, guaranteed to exist as long as this node does via value change callbacks
	 * @TODO: Make that guarantee true.
	 */
	FGuid TransitionID = FGuid();

	/** If we are showing in / out transitions. Init in child classes. */
	EStateTransitionDirection TransitionDirection = EStateTransitionDirection::Out;

private:

	TSharedPtr<FDeferredCleanupSlateBrush> TransitionParentStateBrush;
	TSharedPtr<FDeferredCleanupSlateBrush> TransitionStateBrush;
	TSharedPtr<FDeferredCleanupSlateBrush> TransitionStateBackgroundBrush;

	TSharedPtr<IStructureDetailsView> TransitionDetailsView;
	TSharedPtr<SButton> DeleteTransitionButton;
};


/**
 * Experimental. Specialization of STransitionNode for 'In' transitions
 */
class SInTransitionNode : public STransitionNode
{

public:

	static const FName NodeID;

public:

	//~ Begin SExtendableNode Interface
	virtual FName GetNodeName() const override;
	//~ End SExtendableNode Interface

protected:

	//~ Begin SExtendableNode Interface
	virtual void Construct_InitializeNodePersistentPreLOD() override;
	//~ End SExtendableNode Interface

};


/**
 * Experimental. Specialization of STransitionNode for 'Out' transitions
 */
class SOutTransitionNode : public STransitionNode
{

public:

	static const FName NodeID;

public:

	//~ Begin SExtendableNode Interface
	virtual FName GetNodeName() const override;
	//~ End SExtendableNode Interface

protected:

	//~ Begin SExtendableNode Interface
	virtual void Construct_InitializeNodePersistentPreLOD() override;
	//~ End SExtendableNode Interface

};


} // namespace UE::StateTree::Editor::StateCentricView