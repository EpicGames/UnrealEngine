// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateCentricView/View/StateCentricViewUtils.h"

#include "StateCentricView/StateCentricViewSettings.h"
#include "StateCentricView/ViewModel/StateCentricViewEditorDataExtension.h"
#include "StateCentricView/View/SImplicitRootTransitionNode.h"
#include "StateCentricView/View/SParentSelectionNode.h"
#include "StateCentricView/View/SStateTreeExtendableNode.h"
#include "StateCentricView/View/STransitionNode.h"
#include "StateTreeEditorStyle.h"

#include "Styling/SlateBrush.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "StateCentricViewUtils"

// These CVars exist as a discussion point. Should have a very short lifespan. Probably week or so.

bool bEnableDepthBasedOutline = true;
FAutoConsoleVariableRef CVarEnableDepthBasedOutline(
	TEXT("StateTree.Editor.Experimental.EnableDepthBasedOutline"),
	bEnableDepthBasedOutline,
	TEXT("Set true to change outline style in state centric view when using hierarchy colors")
);

bool bDisableDarkenHeader = false;
FAutoConsoleVariableRef CVarDisableDarkenHeader(
	TEXT("StateTree.Editor.Experimental.DisableDarkenHeader"),
	bDisableDarkenHeader,
	TEXT("Set true to darken the headers in state centric")
);

namespace UE::StateTree::Editor::StateCentricView
{

TSharedRef<SWidget> FStateCentricViewUtils::GenerateTransitionsViewWidget(const EExtendableNodeLOD LOD
	, TSharedRef<FStateTreeViewModel> ViewModel
	, TNotNull<UStateTreeState*> ViewState
	, const EStateTransitionDirection TransitionDirection)
{
	UStateTreeState* CentralState = ViewModel->GetViewState();

	if (LOD == EExtendableNodeLOD::Hidden || LOD == EExtendableNodeLOD::Collapsed)
	{
		return SNullWidget::NullWidget;
	}
	else if (ensure(LOD == EExtendableNodeLOD::Minimal || LOD == EExtendableNodeLOD::Full))
	{
		TArray<FStateTreeTransition*> Transitions;
		if (TransitionDirection == EStateTransitionDirection::In)
		{
			Transitions = ViewModel->GetStateInTransitions(ViewState);
		}
		else if (ensure(TransitionDirection == EStateTransitionDirection::Out))
		{
			Transitions = ViewModel->GetStateOutTransitions(ViewState);
		}

		TSharedRef<SVerticalBox> TransitionListBody = SNew(SVerticalBox);

		if (ViewState == CentralState && TransitionDirection == EStateTransitionDirection::In)
		{
			if (const UStateTreeState* ParentState = ViewState->Parent)
			{
				if (ParentState->SelectionBehavior != EStateTreeStateSelectionBehavior::None
					&& ParentState->SelectionBehavior != EStateTreeStateSelectionBehavior::TryEnterState
					&& ParentState->SelectionBehavior != EStateTreeStateSelectionBehavior::TryFollowTransitions)
				{
					TSharedPtr<SWidget> ParentSelectionWidget = SNew(SParentSelectionNode, ViewModel, ViewState)
						.Clipping(EWidgetClipping::OnDemand)
						.Config({ .InitialLOD = LOD });

					if (UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(&ViewModel.Get()).bJustifyCentralViewTransitions)
					{
						TransitionListBody->AddSlot()
						.FillContentHeight(1.0f)
						.VAlign(VAlign_Center)
						.Padding(FMargin(8.0, 6.0, 8.0, 6.0))
						[
							ParentSelectionWidget.ToSharedRef()
						];
					}
					else
					{
						TransitionListBody->AddSlot()
						.AutoHeight()
						.Padding(FMargin(8.0, 6.0, 8.0, 6.0))
						[
							ParentSelectionWidget.ToSharedRef()
						];
					}
				}
			}
		}

		if (ViewState == CentralState && TransitionDirection == EStateTransitionDirection::Out)
		{
			if (ViewState->Children.IsEmpty() && ViewState->Transitions.IsEmpty())
			{
				TSharedPtr<SWidget> ImplicitRootTransitionWidget = SNew(SImplicitRootTransitionNode, ViewModel, ViewState)
					.Clipping(EWidgetClipping::OnDemand)
					.Config({ .InitialLOD = LOD });

				if (UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(&ViewModel.Get()).bJustifyCentralViewTransitions)
				{
					TransitionListBody->AddSlot()
					.FillContentHeight(1.0f)
					.VAlign(VAlign_Center)
					.Padding(FMargin(8.0, 6.0, 8.0, 6.0))
					[
						ImplicitRootTransitionWidget.ToSharedRef()
					];
				}
				else
				{
					TransitionListBody->AddSlot()
					.AutoHeight()
					.Padding(FMargin(8.0, 6.0, 8.0, 6.0))
					[
						ImplicitRootTransitionWidget.ToSharedRef()
					];
				}
			}
		}

		for (FStateTreeTransition* Transition : Transitions)
		{
			TSharedPtr<SWidget> TransitionNodeWidget;
			if (TransitionDirection == EStateTransitionDirection::In)
			{
				TransitionNodeWidget = SNew(SInTransitionNode, ViewModel, ViewState, Transition)
					.Clipping(EWidgetClipping::OnDemand)
					.Config({ .InitialLOD = LOD });
			}
			else if (ensure(TransitionDirection == EStateTransitionDirection::Out))
			{
				TransitionNodeWidget = SNew(SOutTransitionNode, ViewModel, ViewState, Transition)
					.Clipping(EWidgetClipping::OnDemand)
					.Config({ .InitialLOD = LOD });
			}

			// @TODO, CVar / settings here
			if (ViewState == CentralState && UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(&ViewModel.Get()).bJustifyCentralViewTransitions)
			{
				TransitionListBody->AddSlot()
				.FillContentHeight(1.0f)
				.VAlign(VAlign_Center)
				.Padding(FMargin(8.0, 6.0, 8.0, 6.0))
				[
					TransitionNodeWidget.ToSharedRef()
				];
			}
			else
			{
				TransitionListBody->AddSlot()
				.AutoHeight()
				.Padding(FMargin(8.0, 6.0, 8.0, 6.0))
				[
					TransitionNodeWidget.ToSharedRef()
				];
			}
		}
		
		if (Transitions.Num() > 0)
		{
			SVerticalBox::FSlot& FirstSlot = TransitionListBody->GetSlot(0);
			SVerticalBox::FSlot& LastSlot = TransitionListBody->GetSlot(TransitionListBody->NumSlots() - 1);

			FirstSlot.SetPadding(FirstSlot.GetPadding() + FMargin(0.0, 6.0, 0.0, 0.0));
			LastSlot.SetPadding(LastSlot.GetPadding() + FMargin(0.0, 0.0, 0.0, 6.0));
		}

		return TransitionListBody;
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SButton> FStateCentricViewUtils::GenerateGotoStateButton(const EExtendableNodeLOD LOD, TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> ViewState)
{
	TWeakPtr<FStateTreeViewModel> WeakViewModel = ViewModel;
	auto SelectStateCallback = [WeakViewModel, ViewState](const FGeometry&, const FPointerEvent& )
	{
		if (TSharedPtr<FStateTreeViewModel> ViewModelPinned = WeakViewModel.Pin())
		{
			ViewModelPinned->SetSelection(const_cast<UStateTreeState*>(NotNullGet(ViewState)));
		}

		return FReply::Handled();
	};

	TSharedRef<SButton> Result = SNew(SButton)
		.ButtonStyle(&FStateTreeEditorStyle::Get().GetWidgetStyle<FButtonStyle>("StateTree.CentricView.HoverHighlight"));

	Result->SetOnMouseDoubleClick(FPointerEventHandler::CreateLambda(SelectStateCallback));

	return Result;
}

void FStateCentricViewUtils::AddChildStateFootnoteExtensions(const EExtendableNodeLOD InLOD
	, TNotNull<SStateTreeExtendableNode*> InSourceNode
	, TArray<TSharedPtr<FDeferredCleanupSlateBrush>>& OutChildStateBrushes)
{
	TSharedPtr<FStateTreeViewModel> ViewModel = InSourceNode->ViewModel;
	TWeakObjectPtr<UStateTreeState> ViewState = InSourceNode->ViewState;

	OutChildStateBrushes.Reset();

	if (InLOD == EExtendableNodeLOD::Full)
	{
		// @TODO: This should be be a details entry maybe?

		TArray<UStateTreeState*> ChildStates = ViewModel->GetStateChildren(ViewState.Get());

		if (ChildStates.IsEmpty())
		{
			return;
		}

		TSharedRef<SWidget> ChildStateAreaHeader =
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(FMargin(2.0, 5.0, 0.0, 5.0))
			[
				SNew(SImage)
					.Image(FAppStyle::GetBrush("LevelEditor.Tabs.Outliner"))
					.ColorAndOpacity(FStyleColors::Foreground)
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(FMargin(4.0, 0.0, 0.0, 0.0))
			[
				SNew(STextBlock)
					.Text(LOCTEXT("StateCentricChildHeader", "Children"))
					.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
					.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
					.ColorAndOpacity(FStyleColors::Foreground)
			];

		TSharedRef<SVerticalBox> ChildStateAreaBody = SNew(SVerticalBox);

		for (UStateTreeState* ChildState : ChildStates)
		{	
			TSharedPtr<FDeferredCleanupSlateBrush> ChildStateBrush = FStateCentricViewUtils::MakeStateSubnodeBackgroundOutlined(ViewModel.ToSharedRef(), ChildState);

			TSharedRef<SWidget> ChildStateBottomExtension = SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SImage)
					.Image(ChildStateBrush->GetSlateBrush())
				]
				+ SOverlay::Slot()
				[
					FStateCentricViewUtils::GenerateGotoStateButton(InLOD, ViewModel.ToSharedRef(), ChildState)
				]
				+ SOverlay::Slot()
				[
					SNew(SHorizontalBox)
					.Clipping(EWidgetClipping::OnDemand)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(FMargin(12.0, 0.0, 2.0, 0.0))
					.AutoWidth()
					[
						SNew(SImage)
						.Image(ViewModel->GetSelectorIcon(ChildState))
						.ColorAndOpacity(FStyleColors::Foreground)
						.Visibility(EVisibility::HitTestInvisible)
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(FText::FromName(ChildState->Name))
						.TextStyle(&FStateTreeEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("Normal.Normal"))
						.ColorAndOpacity(FStyleColors::White)
						.Visibility(EVisibility::HitTestInvisible)
						.AutoWrapText(UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(ViewModel.Get()).bWordWrapChildStates)
					]
				];

			OutChildStateBrushes.Add(ChildStateBrush);

			ChildStateAreaBody->AddSlot()
			.Padding(FMargin(16.0, 8.0, 16.0, 0.0))
			.AutoHeight()
			[
				ChildStateBottomExtension
			];
		}
	

		TSharedRef<SExpandableArea> ChildStateArea = SNew(SExpandableArea)
			.Clipping(EWidgetClipping::OnDemand)
			.InitiallyCollapsed(false)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Header"))
			.BodyBorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
			.HeaderContent()
			[
				ChildStateAreaHeader
			]
			.BodyContent()
			[
				ChildStateAreaBody
			];

		InSourceNode->AddExtension(ChildStateArea)
			.SetLocation(EExtensionLocation::Footnote)
			.SetPadding(FMargin(0.0, 0.0, 0.0, 4.0))
			.SetAutoSize();
	}
}

TOptional<EExtendableNodeLOD> FStateCentricViewUtils::GetDesiredParentLOD(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<UStateTreeState*> ParentState)
{
	UStateTreeEditorData* EditorData = const_cast<UStateTreeEditorData*>(ViewModel->GetStateTreeEditorData());
	UStateCentricViewEditorDataExtension* StateCentricEditorData = EditorData->GetOrCreateExtension<UStateCentricViewEditorDataExtension>();

	TOptional<EExtendableNodeLOD> PerUserSettingsParentLOD = UStateCentricViewPerUserSettings::GetSchemaSettingsDefaultFallback(&ViewModel.Get()).GetParentHireachyLOD();
	TOptional<EExtendableNodeLOD> PerStateTreeParentLOD = StateCentricEditorData->PerStateTreeUserSettings.GetParentHireachyLOD();

	if (EExtendableNodeLOD* LastUserSetParentLOD = StateCentricEditorData->LastUserSetParentStateLOD.Find(ParentState->ID))
	{
		return *LastUserSetParentLOD;
	}

	return PerStateTreeParentLOD ? PerStateTreeParentLOD : PerUserSettingsParentLOD;
}

uint32 FStateCentricViewUtils::GetNumParentsToShow(TSharedRef<FStateTreeViewModel> ViewModel)
{
	UStateTreeEditorData* EditorData = const_cast<UStateTreeEditorData*>(ViewModel->GetStateTreeEditorData());
	UStateCentricViewEditorDataExtension* StateCentricEditorData = EditorData->GetOrCreateExtension<UStateCentricViewEditorDataExtension>();

	uint32 GlobalNumParentsToShow = UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(&ViewModel.Get()).MaxNumParentsToShow;
	TOptional<uint32> UserNumParentsToShow = UStateCentricViewPerUserSettings::GetSchemaSettingsDefaultFallback(&ViewModel.Get()).GetNumParentsShown();

	uint32 SettingsNumParentsToShow = UserNumParentsToShow.Get(GlobalNumParentsToShow);
	TOptional<uint32> PerStateTreeNumParentsToShow = StateCentricEditorData->PerStateTreeUserSettings.GetNumParentsShown();

	return PerStateTreeNumParentsToShow.Get(SettingsNumParentsToShow);
}

TSharedPtr<FDeferredCleanupSlateBrush> FStateCentricViewUtils::MakeStateHalfTopOutline(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State)
{
	const FSlateBrush* BaseBrush = FStateTreeEditorStyle::Get().GetBrush("StateTree.CentricView.Parent.HalfTopOutline");

	FSlateBrush Result = *BaseBrush;

	// Use tint with 0 alpha due to slate rendering bug where alpha still matters.
	Result.TintColor = ViewModel->GetStateColor(State, EStateTreeViewModelColorAdjustment::Bright).CopyWithNewOpacity(0.0f);
	Result.OutlineSettings.Color = ViewModel->GetStateColor(State, EStateTreeViewModelColorAdjustment::Bright);

	if (UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(&ViewModel.Get()).bEnableHierarchyDrivenColors)
	{
		Result.TintColor = ViewModel->GetHierarchyBasedHighlightColor(State, EStateTreeViewModelColorAdjustment::Bright).CopyWithNewOpacity(0.0f);
		Result.OutlineSettings.Color = ViewModel->GetHierarchyBasedHighlightColor(State, EStateTreeViewModelColorAdjustment::Bright);
	}

	return FDeferredCleanupSlateBrush::CreateBrush(Result);
}

TSharedPtr<FDeferredCleanupSlateBrush> FStateCentricViewUtils::MakeStateHalfTopOutlineMuted(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State)
{
	const FSlateBrush* BaseBrush = FStateTreeEditorStyle::Get().GetBrush("StateTree.CentricView.Parent.HalfTopOutline");

	FSlateBrush Result = *BaseBrush;

	// Use tint with 0 alpha due to slate rendering bug where alpha still matters.
	Result.TintColor = ViewModel->GetStateColor(State).CopyWithNewOpacity(0.0f);
	Result.OutlineSettings.Color = ViewModel->GetStateColor(State);

	if (UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(&ViewModel.Get()).bEnableHierarchyDrivenColors)
	{
		Result.TintColor = ViewModel->GetHierarchyBasedHighlightColor(State).CopyWithNewOpacity(0.0f);
		Result.OutlineSettings.Color = ViewModel->GetHierarchyBasedHighlightColor(State);

		if (bEnableDepthBasedOutline)
		{
			Result.OutlineSettings.Color = ViewModel->GetHierarchyBasedBackgroundColor(State, EStateTreeViewModelColorAdjustment::Bright);
		}
	}

	return FDeferredCleanupSlateBrush::CreateBrush(Result);
}

TSharedPtr<FDeferredCleanupSlateBrush> FStateCentricViewUtils::MakeStateHalfTopOutlineCollapsed(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State)
{
	const FSlateBrush* BaseBrush = FStateTreeEditorStyle::Get().GetBrush("StateTree.CentricView.Parent.HalfTopOutline");

	FSlateBrush Result = *BaseBrush;

	// Use tint with 0 alpha due to slate rendering bug where alpha still matters.
	Result.TintColor = ViewModel->GetStateColor(State, EStateTreeViewModelColorAdjustment::Dark).CopyWithNewOpacity(0.0f);
	Result.OutlineSettings.Color = ViewModel->GetStateColor(State, EStateTreeViewModelColorAdjustment::Dark);

	if (UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(&ViewModel.Get()).bEnableHierarchyDrivenColors)
	{
		Result.TintColor = ViewModel->GetHierarchyBasedHighlightColor(State, EStateTreeViewModelColorAdjustment::Dark).CopyWithNewOpacity(0.0f);
		Result.OutlineSettings.Color = ViewModel->GetHierarchyBasedHighlightColor(State);

		if (bEnableDepthBasedOutline)
		{
			Result.OutlineSettings.Color = ViewModel->GetHierarchyBasedBackgroundColor(State, EStateTreeViewModelColorAdjustment::Bright);
		}
	}

	return FDeferredCleanupSlateBrush::CreateBrush(Result);
}

TSharedPtr<FDeferredCleanupSlateBrush> FStateCentricViewUtils::MakeStateOutline(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State)
{
	const FSlateBrush* BaseBrush = State->Children.IsEmpty()
		? FStateTreeEditorStyle::Get().GetBrush("StateTree.CentricView.Leaf.Outline")
		: FStateTreeEditorStyle::Get().GetBrush("StateTree.CentricView.Parent.Outline");

	FSlateBrush Result = *BaseBrush;

	// Use tint with 0 alpha due to slate rendering bug where alpha still matters.
	Result.TintColor = ViewModel->GetStateColor(State, EStateTreeViewModelColorAdjustment::Bright).CopyWithNewOpacity(0.0f);
	Result.OutlineSettings.Color = ViewModel->GetStateColor(State, EStateTreeViewModelColorAdjustment::Bright);

	if (UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(&ViewModel.Get()).bEnableHierarchyDrivenColors)
	{
		Result.TintColor = ViewModel->GetHierarchyBasedHighlightColor(State, EStateTreeViewModelColorAdjustment::Bright).CopyWithNewOpacity(0.0f);
		Result.OutlineSettings.Color = ViewModel->GetHierarchyBasedHighlightColor(State);

		if (bEnableDepthBasedOutline)
		{
			Result.OutlineSettings.Color = ViewModel->GetHierarchyBasedBackgroundColor(State, EStateTreeViewModelColorAdjustment::Bright);
		}
	}

	return FDeferredCleanupSlateBrush::CreateBrush(Result);
}

TSharedPtr<FDeferredCleanupSlateBrush> FStateCentricViewUtils::MakeStateOutlineMuted(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State)
{
	const FSlateBrush* BaseBrush = State->Children.IsEmpty()
		? FStateTreeEditorStyle::Get().GetBrush("StateTree.CentricView.Leaf.Outline")
		: FStateTreeEditorStyle::Get().GetBrush("StateTree.CentricView.Parent.Outline");

	FSlateBrush Result = *BaseBrush;

	// Use tint with 0 alpha due to slate rendering bug where alpha still matters.
	Result.TintColor = ViewModel->GetStateColor(State).CopyWithNewOpacity(0.0f);
	Result.OutlineSettings.Color = ViewModel->GetStateColor(State);

	if (UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(&ViewModel.Get()).bEnableHierarchyDrivenColors)
	{
		Result.TintColor = ViewModel->GetHierarchyBasedHighlightColor(State).CopyWithNewOpacity(0.0f);
		Result.OutlineSettings.Color = ViewModel->GetHierarchyBasedHighlightColor(State);

		if (bEnableDepthBasedOutline)
		{
			Result.OutlineSettings.Color = ViewModel->GetHierarchyBasedBackgroundColor(State, EStateTreeViewModelColorAdjustment::Bright);
		}
	}

	return FDeferredCleanupSlateBrush::CreateBrush(Result);
}

TSharedPtr<FDeferredCleanupSlateBrush> FStateCentricViewUtils::MakeStateHeaderBackground(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State, TOptional<FVector2f> ImageSize)
{
	const FSlateBrush* BaseBrush = State->Children.IsEmpty()
		? FStateTreeEditorStyle::Get().GetBrush("StateTree.CentricView.Leaf.Header.Background")
		: FStateTreeEditorStyle::Get().GetBrush("StateTree.CentricView.Parent.Header.Background");

	FSlateBrush Result = *BaseBrush;

	Result.TintColor = ViewModel->GetStateColor(State, EStateTreeViewModelColorAdjustment::Dark);
	Result.OutlineSettings.Color = ViewModel->GetStateColor(State, EStateTreeViewModelColorAdjustment::Dark);
	Result.ImageSize = ImageSize.Get(Result.ImageSize);

	if (UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(&ViewModel.Get()).bEnableHierarchyDrivenColors)
	{
		Result.TintColor = ViewModel->GetHierarchyBasedBackgroundColor(State, EStateTreeViewModelColorAdjustment::Dark);
		Result.OutlineSettings.Color = ViewModel->GetHierarchyBasedBackgroundColor(State, EStateTreeViewModelColorAdjustment::Dark);

		if (bDisableDarkenHeader)
		{
			Result.TintColor = ViewModel->GetHierarchyBasedBackgroundColor(State);
			Result.OutlineSettings.Color = ViewModel->GetHierarchyBasedBackgroundColor(State);
		}
	}

	return FDeferredCleanupSlateBrush::CreateBrush(Result);
}

TSharedPtr<FDeferredCleanupSlateBrush> FStateCentricViewUtils::MakeStateHeaderBackgroundMuted(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State)
{
	const FSlateBrush* BaseBrush = State->Children.IsEmpty()
		? FStateTreeEditorStyle::Get().GetBrush("StateTree.CentricView.Leaf.Header.Background")
		: FStateTreeEditorStyle::Get().GetBrush("StateTree.CentricView.Parent.Header.Background.Muted");

	FSlateBrush Result = *BaseBrush;

	Result.TintColor = State->Children.IsEmpty() ? ViewModel->GetStateColor(State, EStateTreeViewModelColorAdjustment::Dark) : Result.TintColor;
	Result.OutlineSettings.Color = State->Children.IsEmpty() ? ViewModel->GetStateColor(State, EStateTreeViewModelColorAdjustment::Dark) : Result.TintColor;

	if (UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(&ViewModel.Get()).bEnableHierarchyDrivenColors)
	{
		Result.TintColor = State->Children.IsEmpty() ? ViewModel->GetHierarchyBasedBackgroundColor(State, EStateTreeViewModelColorAdjustment::Dark) : Result.TintColor;
		Result.OutlineSettings.Color = State->Children.IsEmpty() ? ViewModel->GetHierarchyBasedBackgroundColor(State, EStateTreeViewModelColorAdjustment::Dark) : Result.TintColor;

		if (bDisableDarkenHeader)
		{
			Result.TintColor = State->Children.IsEmpty() ? ViewModel->GetHierarchyBasedBackgroundColor(State) : Result.TintColor;
			Result.OutlineSettings.Color = State->Children.IsEmpty() ? ViewModel->GetHierarchyBasedBackgroundColor(State) : Result.TintColor;
		}
	}

	return FDeferredCleanupSlateBrush::CreateBrush(Result);
}

TSharedPtr<FDeferredCleanupSlateBrush> FStateCentricViewUtils::MakeStateHeaderBackgroundCollapsed(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State)
{
	const FSlateBrush* BaseBrush = State->Children.IsEmpty()
		? FStateTreeEditorStyle::Get().GetBrush("StateTree.CentricView.Leaf.Header.Background")
		: FStateTreeEditorStyle::Get().GetBrush("StateTree.CentricView.Parent.Header.Background.Muted");

	FSlateBrush Result = *BaseBrush;

	Result.TintColor = ViewModel->GetStateColor(State, EStateTreeViewModelColorAdjustment::Darker);
	Result.OutlineSettings.Color = ViewModel->GetStateColor(State, EStateTreeViewModelColorAdjustment::Darker);

	if (UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(&ViewModel.Get()).bEnableHierarchyDrivenColors)
	{
		Result.TintColor = ViewModel->GetHierarchyBasedBackgroundColor(State, EStateTreeViewModelColorAdjustment::Dark);
		Result.OutlineSettings.Color = ViewModel->GetHierarchyBasedBackgroundColor(State, EStateTreeViewModelColorAdjustment::Dark);

		if (bDisableDarkenHeader)
		{
			Result.TintColor = ViewModel->GetHierarchyBasedBackgroundColor(State);
			Result.OutlineSettings.Color = ViewModel->GetHierarchyBasedBackgroundColor(State);
		}
	}

	return FDeferredCleanupSlateBrush::CreateBrush(Result);
}

TSharedPtr<FDeferredCleanupSlateBrush> FStateCentricViewUtils::MakeStateBackgroundMutedOutlined(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State)
{
	const FSlateBrush* BaseBrush = State->Children.IsEmpty()
		? FStateTreeEditorStyle::Get().GetBrush("StateTree.CentricView.Leaf.Background.Outlined")
		: FStateTreeEditorStyle::Get().GetBrush("StateTree.CentricView.Parent.Background.Muted.Outlined");

	FSlateBrush Result = *BaseBrush;

	Result.TintColor = State->Children.IsEmpty() ? ViewModel->GetStateColor(State, EStateTreeViewModelColorAdjustment::Dark) : Result.TintColor;
	Result.OutlineSettings.Color = ViewModel->GetStateColor(State, EStateTreeViewModelColorAdjustment::Bright);

	if (UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(&ViewModel.Get()).bEnableHierarchyDrivenColors)
	{
		Result.TintColor = State->Children.IsEmpty() ? ViewModel->GetHierarchyBasedBackgroundColor(State, EStateTreeViewModelColorAdjustment::Dark) : Result.TintColor;
		Result.OutlineSettings.Color = ViewModel->GetHierarchyBasedHighlightColor(State, EStateTreeViewModelColorAdjustment::Bright);

		if (bDisableDarkenHeader)
		{
			Result.TintColor = State->Children.IsEmpty() ? ViewModel->GetHierarchyBasedBackgroundColor(State) : Result.TintColor;
		}
	}

	return FDeferredCleanupSlateBrush::CreateBrush(Result);
}

TSharedPtr<FDeferredCleanupSlateBrush> FStateCentricViewUtils::MakeStateBackgroundCollapsedOutlined(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State)
{
	const FSlateBrush* BaseBrush = State->Children.IsEmpty()
		? FStateTreeEditorStyle::Get().GetBrush("StateTree.CentricView.Leaf.Background.Outlined")
		: FStateTreeEditorStyle::Get().GetBrush("StateTree.CentricView.Parent.Background.Muted.Outlined");

	FSlateBrush Result = *BaseBrush;

	Result.TintColor = State->Children.IsEmpty() ? ViewModel->GetStateColor(State, EStateTreeViewModelColorAdjustment::Dark) : Result.TintColor;
	Result.OutlineSettings.Color = ViewModel->GetStateColor(State, EStateTreeViewModelColorAdjustment::Dark);

	if (UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(&ViewModel.Get()).bEnableHierarchyDrivenColors)
	{
		Result.TintColor = State->Children.IsEmpty() ? ViewModel->GetHierarchyBasedBackgroundColor(State, EStateTreeViewModelColorAdjustment::Dark) : Result.TintColor;
		Result.OutlineSettings.Color = ViewModel->GetHierarchyBasedHighlightColor(State);

		if (bEnableDepthBasedOutline)
		{
			Result.OutlineSettings.Color = ViewModel->GetHierarchyBasedBackgroundColor(State, EStateTreeViewModelColorAdjustment::Bright);
		}

		if (bDisableDarkenHeader)
		{
			Result.TintColor = State->Children.IsEmpty() ? ViewModel->GetHierarchyBasedBackgroundColor(State) : Result.TintColor;
		}
	}

	return FDeferredCleanupSlateBrush::CreateBrush(Result);
}

TSharedPtr<FDeferredCleanupSlateBrush> FStateCentricViewUtils::MakeStateFootnoteBackground(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State)
{
	const FSlateBrush* BaseBrush = State->Children.IsEmpty()
		? FStateTreeEditorStyle::Get().GetBrush("StateTree.CentricView.Leaf.Footnote.Background")
		: FStateTreeEditorStyle::Get().GetBrush("StateTree.CentricView.Parent.Footnote.Background");

	FSlateBrush Result = *BaseBrush;
	
	return FDeferredCleanupSlateBrush::CreateBrush(Result);
}

TSharedPtr<FDeferredCleanupSlateBrush> FStateCentricViewUtils::MakeStateSubnodeBackgroundOutlined(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State)
{
	const FSlateBrush* BaseBrush = State->Children.IsEmpty()
		? FStateTreeEditorStyle::Get().GetBrush("StateTree.CentricView.Leaf.Subnode.Background.Outlined")
		: FStateTreeEditorStyle::Get().GetBrush("StateTree.CentricView.Parent.Subnode.Background.Outlined");

	FSlateBrush Result = *BaseBrush;

	Result.TintColor = ViewModel->GetStateColor(State, EStateTreeViewModelColorAdjustment::Dark);
	Result.OutlineSettings.Color = ViewModel->GetStateColor(State);

	if (UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(&ViewModel.Get()).bEnableHierarchyDrivenColors)
	{
		Result.TintColor = ViewModel->GetHierarchyBasedBackgroundColor(State, EStateTreeViewModelColorAdjustment::Dark);
		Result.OutlineSettings.Color = ViewModel->GetHierarchyBasedHighlightColor(State);

		if (bEnableDepthBasedOutline)
		{
			Result.OutlineSettings.Color = ViewModel->GetHierarchyBasedBackgroundColor(State, EStateTreeViewModelColorAdjustment::Bright);
		}

		if (bDisableDarkenHeader)
		{
			Result.TintColor = ViewModel->GetHierarchyBasedBackgroundColor(State);
		}
	}

	return FDeferredCleanupSlateBrush::CreateBrush(Result);
}

TSharedPtr<FDeferredCleanupSlateBrush> FStateCentricViewUtils::MakeStateSubnodeBackground(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State)
{
	const FSlateBrush* BaseBrush = State->Children.IsEmpty()
		? FStateTreeEditorStyle::Get().GetBrush("StateTree.CentricView.Leaf.Subnode.Background")
		: FStateTreeEditorStyle::Get().GetBrush("StateTree.CentricView.Parent.Subnode.Background");

	FSlateBrush Result = *BaseBrush;

	Result.TintColor = ViewModel->GetStateColor(State, EStateTreeViewModelColorAdjustment::Dark);
	Result.OutlineSettings.Color = ViewModel->GetStateColor(State, EStateTreeViewModelColorAdjustment::Dark);

	if (UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(&ViewModel.Get()).bEnableHierarchyDrivenColors)
	{
		Result.TintColor = ViewModel->GetHierarchyBasedBackgroundColor(State, EStateTreeViewModelColorAdjustment::Dark);
		Result.OutlineSettings.Color = ViewModel->GetHierarchyBasedBackgroundColor(State, EStateTreeViewModelColorAdjustment::Dark);

		if (bDisableDarkenHeader)
		{
			Result.TintColor = ViewModel->GetHierarchyBasedBackgroundColor(State);
		}
	}

	return FDeferredCleanupSlateBrush::CreateBrush(Result);
}

TSharedPtr<FDeferredCleanupSlateBrush> FStateCentricViewUtils::MakeStateCollapsedBackground(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State)
{
	const FSlateBrush* BaseBrush =  FStateTreeEditorStyle::Get().GetBrush("StateTree.CentricView.Parent.Subnode.Background");

	FSlateBrush Result = *BaseBrush;

	Result.TintColor = ViewModel->GetStateColor(State, EStateTreeViewModelColorAdjustment::Dark);
	Result.OutlineSettings.Color = ViewModel->GetStateColor(State, EStateTreeViewModelColorAdjustment::Dark);

	if (UStateCentricViewSettings::GetSchemaSettingsDefaultFallback(&ViewModel.Get()).bEnableHierarchyDrivenColors)
	{
		Result.TintColor = ViewModel->GetHierarchyBasedBackgroundColor(State, EStateTreeViewModelColorAdjustment::Dark);
		Result.OutlineSettings.Color = ViewModel->GetHierarchyBasedBackgroundColor(State, EStateTreeViewModelColorAdjustment::Dark);

		if (bDisableDarkenHeader)
		{
			Result.TintColor = ViewModel->GetHierarchyBasedBackgroundColor(State);
		}
	}

	return FDeferredCleanupSlateBrush::CreateBrush(Result);
}

const FSlateBrush* FStateCentricViewUtils::GetSubNodeBackground()
{
	return FStateTreeEditorStyle::Get().GetBrush("StateTree.CentricView.SubNode.Background");
}

const FSlateBrush* FStateCentricViewUtils::GetSubNodeFootnoteBackground()
{
	return FStateTreeEditorStyle::Get().GetBrush("StateTree.CentricView.SubNode.Footnote.Background");
}

} // namespace UE::StateTree::Editor::StateCentricView

#undef LOCTEXT_NAMESPACE

