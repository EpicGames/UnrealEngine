// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "StateCentricView/View/SStateTreeExtendableNode.h"

struct FSlateBrush;

class FDeferredCleanupSlateBrush;
class FStateTreeViewModel;
class SButton;
class SWidget;
class UStateTreeState;

enum class EExtendableNodeLOD : uint8;
enum class EStateTransitionDirection : uint8;

namespace UE::StateTree::Editor::StateCentricView
{

/**
 * Experimental. General Utils for state centric view widget creation.
 * 
 * Currrently contains widget creation that is reused at different places. We prefer duplicated logic be here so it
 * can be re-used in arbitray places if needed. All widget generation related methods must accept a LOD.
 */
struct FStateCentricViewUtils
{
	/** Build a view of transitions for given LOD / state / direction */
	static TSharedRef<SWidget> GenerateTransitionsViewWidget(const EExtendableNodeLOD LOD
		, TSharedRef<FStateTreeViewModel> ViewModel
		, TNotNull<UStateTreeState*> ViewState
		, const EStateTransitionDirection TransitionDirection);

	/** Build a transparent button that jumps to given state */
	static TSharedRef<SButton> GenerateGotoStateButton(const EExtendableNodeLOD LOD
		, TSharedRef<FStateTreeViewModel> ViewModel
		, TNotNull<const UStateTreeState*> ViewState);

	/** Adds list of child state extensions at footnote for Full LOD */
	static void AddChildStateFootnoteExtensions(const EExtendableNodeLOD InLOD
		, TNotNull<SStateTreeExtendableNode*> InSourceNode
		, TArray<TSharedPtr<FDeferredCleanupSlateBrush>>& OutSlateBrushResouces);

	/** Gets desired parent LOD based on various settings */
	static TOptional<EExtendableNodeLOD> GetDesiredParentLOD(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<UStateTreeState*> ParentState);

	/** Gets number of parents to show */
	static uint32 GetNumParentsToShow(TSharedRef<FStateTreeViewModel> ViewModel);

	/** Helpers to get brushes based on if state is parent / leaf w/ appropriate colors */
	[[nodiscard]] static TSharedPtr<FDeferredCleanupSlateBrush> MakeStateHalfTopOutline(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State);
	[[nodiscard]] static TSharedPtr<FDeferredCleanupSlateBrush> MakeStateHalfTopOutlineMuted(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State);
	[[nodiscard]] static TSharedPtr<FDeferredCleanupSlateBrush> MakeStateHalfTopOutlineCollapsed(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State);
	[[nodiscard]] static TSharedPtr<FDeferredCleanupSlateBrush> MakeStateOutline(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State);
	[[nodiscard]] static TSharedPtr<FDeferredCleanupSlateBrush> MakeStateOutlineMuted(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State);

	[[nodiscard]] static TSharedPtr<FDeferredCleanupSlateBrush> MakeStateHeaderBackground(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State, TOptional<FVector2f> ImageSize = {});
	[[nodiscard]] static TSharedPtr<FDeferredCleanupSlateBrush> MakeStateHeaderBackgroundMuted(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State);
	[[nodiscard]] static TSharedPtr<FDeferredCleanupSlateBrush> MakeStateHeaderBackgroundCollapsed(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State);

	[[nodiscard]] static TSharedPtr<FDeferredCleanupSlateBrush> MakeStateBackgroundMutedOutlined(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State);
	[[nodiscard]] static TSharedPtr<FDeferredCleanupSlateBrush> MakeStateBackgroundCollapsedOutlined(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State);

	[[nodiscard]] static TSharedPtr<FDeferredCleanupSlateBrush> MakeStateFootnoteBackground(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State);

	[[nodiscard]] static TSharedPtr<FDeferredCleanupSlateBrush> MakeStateSubnodeBackgroundOutlined(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State);
	[[nodiscard]] static TSharedPtr<FDeferredCleanupSlateBrush> MakeStateSubnodeBackground(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State);
	[[nodiscard]] static TSharedPtr<FDeferredCleanupSlateBrush> MakeStateCollapsedBackground(TSharedRef<FStateTreeViewModel> ViewModel, TNotNull<const UStateTreeState*> State);

	/** Helpers to get standard state centric brushes */
	static const FSlateBrush* GetSubNodeBackground();
	static const FSlateBrush* GetSubNodeFootnoteBackground();
};

} // namespace UE::StateTree::Editor::StateCentricView