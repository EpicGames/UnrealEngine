// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateCentricView/ViewModel/StateCentricViewEditorDataExtension.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateCentricViewEditorDataExtension)

UStateCentricViewEditorDataExtension::UStateCentricViewEditorDataExtension()
{
	InTransitionSplitterSizeRule = SSplitter::ESizeRule::FractionOfParent;
	OutTransitionSplitterSizeRule = SSplitter::ESizeRule::FractionOfParent;
}

void UStateCentricViewEditorDataExtension::ResetSplitters()
{
	LeftSpacerSplitterValue = 0.025f;
	InTransitionSplitterValue = 0.1875f;
	MainNodeSplitterValue = 0.375f;
	OutTransitionSplitterValue = 0.4375f;
	RightSpacerSplitterValue = 0.025f;
}

float UStateCentricViewEditorDataExtension::HandleGetLeftSpacerSplitterValue() const
{
	return LeftSpacerSplitterValue;
}

float UStateCentricViewEditorDataExtension::HandleGetInTransitionSplitterValue() const
{
	return InTransitionSplitterValue;
}

float UStateCentricViewEditorDataExtension::HandleGetMainNodeSplitterValue() const
{
	return MainNodeSplitterValue;
}

float UStateCentricViewEditorDataExtension::HandleGetOutTransitionSplitterValue() const
{
	return OutTransitionSplitterValue;
}

float UStateCentricViewEditorDataExtension::HandleGetRightSpacerSplitterValue() const
{
	return RightSpacerSplitterValue;
}

SSplitter::ESizeRule UStateCentricViewEditorDataExtension::GetInTransitionSplitterSizeRule() const
{
	return InTransitionSplitterSizeRule;
}

SSplitter::ESizeRule UStateCentricViewEditorDataExtension::GetOutTransitionSplitterSizeRule() const
{
	return OutTransitionSplitterSizeRule;
}

void UStateCentricViewEditorDataExtension::HandleOnLeftSpacerSplitterResized(float InSize)
{
	LeftSpacerSplitterValue = InSize;
}

void UStateCentricViewEditorDataExtension::HandleOnInTransitionSplitterResized(float InSize)
{
	InTransitionSplitterValue = InSize;
}

void UStateCentricViewEditorDataExtension::HandleOnMainNodeSplitterResized(float InSize)
{
	MainNodeSplitterValue = InSize;
}

void UStateCentricViewEditorDataExtension::HandleOnOutTransitionSplitterResized(float InSize)
{
	OutTransitionSplitterValue = InSize;
}

void UStateCentricViewEditorDataExtension::HandleOnRightSpacerSplitterResized(float InSize)
{
	RightSpacerSplitterValue = InSize;
}