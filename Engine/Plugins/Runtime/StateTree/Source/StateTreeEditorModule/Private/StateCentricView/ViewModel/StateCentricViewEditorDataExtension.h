// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateCentricView/StateCentricViewPerUserSettings.h"
#include "StateCentricView/StateCentricViewSettings.h"
#include "StateTreeEditorDataExtension.h"

#include "Widgets/Layout/SSplitter.h"

#include "StateCentricViewEditorDataExtension.generated.h"

#define UE_API STATETREEEDITORMODULE_API

class UStateTreeState;

/**
 * Experimental. Extension to store temporary editor data for state centric view & view interactions
 */
UCLASS(Experimental, MinimalAPI)
class UStateCentricViewEditorDataExtension : public UStateTreeEditorDataExtension
{
	GENERATED_BODY()

public:

	UStateCentricViewEditorDataExtension();

	/** User settings for this specific state tree. Ensures consistent behavior when opening different states in this state tree. */
	UPROPERTY(EditAnywhere, Category = "Appearance", meta = (EditCondition = "StateTreeEditorModule.StateCentricViewSettings.IsStateCentricViewEnabled", EditConditionHides))
	FStateCentricViewPerUserSetting PerStateTreeUserSettings;

	/**
	 * LOD to show this parent state at. We use this to ensure the view remains consistent
	 * when clicking between different states with the same parent & when clicking back & forth.
	 * 
	 * @TODO: This is a map, but it would be more idiomatic for it to exist as an editor data
	 * extension on a state. Currently that doesn't exist. See if ST owners are interested in that being added.
	 * Ex: This map becomes ugly when a state is deleted, with correct ownership this is not an issue.
	 */
	UPROPERTY(Transient)
	TMap<FGuid, EExtendableNodeLOD> LastUserSetParentStateLOD;

	UPROPERTY(Transient)
	bool bShouldStoreParentExpansion = true;

public:

	float LeftSpacerSplitterValue = 0.025f;
	float InTransitionSplitterValue = 0.1875f;
	float MainNodeSplitterValue = 0.375f;
	float OutTransitionSplitterValue = 0.4375f;
	float RightSpacerSplitterValue = 0.025f;

	SSplitter::ESizeRule InTransitionSplitterSizeRule;
	SSplitter::ESizeRule OutTransitionSplitterSizeRule;

public:

	UE_API void ResetSplitters();

	UE_API float HandleGetLeftSpacerSplitterValue() const;
	UE_API float HandleGetInTransitionSplitterValue() const;
	UE_API float HandleGetMainNodeSplitterValue() const;
	UE_API float HandleGetOutTransitionSplitterValue() const;
	UE_API float HandleGetRightSpacerSplitterValue() const;

	SSplitter::ESizeRule GetInTransitionSplitterSizeRule() const;
	SSplitter::ESizeRule GetOutTransitionSplitterSizeRule() const;

	UE_API void HandleOnLeftSpacerSplitterResized(float InSize);
	UE_API void HandleOnInTransitionSplitterResized(float InSize);
	UE_API void HandleOnMainNodeSplitterResized(float InSize);
	UE_API void HandleOnOutTransitionSplitterResized(float InSize);
	UE_API void HandleOnRightSpacerSplitterResized(float InSize);

protected:
	
	UStateTreeEditorData* GetStateTreeEditorData() const
	{
		return GetOuterUStateTreeEditorData();
	}
};

#undef UE_API