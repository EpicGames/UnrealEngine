// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeCompiler.h"
#include "StateTree.h"
#include "StateTreeAnyEnum.h"
#include "StateTreeCompilerLog.h"
#include "StateTreeEditorData.h"
#include "StateTreeEditorDataExtension.h"
#include "StateTreeEditorSchema.h"
#include "StateTreeExtension.h"
#include "StateTreeEditorModule.h"
#include "StateTreeEvaluatorBase.h"
#include "StateTreeTaskBase.h"
#include "StateTreeConditionBase.h"
#include "StateTreeConsiderationBase.h"
#include "Serialization/ArchiveUObject.h"
#include "GameFramework/Actor.h"
#include "StateTreePropertyRef.h"
#include "StateTreePropertyRefHelpers.h"
#include "StateTreePropertyHelpers.h"
#include "StateTreeDelegate.h"
#include "Customizations/StateTreeEditorNodeUtils.h"

#include "StructUtils/UserDefinedStruct.h"
#include "StructUtilsEditorUtils.h"

namespace UE::StateTree
{
	struct FCompileNodeContext : ICompileNodeContext
	{
		explicit FCompileNodeContext(const FStateTreeDataView& InDataView, const FStateTreeBindableStructDesc& InDesc, const IStateTreeBindingLookup& InBindingLookup)
			: InstanceDataView(InDataView),
			Desc(InDesc),
			BindingLookup(InBindingLookup)
		{
		}

		virtual void AddValidationError(const FText& Message) override
		{
			ValidationErrors.Add(Message);
		}

		virtual FStateTreeDataView GetInstanceDataView() const override
		{
			return InstanceDataView;
		}

		virtual bool HasBindingForProperty(const FName PropertyName) const override
		{
			const FPropertyBindingPath& PropertyPath = FPropertyBindingPath(Desc.ID, PropertyName);

			return BindingLookup.GetPropertyBindingSource(PropertyPath) != nullptr;
 		}

		TArray<FText> ValidationErrors;
		FStateTreeDataView InstanceDataView;
		const FStateTreeBindableStructDesc& Desc;
		const IStateTreeBindingLookup& BindingLookup;
	};
}


namespace UE::StateTree::Compiler
{
	// Version used by AppendToClassSchema. Bump this version number when a recompile of all assets is needed.
	//Only bump when only the compiler changes.
	//You do not need to bump the version when the memory layout of UStateTree has changed.
	static constexpr int32 ClassSchemaVersion = 6;

	FAutoConsoleVariable CVarLogEnableParameterDelegateDispatcherBinding(
		TEXT("StateTree.Compiler.EnableParameterDelegateDispatcherBinding"),
		false,
		TEXT("Enable binding from delegate dispatchers that are in the state tree parameters.")
	);

	bool bEnablePropertyFunctionWithEvaluationScopeInstanceData = true;
	FAutoConsoleVariableRef CVarEnablePropertyFunctionWithEvaluationScopeInstanceData(
		TEXT("StateTree.Compiler.EnablePropertyFunctionWithEvaluationScopeInstanceData"),
		bEnablePropertyFunctionWithEvaluationScopeInstanceData,
		TEXT("Use EvaluationScope data for property functions instead of SharedInstance data.\n")
		TEXT("SharedInstance is the previous behavior that is deprecated.")
	);

	bool bEnableConditionWithEvaluationScopeInstanceData = true;
	FAutoConsoleVariableRef CVarEnableConditionWithEvaluationScopeInstanceData(
		TEXT("StateTree.Compiler.EnableConditionWithEvaluationScopeInstanceData"),
		bEnableConditionWithEvaluationScopeInstanceData,
		TEXT("Use EvaluationScope data for conditions instead of SharedInstance data.")
	);

	bool bEnableUtilityConsiderationWithEvaluationScopeInstanceData = true;
	FAutoConsoleVariableRef CVarEnableUtilityConsiderationWithEvaluationScopeInstanceData(
		TEXT("StateTree.Compiler.EnableUtilityConsiderationWithEvaluationScopeInstanceData"),
		bEnableUtilityConsiderationWithEvaluationScopeInstanceData,
		TEXT("Use EvaluationScope data for utility considerations instead of SharedInstance data.")
	);

	bool bEnableCheckOutersOnCompilationSucceeded = false;
	FAutoConsoleVariableRef CVarEnableCheckOutersOnCompilationSucceeded(
		TEXT("StateTree.Compiler.bEnableCheckOutersOnCompilationSucceeded"),
		bEnableCheckOutersOnCompilationSucceeded,
		TEXT("After a StateTree successfully compiles, test if there are invalid references in the compiled data.")
	);

	bool bLogResultOnCompilationCompleted = false;
	FAutoConsoleVariableRef CVarLogCompiledStateTree(
		TEXT("StateTree.Compiler.LogResultOnCompilationCompleted"),
		bLogResultOnCompilationCompleted,
		TEXT("After a StateTree compiles, log the internal content of the StateTree.")
	);

	// Helper archive that checks that the all instanced sub-objects have correct outer. 
	class FCheckOutersArchive : public FArchiveUObject
	{
		using Super = FArchiveUObject;
		TNotNull<const UStateTree*> StateTree;
		TNotNull<const UStateTreeEditorData*> EditorData;
		FStateTreeCompilerLog& Log;
	public:

		FCheckOutersArchive(TNotNull<const UStateTree*> InStateTree, TNotNull<const UStateTreeEditorData*> InEditorData, FStateTreeCompilerLog& InLog)
			: StateTree(InStateTree)
			, EditorData(InEditorData)
			, Log(InLog)
		{
			Super::SetIsSaving(true);
			Super::SetIsPersistent(true);
		}

		virtual bool ShouldSkipProperty(const FProperty* InProperty) const
		{
			// Skip editor data.
			if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InProperty))
			{
				if (ObjectProperty->PropertyClass == UStateTreeEditorData::StaticClass())
				{
					return true;
				}
			}
			return false;
		}

		virtual FArchive& operator<<(UObject*& Object) override
		{
			if (Object)
			{
				if (const FProperty* Property = GetSerializedProperty())
				{
					if (Property->HasAnyPropertyFlags(CPF_InstancedReference))
					{
						if (!Object->IsInOuter(StateTree))
						{
							Log.Reportf(EMessageSeverity::Error, TEXT("Compiled StateTree contains instanced object %s (%s), which does not belong to the StateTree. This is due to error in the State Tree node implementation."),
								*GetFullNameSafe(Object), *GetFullNameSafe(Object->GetClass()));
						}

						if (Object->IsInOuter(EditorData))
						{
							Log.Reportf(EMessageSeverity::Error, TEXT("Compiled StateTree contains instanced object %s (%s), which still belongs to the Editor data. This is due to error in the State Tree node implementation."),
								*GetFullNameSafe(Object), *GetFullNameSafe(Object->GetClass()));
						}
					}
				}
			}
			return *this;
		}
	};

	enum class EPropertyVisitorResult : uint8
	{
		Continue,
		Break
	};

	void ScanProperties(FStateTreeDataView Data, TFunctionRef<EPropertyVisitorResult(const FProperty* InProperty, const void* InAddress)> InFunc)
	{
		TSet<const UObject*> Visited;
		auto ScanPropertiesRecursive = [&Visited, InFunc](auto&& Self, FStateTreeDataView CurrentData)
			{
				if (!CurrentData.IsValid())
				{
					return;
				}

				for (TPropertyValueIterator<FProperty> It(CurrentData.GetStruct(), CurrentData.GetMemory()); It; ++It)
				{
					const FProperty* Property = It->Key;
					const void* Address = It->Value;

					if (!Address)
					{
						continue;
					}

					if (InFunc(Property, Address) == EPropertyVisitorResult::Break)
					{
						break;
					}

					if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property))
					{
						if (const UObject* Object = ObjectProperty->GetObjectPropertyValue(Address))
						{
							// Recurse into instanced object
							if (Property->HasAnyPropertyFlags(CPF_InstancedReference))
							{
								if (!Visited.Contains(Object))
								{
									Visited.Add(Object);
									Self(Self, FStateTreeDataView(const_cast<UObject*>(Object)));
								}
							}
						}
					}
				}
			};

		ScanPropertiesRecursive(ScanPropertiesRecursive, Data);
	}

	/** Scans Data for actors that are tied to some level and returns them. */
	void ScanLevelActorReferences(FStateTreeDataView Data, TArray<const AActor*>& OutActors)
	{
		ScanProperties(Data, [&OutActors](const FProperty* InProperty, const void* InAddress)
		{
			if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(InProperty))
			{
				if (const UObject* Object = ObjectProperty->GetObjectPropertyValue(InAddress))
				{
					if (const AActor* Actor = Cast<AActor>(Object))
					{
						if (const ULevel* Level = Actor->GetLevel())
						{
							OutActors.Add(Actor);
						}
					}
				}
			}

			return EPropertyVisitorResult::Continue;
			});
	}

	bool ValidateNoLevelActorReferences(FStateTreeCompilerLog& Log, const FStateTreeBindableStructDesc& NodeDesc, const FStateTreeDataView NodeView, const FStateTreeDataView InstanceView)
	{
		TArray<const AActor*> LevelActors;
		ScanLevelActorReferences(NodeView, LevelActors);
		ScanLevelActorReferences(InstanceView, LevelActors);
		if (!LevelActors.IsEmpty())
		{
			TStringBuilder<256> AllActorsString;
			for (const AActor* Actor : LevelActors)
			{
				if (AllActorsString.Len() > 0)
				{
					AllActorsString += TEXT(", ");
				}
				AllActorsString += *GetNameSafe(Actor);
			}
			Log.Reportf(EMessageSeverity::Error, NodeDesc,
				TEXT("Level Actor references were found: %s. Direct Actor references are not allowed."),
					*AllActorsString);
			return false;
		}
		
		return true;
	}

	template<typename... T>
	bool StructHasAnyStructProperties(FConstStructView StructView)
	{
		bool bResult = false;
		ScanProperties(FStateTreeDataView(StructView.GetScriptStruct(), const_cast<uint8*>(StructView.GetMemory())), 
			[&bResult](const FProperty* InProperty, const void* InAddress)
			{
				if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
				{
					if ((... || StructProperty->Struct->IsChildOf<T>()))
					{
						bResult = true;
						return EPropertyVisitorResult::Break;
					}
				}

				return EPropertyVisitorResult::Continue;
			});

		return bResult;
	}

	template<typename... T>
	bool IsPropertyChildOfAnyStruct(const FStateTreeBindableStructDesc& Struct, const FPropertyBindingPath& Path)
	{
		TArray<FPropertyBindingPathIndirection> Indirection;
		const bool bResolved = Path.ResolveIndirections(Struct.Struct, Indirection);

		if (bResolved && Indirection.Num() > 0)
		{
			const FProperty* Property = Indirection.Last().GetProperty();
			check(Property);

			if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				return StructProperty->Struct && (... || StructProperty->Struct->IsChildOf<T>());
			}
		}

		return false;
	}

	bool IsNodeStructEligibleForBinding(FConstStructView NodeView)
	{
		check(NodeView.GetScriptStruct() && NodeView.GetScriptStruct()->IsChildOf<FStateTreeNodeBase>());

		return StructHasAnyStructProperties<FStateTreePropertyRef, FStateTreeDelegateDispatcher, FStateTreeDelegateListener>(NodeView);
	}

	void FValidationResult::Log(FStateTreeCompilerLog& Log, const TCHAR* ContextText, const FStateTreeBindableStructDesc& ContextStruct) const
	{
		Log.Reportf(EMessageSeverity::Error, ContextStruct, TEXT("The StateTree is too complex. Compact index %s out of range %d/%d."), ContextText, Value, MaxValue);
	}

	const UScriptStruct* GetBaseStructFromMetaData(const FProperty* Property, FString& OutBaseStructName)
	{
		static const FName NAME_BaseStruct = "BaseStruct";

		const UScriptStruct* Result = nullptr;
		OutBaseStructName = Property->GetMetaData(NAME_BaseStruct);
	
		if (!OutBaseStructName.IsEmpty())
		{
			Result = UClass::TryFindTypeSlow<UScriptStruct>(OutBaseStructName);
			if (!Result)
			{
				Result = LoadObject<UScriptStruct>(nullptr, *OutBaseStructName);
			}
		}

		return Result;
	}

	UObject* DuplicateInstanceObject(FStateTreeCompilerLog& Log, const FStateTreeBindableStructDesc& NodeDesc, FGuid NodeID, TNotNull<const UObject*> InstanceObject, TNotNull<UObject*> Owner)
	{
		if (InstanceObject->GetClass()->HasAnyClassFlags(CLASS_NewerVersionExists))
		{
			const UStateTree* OuterStateTree = Owner->GetTypedOuter<UStateTree>();
			Log.Reportf(EMessageSeverity::Warning, NodeDesc,
				TEXT("Duplicating '%s' with an old class '%s' Please resave State Tree asset '%s'."),
				*InstanceObject->GetName(), *InstanceObject->GetClass()->GetName(), *GetFullNameSafe(OuterStateTree));
		}

		// We want the object name to match between compilations.
		//Use the class name and increase the counter internally. We do that to not be influenced by another object in a different outer.
		//The objects from a previous compilation are rename in UStateTree::ResetCompiled.
		FName NewObjectName = InstanceObject->GetClass()->GetFName();
		while (StaticFindObjectFastInternal(nullptr, Owner, NewObjectName, EFindObjectFlags::ExactClass) != nullptr)
		{
			NewObjectName.SetNumber(NewObjectName.GetNumber() + 1);
		}
		return ::DuplicateObject(&(*InstanceObject), &(*Owner), NewObjectName);
	}

	struct FCompletionTasksMaskResult
	{
		FStateTreeTasksCompletionStatus::FMaskType Mask;
		int32 MaskBufferIndex;	// Index of FStateTreeTasksCompletionStatus::Buffer.
		int32 MaskFirstTaskBitOffset; // Inside FStateTreeTasksCompletionStatus::Buffer[MaskBufferIndex], the bit offset of the first task.
		int32 FullMaskEndTaskBitOffset; // the next bit that the next child can take in the full FStateTreeTasksCompletionStatus::Buffer.
	};

	/** Makes the completion mask for the state or frame. */
	FCompletionTasksMaskResult MakeCompletionTasksMask(int32 FullStartBitIndex, TConstArrayView<FStateTreeEditorNode> AllTasks, TConstArrayView<int32> ValidTasks)
	{
		FStateTreeTasksCompletionStatus::FMaskType Mask = 0;
		int32 NumberOfBitsNeeded = 0;
		const int32 NumberOfTasks = ValidTasks.Num();

		TBitArray<> ConsideredForCompletion;
		bool bHasTaskConsideredForCompletion = false;
		for (int32 Index = 0; Index < NumberOfTasks; ++Index)
		{
			const int32 TaskIndex = ValidTasks[Index];
			const bool bIsTaskConsideredForCompletion = UE::StateTreeEditor::EditorNodeUtils::IsTaskEnabled(AllTasks[TaskIndex])
				&& UE::StateTreeEditor::EditorNodeUtils::IsTaskConsideredForCompletion(AllTasks[TaskIndex]);

			ConsideredForCompletion.Add(bIsTaskConsideredForCompletion);
			bHasTaskConsideredForCompletion |= bIsTaskConsideredForCompletion;
		}

		NumberOfBitsNeeded = NumberOfTasks;

		//Each state will take at least 1 bit.
		//No task are marked as completed, we still need a flag to mark the state itself completes (ie. for linked state).
		//Linked state can't have tasks but keep that logic in case it is needed later.
		//A mask of 0, will cause issue with the GetCompletionsStatus.
		if (!bHasTaskConsideredForCompletion)
		{
			Mask |= 1;
			++NumberOfBitsNeeded;
		}

		for (int32 Index = NumberOfTasks - 1; Index >= 0; --Index)
		{
			Mask <<= 1;
			if (ConsideredForCompletion[Index])
			{
				Mask |= 1;
			}
		}

		constexpr int32 NumberOfBitsPerMask = sizeof(FStateTreeTasksCompletionStatus::FMaskType) * 8;

		// Is the new amount of bits bring up over the next buffer?
		const int32 CurrentEndBitIndex = FullStartBitIndex + NumberOfBitsNeeded;
		const int32 NewMaskBufferIndex = (CurrentEndBitIndex - 1) / NumberOfBitsPerMask;
		if (NewMaskBufferIndex != FullStartBitIndex / NumberOfBitsPerMask)
		{
			// Do not shift the mask. Use the next int32
			const int32 NewMaskFirstTaskBitOffset = 0;
			const int32 NewMaskEndTaskBitOffset = (NewMaskBufferIndex * NumberOfBitsPerMask) + NumberOfBitsNeeded;
			return { .Mask = Mask, .MaskBufferIndex = NewMaskBufferIndex, .MaskFirstTaskBitOffset = NewMaskFirstTaskBitOffset, .FullMaskEndTaskBitOffset = NewMaskEndTaskBitOffset };
		}
		else
		{
			const int32 NewMaskFirstTaskBitOffset = FullStartBitIndex % NumberOfBitsPerMask;
			const int32 NewMaskEndTaskBitOffset = CurrentEndBitIndex;

			Mask <<= NewMaskFirstTaskBitOffset;

			return { .Mask = Mask, .MaskBufferIndex = NewMaskBufferIndex, .MaskFirstTaskBitOffset = NewMaskFirstTaskBitOffset, .FullMaskEndTaskBitOffset = NewMaskEndTaskBitOffset };
		}
	}

}; // UE::StateTree::Compiler

void FStateTreeCompiler::AppendToStateTreeClassSchema(FAppendToClassSchemaContext& Context)
{
	Context.Update(&UE::StateTree::Compiler::ClassSchemaVersion, sizeof(UE::StateTree::Compiler::ClassSchemaVersion));

	auto UpdateContext = [&Context](const FStringView Name)
		{
			const IConsoleVariable* Variable = IConsoleManager::Get().FindConsoleVariable(Name.GetData());
			ensure(Variable != nullptr);
			const bool Value = Variable == nullptr || !Variable->GetBool();
			Context.Update(&Value, sizeof(bool));
		};
	UpdateContext(TEXT("StateTree.Compiler.EnableParameterDelegateDispatcherBinding"));
	UpdateContext(TEXT("StateTree.Compiler.EnablePropertyFunctionWithEvaluationScopeInstanceData"));
	UpdateContext(TEXT("StateTree.Compiler.EnableConditionWithEvaluationScopeInstanceData"));
	UpdateContext(TEXT("StateTree.Compiler.EnableUtilityConsiderationWithEvaluationScopeInstanceData"));
}


bool FStateTreeCompiler::FailCompilation(UStateTree::ECompileStatus Status) const
{
	StateTree->ResetCompiled();
	StateTree->CompileStatus = Status;
	// The editor data doesn't match the state tree data, but another compilation would create the same result.
	//Mark it as not dirty and use the CompileStatus to mark the runtime data as invalid.
	//Keeping it dirty would trigger multiple compilations that are not needed.
	StateTree->EditorDataDirtyStatus = UStateTree::EDirtyStatus::None;
	StateTree->LastCompiledEditorDataHash = 0;
	return false;
}

bool FStateTreeCompiler::PreCompile(TNotNull<UStateTree*> InStateTree)
{
	if (bCompiled)
	{
		Log.Reportf(EMessageSeverity::Error, TEXT("Internal error. The compiler has already been executed. Create a new compiler instance."));
		return false;
	}
	bCompiled = true;

	StateTree = InStateTree;
	EditorData = Cast<UStateTreeEditorData>(StateTree->EditorData);
	if (!EditorData)
	{
		return false;
	}

	// Cleanup existing state
	StateTree->ResetCompiled();

	if (!EditorData->Schema)
	{
		Log.Reportf(EMessageSeverity::Error, TEXT("Missing Schema. Please set valid schema in the State Tree Asset settings."));
		return false;
	}

	if (!BindingsCompiler.Init(StateTree->PropertyBindings, Log))
	{
		return false;
	}

	EditorData->GetAllStructValues(IDToStructValue);
	for (const FPropertyBindingBinding& Binding : EditorData->EditorBindings.GetBindings())
	{
		BindingsByTargetStructID.Add(Binding.GetTargetPath().GetStructID(), &Binding);
	}
	EditorBindingsNum = EditorData->EditorBindings.GetBindings().Num();

	return true;
}

bool FStateTreeCompiler::CompilePublic(TNotNull<UStateTree*> InStateTree)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStateTreeCompiler::CompilePublic)
	if (!PreCompile(InStateTree))
	{
		return FailCompilation(UStateTree::ECompileStatus::Public);
	}

	if (!CreateParameters())
	{
		return FailCompilation(UStateTree::ECompileStatus::Public);
	}

	// Public succeeded, the next step is dirty
	StateTree->CompileStatus = UStateTree::ECompileStatus::Internal;
	StateTree->EditorDataDirtyStatus = UStateTree::EDirtyStatus::Internal;

	return true;
}

bool FStateTreeCompiler::CompileInternalImpl()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStateTreeCompiler::CompileInternal)

	check(StateTree);
	check(EditorData);

	// Copy schema the EditorData
	StateTree->Schema = DuplicateObject(EditorData->Schema, StateTree);

	int32 ContextDataIndex = 0;

	// Mark all named external values as binding source
	if (ensure(StateTree->Schema))
	{
		StateTree->ContextDataDescs = StateTree->Schema->GetContextDataDescs();
		for (FStateTreeExternalDataDesc& Desc : StateTree->ContextDataDescs)
		{
			if (Desc.Struct)
			{
				const FStateTreeBindableStructDesc ExtDataDesc = {
					UE::StateTree::Editor::GlobalStateName,
					Desc.Name,
					Desc.Struct,
					FStateTreeDataHandle(EStateTreeDataSourceType::ContextData, ContextDataIndex++),
					EStateTreeBindableStructSource::Context,
					Desc.ID
				};
				BindingsCompiler.AddSourceStruct(ExtDataDesc);
				if (const auto Validation = UE::StateTree::Compiler::IsValidIndex16(ContextDataIndex); Validation.DidFail())
				{
					Validation.Log(Log, TEXT("ExternalStructIndex"), ExtDataDesc);
					return FailCompilation(UStateTree::ECompileStatus::Internal);
				}
				Desc.Handle.DataHandle = ExtDataDesc.DataHandle;
			}
		}
	}

	if (const UE::StateTree::Compiler::FValidationResult Validation = UE::StateTree::Compiler::IsValidIndex16(ContextDataIndex); Validation.DidFail())
	{
		Validation.Log(Log, TEXT("NumContextData"));
		return FailCompilation(UStateTree::ECompileStatus::Internal);
	}
	StateTree->NumContextData = static_cast<uint16>(ContextDataIndex);

	GatherBindingSources();

	if (!CreateStates())
	{
		return FailCompilation(UStateTree::ECompileStatus::Internal);
	}

	// Eval and Global task methods use InstanceStructs.Num() as ID generator.
	check(InstanceStructs.Num() == 0);
	
	if (!CreateEvaluators())
	{
		return FailCompilation(UStateTree::ECompileStatus::Internal);
	}

	if (!CreateGlobalTasks())
	{
		return FailCompilation(UStateTree::ECompileStatus::Internal);
	}

	const int32 NumGlobalInstanceData = InstanceStructs.Num();
	if (const auto Validation = UE::StateTree::Compiler::IsValidIndex16(NumGlobalInstanceData); Validation.DidFail())
	{
		Validation.Log(Log, TEXT("NumGlobalInstanceData"));
		return FailCompilation(UStateTree::ECompileStatus::Internal);
	}
	StateTree->NumGlobalInstanceData = uint16(NumGlobalInstanceData);

	if (!CreateStateTasksAndParameters())
	{
		return FailCompilation(UStateTree::ECompileStatus::Internal);
	}

	if (!CreateStateTransitions())
	{
		return FailCompilation(UStateTree::ECompileStatus::Internal);
	}

	if (!CreateStateConsiderations())
	{
		return FailCompilation(UStateTree::ECompileStatus::Internal);
	}

	StateTree->Nodes = Nodes;
	StateTree->DefaultInstanceData.Init(*StateTree, InstanceStructs, FStateTreeInstanceData::FAddArgs{ .bDuplicateWrappedObject = false });
	StateTree->SharedInstanceData.Init(*StateTree, SharedInstanceStructs, FStateTreeInstanceData::FAddArgs{ .bDuplicateWrappedObject = false });
	StateTree->DefaultEvaluationScopeInstanceData.Init(StateTree, EvaluationScopeStructs, UE::StateTree::InstanceData::FInstanceContainer::FAddArgs{ .bDuplicateWrappedObject = false });
	StateTree->DefaultExecutionRuntimeData.Init(StateTree, ExecutionRuntimeStructs, UE::StateTree::InstanceData::FInstanceContainer::FAddArgs{ .bDuplicateWrappedObject = false });

	if (!CreateTaskCompletionDispatchers())
	{
		return FailCompilation(UStateTree::ECompileStatus::Internal);
	}

	// Store the new compiled dispatchers.
	EditorData->CompiledDispatchers = BindingsCompiler.GetCompiledDelegateDispatchers();

	BindingsCompiler.Finalize();

	// Internal succeeded, the next step is dirty
	StateTree->CompileStatus = UStateTree::ECompileStatus::Link;
	StateTree->EditorDataDirtyStatus = UStateTree::EDirtyStatus::Link;

	if (!StateTree->Link())
	{
		Log.Reportf(EMessageSeverity::Error, TEXT("Unexpected failure to link the StateTree asset. See log for more info."));
		return FailCompilation(UStateTree::ECompileStatus::Link);
	}

	// Store mapping between node unique ID and their compiled index. Used for debugging purposes.
	StateTree->IDToNodeMappings.Reserve(IDToNode.Num());
	for (const TPair<FGuid, int32>& ToNode : IDToNode)
	{
		StateTree->IDToNodeMappings.Emplace(ToNode.Key, FStateTreeIndex16(ToNode.Value));
	}

	// Store mapping between state unique ID and state handle. Used for debugging purposes.
	StateTree->IDToStateMappings.Reserve(IDToState.Num());
	for (const TPair<FGuid, int32>& ToState : IDToState)
	{
		StateTree->IDToStateMappings.Emplace(ToState.Key, FStateTreeStateHandle(ToState.Value));
	}

	// Store mapping between state transition identifier and compact transition index. Used for debugging purposes.
	StateTree->IDToTransitionMappings.Reserve(IDToTransition.Num());
	for (const TPair<FGuid, int32>& ToTransition : IDToTransition)
	{
		StateTree->IDToTransitionMappings.Emplace(ToTransition.Key, FStateTreeIndex16(ToTransition.Value));
	}

	if (!NotifyInternalPost())
	{
		return FailCompilation(UStateTree::ECompileStatus::Internal);
	}

	if (UE::StateTree::Compiler::bEnableCheckOutersOnCompilationSucceeded)
	{
		CheckCompiledStateTreeOuters(StateTree, Log);
	}

	if (UE::StateTree::Compiler::bLogResultOnCompilationCompleted)
	{
		UE_LOGF(LogStateTreeEditor, Log, "%ls", *StateTree->DebugInternalLayoutAsString());
	}

	return true;
}

bool FStateTreeCompiler::CompileInternal(TNotNull<UStateTree*> InStateTree)
{
	// Compile both step to have the public up to date.
	if (!CompilePublic(InStateTree))
	{
		return FailCompilation(UStateTree::ECompileStatus::Public);
	}
	return CompileInternalImpl();
}

bool FStateTreeCompiler::Compile(TNotNull<UStateTree*> InStateTree)
{
	return CompileInternal(InStateTree);
}

bool FStateTreeCompiler::Compile(UStateTree& InStateTree)
{
	return Compile(&InStateTree);
}

void FStateTreeCompiler::CheckCompiledStateTreeOuters(TNotNull<const UStateTree*> InStateTree, FStateTreeCompilerLog& InLog)
{
	const UStateTreeEditorData* EditorData = Cast<UStateTreeEditorData>(InStateTree->EditorData);
	if (!EditorData)
	{
		return;
	}

	// Test if there are invalid references in the compiled data.
	// Serialize is non-const but does not modify the StateTree when used with a saving archive.
	UE::StateTree::Compiler::FCheckOutersArchive CheckOuters(InStateTree, EditorData, InLog);
	const_cast<UStateTree*>(static_cast<const UStateTree*>(InStateTree))->Serialize(CheckOuters);
}

FStateTreeStateHandle FStateTreeCompiler::GetStateHandle(const FGuid& StateID) const
{
	const int32* Idx = IDToState.Find(StateID);
	if (Idx == nullptr)
	{
		return FStateTreeStateHandle::Invalid;
	}

	return FStateTreeStateHandle(uint16(*Idx));
}

UStateTreeState* FStateTreeCompiler::GetState(const FGuid& StateID) const
{
	const int32* Idx = IDToState.Find(StateID);
	if (Idx == nullptr)
	{
		return nullptr;
	}

	return SourceStates[*Idx];
}

void FStateTreeCompiler::GatherBindingSources()
{
	check(EditorData);

	// Build the list of all the binding sources. Allow the compiler to know if a struct is used when compiling it by a child state (not yet fully compiled).
	for (const FPropertyBindingBinding& BindingIt : EditorData->EditorBindings.GetBindings())
	{
		const FStateTreePropertyPathBinding& Binding = static_cast<const FStateTreePropertyPathBinding&>(BindingIt);
		const FGuid SourceStructID = Binding.GetSourcePath().GetStructID();
		if (Binding.IsTaskCompletionBinding())
		{
			TaskCompletionDispatcherIDs.Add(SourceStructID);
		}
	}
}

bool FStateTreeCompiler::CreateParameters()
{
	// Copy parameters from EditorData
	StateTree->ParameterDataType = EditorData->Schema->GetGlobalParameterDataType();

	if (StateTree->ParameterDataType == EStateTreeParameterDataType::GlobalParameterData)
	{
		StateTree->Parameters = EditorData->GetRootParametersPropertyBag();
	}
	else
	{
		ensure(StateTree->ParameterDataType == EStateTreeParameterDataType::ExternalGlobalParameterData);
		StateTree->Parameters = EditorData->GetRootParametersPropertyBag();
	}

	// Mark parameters as binding source
	const EStateTreeDataSourceType GlobalParameterDataType = UE::StateTree::CastToDataSourceType(StateTree->ParameterDataType);
	const FStateTreeBindableStructDesc ParametersDesc = {
			UE::StateTree::Editor::GlobalStateName,
			TEXT("Parameters"),
			StateTree->Parameters.GetPropertyBagStruct(),
			FStateTreeDataHandle(GlobalParameterDataType),
			EStateTreeBindableStructSource::Parameter,
			EditorData->GetRootParametersGuid()
	};
	BindingsCompiler.AddSourceStruct(ParametersDesc);

	const FStateTreeDataView PropertyBagView(EditorData->GetRootParametersPropertyBag().GetPropertyBagStruct(), (uint8*)EditorData->GetRootParametersPropertyBag().GetValue().GetMemory());
	if (!UE::StateTree::Compiler::ValidateNoLevelActorReferences(Log, ParametersDesc, FStateTreeDataView(), PropertyBagView))
	{
		return false;
	}

	// Compile the delegate dispatcher.
	if (UE::StateTree::Compiler::CVarLogEnableParameterDelegateDispatcherBinding->GetBool())
	{
		FValidatedPathBindings Bindings;
		FStateTreeDataView SourceValue(StateTree->Parameters.GetMutableValue());
		if (!GetAndValidateBindings(ParametersDesc, SourceValue, Bindings))
		{
			Log.Reportf(EMessageSeverity::Error, TEXT("Failed to create bindings for global parameters."));
			return false;
		}

		if (Bindings.CopyBindings.Num() != 0 || Bindings.DelegateListeners.Num() != 0 || Bindings.ReferenceBindings.Num() != 0)
		{
			Log.Reportf(EMessageSeverity::Warning, TEXT("The global parameters should not target have binding."));
			return false;
		}

		if (!BindingsCompiler.CompileDelegateDispatchers(ParametersDesc, EditorData->CompiledDispatchers, Bindings.DelegateDispatchers, SourceValue))
		{
			Log.Reportf(EMessageSeverity::Error, TEXT("Failed to create delegate dispatcher bindings."));
			return false;
		}
	}

	return true;
}

bool FStateTreeCompiler::CreateStates()
{
	check(EditorData);
	
	// Create main tree (omit subtrees)
	for (UStateTreeState* SubTree : EditorData->SubTrees)
	{
		if (SubTree != nullptr
			&& SubTree->Type != EStateTreeStateType::Subtree)
		{
			if (!CreateStateRecursive(*SubTree, FStateTreeStateHandle::Invalid))
			{
				return false;
			}
		}
	}

	// Create Subtrees
	for (UStateTreeState* SubTree : EditorData->SubTrees)
	{
		TArray<UStateTreeState*> Stack;
		Stack.Push(SubTree);
		while (!Stack.IsEmpty())
		{
			if (UStateTreeState* State = Stack.Pop())
			{
				if (State->Type == EStateTreeStateType::Subtree)
				{
					if (!CreateStateRecursive(*State, FStateTreeStateHandle::Invalid))
					{
						return false;
					}
				}
				Stack.Append(State->Children);
			}
		}
	}

	return true;
}

bool FStateTreeCompiler::CreateStateRecursive(UStateTreeState& State, const FStateTreeStateHandle Parent)
{
	check(StateTree);
	check(StateTree->Schema);

	FStateTreeCompilerLogStateScope LogStateScope(&State, Log);

	if ((State.Type == EStateTreeStateType::LinkedAsset
		|| State.Type == EStateTreeStateType::Linked)
		&& State.Children.Num() > 0)
	{
		Log.Reportf(EMessageSeverity::Warning,
			TEXT("Linked State cannot have child states, because the state selection will enter to the linked state on activation."));
	}

	if (const auto Validation = UE::StateTree::Compiler::IsValidCount16(StateTree->States.Num()); Validation.DidFail())
	{
		Validation.Log(Log, TEXT("States"));
		return false;
	}
	const int32 StateIdx = StateTree->States.AddDefaulted();
	const FStateTreeStateHandle StateHandle = FStateTreeStateHandle(StateIdx);
	FCompactStateTreeState& CompactState = StateTree->States[StateIdx];
	CompactState.Name = State.Name;
	CompactState.Tag = State.Tag;
	CompactState.Parent = Parent;
	CompactState.bEnabled = State.bEnabled;
	CompactState.bCheckPrerequisitesWhenActivatingChildDirectly = State.bCheckPrerequisitesWhenActivatingChildDirectly;
	CompactState.Weight = State.Weight;

	CompactState.bHasCustomTickRate = State.bHasCustomTickRate && StateTree->Schema->IsScheduledTickAllowed();
	CompactState.CustomTickRate = FMath::Max(State.CustomTickRate, 0.0f);
	CompactState.bCopyParameterBindingsOnTick = State.bCopyParameterBindingsOnTick;
	if (CompactState.bHasCustomTickRate && State.CustomTickRate < 0.0f)
	{
		Log.Reportf(EMessageSeverity::Warning, TEXT("The custom tick rate has to be greater than or equal to 0."));
	}

	CompactState.Type = State.Type;
	CompactState.SelectionBehavior = State.SelectionBehavior;

	// create sub frame info
	if (!Parent.IsValid())
	{
		const int32 NewFrameIndex = StateTree->Frames.Num();
		FCompactStateTreeFrame& CompactFrame = StateTree->Frames.AddDefaulted_GetRef();
		CompactFrame.RootState = FStateTreeStateHandle((uint16)StateIdx);
		CompactFrame.NumberOfTasksStatusMasks = 0;

		StateHandleToFrameIndex.Add(StateHandle, NewFrameIndex);
	}
	else
	{
		const int32* ParentFrameIndexPtr = StateHandleToFrameIndex.Find(Parent);
		if (ensure(ParentFrameIndexPtr != nullptr))
		{
			const int32 ParentFrameIndex = *ParentFrameIndexPtr;
			StateHandleToFrameIndex.Add(StateHandle, ParentFrameIndex);
		}
	}

	if (!StateTree->Schema->IsStateTypeAllowed(CompactState.Type))
	{
		Log.Reportf(EMessageSeverity::Warning,
			TEXT("The State '%s' has a restricted type for the schema."),
			*CompactState.Name.ToString());
		return false;
	}

	const bool bHasPredefinedSelectionBehavior = (CompactState.Type == EStateTreeStateType::Linked || CompactState.Type == EStateTreeStateType::LinkedAsset);
	if (bHasPredefinedSelectionBehavior)
	{
		CompactState.SelectionBehavior = EStateTreeStateSelectionBehavior::TryEnterState;
	}
	else if (!StateTree->Schema->IsStateSelectionAllowed(CompactState.SelectionBehavior))
	{
		Log.Reportf(EMessageSeverity::Warning,
			TEXT("The State '%s' has a restricted selection behavior for the schema."),
			*CompactState.Name.ToString());
		return false;
	}

	if (State.Type == EStateTreeStateType::LinkedAsset && State.Tag.IsValid())
	{
		const EStateTreeVisitor VisitResult = EditorData->VisitStateBindings(&State, [](TNotNull<const UStateTreeState*> State, const FStateTreePropertyPathBinding& Binding, const FStateTreeBindableStructDesc& BindingTargetNodeDesc, const FStateTreeDataView BindingTargetNodeValue)
			{
				if (Binding.GetSourcePath().GetStructID() == State->Parameters.ID || Binding.GetTargetPath().GetStructID() == State->Parameters.ID)
				{
					return EStateTreeVisitor::Break;
				}

				return EStateTreeVisitor::Continue;
			});

		if (VisitResult == EStateTreeVisitor::Break)
		{
			CompactState.bCanOverrideLinkedAssetAtRuntime = false;
		}
	}

	SourceStates.Add(&State);
	StatePaths.Add(State.GetPath());
	IDToState.Add(State.ID, StateIdx);

	// Child states
	const int32 ChildrenBegin = StateTree->States.Num();
	if (const auto Validation = UE::StateTree::Compiler::IsValidCount16(ChildrenBegin); Validation.DidFail())
	{
		Validation.Log(Log, TEXT("ChildrenBegin"));
		return false;
	}
	CompactState.ChildrenBegin = uint16(ChildrenBegin);

	for (UStateTreeState* Child : State.Children)
	{
		if (Child != nullptr && Child->Type != EStateTreeStateType::Subtree)
		{
			if (!CreateStateRecursive(*Child, StateHandle))
			{
				return false;
			}
		}
	}
	
	const int32 ChildrenEnd = StateTree->States.Num();
	if (const auto Validation = UE::StateTree::Compiler::IsValidCount16(ChildrenEnd); Validation.DidFail())
	{
		Validation.Log(Log, TEXT("ChildrenEnd"));
		return false;
	}
	StateTree->States[StateIdx].ChildrenEnd = uint16(ChildrenEnd); // Not using CompactState here because the array may have changed.

	return true;
}

bool FStateTreeCompiler::CreateConditions(UStateTreeState& State, const FString& StatePath, TConstArrayView<FStateTreeEditorNode> Conditions)
{
	bool bSucceeded = true;

	for (int32 Index = 0; Index < Conditions.Num(); Index++)
	{
		const bool bIsFirst = Index == 0;
		const FStateTreeEditorNode& CondNode = Conditions[Index];
		// First operand should be copied as we don't have a previous item to operate on.
		const EStateTreeExpressionOperand Operand = bIsFirst ? EStateTreeExpressionOperand::Copy : CondNode.ExpressionOperand;
		// First indent must be 0 to make the parentheses calculation match.
		const int32 CurrIndent = bIsFirst ? 0 : FMath::Clamp((int32)CondNode.ExpressionIndent, 0, UE::StateTree::MaxExpressionIndent);
		// Next indent, or terminate at zero.
		const int32 NextIndent = Conditions.IsValidIndex(Index + 1) ? FMath::Clamp((int32)Conditions[Index + 1].ExpressionIndent, 0, UE::StateTree::MaxExpressionIndent) : 0;
		
		const int32 DeltaIndent = NextIndent - CurrIndent;

		if (!CreateCondition(State, StatePath, CondNode, Operand, (int8)DeltaIndent))
		{
			bSucceeded = false;
			continue;
		}
	}

	return bSucceeded;
}

bool FStateTreeCompiler::CreateEvaluators()
{
	check(EditorData);
	check(StateTree);

	bool bSucceeded = true;

	const int32 EvaluatorsBegin = Nodes.Num();
	if (const auto Validation = UE::StateTree::Compiler::IsValidCount16(EvaluatorsBegin); Validation.DidFail())
	{
		Validation.Log(Log, TEXT("EvaluatorsBegin"));
		return false;
	}
	StateTree->EvaluatorsBegin = uint16(EvaluatorsBegin);

	for (FStateTreeEditorNode& EvalNode : EditorData->Evaluators)
	{
		const int32 GlobalInstanceIndex = InstanceStructs.Num();
		const FStateTreeDataHandle EvalDataHandle(EStateTreeDataSourceType::GlobalInstanceData, GlobalInstanceIndex);
		if (!CreateEvaluator(EvalNode, EvalDataHandle))
		{
			bSucceeded = false;
			continue;
		}
	}
	
	const int32 EvaluatorsNum = Nodes.Num() - EvaluatorsBegin;
	if (const auto Validation = UE::StateTree::Compiler::IsValidCount16(EvaluatorsNum); Validation.DidFail())
	{
		Validation.Log(Log, TEXT("EvaluatorsNum"));
		return false;
	}
	StateTree->EvaluatorsNum = uint16(EvaluatorsNum);

	return bSucceeded && CreateBindingsForNodes(EditorData->Evaluators, FStateTreeIndex16(EvaluatorsBegin), InstanceStructs);
}

bool FStateTreeCompiler::CreateGlobalTasks()
{
	check(EditorData);
	check(StateTree);

	bool bSucceeded = true;

	const int32 GlobalTasksBegin = Nodes.Num();
	if (const auto Validation = UE::StateTree::Compiler::IsValidCount16(GlobalTasksBegin); Validation.DidFail())
	{
		Validation.Log(Log, TEXT("GlobalTasksBegin"));
		return false;
	}
	StateTree->GlobalTasksBegin = uint16(GlobalTasksBegin);
	StateTree->CompletionGlobalTasksMask = 0;

	TArray<int32, TInlineAllocator<32>> ValidTaskNodeIndex;
	for (int32 TaskIndex = 0; TaskIndex < EditorData->GlobalTasks.Num(); ++TaskIndex)
	{
		FStateTreeEditorNode& TaskNode = EditorData->GlobalTasks[TaskIndex];
		// Silently ignore empty nodes.
		if (!TaskNode.GetNode().IsValid())
		{
			continue;
		}

		const int32 GlobalInstanceIndex = InstanceStructs.Num();
		const FStateTreeDataHandle TaskDataHandle(EStateTreeDataSourceType::GlobalInstanceData, GlobalInstanceIndex);
		if (!CreateTask(nullptr, TaskNode, TaskDataHandle))
		{
			bSucceeded = false;
			continue;
		}

		ValidTaskNodeIndex.Add(TaskIndex);
	}

	if (ValidTaskNodeIndex.Num() > FStateTreeTasksCompletionStatus::MaxNumberOfTasksPerGroup)
	{
		Log.Reportf(EMessageSeverity::Error, FStateTreeBindableStructDesc(),
			TEXT("Exceeds the maximum number of global tasks (%d)"), FStateTreeTasksCompletionStatus::MaxNumberOfTasksPerGroup);
		return false;
	}
	
	constexpr int32 CompletionGlobalTaskStartBitIndex = 0;
	const UE::StateTree::Compiler::FCompletionTasksMaskResult MaskResult = UE::StateTree::Compiler::MakeCompletionTasksMask(CompletionGlobalTaskStartBitIndex, EditorData->GlobalTasks, ValidTaskNodeIndex);
	StateTree->CompletionGlobalTasksMask = MaskResult.Mask;
	GlobalTaskEndBit = MaskResult.FullMaskEndTaskBitOffset;
	StateTree->CompletionGlobalTasksControl = StateTree->Schema->AllowTasksCompletion() ? EditorData->GlobalTasksCompletion : EStateTreeTaskCompletionType::Any;

	if (MaskResult.MaskFirstTaskBitOffset != 0)
	{
		ensureMsgf(false, TEXT("Invalid bit offset %d. The Global task should start at 0."), MaskResult.MaskFirstTaskBitOffset);
		Log.Reportf(EMessageSeverity::Error, FStateTreeBindableStructDesc(), TEXT("Internal Error. Global task bit offset starts at 0."));
		return false;
	}
	
	const int32 GlobalTasksNum = Nodes.Num() - GlobalTasksBegin;
	if (const auto Validation = UE::StateTree::Compiler::IsValidCount16(GlobalTasksNum); Validation.DidFail())
	{
		Validation.Log(Log, TEXT("GlobalTasksNum"));
		return false;
	}
	StateTree->GlobalTasksNum = uint16(GlobalTasksNum);

	return bSucceeded && CreateBindingsForNodes(EditorData->GlobalTasks, FStateTreeIndex16(GlobalTasksBegin), InstanceStructs);
}

bool FStateTreeCompiler::CreateStateTasksAndParameters()
{
	check(StateTree);

	bool bSucceeded = true;

	// Index of the first instance data per state. Accumulated depth first.
	struct FTaskAndParametersCompactState
	{
		int32 FirstInstanceDataIndex = 0;
		int32 NextBitIndexForCompletionMask = 0;
		bool bProcessed = false;
	};
	TArray<FTaskAndParametersCompactState> StateInfos;
	StateInfos.SetNum(StateTree->States.Num());
	
	for (int32 StateIndex = 0; StateIndex < StateTree->States.Num(); ++StateIndex)
	{
		FCompactStateTreeState& CompactState = StateTree->States[StateIndex];
		const FStateTreeStateHandle CompactStateHandle(StateIndex);
		UStateTreeState* State = SourceStates[StateIndex];
		const FString& StatePath = GetStatePath(StateIndex);
		check(State != nullptr);

		// Carry over instance data count from parent.
		if (CompactState.Parent.IsValid())
		{
			const FCompactStateTreeState& ParentCompactState = StateTree->States[CompactState.Parent.Index];

			check(StateInfos[StateIndex].bProcessed == false);
			check(!bSucceeded || StateInfos[CompactState.Parent.Index].bProcessed == true);

			const int32 InstanceDataBegin = StateInfos[CompactState.Parent.Index].FirstInstanceDataIndex + (int32)ParentCompactState.InstanceDataNum;
			StateInfos[StateIndex].FirstInstanceDataIndex = InstanceDataBegin;

			CompactState.Depth = ParentCompactState.Depth + 1;
		}

		int32 InstanceDataIndex = StateInfos[StateIndex].FirstInstanceDataIndex;

		FStateTreeCompilerLogStateScope LogStateScope(State, Log);

		// Create parameters
		
		// Each state has their parameters as instance data.
		FInstancedStruct& Instance = InstanceStructs.AddDefaulted_GetRef();
		Instance.InitializeAs<FCompactStateTreeParameters>(State->Parameters.Parameters);
		FCompactStateTreeParameters& CompactStateTreeParameters = Instance.GetMutable<FCompactStateTreeParameters>(); 
			
		const int32 InstanceIndex = InstanceStructs.Num() - 1;
		if (const auto Validation = UE::StateTree::Compiler::IsValidIndex16(InstanceIndex); Validation.DidFail())
		{
			Validation.Log(Log, TEXT("InstanceIndex"));
			return false;
		}
		CompactState.ParameterTemplateIndex = FStateTreeIndex16(InstanceIndex);

		if (State->Type == EStateTreeStateType::Subtree)
		{
			CompactState.ParameterDataHandle = FStateTreeDataHandle(EStateTreeDataSourceType::SubtreeParameterData, InstanceDataIndex++, CompactStateHandle);
		}
		else
		{
			CompactState.ParameterDataHandle = FStateTreeDataHandle(EStateTreeDataSourceType::StateParameterData, InstanceDataIndex++, CompactStateHandle);
		}

		// @todo: We should be able to skip empty parameter data.

		// Binding target
		FStateTreeBindableStructDesc LinkedParamsDesc = {
			StatePath,
			FName("Parameters"),
			State->Parameters.Parameters.GetPropertyBagStruct(),
			CompactState.ParameterDataHandle,
			EStateTreeBindableStructSource::StateParameter,
			State->Parameters.ID
		};

		if (!UE::StateTree::Compiler::ValidateNoLevelActorReferences(Log, LinkedParamsDesc, FStateTreeDataView(), FStateTreeDataView(CompactStateTreeParameters.GetMutableValue())))
		{
			bSucceeded = false;
			continue;
		}

		// Add as binding source.
		BindingsCompiler.AddSourceStruct(LinkedParamsDesc);

		if (State->bHasRequiredEventToEnter)
		{
			CompactState.EventDataIndex = FStateTreeIndex16(InstanceDataIndex++);
			CompactState.RequiredEventToEnter.Tag = State->RequiredEventToEnter.Tag;
			CompactState.RequiredEventToEnter.PayloadStruct = State->RequiredEventToEnter.PayloadStruct;
			CompactState.bConsumeEventOnSelect = State->RequiredEventToEnter.bConsumeEventOnSelect;

			const FString StatePathWithConditions = StatePath + TEXT("/EnterConditions");

			FStateTreeBindableStructDesc Desc;
			Desc.StatePath = StatePathWithConditions,
			Desc.Struct = FStateTreeEvent::StaticStruct();
			Desc.Name = FName("Enter Event");
			Desc.ID = State->GetEventID();
			Desc.DataSource = EStateTreeBindableStructSource::StateEvent;
			Desc.DataHandle = FStateTreeDataHandle(EStateTreeDataSourceType::StateEvent, CompactState.EventDataIndex.Get(), CompactStateHandle);

			BindingsCompiler.AddSourceStruct(Desc);

			if (!CompactState.RequiredEventToEnter.IsValid())
			{
				Log.Reportf(EMessageSeverity::Error, Desc,
					TEXT("Event is marked as required, but isn't set up."));
				bSucceeded = false;
				continue;
			}
		}

		if (CompactState.Depth >= FStateTreeActiveStates::MaxStates)
		{
			Log.Reportf(EMessageSeverity::Error, LinkedParamsDesc,
				TEXT("Exceeds the maximum depth of execution (%u)"), FStateTreeActiveStates::MaxStates);
			bSucceeded = false;
			continue;
		}

		// Subtrees parameters cannot have bindings
		if (State->Type != EStateTreeStateType::Subtree)
		{
			FStateTreeIndex16 PropertyFunctionsBegin(Nodes.Num());
			if (!CreatePropertyFunctionsForStruct(LinkedParamsDesc.ID))
			{
				bSucceeded = false;
				continue;
			}

			FStateTreeIndex16 PropertyFunctionsEnd(Nodes.Num());
		
			if (PropertyFunctionsBegin == PropertyFunctionsEnd)
			{
				PropertyFunctionsBegin = FStateTreeIndex16::Invalid;
				PropertyFunctionsEnd = FStateTreeIndex16::Invalid;
			}

			// Only nodes support output bindings
			constexpr FStateTreeIndex16* OutputBindingsBatch = nullptr;
			if (!CreateBindingsForStruct(LinkedParamsDesc, FStateTreeDataView(CompactStateTreeParameters.GetMutableValue()), PropertyFunctionsBegin, PropertyFunctionsEnd, CompactState.ParameterBindingsBatch, OutputBindingsBatch))
			{
				bSucceeded = false;
				continue;
			}
		}

		// Create tasks
		const int32 TasksBegin = Nodes.Num();
		if (const auto Validation = UE::StateTree::Compiler::IsValidCount16(TasksBegin); Validation.DidFail())
		{
			Validation.Log(Log, TEXT("TasksBegin"));
			return false;
		}
		CompactState.TasksBegin = uint16(TasksBegin);
		
		TArrayView<FStateTreeEditorNode> Tasks;
		if (State->Tasks.Num())
		{
			Tasks = State->Tasks;
		}
		else if (State->SingleTask.GetNode().IsValid())
		{
			Tasks = TArrayView<FStateTreeEditorNode>(&State->SingleTask, 1);
		}
		
		bool bCreateTaskSucceeded = true;
		int32 EnabledTasksNum = 0;
		TArray<int32, TInlineAllocator<32>> ValidTaskNodeIndex;
		for (int32 TaskIndex = 0; TaskIndex < Tasks.Num(); ++TaskIndex)
		{
			FStateTreeEditorNode& TaskNode = Tasks[TaskIndex];
			// Silently ignore empty nodes.
			if (!TaskNode.GetNode().IsValid())
			{
				continue;
			}

			FStateTreeTaskBase& Task = TaskNode.GetNode().Get<FStateTreeTaskBase>();
			if(Task.bTaskEnabled)
			{
				EnabledTasksNum += 1;
			}

			const FStateTreeDataHandle TaskDataHandle(EStateTreeDataSourceType::ActiveInstanceData, InstanceDataIndex++, CompactStateHandle);
			if (!CreateTask(State, TaskNode, TaskDataHandle))
			{
				bSucceeded = false;
				bCreateTaskSucceeded = false;
				continue;
			}

			ValidTaskNodeIndex.Add(TaskIndex);
		}

		if (!bCreateTaskSucceeded)
		{
			continue;
		}
		
		const int32 TasksNum = Nodes.Num() - TasksBegin;
		check(ValidTaskNodeIndex.Num() == TasksNum);
		if (const auto Validation = UE::StateTree::Compiler::IsValidCount8(TasksNum); Validation.DidFail())
		{
			Validation.Log(Log, TEXT("TasksNum"));
			return false;
		}

		// Create tasks
		if (TasksNum > FStateTreeTasksCompletionStatus::MaxNumberOfTasksPerGroup)
		{
			Log.Reportf(EMessageSeverity::Error, LinkedParamsDesc,
				TEXT("Exceeds the maximum number of tasks (%d)"), FStateTreeTasksCompletionStatus::MaxNumberOfTasksPerGroup);
			bSucceeded = false;
			continue;
		}

		const int32 InstanceDataNum = InstanceDataIndex - StateInfos[StateIndex].FirstInstanceDataIndex;
		if (const auto Validation = UE::StateTree::Compiler::IsValidCount8(InstanceDataNum); Validation.DidFail())
		{
			Validation.Log(Log, TEXT("InstanceDataNum"));
			return false;
		}

		CompactState.TasksNum = uint8(TasksNum);
		CompactState.EnabledTasksNum = uint8(EnabledTasksNum);
		CompactState.InstanceDataNum = uint8(InstanceDataNum);

		// Create completion mask
		{
			int32 StartBitIndex = 0;
			if (CompactState.Parent.IsValid())
			{
				StartBitIndex = StateInfos[CompactState.Parent.Index].NextBitIndexForCompletionMask;
			}
			else
			{
				// Frame need an extra buffer for global tasks.
				//Linked sub-frames do not support global tasks but they can be use to replace root (old behavior).
				StartBitIndex = GlobalTaskEndBit;
			}

			UE::StateTree::Compiler::FCompletionTasksMaskResult MaskResult = UE::StateTree::Compiler::MakeCompletionTasksMask(StartBitIndex, Tasks, ValidTaskNodeIndex);

			const int32 CompletionTasksMaskBufferIndex = MaskResult.MaskBufferIndex;
			if (const auto Validation = UE::StateTree::Compiler::IsValidCount8(CompletionTasksMaskBufferIndex); Validation.DidFail())
			{
				Validation.Log(Log, TEXT("CompletionTasksMaskBufferIndex"));
				bSucceeded = false;
				continue;
			}
			const int32 CompletionTasksMaskBitsOffset = MaskResult.MaskFirstTaskBitOffset;
			if (const auto Validation = UE::StateTree::Compiler::IsValidCount8(CompletionTasksMaskBitsOffset); Validation.DidFail())
			{
				Validation.Log(Log, TEXT("CompletionTasksMaskBitsOffset"));
				bSucceeded = false;
				continue;
			}

			CompactState.CompletionTasksMask = MaskResult.Mask;
			CompactState.CompletionTasksControl = StateTree->Schema->AllowTasksCompletion() ? State->TasksCompletion : EStateTreeTaskCompletionType::Any;
			CompactState.CompletionTasksMaskBufferIndex = static_cast<uint8>(CompletionTasksMaskBufferIndex);
			CompactState.CompletionTasksMaskBitsOffset = static_cast<uint8>(CompletionTasksMaskBitsOffset);
			StateInfos[StateIndex].NextBitIndexForCompletionMask = MaskResult.FullMaskEndTaskBitOffset;

			// Find Frame and update the number of masks.
			{
				const int32* FrameIndexPtr = StateHandleToFrameIndex.Find(CompactStateHandle);
				if (FrameIndexPtr == nullptr)
				{
					Log.Reportf(EMessageSeverity::Error, LinkedParamsDesc, TEXT("The parent frame can't be found"));
					bSucceeded = false;
					continue;
				}

				if (!StateTree->Frames.IsValidIndex(*FrameIndexPtr))
				{
					Log.Reportf(EMessageSeverity::Error, LinkedParamsDesc, TEXT("The parent frame is invalid"));
					bSucceeded = false;
					continue;
				}

				FCompactStateTreeFrame& Frame = StateTree->Frames[*FrameIndexPtr];
				Frame.NumberOfTasksStatusMasks = FMath::Max(Frame.NumberOfTasksStatusMasks, static_cast<uint8>(CompactState.CompletionTasksMaskBufferIndex+1));
			}
		}

		if (!CreateBindingsForNodes(Tasks, FStateTreeIndex16(TasksBegin), InstanceStructs))
		{
			bSucceeded = false;
			continue;
		}

		StateInfos[StateIndex].bProcessed = true;
	}
	
	return bSucceeded;
}

bool FStateTreeCompiler::CreateStateEnterConditions(int32 StateIndex)
{
	FCompactStateTreeState& CompactState = StateTree->States[StateIndex];
	UStateTreeState* SourceState = SourceStates[StateIndex];
	check(SourceState != nullptr);

	const FString& StatePath = GetStatePath(StateIndex);

	// Enter conditions.
	const int32 EnterConditionsBegin = Nodes.Num();
	if (const auto Validation = UE::StateTree::Compiler::IsValidCount16(EnterConditionsBegin); Validation.DidFail())
	{
		Validation.Log(Log, TEXT("EnterConditionsBegin"));
		return false;
	}
	CompactState.EnterConditionsBegin = uint16(EnterConditionsBegin);

	const FString StatePathWithConditions = StatePath + TEXT("/EnterConditions");
	if (!CreateConditions(*SourceState, StatePathWithConditions, SourceState->EnterConditions))
	{
		Log.Reportf(EMessageSeverity::Error,
			TEXT("Failed to create state enter condition."));
		return false;
	}

	const int32 EnterConditionsNum = Nodes.Num() - EnterConditionsBegin;
	if (const auto Validation = UE::StateTree::Compiler::IsValidCount8(EnterConditionsNum); Validation.DidFail())
	{
		Validation.Log(Log, TEXT("EnterConditionsNum"));
		return false;
	}
	CompactState.EnterConditionsNum = uint8(EnterConditionsNum);

	const bool bUseEvaluationScopeInstanceData = UE::StateTree::Compiler::bEnableConditionWithEvaluationScopeInstanceData;
	TArray<FInstancedStruct>& InstancedStructContainer = bUseEvaluationScopeInstanceData ? EvaluationScopeStructs : SharedInstanceStructs;
	if (!CreateBindingsForNodes(SourceState->EnterConditions, FStateTreeIndex16(EnterConditionsBegin), InstancedStructContainer))
	{
		return false;
	}

	// Check if any of the enter conditions require state completion events, and cache that.
	for (int32 ConditionIndex = EnterConditionsBegin; ConditionIndex < EnterConditionsBegin + EnterConditionsNum; ConditionIndex++)
	{
		if (const FStateTreeConditionBase* Cond = Nodes[ConditionIndex].GetPtr<const FStateTreeConditionBase>())
		{
			if (Cond->bHasShouldCallStateChangeEvents)
			{
				CompactState.bHasStateChangeConditions = true;
				break;
			}
		}
	}

	return true;
}

bool FStateTreeCompiler::CreateStateTransitions()
{
	check(StateTree);

	bool bSucceeded = true;

	for (int32 StateIndex = 0; StateIndex < StateTree->States.Num(); ++StateIndex)
	{
		FCompactStateTreeState& CompactState = StateTree->States[StateIndex];
		UStateTreeState* SourceState = SourceStates[StateIndex];
		check(SourceState != nullptr);

		FStateTreeCompilerLogStateScope LogStateScope(SourceState, Log);

		if (!CreateStateEnterConditions(StateIndex))
		{
			bSucceeded = false;
			continue;
		}

		const FString& StatePath = GetStatePath(StateIndex);

		// Linked state
		if (SourceState->Type == EStateTreeStateType::Linked)
		{
			// Make sure the linked state is not self or parent to this state.
			const UStateTreeState* LinkedParentState = nullptr;
			for (const UStateTreeState* State = SourceState; State != nullptr; State = State->Parent)
			{
				if (State->ID == SourceState->LinkedSubtree.ID)
				{
					LinkedParentState = State;
					break;
				}
			}

			if (LinkedParentState != nullptr)
			{
				Log.Reportf(EMessageSeverity::Error,
					TEXT("State is linked to it's parent subtree '%s', which will create infinite loop."),
					*LinkedParentState->Name.ToString());
				bSucceeded = false;
				continue;
			}

			// The linked state must be a subtree.
			const UStateTreeState* TargetState = GetState(SourceState->LinkedSubtree.ID);
			if (TargetState == nullptr)
			{
				Log.Reportf(EMessageSeverity::Error,
					TEXT("Failed to resolve linked subtree '%s'."),
					*SourceState->LinkedSubtree.Name.ToString());
				bSucceeded = false;
				continue;
			}

			if (TargetState->Type != EStateTreeStateType::Subtree)
			{
				Log.Reportf(EMessageSeverity::Error,
					TEXT("State '%s' is linked to subtree '%s', which is not a subtree."),
					*SourceState->Name.ToString(), *TargetState->Name.ToString());
				bSucceeded = false;
				continue;
			}

			CompactState.LinkedState = GetStateHandle(SourceState->LinkedSubtree.ID);

			if (!CompactState.LinkedState.IsValid())
			{
				Log.Reportf(EMessageSeverity::Error,
					TEXT("Failed to resolve linked subtree '%s'."),
					*SourceState->LinkedSubtree.Name.ToString());
				bSucceeded = false;
				continue;
			}
		}
		else if (SourceState->Type == EStateTreeStateType::LinkedAsset)
		{
			// Do not allow to link to the same asset (might create recursion)
			if (SourceState->LinkedAsset == StateTree)
			{
				Log.Reportf(EMessageSeverity::Error,
					TEXT("It is not allowed to link to the same tree, as it might create infinite loop."));
				bSucceeded = false;
				continue;
			}

			if (SourceState->LinkedAsset)
			{
				// Linked asset must have same schema.
				const UStateTreeSchema* LinkedAssetSchema = SourceState->LinkedAsset->GetSchema();

				if (!LinkedAssetSchema)
				{
					Log.Reportf(EMessageSeverity::Error,
						TEXT("Linked State Tree asset must have valid schema."));
					bSucceeded = false;
					continue;
				}

				check(StateTree->Schema);
				if (LinkedAssetSchema->GetClass() != StateTree->Schema->GetClass())
				{
					Log.Reportf(EMessageSeverity::Error,
						TEXT("Linked State Tree asset '%s' must have same schema class as this asset. Linked asset has '%s', expected '%s'."),
						*GetFullNameSafe(SourceState->LinkedAsset),
						*LinkedAssetSchema->GetClass()->GetDisplayNameText().ToString(),
						*StateTree->Schema->GetClass()->GetDisplayNameText().ToString()
					);
					bSucceeded = false;
					continue;
				}
			}

			CompactState.LinkedAsset = SourceState->LinkedAsset;
		}

		// Transitions
		const int32 TransitionsBegin = StateTree->Transitions.Num();
		if (const auto Validation = UE::StateTree::Compiler::IsValidCount16(TransitionsBegin); Validation.DidFail())
		{
			Validation.Log(Log, TEXT("TransitionsBegin"));
			return false;
		}
		CompactState.TransitionsBegin = uint16(TransitionsBegin);

		bool bTransitionSucceeded = true;
		for (FStateTreeTransition& Transition : SourceState->Transitions)
		{
			const int32 TransitionIndex = StateTree->Transitions.Num();
			IDToTransition.Add(Transition.ID, TransitionIndex);

			FCompactStateTransition& CompactTransition = StateTree->Transitions.AddDefaulted_GetRef();
			CompactTransition.Trigger = Transition.Trigger;
			CompactTransition.Priority = Transition.Priority;

			if (Transition.Trigger == EStateTreeTransitionTrigger::OnDelegate)
			{
				FPropertyBindingPath DelegateBindingPath(Transition.ID, GET_MEMBER_NAME_CHECKED(FStateTreeTransition, DelegateListener));

				const FStateTreePropertyPathBinding* Binding = static_cast<const FStateTreePropertyPathBinding*>(EditorData->EditorBindings.FindBinding(DelegateBindingPath));

				if (Binding == nullptr)
				{
					bTransitionSucceeded = false;
					Log.Reportf(EMessageSeverity::Error,
						TEXT("On Delegate Transition to '%s' requires to be bound to some delegate dispatcher."),
						*Transition.State.Name.ToString());
					continue;
				}

				CompactTransition.RequiredDelegateDispatcher = BindingsCompiler.GetDispatcherFromBinding(*Binding);
				if (!CompactTransition.RequiredDelegateDispatcher.IsValid())
				{
					bTransitionSucceeded = false;
					Log.Reportf(EMessageSeverity::Error,
						TEXT("On Delegate Transition to '%s' is bound to unknown delegate dispatcher"),
						*Transition.State.Name.ToString());
					continue;
				}
			}

			CompactTransition.ChangeTypeTargetStateRule = Transition.ReactivateTargetState;
			CompactTransition.bTransitionEnabled = Transition.bTransitionEnabled;

			if (Transition.bDelayTransition)
			{
				CompactTransition.Delay.Set(Transition.DelayDuration, Transition.DelayRandomVariance);
			}

			if (CompactState.SelectionBehavior == EStateTreeStateSelectionBehavior::TryFollowTransitions
				&& Transition.bDelayTransition)
			{
				Log.Reportf(EMessageSeverity::Warning,
					TEXT("Transition to '%s' with delay will be ignored during state selection."),
					*Transition.State.Name.ToString());
			}

			if (EnumHasAnyFlags(Transition.Trigger, EStateTreeTransitionTrigger::OnStateCompleted))
			{
				// Completion transitions dont have priority.
				CompactTransition.Priority = EStateTreeTransitionPriority::None;

				// Completion transitions cannot have delay.
				CompactTransition.Delay.Reset();

				// Completion transitions must have valid target state.
				if (Transition.State.LinkType == EStateTreeTransitionType::None)
				{
					Log.Reportf(EMessageSeverity::Error,
						TEXT("State completion transition to '%s' must have transition to valid state, 'None' not accepted."),
						*Transition.State.Name.ToString());
					bTransitionSucceeded = false;
					continue;
				}
			}

			CompactTransition.State = FStateTreeStateHandle::Invalid;
			if (!ResolveTransitionStateAndFallback(SourceState, Transition.State, CompactTransition.State, CompactTransition.Fallback))
			{
				bTransitionSucceeded = false;
				continue;
			}

			if (CompactTransition.State.IsValid()
				&& !CompactTransition.State.IsCompletionState())
			{
				FCompactStateTreeState& TransitionTargetState = StateTree->States[CompactTransition.State.Index];
				if (TransitionTargetState.Type == EStateTreeStateType::Subtree)
				{
					Log.Reportf(EMessageSeverity::Warning,
						TEXT("Transitioning directly to a Subtree State '%s' is not recommended, as it may have unexpected results. Subtree States should be used with Linked States instead."),
						*TransitionTargetState.Name.ToString());
				}
			}

			const FString StatePathWithTransition = StatePath + FString::Printf(TEXT("/Transition[%d]"), TransitionIndex - TransitionsBegin);

			if (Transition.Trigger == EStateTreeTransitionTrigger::OnEvent)
			{
				CompactTransition.RequiredEvent.Tag = Transition.RequiredEvent.Tag;
				CompactTransition.RequiredEvent.PayloadStruct = Transition.RequiredEvent.PayloadStruct;
				CompactTransition.bConsumeEventOnSelect = Transition.RequiredEvent.bConsumeEventOnSelect;

				FStateTreeBindableStructDesc Desc;
				Desc.StatePath = StatePathWithTransition;
				Desc.Struct = FStateTreeEvent::StaticStruct();
				Desc.Name = FName(TEXT("Transition Event"));
				Desc.ID = Transition.GetEventID();
				Desc.DataSource = EStateTreeBindableStructSource::TransitionEvent;
				Desc.DataHandle = FStateTreeDataHandle(EStateTreeDataSourceType::TransitionEvent, TransitionIndex);

				if (!Transition.RequiredEvent.IsValid())
				{
					Log.Reportf(EMessageSeverity::Error, Desc,
						TEXT("On Event Transition requires at least tag or payload to be set up."));
					bTransitionSucceeded = false;
					continue;
				}

				if (CompactTransition.State.IsValid()
					&& !CompactTransition.State.IsCompletionState())
				{
					FCompactStateTreeState& TransitionTargetState = StateTree->States[CompactTransition.State.Index];
					if (TransitionTargetState.RequiredEventToEnter.IsValid() && !TransitionTargetState.RequiredEventToEnter.IsSubsetOfAnotherDesc(CompactTransition.RequiredEvent))
					{
						Log.Reportf(EMessageSeverity::Error, Desc,
							TEXT("On Event transition to '%s' will never succeed as transition and state required events are incompatible."),
							*TransitionTargetState.Name.ToString());
						bTransitionSucceeded = false;
						continue;
					}
				}

				BindingsCompiler.AddSourceStruct(Desc);
			}

			if (CompactTransition.bTransitionEnabled)
			{
				CompactState.bHasTickTriggerTransitions |= EnumHasAnyFlags(Transition.Trigger, EStateTreeTransitionTrigger::OnTick);
				CompactState.bHasEventTriggerTransitions |= EnumHasAnyFlags(Transition.Trigger, EStateTreeTransitionTrigger::OnEvent);
				CompactState.bHasDelegateTriggerTransitions |= EnumHasAnyFlags(Transition.Trigger, EStateTreeTransitionTrigger::OnDelegate);
				CompactState.bHasCompletedTriggerTransitions |= EnumHasAnyFlags(Transition.Trigger, EStateTreeTransitionTrigger::OnStateCompleted);
				CompactState.bHasSucceededTriggerTransitions |= EnumHasAnyFlags(Transition.Trigger, EStateTreeTransitionTrigger::OnStateSucceeded);
				CompactState.bHasFailedTriggerTransitions |= EnumHasAnyFlags(Transition.Trigger, EStateTreeTransitionTrigger::OnStateFailed);
			}

			const int32 ConditionsBegin = Nodes.Num();
			if (const auto Validation = UE::StateTree::Compiler::IsValidCount16(ConditionsBegin); Validation.DidFail())
			{
				Validation.Log(Log, TEXT("ConditionsBegin"));
				return false;
			}
			CompactTransition.ConditionsBegin = uint16(ConditionsBegin);

			if (!CreateConditions(*SourceState, StatePathWithTransition, Transition.Conditions))
			{
				Log.Reportf(EMessageSeverity::Error,
					TEXT("Failed to create condition for transition to '%s'."),
					*Transition.State.Name.ToString());
				bTransitionSucceeded = false;
				continue;
			}

			const int32 ConditionsNum = Nodes.Num() - ConditionsBegin;
			if (const auto Validation = UE::StateTree::Compiler::IsValidCount8(ConditionsNum); Validation.DidFail())
			{
				Validation.Log(Log, TEXT("ConditionsNum"));
				return false;
			}
			CompactTransition.ConditionsNum = uint8(ConditionsNum);

			const bool bUseEvaluationScopeInstanceData = UE::StateTree::Compiler::bEnableConditionWithEvaluationScopeInstanceData;
			TArray<FInstancedStruct>& InstancedStructContainer = bUseEvaluationScopeInstanceData ? EvaluationScopeStructs : SharedInstanceStructs;
			if (!CreateBindingsForNodes(Transition.Conditions, FStateTreeIndex16(ConditionsBegin), InstancedStructContainer))
			{
				bTransitionSucceeded = false;
				continue;
			}
		}

		if (!bTransitionSucceeded)
		{
			bSucceeded = false;
			continue;
		}

		const int32 TransitionsNum = StateTree->Transitions.Num() - TransitionsBegin;
		if (const auto Validation = UE::StateTree::Compiler::IsValidCount8(TransitionsNum); Validation.DidFail())
		{
			Validation.Log(Log, TEXT("TransitionsNum"));
			return false;
		}
		CompactState.TransitionsNum = uint8(TransitionsNum);
	}

	// @todo: Add test to check that all success/failure transition is possible (see editor).

	return bSucceeded;
}

bool FStateTreeCompiler::CreateStateConsideration(int32 StateIndex)
{
	FCompactStateTreeState& CompactState = StateTree->States[StateIndex];
	UStateTreeState* SourceState = SourceStates[StateIndex];
	check(SourceState != nullptr);

	const FString& StatePath = GetStatePath(StateIndex);

	const int32 UtilityConsiderationsBegin = Nodes.Num();
	if (const auto Validation = UE::StateTree::Compiler::IsValidCount16(UtilityConsiderationsBegin); Validation.DidFail())
	{
		Validation.Log(Log, TEXT("UtilityConsiderationsBegin"));
		return false;
	}
	CompactState.UtilityConsiderationsBegin = uint16(UtilityConsiderationsBegin);

	const FString StatePathWithConsiderations = StatePath + TEXT("/Considerations");
	if (!CreateConsiderations(*SourceState, StatePathWithConsiderations, SourceState->Considerations))
	{
		Log.Reportf(EMessageSeverity::Error,
			TEXT("Failed to create state utility considerations."));
		return false;
	}

	const int32 UtilityConsiderationsNum = Nodes.Num() - UtilityConsiderationsBegin;
	if (const auto Validation = UE::StateTree::Compiler::IsValidCount8(UtilityConsiderationsNum); Validation.DidFail())
	{
		Validation.Log(Log, TEXT("UtilityConsiderationsNum"));
		return false;
	}
	CompactState.UtilityConsiderationsNum = uint8(UtilityConsiderationsNum);

	const bool bUseEvaluationScopeInstanceData = UE::StateTree::Compiler::bEnableUtilityConsiderationWithEvaluationScopeInstanceData;
	TArray<FInstancedStruct>& InstancedStructContainer = bUseEvaluationScopeInstanceData ? EvaluationScopeStructs : SharedInstanceStructs;
	if (!CreateBindingsForNodes(SourceState->Considerations, FStateTreeIndex16(UtilityConsiderationsBegin), InstancedStructContainer))
	{
		return false;
	}

	return true;
}

bool FStateTreeCompiler::CreateStateConsiderations()
{
	check(StateTree);

	bool bSucceeded = true;

	for (int32 StateIndex = 0; StateIndex < StateTree->States.Num(); ++StateIndex)
	{
		UStateTreeState* SourceState = SourceStates[StateIndex];
		check(SourceState != nullptr);

		FStateTreeCompilerLogStateScope LogStateScope(SourceState, Log);

		if (!CreateStateConsideration(StateIndex))
		{
			bSucceeded = false;
			continue;
		}
	}

	return bSucceeded;
}

bool FStateTreeCompiler::CreateBindingsForNodes(TConstArrayView<FStateTreeEditorNode> EditorNodes, FStateTreeIndex16 NodesBegin, TArray<FInstancedStruct>& Instances)
{
	check(NodesBegin.IsValid());

	bool bSucceeded = true;

	int32 NodeIndex = NodesBegin.Get();
	for (const FStateTreeEditorNode& EditorNode : EditorNodes)
	{
		// Node might be an empty line in Editor.
		if (!EditorNode.GetNode().IsValid())
		{
			continue;
		}

		FInstancedStruct& NodeInstancedStruct = Nodes[NodeIndex++];
		FStateTreeNodeBase& Node = NodeInstancedStruct.GetMutable<FStateTreeNodeBase>();

		FStateTreeIndex16 PropertyFunctionsBegin(Nodes.Num());
		if (!CreatePropertyFunctionsForStruct(EditorNode.ID))
		{
			bSucceeded = false;
			continue;
		}
		FStateTreeIndex16 PropertyFunctionsEnd(Nodes.Num());
		
		if (PropertyFunctionsBegin == PropertyFunctionsEnd)
		{
			PropertyFunctionsBegin = FStateTreeIndex16::Invalid;
			PropertyFunctionsEnd = FStateTreeIndex16::Invalid;
		}

		FStateTreeDataView InstanceView;
		check(Instances.IsValidIndex(Node.InstanceTemplateIndex.Get()));

		FInstancedStruct& Instance = Instances[Node.InstanceTemplateIndex.Get()];
		if (FStateTreeInstanceObjectWrapper* ObjectWrapper = Instance.GetMutablePtr<FStateTreeInstanceObjectWrapper>())
		{
			check(EditorNode.GetInstance().GetStruct() == ObjectWrapper->InstanceObject->GetClass());
			InstanceView = FStateTreeDataView(ObjectWrapper->InstanceObject);
		}
		else
		{
			check(EditorNode.GetInstance().GetStruct() == Instance.GetScriptStruct());
			InstanceView = FStateTreeDataView(Instance);
		}

		{
			const FStateTreeBindableStructDesc* BindableInstanceStruct = BindingsCompiler.GetSourceStructDescByID(EditorNode.ID);
			check(BindableInstanceStruct);
			if (!CreateBindingsForStruct(*BindableInstanceStruct, InstanceView, PropertyFunctionsBegin, PropertyFunctionsEnd, Node.BindingsBatch, &Node.OutputBindingsBatch))
			{
				bSucceeded = false;
				continue;
			}

			if (const FStateTreeBindableStructDesc* BindableNodeStruct = BindingsCompiler.GetSourceStructDescByID(EditorNode.GetNodeID()))
			{
				if (UE::StateTree::Compiler::IsNodeStructEligibleForBinding(NodeInstancedStruct))
				{
					if (!CreateBindingsForStruct(*BindableNodeStruct, static_cast<FStructView>(NodeInstancedStruct), PropertyFunctionsBegin, PropertyFunctionsEnd, Node.BindingsBatch))
					{
						bSucceeded = false;
						continue;
					}
				}

				// CreateBindingsForStruct Compile the regular FStateTreeDelegateDispatcher. Task completion delegate dispatchers need to be processed separately.
				if (TaskCompletionDispatcherIDs.Contains(EditorNode.GetNodeID()))
				{
					const FStateTreeBindableStructDesc& TargetStruct = *BindableNodeStruct;
					if (!BindingsCompiler.CompileTaskCompletionDelegateDispatchers(TargetStruct, EditorData->EditorBindings.GetBindings()))
					{
						return false;
					}
				}
			}
		}
	}

	return bSucceeded;
}

bool FStateTreeCompiler::CreateBindingsForStruct(
	const FStateTreeBindableStructDesc& TargetStruct,
	FStateTreeDataView TargetValue,
	FStateTreeIndex16 PropertyFuncsBegin,
	FStateTreeIndex16 PropertyFuncsEnd,
	FStateTreeIndex16& OutBatchIndex,
	FStateTreeIndex16* OutOutputBindingBatchIndex /* nullptr */)
{
	auto CopyBatch = [Self = this, &TargetStruct]
		(TConstArrayView<FStateTreePropertyPathBinding> InCopyBindings, FStateTreeIndex16 InPropertyFuncsBegin, FStateTreeIndex16 InPropertyFuncsEnd, const TCHAR* InLogContextText, FStateTreeIndex16& OutBatchIndex)
		{
			int32 BatchIndex = INDEX_NONE;

			// Compile batch copy for this struct, we pass in all the bindings, the compiler will pick up the ones for the target structs.
			if (!Self->BindingsCompiler.CompileBatch(TargetStruct, InCopyBindings, InPropertyFuncsBegin, InPropertyFuncsEnd, BatchIndex))
			{
				return false;
			}

			if (const auto Validation = UE::StateTree::Compiler::IsValidIndex16(BatchIndex); Validation.DidFail())
			{
				Validation.Log(Self->Log, InLogContextText, TargetStruct);
				return false;
			}

			OutBatchIndex = FStateTreeIndex16(BatchIndex);
			return true;
		};

	FValidatedPathBindings Bindings;

	// Check that the bindings for this struct are still all valid.
	if (!GetAndValidateBindings(TargetStruct, TargetValue, Bindings))
	{
		return false;
	}

	// Copy Bindings
	{
		if (!CopyBatch(Bindings.CopyBindings, PropertyFuncsBegin, PropertyFuncsEnd, TEXT("CopiesBatchIndex"), OutBatchIndex))
		{
			return false;
		}

		if (OutOutputBindingBatchIndex)
		{
			// For output binding, we don't run any property functions
			if (!CopyBatch(Bindings.OutputCopyBindings, FStateTreeIndex16::Invalid, FStateTreeIndex16::Invalid, TEXT("OutputCopiesBatchIndex"), *OutOutputBindingBatchIndex))
			{
				return false;
			}
		}
	}

	// Delegate Dispatcher
	if (!BindingsCompiler.CompileDelegateDispatchers(TargetStruct, EditorData->CompiledDispatchers, Bindings.DelegateDispatchers, TargetValue))
	{
		return false;
	}

	// Delegate Listener
	if (!BindingsCompiler.CompileDelegateListeners(TargetStruct, Bindings.DelegateListeners, TargetValue))
	{
		return false;
	}

	// Reference Bindings
	if (!BindingsCompiler.CompileReferences(TargetStruct, Bindings.ReferenceBindings, TargetValue, IDToStructValue))
	{
		return false;
	}

	return true;
}

bool FStateTreeCompiler::CreatePropertyFunctionsForStruct(FGuid StructID)
{
	check(EditorBindingsNum == EditorData->EditorBindings.GetBindings().Num());
	for (TMultiMap<FGuid, const FPropertyBindingBinding*>::TKeyIterator It = BindingsByTargetStructID.CreateKeyIterator(StructID); It; ++It)
	{
		const FPropertyBindingBinding& Binding = *(It.Value());
		if (!ensure(Binding.GetTargetPath().GetStructID() == StructID))
		{
			continue;
		}

		const FConstStructView NodeView = Binding.GetPropertyFunctionNode();
		if (!NodeView.IsValid())
		{
			continue;
		}

		const FStateTreeEditorNode& FuncEditorNode = NodeView.Get<const FStateTreeEditorNode>();
		if (!CreatePropertyFunction(FuncEditorNode))
		{
			return false;
		}
	}

	return true;
}

bool FStateTreeCompiler::CreatePropertyFunction(const FStateTreeEditorNode& FuncEditorNode)
{
	if (!CreatePropertyFunctionsForStruct(FuncEditorNode.ID))
	{
		return false;
	}

	FStateTreeBindableStructDesc StructDesc;
	StructDesc.StatePath = UE::StateTree::Editor::PropertyFunctionStateName;
	StructDesc.ID = FuncEditorNode.ID;
	StructDesc.Name = FuncEditorNode.GetName();
	StructDesc.DataSource = EStateTreeBindableStructSource::PropertyFunction;

	const bool bUseEvaluationScopeInstanceData = UE::StateTree::Compiler::bEnablePropertyFunctionWithEvaluationScopeInstanceData;
	const FStateTreeDataHandle DataHandle = bUseEvaluationScopeInstanceData
		? FStateTreeDataHandle(EStateTreeDataSourceType::EvaluationScopeInstanceData, EvaluationScopeStructs.Num())
		: FStateTreeDataHandle(EStateTreeDataSourceType::SharedInstanceData, SharedInstanceStructs.Num());
	TArray<FInstancedStruct>& InstancedStructContainer = bUseEvaluationScopeInstanceData ? EvaluationScopeStructs : SharedInstanceStructs;
	FInstancedStruct* CreatedNode = CreateNode(nullptr, FuncEditorNode, StructDesc, DataHandle, InstancedStructContainer);
	if (CreatedNode == nullptr)
	{
		return false;
	}

	FStateTreeNodeBase* Function = CreatedNode->GetMutablePtr<FStateTreeNodeBase>();
	if (Function == nullptr)
	{
		return false;
	}

	const FStateTreeBindableStructDesc* BindableStruct = BindingsCompiler.GetSourceStructDescByID(FuncEditorNode.ID);
	check(BindableStruct);

	FStateTreeDataView InstanceView;
	check(InstancedStructContainer.IsValidIndex(Function->InstanceTemplateIndex.Get()));

	FInstancedStruct& Instance = InstancedStructContainer[Function->InstanceTemplateIndex.Get()];
	if (FStateTreeInstanceObjectWrapper* ObjectWrapper = Instance.GetMutablePtr<FStateTreeInstanceObjectWrapper>())
	{
		check(FuncEditorNode.GetInstance().GetStruct() == ObjectWrapper->InstanceObject->GetClass());
		InstanceView = FStateTreeDataView(ObjectWrapper->InstanceObject);
	}
	else
	{
		check(FuncEditorNode.GetInstance().GetStruct() == Instance.GetScriptStruct());
		InstanceView = FStateTreeDataView(Instance);
	}

	return CreateBindingsForStruct(*BindableStruct, InstanceView, FStateTreeIndex16::Invalid, FStateTreeIndex16::Invalid, Function->BindingsBatch);
}

FInstancedStruct* FStateTreeCompiler::CreateNode(UStateTreeState* State, const FStateTreeEditorNode& EditorNode, FStateTreeBindableStructDesc& InstanceDesc, const FStateTreeDataHandle DataHandle, TArray<FInstancedStruct>& InstancedStructContainer)
{
	if (!EditorNode.GetNode().IsValid())
	{
		return nullptr;
	}

	// Check that item has valid instance initialized.
	if (!EditorNode.GetInstance().IsValid())
	{
		Log.Reportf(EMessageSeverity::Error, InstanceDesc,
			TEXT("Malformed node, missing instance value."));
		return nullptr;
	}

	// Copy the node
	IDToNode.Add(EditorNode.ID, Nodes.Num());
	FInstancedStruct& RawNode = Nodes.Add_GetRef(FInstancedStruct(EditorNode.GetNode()));
	InstantiateStructSubobjects(RawNode);

	FStateTreeNodeBase& Node = RawNode.GetMutable<FStateTreeNodeBase>();

	TOptional<FStateTreeDataView> InstanceDataView = CreateNodeInstanceData(EditorNode, Node, InstanceDesc, DataHandle, InstancedStructContainer);
	if (!InstanceDataView.IsSet())
	{
		return nullptr;
	}

	if (!CompileAndValidateNode(State, InstanceDesc, RawNode, InstanceDataView.GetValue()))
	{
		return nullptr;
	}

	CreateBindingSourceStructsForNode(EditorNode, InstanceDesc);

	return &RawNode;
}

FInstancedStruct* FStateTreeCompiler::CreateNodeWithSharedInstanceData(UStateTreeState* State, const FStateTreeEditorNode& EditorNode, FStateTreeBindableStructDesc& InstanceDesc)
{
	const FStateTreeDataHandle DataHandle = FStateTreeDataHandle(EStateTreeDataSourceType::SharedInstanceData, SharedInstanceStructs.Num());
	return CreateNode(State, EditorNode, InstanceDesc, DataHandle, SharedInstanceStructs);
}

TOptional<FStateTreeDataView> FStateTreeCompiler::CreateNodeInstanceData(const FStateTreeEditorNode& EditorNode, FStateTreeNodeBase& Node, FStateTreeBindableStructDesc& StructDesc, const FStateTreeDataHandle DataHandle, TArray<FInstancedStruct>& InstancedStructContainer)
{
	FStateTreeDataView InstanceDataView;

	// Update Node name as description for runtime.
	Node.Name = EditorNode.GetName();

	// Update the instance data
	{
		const FStateTreeDataView InstanceView = EditorNode.GetInstance();
		if (InstanceView.IsValid())
		{
			if (InstanceView.GetStruct() == Node.GetInstanceDataType())
			{
				if (const UScriptStruct* InstanceStruct = Cast<const UScriptStruct>(InstanceView.GetStruct()))
				{
					// Struct Instance
					FInstancedStruct InstancedStruct;
					InstancedStruct.InitializeAs(InstanceStruct, static_cast<const uint8*>(InstanceView.GetMemory()));
					const int32 InstanceIndex = InstancedStructContainer.Add(MoveTemp(InstancedStruct));
					InstantiateStructSubobjects(InstancedStructContainer[InstanceIndex]);

					// Create binding source struct FInstancedStruct descriptor.
					StructDesc.Struct = InstanceStruct;

					if (const auto Validation = UE::StateTree::Compiler::IsValidIndex16(InstanceIndex); Validation.DidFail())
					{
						Validation.Log(Log, TEXT("InstanceIndex"), StructDesc);
						return {};
					}
					Node.InstanceTemplateIndex = FStateTreeIndex16(InstanceIndex);
					Node.InstanceDataHandle = DataHandle;
					InstanceDataView = FStateTreeDataView(InstancedStructContainer[InstanceIndex]);
				}
				else if (const UClass* InstanceClass = Cast<const UClass>(InstanceView.GetStruct()))
				{
					const UObject* InstanceObject = InstanceView.GetPtr<const UObject>();
					UObject* Instance = UE::StateTree::Compiler::DuplicateInstanceObject(Log, StructDesc, EditorNode.ID, InstanceObject, StateTree);

					FInstancedStruct Wrapper;
					Wrapper.InitializeAs<FStateTreeInstanceObjectWrapper>(Instance);
					const int32 InstanceIndex = InstancedStructContainer.Add(MoveTemp(Wrapper));

					// Create binding source struct descriptor.
					StructDesc.Struct = InstanceClass;

					if (const auto Validation = UE::StateTree::Compiler::IsValidIndex16(InstanceIndex); Validation.DidFail())
					{
						Validation.Log(Log, TEXT("InstanceIndex"), StructDesc);
						return {};
					}
					Node.InstanceTemplateIndex = FStateTreeIndex16(InstanceIndex);
					Node.InstanceDataHandle = DataHandle.ToObjectSource();
					InstanceDataView = FStateTreeDataView(Instance);
				}
				else
				{
					check(false); // Unsupported instance data type. It should be either UScriptStruct or UClass.
					return {};
				}
			}
			else
			{
				Log.Reportf(EMessageSeverity::Error, StructDesc, TEXT("The instance data type does not match."));
				return {};
			}
		}
		else if (Node.GetInstanceDataType() != nullptr)
		{
			Log.Reportf(EMessageSeverity::Error, StructDesc, TEXT("The instance data is missing."));
			return {};
		}
		StructDesc.DataHandle = Node.InstanceDataHandle;
	}

	{
		const FStateTreeDataView ExecutionRuntimeView = EditorNode.GetExecutionRuntimeData();
		if (ExecutionRuntimeView.IsValid())
		{
			if (ExecutionRuntimeView.GetStruct() == Node.GetExecutionRuntimeDataType())
			{
				if (const UScriptStruct* ExecutionRuntimeStruct = Cast<const UScriptStruct>(ExecutionRuntimeView.GetStruct()))
				{
					// Struct Instance
					FInstancedStruct InstancedStruct;
					InstancedStruct.InitializeAs(ExecutionRuntimeStruct, static_cast<const uint8*>(ExecutionRuntimeView.GetMemory()));
					const int32 InstanceIndex = ExecutionRuntimeStructs.Add(MoveTemp(InstancedStruct));
					InstantiateStructSubobjects(ExecutionRuntimeStructs[InstanceIndex]);

					if (const auto Validation = UE::StateTree::Compiler::IsValidIndex16(InstanceIndex); Validation.DidFail())
					{
						Validation.Log(Log, TEXT("ExecutionRuntime Index"), StructDesc);
						return {};
					}
					Node.ExecutionRuntimeTemplateIndex = FStateTreeIndex16(InstanceIndex);
				}
				else if (const UClass* ExecutionRuntimeClass = Cast<const UClass>(ExecutionRuntimeView.GetStruct()))
				{
					// Object Instance
					const UObject* ExecutionRuntimeObject = ExecutionRuntimeView.GetPtr<const UObject>();
					UObject* Instance = UE::StateTree::Compiler::DuplicateInstanceObject(Log, StructDesc, EditorNode.ID, ExecutionRuntimeObject, StateTree);

					FInstancedStruct Wrapper;
					Wrapper.InitializeAs<FStateTreeInstanceObjectWrapper>(Instance);
					const int32 InstanceIndex = ExecutionRuntimeStructs.Add(MoveTemp(Wrapper));

					if (const auto Validation = UE::StateTree::Compiler::IsValidIndex16(InstanceIndex); Validation.DidFail())
					{
						Validation.Log(Log, TEXT("ExecutionRuntime Index"), StructDesc);
						return {};
					}
					Node.ExecutionRuntimeTemplateIndex = FStateTreeIndex16(InstanceIndex);
				}
				else
				{
					check(false); // Unsupported instance data type. It should be either UScriptStruct or UClass.
					return {};
				}
			}
			else
			{
				Log.Reportf(EMessageSeverity::Error, StructDesc, TEXT("The execution runtime data type does not match."));
				return {};
			}
		}
		else if (Node.GetExecutionRuntimeDataType() != nullptr)
		{
			Log.Reportf(EMessageSeverity::Error, StructDesc, TEXT("The execution runtime data is missing."));
			return {};
		}
	}

	return TOptional<FStateTreeDataView>(InstanceDataView);
}

bool FStateTreeCompiler::CreateTaskCompletionDispatchers()
{
	// Transform the compiled version into the asset version.
	bool bTransitionSucceeded = true;
	for (const FStateTreePropertyBindingCompiler::FTaskCompletionDispatcher& Info : BindingsCompiler.GetCompiledTaskCompletionDispatchers())
	{
		if (const int32* NodeIndexPtr = IDToNode.Find(Info.TaskNodeID))
		{
			const int32 TaskIndex = *NodeIndexPtr;
			if (!StateTree->Nodes.IsValidIndex(TaskIndex))
			{
				Log.Reportf(EMessageSeverity::Error, TEXT("Internal error. A task completion delegate dispatcher index is invalid. (%d)"), TaskIndex);
				bTransitionSucceeded = false;
				continue;
			}

			FStateTreeTaskBase* Task = StateTree->Nodes[TaskIndex].GetPtr<FStateTreeTaskBase>();
			if (Task == nullptr)
			{
				Log.Reportf(EMessageSeverity::Error, TEXT("Internal error. A task completion delegate dispatcher index is not a task. (%d)"), TaskIndex);
				bTransitionSucceeded = false;
				continue;
			}
			Task->bHasTaskCompletionDelegateDispatcher = true;

			if (!Info.Dispatcher.IsValid())
			{
				Log.Reportf(EMessageSeverity::Error, TEXT("Internal error. A task completion delegate dispatcher is invalid. (%d)"), TaskIndex);
				bTransitionSucceeded = false;
				continue;
			}

			UE::StateTree::FTaskCompletionDispatcher CompiledDispatcher;
			CompiledDispatcher.TaskNodeIndex = FStateTreeIndex16(TaskIndex);
			CompiledDispatcher.Condition = Info.Condition;
			CompiledDispatcher.Dispatcher = Info.Dispatcher;
			StateTree->TaskCompletionDispatchers.Add(CompiledDispatcher);
		}
		else
		{
			Log.Reportf(EMessageSeverity::Error, TEXT("Internal error. Failed to resolve task node GUID for task completion dispatcher."));
			bTransitionSucceeded = false;
		}
	}
	return bTransitionSucceeded;
}

bool FStateTreeCompiler::ResolveTransitionStateAndFallback(const UStateTreeState* SourceState, const FStateTreeStateLink& Link, FStateTreeStateHandle& OutTransitionHandle, EStateTreeSelectionFallback& OutFallback) const 
{
	if (Link.LinkType == EStateTreeTransitionType::GotoState)
	{
		// Warn if goto state points to another subtree.
		if (const UStateTreeState* TargetState = GetState(Link.ID))
		{
			if (SourceState && TargetState->GetRootState() != SourceState->GetRootState())
			{
				Log.Reportf(EMessageSeverity::Warning,
					TEXT("Target state '%s' is in different subtree. Verify that this is intentional."),
					*Link.Name.ToString());
			}

			if (TargetState->SelectionBehavior == EStateTreeStateSelectionBehavior::None)
			{
				Log.Reportf(EMessageSeverity::Error,
					TEXT("The target State '%s' is not selectable, it's selection behavior is set to None."),
					*Link.Name.ToString());
				return false;
			}
		}
		
		OutTransitionHandle = GetStateHandle(Link.ID);
		if (!OutTransitionHandle.IsValid())
		{
			Log.Reportf(EMessageSeverity::Error,
				TEXT("Failed to resolve transition to state '%s'."),
				*Link.Name.ToString());
			return false;
		}
	}
	else if (Link.LinkType == EStateTreeTransitionType::Parent)
	{
		// Find next parent.
		const UStateTreeState* Parent = SourceState && SourceState->Parent ? SourceState->Parent.Get() : nullptr;
		if (Parent == nullptr)
		{
			Log.Reportf(EMessageSeverity::Error,
				TEXT("Failed to resolve transition, there's no selectable parent state."));
			return false;
		}
		OutTransitionHandle = GetStateHandle(Parent->ID);
		if (!OutTransitionHandle.IsValid())
		{
			Log.Reportf(EMessageSeverity::Error,
				TEXT("Failed to resolve transition parent state, no handle found for '%s'."),
				*Parent->Name.ToString());
			return false;
		}
	}
	else if (Link.LinkType == EStateTreeTransitionType::NextState
		|| Link.LinkType == EStateTreeTransitionType::NextSelectableState)
	{
		// Find next state.
		const UStateTreeState* NextState = SourceState ? SourceState->GetNextSelectableSiblingState() : nullptr;
		if (NextState == nullptr)
		{
			Log.Reportf(EMessageSeverity::Error,
				TEXT("Failed to resolve transition, there's no selectable next state."));
			return false;
		}
		OutTransitionHandle = GetStateHandle(NextState->ID);
		if (!OutTransitionHandle.IsValid())
		{
			Log.Reportf(EMessageSeverity::Error,
				TEXT("Failed to resolve transition next state, no handle found for '%s'."),
				*NextState->Name.ToString());
			return false;
		}
	}
	else if (Link.LinkType == EStateTreeTransitionType::NextParent
		|| Link.LinkType == EStateTreeTransitionType::NextSelectableParent)
	{
		// Find next parent.
		const UStateTreeState* NextParent = SourceState && SourceState->Parent ? SourceState->Parent->GetNextSelectableSiblingState() : nullptr;
		if (NextParent == nullptr)
		{
			Log.Reportf(EMessageSeverity::Error,
				TEXT("Failed to resolve transition, there's no selectable next state for the parent state."));
			return false;
		}
		OutTransitionHandle = GetStateHandle(NextParent->ID);
		if (!OutTransitionHandle.IsValid())
		{
			Log.Reportf(EMessageSeverity::Error,
				TEXT("Failed to resolve transition next state, no handle found for '%s'."),
				*NextParent->Name.ToString());
			return false;
		}
	}
	else if(Link.LinkType == EStateTreeTransitionType::Failed)
	{
		OutTransitionHandle = FStateTreeStateHandle::Failed;
	}
	else if(Link.LinkType == EStateTreeTransitionType::Succeeded)
	{
		OutTransitionHandle = FStateTreeStateHandle::Succeeded;
	}
	else if(Link.LinkType == EStateTreeTransitionType::None)
	{
		OutTransitionHandle = FStateTreeStateHandle::Invalid;
	}
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	else if (Link.LinkType == EStateTreeTransitionType::NotSet)
	{
		OutTransitionHandle = FStateTreeStateHandle::Invalid;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Set fallback type
	if (Link.LinkType == EStateTreeTransitionType::NextSelectableState
		|| Link.LinkType == EStateTreeTransitionType::NextSelectableParent)
	{
		OutFallback = EStateTreeSelectionFallback::NextSelectableSibling;
	}
	else
	{
		OutFallback = EStateTreeSelectionFallback::None;
	}

	return true;
}

bool FStateTreeCompiler::CreateCondition(UStateTreeState& State, const FString& StatePath, const FStateTreeEditorNode& CondNode, const EStateTreeExpressionOperand Operand, const int8 DeltaIndent)
{
	FStateTreeBindableStructDesc InstanceDesc;
	InstanceDesc.StatePath = StatePath;
	InstanceDesc.ID = CondNode.ID;
	InstanceDesc.Name = CondNode.GetName();
	InstanceDesc.DataSource = EStateTreeBindableStructSource::Condition;

	const bool bUseEvaluationScopeInstanceData = UE::StateTree::Compiler::bEnableConditionWithEvaluationScopeInstanceData;
	const FStateTreeDataHandle DataHandle = bUseEvaluationScopeInstanceData
		? FStateTreeDataHandle(EStateTreeDataSourceType::EvaluationScopeInstanceData, EvaluationScopeStructs.Num())
		: FStateTreeDataHandle(EStateTreeDataSourceType::SharedInstanceData, SharedInstanceStructs.Num());
	TArray<FInstancedStruct>& InstancedStructContainer = bUseEvaluationScopeInstanceData ? EvaluationScopeStructs : SharedInstanceStructs;
	FInstancedStruct* CreatedNode = CreateNode(&State, CondNode, InstanceDesc, DataHandle, InstancedStructContainer);
	if (CreatedNode == nullptr)
	{
		return false;
	}

	FStateTreeConditionBase* Cond = CreatedNode->GetMutablePtr<FStateTreeConditionBase>();
	if (ensure(Cond))
	{
		if (Cond->EvaluationMode == EStateTreeConditionEvaluationMode::ForcedFalse
			|| Cond->EvaluationMode == EStateTreeConditionEvaluationMode::ForcedTrue)
		{
			Log.Reportf(EMessageSeverity::Info, InstanceDesc,
					TEXT("The condition result will always be %s."),
					Cond->EvaluationMode == EStateTreeConditionEvaluationMode::ForcedTrue ? TEXT("True") : TEXT("False"));
		}

		Cond->Operand = Operand;
		Cond->DeltaIndent = DeltaIndent;
		return true;
	}

	return false;
}

bool FStateTreeCompiler::CreateConsiderations(UStateTreeState& State, const FString& StatePath, TConstArrayView<FStateTreeEditorNode> Considerations)
{
	if (State.Considerations.Num() != 0)
	{
		if (State.Parent == nullptr
			|| !UE::StateTree::IsSelectionBehaviorUsingUtility(State.Parent->SelectionBehavior))
		{
			Log.Reportf(EMessageSeverity::Warning, TEXT("State's Utility Considerations data are compiled but they don't have effect."
					"The Utility Considerations are used only when parent State's Selection Behavior is:"
					"\"Try Select Children with Highest Utility\" or \"Try Select Children At Random Weighted By Utility\"."));
		}
	}

	for (int32 Index = 0; Index < Considerations.Num(); Index++)
	{
		const bool bIsFirst = Index == 0;
		const FStateTreeEditorNode& ConsiderationNode = Considerations[Index];
		// First operand should be copy as we dont have a previous item to operate on.
		const EStateTreeExpressionOperand Operand = bIsFirst ? EStateTreeExpressionOperand::Copy : ConsiderationNode.ExpressionOperand;
		// First indent must be 0 to make the parentheses calculation match.
		const int32 CurrIndent = bIsFirst ? 0 : FMath::Clamp((int32)ConsiderationNode.ExpressionIndent, 0, UE::StateTree::MaxExpressionIndent);
		// Next indent, or terminate at zero.
		const int32 NextIndent = Considerations.IsValidIndex(Index + 1) ? FMath::Clamp((int32)Considerations[Index + 1].ExpressionIndent, 0, UE::StateTree::MaxExpressionIndent) : 0;

		const int32 DeltaIndent = NextIndent - CurrIndent;

		if (!CreateConsideration(State, StatePath, ConsiderationNode, Operand, (int8)DeltaIndent))
		{
			return false;
		}
	}

	return true;
}

bool FStateTreeCompiler::CreateConsideration(UStateTreeState& State, const FString& StatePath, const FStateTreeEditorNode& ConsiderationNode, const EStateTreeExpressionOperand Operand, const int8 DeltaIndent)
{
	FStateTreeBindableStructDesc InstanceDesc;
	InstanceDesc.StatePath = StatePath;
	InstanceDesc.ID = ConsiderationNode.ID;
	InstanceDesc.Name = ConsiderationNode.GetName();
	InstanceDesc.DataSource = EStateTreeBindableStructSource::Consideration;

	const bool bUseEvaluationScopeInstanceData = UE::StateTree::Compiler::bEnableUtilityConsiderationWithEvaluationScopeInstanceData;
	const FStateTreeDataHandle DataHandle = bUseEvaluationScopeInstanceData
		? FStateTreeDataHandle(EStateTreeDataSourceType::EvaluationScopeInstanceData, EvaluationScopeStructs.Num())
		: FStateTreeDataHandle(EStateTreeDataSourceType::SharedInstanceData, SharedInstanceStructs.Num());
	TArray<FInstancedStruct>& InstancedStructContainer = bUseEvaluationScopeInstanceData ? EvaluationScopeStructs : SharedInstanceStructs;
	FInstancedStruct* CreatedNode = CreateNode(&State, ConsiderationNode, InstanceDesc, DataHandle, InstancedStructContainer);
	if (CreatedNode == nullptr)
	{
		return false;
	}

	FStateTreeConsiderationBase* Consideration = CreatedNode->GetMutablePtr<FStateTreeConsiderationBase>();
	if (ensure(Consideration))
	{
		Consideration->Operand = Operand;
		Consideration->DeltaIndent = DeltaIndent;
		return true;
	}

	return false;
}

bool FStateTreeCompiler::CompileAndValidateNode(const UStateTreeState* SourceState, const FStateTreeBindableStructDesc& InstanceDesc, FStructView NodeView, const FStateTreeDataView InstanceData)
{
	if (!NodeView.IsValid())
	{
		return false;
	}
	
	FStateTreeNodeBase& Node = NodeView.Get<FStateTreeNodeBase>();
	check(InstanceData.IsValid());

	auto ValidateStateLinks = [this, SourceState](TPropertyValueIterator<FStructProperty> It)
	{
		for ( ; It; ++It)
		{
			if (It->Key->Struct == TBaseStructure<FStateTreeStateLink>::Get())
			{
				FStateTreeStateLink& StateLink = *static_cast<FStateTreeStateLink*>(const_cast<void*>(It->Value));

				if (!ResolveTransitionStateAndFallback(SourceState, StateLink, StateLink.StateHandle, StateLink.Fallback))
				{
					return false;
				}
			}
		}

		return true;
	};
	
	// Validate any state links.
	if (!ValidateStateLinks(TPropertyValueIterator<FStructProperty>(InstanceData.GetStruct(), InstanceData.GetMutableMemory())))
	{
		return false;
	}
	if (!ValidateStateLinks(TPropertyValueIterator<FStructProperty>(NodeView.GetScriptStruct(), NodeView.GetMemory())))
	{
		return false;
	}

	const FStateTreeBindingLookup& BindingLookup = FStateTreeBindingLookup(EditorData);
	UE::StateTree::FCompileNodeContext CompileContext(InstanceData, InstanceDesc, BindingLookup);
	const EDataValidationResult Result = Node.Compile(CompileContext);

	if (Result == EDataValidationResult::Invalid && CompileContext.ValidationErrors.IsEmpty())
	{
		Log.Report(EMessageSeverity::Error, InstanceDesc, TEXT("Node validation failed."));
	}
	else
	{
		const EMessageSeverity::Type Severity = Result == EDataValidationResult::Invalid ? EMessageSeverity::Error : EMessageSeverity::Warning;
		for (const FText& Error : CompileContext.ValidationErrors)
		{
			Log.Report(Severity, InstanceDesc, Error.ToString());
		}
	}

	// Make sure there's no level actor references in the data.
	if (!UE::StateTree::Compiler::ValidateNoLevelActorReferences(Log, InstanceDesc, NodeView, InstanceData))
	{
		return false;
	}
	
	if (const UClass* InstanceDataClass = Cast<UClass>(InstanceData.GetStruct()); InstanceDataClass && InstanceDataClass->HasAllClassFlags(CLASS_Deprecated))
	{
		Log.Report(EMessageSeverity::Warning, InstanceDesc, TEXT("The instance data class is deprecated. It won't work in a cooked build."));
	}

	return Result != EDataValidationResult::Invalid;
}

bool FStateTreeCompiler::CreateTask(UStateTreeState* State, const FStateTreeEditorNode& TaskNode, const FStateTreeDataHandle TaskDataHandle)
{
	if (!TaskNode.GetNode().IsValid())
	{
		return false;
	}
	
	// Create binding source struct descriptor.
	FStateTreeBindableStructDesc InstanceDesc;
	InstanceDesc.StatePath = State ? State->GetPath() : UE::StateTree::Editor::GlobalStateName;
	InstanceDesc.ID = TaskNode.ID;
	InstanceDesc.Name = TaskNode.GetName();
	InstanceDesc.DataSource = EStateTreeBindableStructSource::Task;

	// Check that node has valid instance initialized.
	if (!TaskNode.GetInstance().IsValid())
	{
		Log.Reportf(EMessageSeverity::Error, InstanceDesc,
			TEXT("Malformed task, missing instance value."));
		return false;
	}

	// Copy the task
	const int32 NewNodeIndex = Nodes.Num();
	IDToNode.Add(TaskNode.ID, NewNodeIndex);	// for the instance data
	if (TaskCompletionDispatcherIDs.Contains(TaskNode.GetNodeID()))
	{
		IDToNode.Add(TaskNode.GetNodeID(), NewNodeIndex); // for the template data
	}
	FInstancedStruct& Node = Nodes.Add_GetRef(FInstancedStruct(TaskNode.GetNode()));
	InstantiateStructSubobjects(Node);
	
	FStateTreeTaskBase& Task = Node.GetMutable<FStateTreeTaskBase>();
	TOptional<FStateTreeDataView> InstanceDataView = CreateNodeInstanceData(TaskNode, Task, InstanceDesc, TaskDataHandle, InstanceStructs);
	if (!InstanceDataView.IsSet())
	{
		return false;
	}

	if (!Task.bTaskEnabled)
	{
		Log.Reportf(EMessageSeverity::Info, InstanceDesc, TEXT("Task is disabled and will have no effect."));
	}

	if (!CompileAndValidateNode(State, InstanceDesc, Node,  InstanceDataView.GetValue()))
	{
		return false;
	}

	CreateBindingSourceStructsForNode(TaskNode, InstanceDesc);

	return true;
}

bool FStateTreeCompiler::CreateEvaluator(const FStateTreeEditorNode& EvalNode, const FStateTreeDataHandle EvalDataHandle)
{
	// Silently ignore empty nodes.
	if (!EvalNode.GetNode().IsValid())
	{
		return true;
	}

	// Create binding source descriptor for instance.
	FStateTreeBindableStructDesc InstanceDesc;
	InstanceDesc.StatePath = UE::StateTree::Editor::GlobalStateName;
	InstanceDesc.ID = EvalNode.ID;
	InstanceDesc.Name = EvalNode.GetName();
	InstanceDesc.DataSource = EStateTreeBindableStructSource::Evaluator;

	// Check that node has valid instance initialized.
	if (!EvalNode.GetInstance().IsValid())
	{
		Log.Reportf(EMessageSeverity::Error, InstanceDesc,
			TEXT("Malformed evaluator, missing instance value."));
		return false;
	}

	// Copy the evaluator
	IDToNode.Add(EvalNode.ID, Nodes.Num());
	FInstancedStruct& Node = Nodes.Add_GetRef(FInstancedStruct(EvalNode.GetNode()));
	InstantiateStructSubobjects(Node);
	
	FStateTreeEvaluatorBase& Eval = Node.GetMutable<FStateTreeEvaluatorBase>();
	TOptional<FStateTreeDataView> InstanceDataView = CreateNodeInstanceData(EvalNode, Eval, InstanceDesc, EvalDataHandle, InstanceStructs);
	if (!InstanceDataView.IsSet())
	{
		return false;
	}

	if (!CompileAndValidateNode(nullptr, InstanceDesc, Node,  InstanceDataView.GetValue()))
	{
		return false;
	}

	CreateBindingSourceStructsForNode(EvalNode, InstanceDesc);

	return true;
}

bool FStateTreeCompiler::ValidateStructRef(const FStateTreeBindableStructDesc& SourceStruct, const FPropertyBindingPath& SourcePath,
											const FStateTreeBindableStructDesc& TargetStruct, const FPropertyBindingPath& TargetPath) const
{
	FString ResolveError;
	TArray<FPropertyBindingPathIndirection> TargetIndirection;
	if (!TargetPath.ResolveIndirections(TargetStruct.Struct, TargetIndirection, &ResolveError))
	{
		// This will later be reported by the bindings compiler.
		Log.Reportf(EMessageSeverity::Error, TargetStruct, TEXT("Failed to resolve binding path in %s: %s"), *TargetStruct.ToString(), *ResolveError);
		return false;
	}
	const FProperty* TargetLeafProperty = TargetIndirection.Num() > 0 ? TargetIndirection.Last().GetProperty() : nullptr;

	// Early out if the target is not FStateTreeStructRef.
	const FStructProperty* TargetStructProperty = CastField<FStructProperty>(TargetLeafProperty);
	if (TargetStructProperty == nullptr || TargetStructProperty->Struct != TBaseStructure<FStateTreeStructRef>::Get())
	{
		return true;
	}

	FString TargetBaseStructName;
	const UScriptStruct* TargetBaseStruct = UE::StateTree::Compiler::GetBaseStructFromMetaData(TargetStructProperty, TargetBaseStructName);
	if (TargetBaseStruct == nullptr)
	{
		Log.Reportf(EMessageSeverity::Error, TargetStruct,
				TEXT("Could not find base struct type '%s' for target %s'."),
				*TargetBaseStructName, *UE::StateTree::GetDescAndPathAsString(TargetStruct, TargetPath));
		return false;
	}

	TArray<FPropertyBindingPathIndirection> SourceIndirection;
	if (!SourcePath.ResolveIndirections(SourceStruct.Struct, SourceIndirection, &ResolveError))
	{
		// This will later be reported by the bindings compiler.
		Log.Reportf(EMessageSeverity::Error, SourceStruct, TEXT("Failed to resolve binding path in %s: %s"), *SourceStruct.ToString(), *ResolveError);
		return false;
	}
	const FProperty* SourceLeafProperty = SourceIndirection.Num() > 0 ? SourceIndirection.Last().GetProperty() : nullptr;

	// Exit if the source is not a struct property.
	const FStructProperty* SourceStructProperty = CastField<FStructProperty>(SourceLeafProperty);
	if (SourceStructProperty == nullptr)
	{
		return true;
	}
	
	if (SourceStructProperty->Struct == TBaseStructure<FStateTreeStructRef>::Get())
	{
		// Source is struct ref too, check the types match.
		FString SourceBaseStructName;
		const UScriptStruct* SourceBaseStruct = UE::StateTree::Compiler::GetBaseStructFromMetaData(SourceStructProperty, SourceBaseStructName);
		if (SourceBaseStruct == nullptr)
		{
			Log.Reportf(EMessageSeverity::Error, TargetStruct,
					TEXT("Could not find base struct '%s' for binding source %s."),
					*SourceBaseStructName, *UE::StateTree::GetDescAndPathAsString(SourceStruct, SourcePath));
			return false;
		}

		if (SourceBaseStruct->IsChildOf(TargetBaseStruct) == false)
		{
			Log.Reportf(EMessageSeverity::Error, TargetStruct,
						TEXT("Type mismatch between source %s and target %s types, '%s' is not child of '%s'."),
						*UE::StateTree::GetDescAndPathAsString(SourceStruct, SourcePath),
						*UE::StateTree::GetDescAndPathAsString(TargetStruct, TargetPath),
						*GetNameSafe(SourceBaseStruct), *GetNameSafe(TargetBaseStruct));
			return false;
		}
	}
	else
	{
		if (!SourceStructProperty->Struct || SourceStructProperty->Struct->IsChildOf(TargetBaseStruct) == false)
		{
			Log.Reportf(EMessageSeverity::Error, TargetStruct,
						TEXT("Type mismatch between source %s and target %s types, '%s' is not child of '%s'."),
						*UE::StateTree::GetDescAndPathAsString(SourceStruct, SourcePath),
						*UE::StateTree::GetDescAndPathAsString(TargetStruct, TargetPath),
						*GetNameSafe(SourceStructProperty->Struct), *GetNameSafe(TargetBaseStruct));
			return false;
		}
	}

	return true;
}

bool FStateTreeCompiler::ValidateBindingOnNode(const FStateTreeBindableStructDesc& TargetStruct, const FPropertyBindingPath& TargetPath) const
{
	FString ResolveError;
	TArray<FPropertyBindingPathIndirection> TargetIndirection;
	if (!TargetPath.ResolveIndirections(TargetStruct.Struct, TargetIndirection, &ResolveError))
	{
		// This will later be reported by the bindings compiler.
		Log.Reportf(EMessageSeverity::Error, TargetStruct, TEXT("Failed to resolve binding path in %s: %s"), *TargetStruct.ToString(), *ResolveError);
		return false;
	}

	check(TargetStruct.Struct);
	if (TargetStruct.Struct->IsChildOf<FStateTreeNodeBase>())
	{
		if (!UE::StateTree::Compiler::IsPropertyChildOfAnyStruct<FStateTreePropertyRef, FStateTreeDelegateListener>(TargetStruct, TargetPath))
		{
			Log.Reportf(EMessageSeverity::Error, TargetStruct,
				TEXT("Node struct(non-instance) only supports DelegateListener and PropertyReference as binding targets. %s is not supported."),
				*UE::StateTree::GetDescAndPathAsString(TargetStruct, TargetPath));

			return false;
		}
	}

	return true;
}

bool FStateTreeCompiler::GetAndValidateBindings(const FStateTreeBindableStructDesc& TargetStruct, FStateTreeDataView TargetValue, FValidatedPathBindings& OutValidatedBindings) const
{
	using namespace UE::StateTree::Compiler;

	check(EditorData);
	
	OutValidatedBindings = FValidatedPathBindings();

	// If target struct is not set, nothing to do.
	if (TargetStruct.Struct == nullptr)
	{
		return true;
	}

	bool bSucceeded = true;

	// Handle sources. Need to handle them now while we have the instance.
	for (FStateTreePropertyPathBinding& Binding : EditorData->EditorBindings.GetMutableBindings())
	{
		if (Binding.GetSourcePath().GetStructID() == TargetStruct.ID)
		{
			if (IsPropertyChildOfAnyStruct<FStateTreeDelegateDispatcher>(TargetStruct, Binding.GetSourcePath()))
			{
				FStateTreePropertyPathBinding& BindingCopy = OutValidatedBindings.DelegateDispatchers.Add_GetRef(Binding);
				BindingCopy.SetSourceDataHandle(TargetStruct.DataHandle);
			}
		}
	}

	// Handle targets.
	for (FStateTreePropertyPathBinding& Binding : EditorData->EditorBindings.GetMutableBindings())
	{
		if (Binding.GetTargetPath().GetStructID() != TargetStruct.ID)
		{
			continue;
		}

		const FGuid SourceStructID = Binding.GetSourcePath().GetStructID();
		const FStateTreeBindableStructDesc* SourceStruct = BindingsCompiler.GetSourceStructDescByID(SourceStructID);

		// Validate and try to fix up binding paths to be up to date
		// @todo: check if Delegate and PropertyRef is put on Instance Data and throw warning if so
		// @todo: not liking how this mutates the Binding.TargetPath, but currently we dont track well the instanced object changes.
		{
			// Source must be one of the source structs we have discovered in the tree.
			if (!SourceStruct)
			{
				Log.Reportf(EMessageSeverity::Error, TargetStruct,
					TEXT("Failed to find binding source property '%s' for target %s."),
					*Binding.GetSourcePath().ToString(), *UE::StateTree::GetDescAndPathAsString(TargetStruct, Binding.GetTargetPath()));
				bSucceeded = false;
				continue;
			}

			// Source must be accessible by the target struct via all execution paths.
			TArray<TInstancedStruct<FPropertyBindingBindableStructDescriptor>> AccessibleStructs;
			EditorData->GetBindableStructs(Binding.GetTargetPath().GetStructID(), AccessibleStructs);

			const bool bSourceAccessible = AccessibleStructs.ContainsByPredicate([SourceStructID](TConstStructView<FPropertyBindingBindableStructDescriptor> Struct)
				{
					return (Struct.Get().ID == SourceStructID);
				});

			if (!bSourceAccessible)
			{
				TInstancedStruct<FPropertyBindingBindableStructDescriptor> SourceStructDescriptor;
				const bool bFoundSourceStructDescriptor = EditorData->GetBindableStructByID(SourceStructID, SourceStructDescriptor);
				if (bFoundSourceStructDescriptor
					&& SourceStructDescriptor.Get<FStateTreeBindableStructDesc>().DataSource == EStateTreeBindableStructSource::Task
					&& !UE::StateTree::AcceptTaskInstanceData(TargetStruct.DataSource))
				{
					Log.Reportf(EMessageSeverity::Error, TargetStruct,
						TEXT("Property at %s cannot be bound to %s, because the binding source %s is a task instance data that is possibly not instantiated before %s in the tree."),
						*UE::StateTree::GetDescAndPathAsString(*SourceStruct, Binding.GetSourcePath()),
						*UE::StateTree::GetDescAndPathAsString(TargetStruct, Binding.GetTargetPath()),
						*SourceStruct->ToString(), *TargetStruct.ToString());
				}
				else
				{
					Log.Reportf(EMessageSeverity::Error, TargetStruct,
						TEXT("Property at %s cannot be bound to %s, because the binding source %s is not updated before %s in the tree."),
						*UE::StateTree::GetDescAndPathAsString(*SourceStruct, Binding.GetSourcePath()),
						*UE::StateTree::GetDescAndPathAsString(TargetStruct, Binding.GetTargetPath()),
						*SourceStruct->ToString(), *TargetStruct.ToString());
				}
				bSucceeded = false;
				continue;
			}

			if (!IDToStructValue.Contains(SourceStructID))
			{
				Log.Reportf(EMessageSeverity::Error, TargetStruct,
					TEXT("Failed to find value for binding source property '%s' for target %s."),
					*Binding.GetSourcePath().ToString(), *UE::StateTree::GetDescAndPathAsString(TargetStruct, Binding.GetTargetPath()));
				bSucceeded = false;
				continue;
			}

			// Update the source structs only if we have value for it. For some sources (e.g. context structs) we know only type, and in that case there are no instance structs.
			const FStateTreeDataView SourceValue = IDToStructValue[SourceStructID];
			if (SourceValue.GetStruct() != nullptr)
			{
				if (!Binding.GetMutableSourcePath().UpdateSegmentsFromValue(SourceValue))
				{
					Log.Reportf(EMessageSeverity::Error, TargetStruct,
						TEXT("Malformed target property path for binding source property '%s' for source %s."),
						*Binding.GetSourcePath().ToString(), *UE::StateTree::GetDescAndPathAsString(TargetStruct, Binding.GetTargetPath()));
					bSucceeded = false;
					continue;
				}
			}

			if (SourceStruct->Struct && !SourceStruct->Struct->IsChildOf<FStateTreeNodeBase>())
			{
				if (!SourceStruct->DataHandle.IsValid())
				{
					Log.Reportf(EMessageSeverity::Error, TargetStruct,
						TEXT("Malformed source'%s for property binding property '%s'."),
						*UE::StateTree::GetDescAndPathAsString(*SourceStruct, Binding.GetSourcePath()), *Binding.GetSourcePath().ToString());
					bSucceeded = false;
					continue;
				}
			}

			// Update path instance types from latest data. E.g. binding may have been created for instanced object of type FooB, and changed to FooA.
			if (!Binding.GetMutableTargetPath().UpdateSegmentsFromValue(TargetValue))
			{
				Log.Reportf(EMessageSeverity::Error, TargetStruct,
					TEXT("Malformed target property path for binding source property '%s' for target %s."),
					*Binding.GetSourcePath().ToString(), *UE::StateTree::GetDescAndPathAsString(TargetStruct, Binding.GetTargetPath()));
				bSucceeded = false;
				continue;
			}

			TArray<FPropertyBindingPathIndirection, TInlineAllocator<16>> Indirections;
			const bool bResolveSucceeded = Binding.GetMutableTargetPath().ResolveIndirections(TargetStruct.Struct, Indirections);
			check(bResolveSucceeded);

			if (Indirections.IsEmpty())
			{
				Log.Reportf(EMessageSeverity::Error, TargetStruct,
					TEXT("Malformed target property path. Target Struct can not be bound directly."));

				bSucceeded = false;
				continue;
			}

			const FProperty* TargetStructRootProperty = Indirections[0].GetProperty();
			check(TargetStructRootProperty);

			// Fix Binding output flag based on target path's root property type
			const bool bIsOutputBinding = Binding.IsOutputBinding();
			if (UE::StateTree::GetUsageFromMetaData(TargetStructRootProperty) == EStateTreePropertyUsage::Output)
			{
				if (!bIsOutputBinding)
				{
					Log.Reportf(EMessageSeverity::Info, TargetStruct,
						TEXT("Automatically fixed Binding to be output one because target property %s is output property."),
						*UE::StateTree::GetDescAndPathAsString(TargetStruct, Binding.GetTargetPath()));

					constexpr bool bOutputBinding = true;
					Binding.SetIsOutputBinding(bOutputBinding);
				}

				if (IsPropertyChildOfAnyStruct<FStateTreeDelegateDispatcher, FStateTreeDelegateListener, FStateTreePropertyRef>(*SourceStruct, Binding.GetSourcePath()))
				{
					Log.Reportf(EMessageSeverity::Error, TargetStruct,
						TEXT("Property at %s cannot be bound to %s. Output binding doesn't accept Delegate or PropertyReference as source property."),
						*UE::StateTree::GetDescAndPathAsString(TargetStruct, Binding.GetTargetPath()), 
						*UE::StateTree::GetDescAndPathAsString(TargetStruct, Binding.GetSourcePath()));
					
					bSucceeded = false;
					continue;
				}
			}
			else if (bIsOutputBinding)
			{
				Log.Reportf(EMessageSeverity::Info, TargetStruct,
					TEXT("Automatically fixed Binding to be non output one because target property %s is not output property."),
					*UE::StateTree::GetDescAndPathAsString(TargetStruct, Binding.GetTargetPath()));

				constexpr bool bOutputBinding = false;
				Binding.SetIsOutputBinding(bOutputBinding);
			}
		}

		TArray<FStateTreePropertyPathBinding>* OutputBindings = nullptr;
		if (IsPropertyChildOfAnyStruct<FStateTreeDelegateListener>(TargetStruct, Binding.GetTargetPath()))
		{
			OutputBindings = &OutValidatedBindings.DelegateListeners;
		}
		else if (IsPropertyChildOfAnyStruct<FStateTreePropertyRef>(TargetStruct, Binding.GetTargetPath()))
		{
			OutputBindings = &OutValidatedBindings.ReferenceBindings;
		}
		else if (Binding.IsOutputBinding())
		{
			OutputBindings = &OutValidatedBindings.OutputCopyBindings;
		}
		else
		{
			OutputBindings = &OutValidatedBindings.CopyBindings;
		}

		check(OutputBindings);

		FStateTreePropertyPathBinding& BindingCopy = OutputBindings->Add_GetRef(Binding);
		BindingCopy.SetSourceDataHandle(SourceStruct->DataHandle);

		// Special case for AnyEnum. StateTreeBindingExtension allows AnyEnums to bind to other enum types.
		// The actual copy will be done via potential type promotion copy, into the value property inside the AnyEnum.
		// We amend the paths here to point to the 'Value' property.
		const bool bSourceIsAnyEnum = IsPropertyChildOfAnyStruct<FStateTreeAnyEnum>(*SourceStruct, Binding.GetSourcePath());
		const bool bTargetIsAnyEnum = IsPropertyChildOfAnyStruct<FStateTreeAnyEnum>(TargetStruct, Binding.GetTargetPath());
		if (bSourceIsAnyEnum || bTargetIsAnyEnum)
		{
			if (bSourceIsAnyEnum)
			{
				BindingCopy.GetMutableSourcePath().AddPathSegment(GET_MEMBER_NAME_STRING_CHECKED(FStateTreeAnyEnum, Value));
			}
			if (bTargetIsAnyEnum)
			{
				BindingCopy.GetMutableTargetPath().AddPathSegment(GET_MEMBER_NAME_STRING_CHECKED(FStateTreeAnyEnum, Value));
			}
		}

		// Check if the bindings is for struct ref and validate the types.
		if (!ValidateStructRef(*SourceStruct, Binding.GetSourcePath(), TargetStruct, Binding.GetTargetPath()))
		{
			bSucceeded = false;
			continue;
		}

		if (!ValidateBindingOnNode(TargetStruct, Binding.GetTargetPath()))
		{
			bSucceeded = false;
			continue;
		}
	}

	if (!bSucceeded)
	{
		return false;
	}

	auto IsPropertyBound = [](const FName& PropertyName, TConstArrayView<FStateTreePropertyPathBinding> Bindings)
	{
		return Bindings.ContainsByPredicate([&PropertyName](const FStateTreePropertyPathBinding& Binding)
			{
				// We're looping over just the first level of properties on the struct, so we assume that the path is just one item
				// (or two in case of AnyEnum, because we expand the path to Property.Value, see code above).
				return Binding.GetTargetPath().GetSegments().Num() >= 1 && Binding.GetTargetPath().GetSegments()[0].GetName() == PropertyName;
			});
	};
	
	// Validate that Input and Context bindings
	for (TFieldIterator<FProperty> It(TargetStruct.Struct); It; ++It)
	{
		const FProperty* Property = *It;
		check(Property);
		const FName PropertyName = Property->GetFName();

		if (UE::StateTree::PropertyRefHelpers::IsPropertyRef(*Property))
		{
			TArray<FPropertyBindingPathIndirection> TargetIndirections;
			FPropertyBindingPath TargetPath(TargetStruct.ID, PropertyName);
			if (!TargetPath.ResolveIndirectionsWithValue(TargetValue, TargetIndirections))
			{
				Log.Reportf(EMessageSeverity::Error, TargetStruct,
						TEXT("Couldn't resolve path to '%s' for target %s."),
						*PropertyName.ToString(), *TargetStruct.ToString());
				bSucceeded = false;
				continue;
			}
			else
			{
				const void* PropertyRef = TargetIndirections.Last().GetPropertyAddress();
				const bool bIsOptional = UE::StateTree::PropertyRefHelpers::IsPropertyRefMarkedAsOptional(*Property, PropertyRef);

				if (bIsOptional == false && !IsPropertyBound(PropertyName, OutValidatedBindings.ReferenceBindings))
				{
					Log.Reportf(EMessageSeverity::Error, TargetStruct,
						TEXT("Property reference '%s' on % s is expected to have a binding."),
						*PropertyName.ToString(), *TargetStruct.ToString());
					bSucceeded = false;
					continue;
				}
			}
		}
		else
		{
			const bool bIsOptional = UE::StateTree::PropertyHelpers::HasOptionalMetadata(*Property);
			const EStateTreePropertyUsage Usage = UE::StateTree::GetUsageFromMetaData(Property);
			if (Usage == EStateTreePropertyUsage::Input)
			{
				// Make sure that an Input property is bound unless marked optional.
				if (bIsOptional == false
					&& !IsPropertyBound(PropertyName, OutValidatedBindings.CopyBindings)
					&& !IsPropertyBound(PropertyName, OutValidatedBindings.DelegateListeners))
				{
					Log.Reportf(EMessageSeverity::Error, TargetStruct,
						TEXT("Input property '%s' on %s is expected to have a binding."),
						*PropertyName.ToString(), *TargetStruct.ToString());
					bSucceeded = false;
					continue;
				}
			}
			else if (Usage == EStateTreePropertyUsage::Context)
			{
				// Make sure a Context property is manually or automatically bound.
				const UStruct* ContextObjectType = nullptr; 
				if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
				{
					ContextObjectType = StructProperty->Struct;
				}		
				else if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property))
				{
					ContextObjectType = ObjectProperty->PropertyClass;
				}

				if (ContextObjectType == nullptr)
				{
					Log.Reportf(EMessageSeverity::Error, TargetStruct,
						TEXT("The type of Context property '%s' on %s is expected to be Object Reference or Struct."),
						*PropertyName.ToString(), *TargetStruct.ToString());
					bSucceeded = false;
					continue;
				}

				const bool bIsBound = IsPropertyBound(PropertyName, OutValidatedBindings.CopyBindings);

				if (!bIsBound)
				{
					const FStateTreeBindableStructDesc Desc = EditorData->FindContextData(ContextObjectType, PropertyName.ToString());

					if (Desc.IsValid())
					{
						// Add automatic binding to Context data.
						constexpr bool bIsOutputBinding = false;
						OutValidatedBindings.CopyBindings.Emplace(FPropertyBindingPath(Desc.ID), FPropertyBindingPath(TargetStruct.ID, PropertyName), bIsOutputBinding);
					}
					else
					{
						Log.Reportf(EMessageSeverity::Error, TargetStruct,
							TEXT("Could not find matching Context object for Context property '%s' on '%s'. Property must have manual binding."),
							*PropertyName.ToString(), *TargetStruct.ToString());
						bSucceeded = false;
						continue;
					}
				}
			}
		}
	}

	return bSucceeded;
}

void FStateTreeCompiler::InstantiateStructSubobjects(FStructView Struct)
{
	check(StateTree);
	check(EditorData);
	
	// Empty struct, nothing to do.
	if (!Struct.IsValid())
	{
		return;
	}

	for (TPropertyValueIterator<FProperty> It(Struct.GetScriptStruct(), Struct.GetMemory()); It; ++It)
	{
		if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(It->Key))
		{
			// Duplicate instanced objects.
			if (ObjectProperty->HasAnyPropertyFlags(CPF_InstancedReference))
			{
				if (UObject* Object = ObjectProperty->GetObjectPropertyValue(It->Value))
				{
					UObject* OuterObject = Object->GetOuter();
					// If the instanced object was created as Editor Data as outer,
					// change the outer to State Tree to prevent references to editor only data.
					if (Object->IsInOuter(EditorData))
					{
						OuterObject = StateTree;
					}
					UObject* DuplicatedObject = DuplicateObject(Object, OuterObject);
					ObjectProperty->SetObjectPropertyValue(const_cast<void*>(It->Value), DuplicatedObject);
				}
			}
		}
	}
}

bool FStateTreeCompiler::NotifyInternalPost()
{
	struct FPostInternalContextImpl : UE::StateTree::Compiler::FPostInternalContext
	{
		virtual TNotNull<const UStateTree*> GetStateTree() const override
		{
			return StateTree;
		}
		virtual TNotNull<const UStateTreeEditorData*> GetEditorData() const override
		{
			return EditorData;
		}

		virtual FStateTreeCompilerLog& GetLog() const override
		{
			return Log;
		}

		virtual UStateTreeExtension* AddExtension(TNotNull<TSubclassOf<UStateTreeExtension>> ExtensionType) const override
		{
			UClass* ExtensionClass = static_cast<TSubclassOf<UStateTreeExtension>>(ExtensionType).Get();
			UStateTreeExtension* Result = NewObject<UStateTreeExtension>(StateTree.Get(), ExtensionClass);
			StateTree->Extensions.Add(Result);
			return Result;
		}

		FPostInternalContextImpl(TNotNull<UStateTree*> InStateTree, TNotNull<UStateTreeEditorData*> InEditorData, FStateTreeCompilerLog& InLog)
			: StateTree(InStateTree)
			, EditorData(InEditorData)
			, Log(InLog)
		{ }

		TObjectPtr<UStateTree> StateTree = nullptr;
		TObjectPtr<UStateTreeEditorData> EditorData = nullptr;
		FStateTreeCompilerLog& Log;
	};

	FPostInternalContextImpl Context = FPostInternalContextImpl(StateTree, EditorData, Log);

	// Notify the module, schema, and extension. Use that order to go from the less specific to the more specific.
	FStateTreeEditorModule::GetModule().OnPostInternalCompile().Broadcast(Context);

	if (EditorData->EditorSchema)
	{
		if (!EditorData->EditorSchema->HandlePostInternalCompile(Context))
		{
			Log.Reportf(EMessageSeverity::Error, TEXT("The schema failed compilation. See log for more info."));
			return false;
		}
	}

	for (UStateTreeEditorDataExtension* Extension : EditorData->Extensions)
	{
		if (Extension)
		{
			if (!Extension->HandlePostInternalCompile(Context))
			{
				Log.Reportf(EMessageSeverity::Error, TEXT("An extension failed compilation. See log for more info."));
				return false;
			}
		}
	}

	return true;
}

void FStateTreeCompiler::CreateBindingSourceStructsForNode(const FStateTreeEditorNode& EditorNode, const FStateTreeBindableStructDesc& InstanceDesc)
{
	using namespace UE::StateTree::Compiler;

	check(EditorNode.GetNode().IsValid());

	BindingsCompiler.AddSourceStruct(InstanceDesc);

	const bool bIsTaskCompletionNode = TaskCompletionDispatcherIDs.Contains(EditorNode.GetNodeID())
		&& EditorNode.GetNode().GetScriptStruct()->IsChildOf(FStateTreeTaskBase::StaticStruct());
	if (bIsTaskCompletionNode || IsNodeStructEligibleForBinding(EditorNode.GetNode()))
	{
		FStateTreeBindableStructDesc NodeDesc = InstanceDesc;
		NodeDesc.ID = EditorNode.GetNodeID();
		NodeDesc.Struct = EditorNode.GetNode().GetScriptStruct();
		NodeDesc.DataHandle = FStateTreeDataHandle::Invalid;	// DataHandle is only for Instance data

		BindingsCompiler.AddSourceStruct(NodeDesc);
	}
}

const FString& FStateTreeCompiler::GetStatePath(int32 StateIndex) const
{
	if (ensure(StatePaths.IsValidIndex(StateIndex)))
	{
		return StatePaths[StateIndex];
	}
	static FString EmptyString;
	return EmptyString;
}
