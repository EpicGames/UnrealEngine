// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeCompilerLog.h"
#include "IMessageLogListing.h"
#include "Misc/UObjectToken.h"
#include "StateTree.h"
#include "StateTreeState.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeCompilerLog)

#define LOCTEXT_NAMESPACE "StateTreeEditor"

TArray<TSharedRef<FTokenizedMessage>> FStateTreeCompilerLog::ToTokenizedMessages() const
{
	TArray<TSharedRef<FTokenizedMessage>> TokenisedMessages;
	for (const FStateTreeCompilerLogMessage& StateTreeMessage : Messages)
	{
		TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create((EMessageSeverity::Type)StateTreeMessage.Severity);

		if (StateTreeMessage.State != nullptr)
		{
			TSharedRef<FUObjectToken> ObjectToken = FUObjectToken::Create(StateTreeMessage.State, FText::FromName(StateTreeMessage.State->Name));

			// Add a dummy activation since default behavior opens content browser.
			static auto DummyActivation = [](const TSharedRef<class IMessageToken>&) {};
			ObjectToken->OnMessageTokenActivated(FOnMessageTokenActivated::CreateLambda(DummyActivation));
			
			Message->AddToken(ObjectToken);
		}

		if (StateTreeMessage.Item.ID.IsValid())
		{
			Message->AddToken(FTextToken::Create(FText::Format(LOCTEXT("LogMessageItem", " {0} '{1}': "),
				UEnum::GetDisplayValueAsText(StateTreeMessage.Item.DataSource), FText::FromName(StateTreeMessage.Item.Name))));
		}

		if (!StateTreeMessage.Message.IsEmpty())
		{
			Message->AddToken(FTextToken::Create(FText::FromString(StateTreeMessage.Message)));
		}

		TokenisedMessages.Add(Message);
	}

	return TokenisedMessages;
}

void FStateTreeCompilerLog::AppendToLog(IMessageLogListing* LogListing) const
{
	LogListing->AddMessages(ToTokenizedMessages());
}

void FStateTreeCompilerLog::DumpToLog(const FLogCategoryBase& Category) const
{
	DumpToLog(nullptr, Category);
}

void FStateTreeCompilerLog::DumpToLog(const UStateTree* Asset, const FLogCategoryBase& Category) const
{
	if (Messages.Num() > 0 && Asset)
	{
		UE_LOG_REF(Category, Log, TEXT("Compile asset '%s' "), *Asset->GetPathName());
	}

	for (const FStateTreeCompilerLogMessage& StateTreeMessage : Messages)
	{
		FString Message;
		
		if (StateTreeMessage.State != nullptr)
		{
			Message += FString::Printf(TEXT("State '%s': "), *StateTreeMessage.State->Name.ToString());
		}

		if (StateTreeMessage.Item.ID.IsValid())
		{
			Message += FString::Printf(TEXT("%s '%s': "), *UEnum::GetDisplayValueAsText(StateTreeMessage.Item.DataSource).ToString(), *StateTreeMessage.Item.Name.ToString());
		}

		Message += StateTreeMessage.Message;

		switch (StateTreeMessage.Severity)
		{
		case EMessageSeverity::Error:
			UE_LOG_REF(Category, Error, TEXT("%s"), *Message);
			break;
		case EMessageSeverity::PerformanceWarning:
		case EMessageSeverity::Warning:
			UE_LOG_REF(Category, Warning, TEXT("%s"), *Message);
			break;
		case EMessageSeverity::Info:
		default:
			UE_LOG_REF(Category, Log, TEXT("%s"), *Message);
			break;
		};
	}
}


#undef LOCTEXT_NAMESPACE

