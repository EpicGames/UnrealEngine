// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeCompilerManager.h"
#include "StateTree.h"
#include "StateTreeCompilerLog.h"
#include "StateTreeDelegates.h"
#include "StateTreeDelegatesInternal.h"
#include "StateTreeEditingSubsystem.h"
#include "StateTreeEditorData.h"
#include "StateTreeEditorModule.h"
#include "StateTreeEditorPropertyBindings.h"
#include "StateTreeSchema.h"

#include "Algo/Copy.h"
#include "Containers/Ticker.h"
#include "Editor.h"
#include "StructUtilsDelegates.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Templates/GuardValueAccessors.h"
#include "UObject/ObjectKey.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectIterator.h"

namespace UE::StateTree::Compiler::Private
{
FAutoConsoleVariable CVarLogStateTreeDependencies(
	TEXT("StateTree.Compiler.LogDependenciesOnCompilation"),
	false,
	TEXT("After a StateTree compiles, log the dependencies that will be required for the asset to recompile.")
);

bool bUseDependenciesToTriggerCompilation = true;
FAutoConsoleVariableRef CVarUseDependenciesToTriggerCompilation(
	TEXT("StateTree.Compiler.UseDependenciesToTriggerCompilation"),
	bUseDependenciesToTriggerCompilation,
	TEXT("Use the build dependencies to detect when a state tree needs to be linked or compiled.")
);

bool bEnableForceCompileSynchronouslyInPIESession = true;
FAutoConsoleVariableRef CVarEnableCompileSynchronouslyInPIESession(
	TEXT("StateTree.Compiler.EnableForceCompileSynchronouslyInPIESession"),
	bEnableForceCompileSynchronouslyInPIESession,
	TEXT("When PIE starts, flush the compilation queue.\n")
	TEXT("Any new asset will be compile synchronously event is async was requested.")
);

bool bEnableCompileAllIfChangedOnBeginPIE = true;
FAutoConsoleVariableRef CVarEnableCompileAllIfChangedOnBeginPIE(
	TEXT("StateTree.Compiler.EnableCompileAllIfChangedOnBeginPIE"),
	bEnableCompileAllIfChangedOnBeginPIE,
	TEXT("When PIE starts, compile all state tree asset (if needed).")
);

bool bEnableCompileAllIfChangedOnBeginPIE_UseCachedList = true;
FAutoConsoleVariableRef CVarEnableCompileAllIfChangedOnBeginPIE_UseCachedList(
	TEXT("StateTree.Compiler.EnableCompileAllIfChangedOnBeginPIE_UseCachedList"),
	bEnableCompileAllIfChangedOnBeginPIE_UseCachedList,
	TEXT("Use the cached list instead of iterating over all assets.")
);

int32 NumberOfQueuedCompilationPerBatch = 1;
FAutoConsoleVariableRef CVarNumberOfQueuedCompilationPerBatch(
	TEXT("StateTree.Compiler.NumberOfQueuedCompilationPerBatch"),
	NumberOfQueuedCompilationPerBatch,
	TEXT("Amount of queued state tree asset to compile per engine tick.")
);

FAutoConsoleCommand FlushCompilationQueue(
	TEXT("StateTree.Compiler.FlushCompilationQueue"),
	TEXT("Compile all pending state tree asset that requested a compilation."),
	FConsoleCommandDelegate::CreateLambda([]()
		{
			FCompilerManager::FlushCompilationQueue();
		})
);

struct FStateTreeDependencies
{
	enum EDependencyType
	{
		DT_None = 0,
		DT_Link = 1 << 0,
		DT_Internal = 1<< 1,
		DT_Public = 1 << 2,
	};
	struct FItem
	{
		FObjectKey Key;
		EDependencyType Type = EDependencyType::DT_None;
	};
	TArray<FItem> Dependencies;
};
ENUM_CLASS_FLAGS(FStateTreeDependencies::EDependencyType);

bool IsStructCoreOrStateTreeInternal(TNotNull<const UStruct*> InStruct)
{
	const TNotNull<const UPackage*> StateTreeModulePackage = UStateTree::StaticClass()->GetOutermost();
	const TNotNull<const UPackage*> StateTreeEditorModulePackage = UStateTreeEditorData::StaticClass()->GetOutermost();
	const TNotNull<const UPackage*> CoreUObjectModulePackage = UObject::StaticClass()->GetOutermost();

	return InStruct->IsInPackage(StateTreeModulePackage)
		|| InStruct->IsInPackage(StateTreeEditorModulePackage)
		|| InStruct->IsInPackage(CoreUObjectModulePackage);
}
/** Find the references that are needed by the asset. */
class FArchiveReferencingPropertiesBase : public FArchiveUObject
{
public:
	FArchiveReferencingPropertiesBase()
	{
		ArIsObjectReferenceCollector = true;
		ArIgnoreOuterRef = true;
		ArIgnoreArchetypeRef = true;
		ArIgnoreClassGeneratedByRef = true;
		ArIgnoreClassRef = true;

		SetShouldSkipCompilingAssets(false);
	}

	static bool IsSupportedObject(TNotNull<const UStruct*> Struct)
	{
		/** As an optimization, do not include basic structures like FVector and state tree internal types. */
		return !IsStructCoreOrStateTreeInternal(Struct);
	}
};

/** Find the references that are needed by the asset. */
class FArchiveReferencingProperties : public FArchiveReferencingPropertiesBase
{
public:
	FArchiveReferencingProperties(TNotNull<const UObject*> InReferencingObject)
		: ReferencingObjectPackage(InReferencingObject->GetPackage())
	{ }

	virtual FArchive& operator<<(UObject*& InSerializedObject) override
	{
		if (InSerializedObject)
		{
			if (const UStruct* AsStruct = Cast<const UStruct>(InSerializedObject))
			{
				if (IsSupportedObject(AsStruct))
				{
					Dependencies.Add(AsStruct);
				}
			}
			else
			{
				if (IsSupportedObject(InSerializedObject->GetClass()))
				{
					Dependencies.Add(InSerializedObject->GetClass());
				}
			}

			// Traversing the asset inner dependencies (instanced type).
			if (InSerializedObject->IsInPackage(ReferencingObjectPackage))
			{
				bool bAlreadyExists;
				SerializedObjects.Add(InSerializedObject, &bAlreadyExists);

				if (!bAlreadyExists)
				{
					InSerializedObject->Serialize(*this);
				}
			}
		}

		return *this;
	}

	TSet<TNotNull<const UStruct*>> Dependencies;

private:
	/** Tracks the objects which have been serialized by this archive, to prevent recursion */
	TSet<UObject*> SerializedObjects;

	TNotNull<const UPackage*> ReferencingObjectPackage;
};

/** Find the state tree asset references that are needed by the asset. */
class FArchiveReferencingStateTree : public FArchiveReferencingPropertiesBase
{
public:
	FArchiveReferencingStateTree(TNotNull<const UStateTree*> InReferencingObject)
		: ReferencingObjectPackage(InReferencingObject->GetPackage())
	{
	}

	virtual FArchive& operator<<(UObject*& InSerializedObject) override
	{
		if (UStateTree* StateTree = Cast<UStateTree>(InSerializedObject))
		{
			Dependencies.Add(StateTree);
		}
		else if (InSerializedObject)
		{
			// Traversing the asset inner dependencies (instanced type).
			if (InSerializedObject->IsInPackage(ReferencingObjectPackage))
			{
				bool bAlreadyExists;
				SerializedObjects.Add(InSerializedObject, &bAlreadyExists);

				if (!bAlreadyExists)
				{
					InSerializedObject->Serialize(*this);
				}
			}
		}

		return *this;
	}

	TSet<UStateTree*> Dependencies;

private:
	/** Tracks the objects which have been serialized by this archive, to prevent recursion */
	TSet<UObject*> SerializedObjects;

	TNotNull<const UPackage*> ReferencingObjectPackage;
};

class FCompilerManagerImpl
{
public:
	FCompilerManagerImpl();
	~FCompilerManagerImpl();
	FCompilerManagerImpl(const FCompilerManagerImpl&) = delete;
	FCompilerManagerImpl& operator=(const FCompilerManagerImpl&) = delete;

	void MarkAsModified(TNotNull<UStateTree*> InStateTree);
	void QueueForCompilation(TNotNull<UStateTree*> InStateTree);
	void FlushCompilationQueue();

	bool CompilePublicSynchronously(TNotNull<UStateTree*> InStateTree);
	bool CompileInternalSynchronously(TNotNull<UStateTree*> InStateTree, FStateTreeCompilerLog& InOutLog);
	TOptional<bool> CompileIfNeededSynchronously(TNotNull<UStateTree*> StateTree);

	void CacheEditorBindingExternalDependencies(TNotNull<UStateTreeEditorData*> InEditorData);

private:
	void AddToQueue_AnyThread(TNotNull<UStateTree*> InStateTree);

	/**
	 * Fix up editor bindings when external Blueprint or UserDefinedStruct that an EditorData is dependent on got changed and reinstanced
	 * For editor bindings fixup, we don't need to recompile. Editor bindings are in sync with compiled bindings(otherwise the compilation would have already been dirty). 
	 * When a UDS changes or an UObject reinstanced, we perform fix up on editor bindings, and relink the compiled data, during which it will also fix up the compiled bindings. So compiled bindings are still in sync with editor bindings.
	 * @param InChangedStructs changed User Defined Structs or reinstanced UObjects
	 */
	void UpdateEditorBindingsIfNeeded(TSet<const UStruct*>& InChangedStructs);

	void GatherDependencies(TNotNull<UStateTree*> StateTree);
	TSet<UStateTree*> GatherStateTreeDependencies_AnyThread(TNotNull<UStateTree*> StateTree) const;
	void LogDependencies(TNotNull<UStateTree*> StateTree) const;

	void SendToBatchCompilationQueue_AnyThread();
	bool BatchCompilationQueue();
	int32 CompileWithQueue_AnyThread(TNotNull<UStateTree*> StateTree, FStateTreeCompilerLog& Log);
	int32 CompileWithQueueImpl_AnyThread(TNotNull<UStateTree*> StateTree, FStateTreeCompilerLog& Log, TSet<TNotNull<const UStateTree*>>& ProcessedAssets);
	void CompileStateTree_AnyThread(TNotNull<UStateTree*> StateTree, FStateTreeCompilerLog& Log);
	bool AllowQueuedCompilation(TNotNull<UStateTree*> StateTree) const;

	void HandleStateTreeCooking(TNotNull<UStateTree*> StateTree);
	using FReplacementObjectMap = TMap<UObject*, UObject*>;
	void HandleObjectsReinstanced(const FReplacementObjectMap& ObjectMap);
	void HandlePreBeginPIE(bool bIsSimulating);
	void HandleEndPIE(bool bIsSimulating);
	void HandleUserDefinedStructReinstanced(const UUserDefinedStruct& UserDefinedStruct);

	void PurgeEditorBindingExternalDependencies();

private:
	FDelegateHandle ObjectsReinstancedHandle;
	FDelegateHandle UserDefinedStructReinstancedHandle;
	FDelegateHandle PreBeginPIEHandle;
	FDelegateHandle EndPIEHandle;

	struct FEditorBindingDependencies
	{
		using FTreeToDependencyMap = TMap<TObjectKey<const UStateTree>, TSet<TObjectKey<const UStruct>>>;
		using FDependencyToTreeMap = TMap<TObjectKey<const UStruct>, TSet<TObjectKey<const UStateTree>>>;
		FTreeToDependencyMap TreeToDependencyMap;
		FDependencyToTreeMap DependencyToTreeMap;

		static bool IsExternalDependency(TNotNull<const UStruct*> InStruct)
		{
			return IsStructCoreOrStateTreeInternal(InStruct) == false;
		}
	};

	/** Dependencies that are state tree external, and not in core */
	FEditorBindingDependencies EditorBindingExternalDependencies;

	/** Referencee to dependencies. Generated with GatherDependencies and used when a dependencies is "recreated" (blueprint compilation). */
	TMap<TObjectKey<UStateTree>, TSharedPtr<FStateTreeDependencies>> StateTreeToDependencies;
	TMap<FObjectKey, TArray<TObjectKey<UStateTree>>> DependenciesToStateTree;

	using FDirtyAssetArray = TArray<TObjectKey<UStateTree>, TInlineAllocator<4>>;
	FCriticalSection QueueLock;
	FDirtyAssetArray DirtyAssets_AnyThread;
	TSet<TObjectKey<UStateTree>> DirtyStateTrees_AnyThread;
	bool bPushedBatchOnGameThread_AnyThread = false;
	std::atomic<bool> bCompileWithQueueInProgress_AnyThread = false;

	bool bCanQueueCompilation = true;
};
static TUniquePtr<FCompilerManagerImpl> CompilerManagerImpl;

FCompilerManagerImpl::FCompilerManagerImpl()
{
	UE::StateTree::Delegates::Private::OnPreCookStateTreeAsset.BindRaw(this, &FCompilerManagerImpl::HandleStateTreeCooking);
	ObjectsReinstancedHandle = FCoreUObjectDelegates::OnObjectsReinstanced.AddRaw(this, &FCompilerManagerImpl::HandleObjectsReinstanced);
	UserDefinedStructReinstancedHandle = UE::StructUtils::Delegates::OnUserDefinedStructReinstanced.AddRaw(this, &FCompilerManagerImpl::HandleUserDefinedStructReinstanced);
	PreBeginPIEHandle = FEditorDelegates::PreBeginPIE.AddRaw(this, &FCompilerManagerImpl::HandlePreBeginPIE);
	EndPIEHandle = FEditorDelegates::EndPIE.AddRaw(this, &FCompilerManagerImpl::HandleEndPIE);

	// When running in standalone
	if (IsRunningGame())
	{
		bCanQueueCompilation = !Private::bEnableForceCompileSynchronouslyInPIESession;
	}
}

FCompilerManagerImpl::~FCompilerManagerImpl()
{
	FEditorDelegates::EndPIE.Remove(EndPIEHandle);
	FEditorDelegates::PreBeginPIE.Remove(PreBeginPIEHandle);
	UE::StructUtils::Delegates::OnUserDefinedStructReinstanced.Remove(UserDefinedStructReinstancedHandle);
	FCoreUObjectDelegates::OnObjectsReinstanced.Remove(ObjectsReinstancedHandle);
	UE::StateTree::Delegates::Private::OnPreCookStateTreeAsset.Unbind();
}

bool FCompilerManagerImpl::CompilePublicSynchronously(TNotNull<UStateTree*> StateTree)
{
	check(IsInGameThread() || IsInAsyncLoadingThread());

	UStateTreeEditingSubsystem::ValidateStateTree(StateTree);

	StateTree->CompileStatus = UStateTree::ECompileStatus::Public;

	FStateTreeCompilerLog Log;
	FStateTreeCompiler Compiler(Log);
	return Compiler.CompilePublic(StateTree);
}

bool FCompilerManagerImpl::CompileInternalSynchronously(TNotNull<UStateTree*> StateTree, FStateTreeCompilerLog& Log)
{
	check(IsInGameThread() || IsInAsyncLoadingThread());

	UStateTreeEditingSubsystem::ValidateStateTree(StateTree);

	{
		UE::TScopeLock _(QueueLock);
		const int32 DirtyAssetsIndex = DirtyAssets_AnyThread.IndexOfByKey(TObjectKey<UStateTree>(StateTree));
		if (DirtyAssetsIndex != INDEX_NONE)
		{
			DirtyAssets_AnyThread.RemoveAtSwap(DirtyAssetsIndex);
		}
		CompileWithQueue_AnyThread(StateTree, Log);
	}

	return StateTree->CompileStatus == UStateTree::ECompileStatus::Executable;
}

TOptional<bool> FCompilerManagerImpl::CompileIfNeededSynchronously(TNotNull<UStateTree*> StateTree)
{
	check(IsInGameThread() || IsInAsyncLoadingThread());

	// Test if the asset is already in the queue. Faster than hashing the editor.
	const bool bIsDirty = StateTree->bCompilationPending
		|| UStateTreeEditingSubsystem::NeedsRecompile(StateTree);
	if (bIsDirty)
	{
		FStateTreeCompilerLog Log;
		const bool bResult = CompileInternalSynchronously(StateTree, Log);
		return TOptional<bool>(bResult);
	}
	return {};
}

void FCompilerManagerImpl::CacheEditorBindingExternalDependencies(TNotNull<UStateTreeEditorData*> InEditorData)
{
	// For now, state tree asset only gets loaded on game thread and we don't use StrongObjPtr here.
	check(IsInGameThread());

	PurgeEditorBindingExternalDependencies();

	const UStateTree* StateTree = InEditorData->GetTypedOuter<UStateTree>();
	check(StateTree);

	TSet<TObjectKey<const UStruct>>& CurrentExternalDependencies = EditorBindingExternalDependencies.TreeToDependencyMap.FindOrAdd(StateTree);

	// Clean up registered dependencies
	for (TObjectKey<const UStruct> DependentStruct : CurrentExternalDependencies)
	{
		TSet<TObjectKey<const UStateTree>>* Trees = EditorBindingExternalDependencies.DependencyToTreeMap.Find(DependentStruct);
		check(Trees);

		const int32 RemovedCnt = Trees->Remove(StateTree);
		check(RemovedCnt == 1);
		if (Trees->IsEmpty())
		{
			EditorBindingExternalDependencies.DependencyToTreeMap.Remove(DependentStruct);
		}
	}

	CurrentExternalDependencies.Reset();
	
	const FStateTreeEditorPropertyBindings* EditorBindings = InEditorData->GetPropertyEditorBindings();
	check(EditorBindings);

	const TArray<TNotNull<const UStruct*>> NewExternalDependencies = EditorBindings->GatherDependenciesByPredicate(FEditorBindingDependencies::IsExternalDependency);
	Algo::Copy(NewExternalDependencies, CurrentExternalDependencies, Algo::NoRef);

	for (TObjectKey<const UStruct> DependentStruct : CurrentExternalDependencies)
	{
		TSet<TObjectKey<const UStateTree>>& Trees = EditorBindingExternalDependencies.DependencyToTreeMap.FindOrAdd(DependentStruct);
		Trees.Emplace(StateTree);
	}
}

void FCompilerManagerImpl::QueueForCompilation(TNotNull<UStateTree*> StateTree)
{
	check(IsInGameThread() || IsInAsyncLoadingThread());

	// Asset that runs on any thread (ex: mass), can't be queued.
	//They might be compiled via CompileIfNeededSynchronously on a worker thread.
	if (bCanQueueCompilation && AllowQueuedCompilation(StateTree))
	{
		const bool bCompiled = CompilePublicSynchronously(StateTree);
		if (bCompiled)
		{
			check(StateTree->CompileStatus == UStateTree::ECompileStatus::Internal);

			// Queue the internal steps
			UE::TScopeLock _(QueueLock);
			AddToQueue_AnyThread(StateTree);
		}
	}
	else
	{
		FStateTreeCompilerLog Log;
		CompileInternalSynchronously(StateTree, Log);
	}
}

void FCompilerManagerImpl::FlushCompilationQueue()
{
	check(IsInGameThread());

	UE::TScopeLock _(QueueLock);
	while (!DirtyAssets_AnyThread.IsEmpty())
	{
		TObjectKey<UStateTree> StateTreeKey = DirtyAssets_AnyThread.Pop();
		UStateTree* StateTree = StateTreeKey.ResolveObjectPtr();
		if (StateTree == nullptr)
		{
			continue;
		}

		FStateTreeCompilerLog Log;
		CompileWithQueue_AnyThread(StateTree, Log);
	}
	bPushedBatchOnGameThread_AnyThread = !DirtyAssets_AnyThread.IsEmpty();
}

bool FCompilerManagerImpl::AllowQueuedCompilation(TNotNull<UStateTree*> StateTree) const
{
	if (UStateTreeEditorData* EditorData = Cast<UStateTreeEditorData>(StateTree->EditorData))
	{
		if (ensure(EditorData->Schema))
		{
			return EditorData->Schema->AllowQueuedCompilation();
		}
	}
	return true;
}

void FCompilerManagerImpl::AddToQueue_AnyThread(TNotNull<UStateTree*> StateTree)
{
	TObjectKey<UStateTree> StateTreeKey = StateTree;
	DirtyAssets_AnyThread.AddUnique(StateTreeKey);

	StateTree->bCompilationPending = true;

	SendToBatchCompilationQueue_AnyThread();
}

void FCompilerManagerImpl::SendToBatchCompilationQueue_AnyThread()
{
	if (!bPushedBatchOnGameThread_AnyThread && NumberOfQueuedCompilationPerBatch > 0)
	{
		bPushedBatchOnGameThread_AnyThread = true;
		FTSTicker::GetCoreTicker().AddTicker(UE_SOURCE_LOCATION, UE_SMALL_NUMBER,
			[](float)
			{
				if (CompilerManagerImpl.IsValid())
				{
					const bool bContinue = CompilerManagerImpl->BatchCompilationQueue();
					return bContinue;
				}
				return false;
			}
		);
	}
}

bool FCompilerManagerImpl::BatchCompilationQueue()
{
	// If locked for compilation... wait for next frame to compile the dirty assets.
	if (bCompileWithQueueInProgress_AnyThread)
	{
		constexpr bool bContinue = true;
		return bContinue;
	}

	UE::TScopeLock _(QueueLock);

	while (!DirtyAssets_AnyThread.IsEmpty())
	{
		TObjectKey<UStateTree> StateTreeKey = DirtyAssets_AnyThread.Pop();
		UStateTree* StateTree = StateTreeKey.ResolveObjectPtr();
		if (StateTree == nullptr)
		{
			continue;
		}

		FStateTreeCompilerLog Log;
		const int32 CompileCounter = CompileWithQueue_AnyThread(StateTree, Log);
		if (CompileCounter >= NumberOfQueuedCompilationPerBatch)
		{
			break;
		}
	}
	const bool bContinue = !DirtyAssets_AnyThread.IsEmpty();
	bPushedBatchOnGameThread_AnyThread = bContinue;
	return bContinue;
}

int32 FCompilerManagerImpl::CompileWithQueue_AnyThread(TNotNull<UStateTree*> StateTree, FStateTreeCompilerLog& Log)
{
	bCompileWithQueueInProgress_AnyThread = true;

	TSet<TNotNull<const UStateTree*>> ProcessedAssets;
	ProcessedAssets.Add(StateTree);
	int32 CompiledStateTreeCount = CompileWithQueueImpl_AnyThread(StateTree, Log, ProcessedAssets);

	bCompileWithQueueInProgress_AnyThread = false;

	return CompiledStateTreeCount;
}

int32 FCompilerManagerImpl::CompileWithQueueImpl_AnyThread(TNotNull<UStateTree*> StateTree, FStateTreeCompilerLog& Log, TSet<TNotNull<const UStateTree*>>& ProcessedAssets)
{
	// State tree asset may depends on each other via linked asset or special task like FStateTreeRunParallelStateTreeTask.
	//The assets should be compiled in order. The referencer asset reference the global parameters of the referencee asset.
	//Depending if cvar bEnableQueuedCompilationWhenAssetDirty is enabled, the "public exported" can always be up-to-date.
	//We don't take any chances. We sort them base on their dependencies.

	int32 CompiledStateTreeCount = 0;
	if (DirtyAssets_AnyThread.Num() == 0)
	{
		CompileStateTree_AnyThread(StateTree, Log);
		++CompiledStateTreeCount;
	}
	else
	{
		const TSet<UStateTree*> Dependencies = GatherStateTreeDependencies_AnyThread(StateTree);
		for (UStateTree* Dependency : Dependencies)
		{
			if (Dependency->IsEditorDataDirty())
			{
				if (ProcessedAssets.Contains(Dependency))
				{
					UE_LOGF(LogStateTree, Warning, "Circular dependency detected while processing %ls and %ls.", *StateTree->GetFullName(), *Dependency->GetFullName());
				}
				else
				{
					const int32 DirtyAssetsIndex = DirtyAssets_AnyThread.IndexOfByKey(TObjectKey<UStateTree>(Dependency));
					if (DirtyAssetsIndex != INDEX_NONE)
					{
						DirtyAssets_AnyThread.RemoveAtSwap(DirtyAssetsIndex, EAllowShrinking::No);

						FStateTreeCompilerLog NewLog;
						ProcessedAssets.Add(Dependency);
						CompiledStateTreeCount += CompileWithQueueImpl_AnyThread(Dependency, NewLog, ProcessedAssets);
					}
				}
			}
		}

		CompileStateTree_AnyThread(StateTree, Log);
		++CompiledStateTreeCount;
	}
	return CompiledStateTreeCount;
}

void FCompilerManagerImpl::CompileStateTree_AnyThread(TNotNull<UStateTree*> StateTree, FStateTreeCompilerLog& Log)
{
	FStateTreeCompiler Compiler(Log);
	const bool bCompilationResult = Compiler.Compile(StateTree);

	const uint32 NewHash = UStateTreeEditingSubsystem::CalculateStateTreeHash(StateTree);
	checkf(StateTree->LastCompiledEditorDataHash == 0, TEXT("The compilation has to reset the statetree data. Including the hash value."));

	StateTree->bCompilationPending = false;

	if (bCompilationResult)
	{
		StateTree->LastCompiledEditorDataHash = NewHash;
		UE_LOGF(LogStateTreeEditor, Log, "Compile StateTree '%ls' succeeded.", *StateTree->GetFullName());
	}
	else
	{
		UE_LOGF(LogStateTreeEditor, Error, "Failed to compile '%ls', errors follow.", *StateTree->GetFullName());
		Log.DumpToLog(StateTree, LogStateTreeEditor);
	}

	DirtyStateTrees_AnyThread.Remove(TObjectKey<UStateTree>(StateTree));
	if (IsInGameThread())
	{
		UE::StateTree::Delegates::OnPostCompile.Broadcast(*StateTree);
	}
	else
	{
		ExecuteOnGameThread(UE_SOURCE_LOCATION, [WeakStateTree = TWeakObjectPtr<UStateTree>(StateTree)]()
			{
				if (UStateTree* StateTree = WeakStateTree.Get())
				{
					UE::StateTree::Delegates::OnPostCompile.Broadcast(*StateTree);
				}
			});
	}

	GatherDependencies(StateTree);

	if (CVarLogStateTreeDependencies->GetBool())
	{
		LogDependencies(StateTree);
	}
}

void FCompilerManagerImpl::UpdateEditorBindingsIfNeeded(TSet<const UStruct*>& InChangedStructs)
{
	for (const UStruct* ChangedStruct : InChangedStructs)
	{
		if (const TSet<TObjectKey<const UStateTree>>* Trees = EditorBindingExternalDependencies.DependencyToTreeMap.Find(ChangedStruct))
		{
			for (TObjectKey<const UStateTree> Tree : *Trees)
			{
				if (const UStateTree* TreePtr = Tree.ResolveObjectPtr())
				{
					UStateTreeEditorData* EditorDataPtr = Cast<UStateTreeEditorData>(TreePtr->EditorData);
					check(EditorDataPtr);

					FStateTreeEditorPropertyBindings* EditorBindings = EditorDataPtr->GetPropertyEditorBindings();
					check(EditorBindings);

					TMap<FGuid, const FStateTreeDataView> AllValues;
					EditorDataPtr->GetAllStructValues(AllValues);

					bool bAnyBindingChanged = false;

					TArray<FPropertyBindingPathIndirection, TInlineAllocator<16>> Indirections;
					auto UpdateBindingPathIfNeeded = [&AllValues, &bAnyBindingChanged, &Indirections, ChangedStruct](FPropertyBindingPath& InBindingPath)
						{
							if (const FStateTreeDataView* StructValue = AllValues.Find(InBindingPath.GetStructID()))
							{
								constexpr FString* Error = nullptr;
								constexpr bool bHandleRedirects = true;

								// Will reset the last Indirections
								if (InBindingPath.ResolveIndirectionsWithValue(*StructValue, Indirections, Error, bHandleRedirects))
								{
									if (FPropertyBindingPath::IndirectionsContainAnyStruct({ ChangedStruct }, Indirections))
									{
										const FPropertyBindingPath BindingPathBeforeUpdate = InBindingPath;
										InBindingPath.UpdateSegmentsFromIndirections(Indirections);

										bAnyBindingChanged |= (InBindingPath != BindingPathBeforeUpdate);
									}
								}
							}
						};

					EditorBindings->ForEachMutableBinding([&UpdateBindingPathIfNeeded](FPropertyBindingBinding& Binding)
						{
							UpdateBindingPathIfNeeded(Binding.GetMutableSourcePath());
						});

					const UPackage* Package = EditorDataPtr->GetPackage();
					check(Package);

					// During object re-instancing, blueprint intentionally un-dirtied packages that were not dirty before the process but got dirtied during the process. This is to prevent dirtying assets on load.
					// During load, bindings either will fail to resolve indirections, or are already up to date. So no package dirtying will happen during load.
					// Outside loading phase, we need to defer dirtying the package to next frame to work around the blueprint mechanism.
					// Object re-instancing is not transactable. We call this function when a UDS change is undone/redone. It is expected that a dirty package remains dirty after a UDS change is undone.
					if (!Package->IsDirty() && bAnyBindingChanged)
					{
						ExecuteOnGameThread(UE_SOURCE_LOCATION, [WeakEditorData = TWeakObjectPtr<UStateTreeEditorData>(EditorDataPtr)]()
						{
							if (UStateTreeEditorData* EditorDataPtr = WeakEditorData.Get())
							{
								EditorDataPtr->MarkPackageDirty();
							}
						});
					}
				}
			}
		}
	}
}

void FCompilerManagerImpl::HandleStateTreeCooking(TNotNull<UStateTree*> StateTree)
{
	if (StateTree->IsEditorDataDirty())
	{
		switch (StateTree->CompileStatus)
		{
		case UStateTree::ECompileStatus::Public:
		case UStateTree::ECompileStatus::Internal:
		case UStateTree::ECompileStatus::Executable:
		{
			FStateTreeCompilerLog Log;
			CompileInternalSynchronously(StateTree, Log);
			break;
		}
		case UStateTree::ECompileStatus::Link:
			std::ignore = StateTree->Link();
			break;
		}
	}
	else
	{
		// Data that is not dirty and requests a compilation should not need a new compilation.
		//The last Link failed.
		//Since we are cooking, compile again and report the errors for the build machine to report the errors.
		if (StateTree->CompileStatus == UStateTree::ECompileStatus::Link)
		{

			std::ignore = StateTree->Link();
		}
		else if (StateTree->CompileStatus != UStateTree::ECompileStatus::Executable)
		{
			FStateTreeCompilerLog Log;
			CompileInternalSynchronously(StateTree, Log);
		}
	}

	if (StateTree->CompileStatus != UStateTree::ECompileStatus::Executable)
	{
		UE_LOGF(LogStateTree, Error, "StateTree asset '%ls' failed compilation.", *StateTree->GetPathName());
	}
}

void FCompilerManagerImpl::MarkAsModified(TNotNull<UStateTree*> InStateTree)
{
	check(IsInGameThread());

	UE::TScopeLock _(QueueLock);
	DirtyStateTrees_AnyThread.Add(TObjectKey<UStateTree>(InStateTree));
}

void FCompilerManagerImpl::HandlePreBeginPIE(const bool bIsSimulating)
{
	check(IsInGameThread());

	if (Private::bEnableCompileAllIfChangedOnBeginPIE)
	{
		TArray<TNotNull<UStateTree*>> DirtyStateTrees;
		{
			UE::TScopeLock _(QueueLock);
			if (Private::bEnableCompileAllIfChangedOnBeginPIE_UseCachedList)
			{
				DirtyStateTrees.Reserve(DirtyStateTrees_AnyThread.Num());
				for (const TObjectKey<UStateTree>& StateTreeKey : DirtyStateTrees_AnyThread)
				{
					if (UStateTree* StateTree = StateTreeKey.ResolveObjectPtr())
					{
						check(!StateTree->HasAnyFlags(RF_ClassDefaultObject));
						DirtyStateTrees.Add(StateTree);
					}
				}
			}
			DirtyStateTrees_AnyThread.Reset();
		}

		if (!Private::bEnableCompileAllIfChangedOnBeginPIE_UseCachedList)
		{
			for (TObjectIterator<UStateTree> It; It; ++It)
			{
				check(!It->HasAnyFlags(RF_ClassDefaultObject));
				if (It->IsEditorDataDirty())
				{
					DirtyStateTrees.Add(*It);
				}
			}
		}

		// Add all dirty StateTrees to the compilation queue. It will respect the compilation order base on the dependencies between StateTree assets.
		for (TNotNull<UStateTree*> StateTree : DirtyStateTrees)
		{
			QueueForCompilation(StateTree);
		}
	}

	bCanQueueCompilation = !Private::bEnableForceCompileSynchronouslyInPIESession;
	FlushCompilationQueue();
}

void FCompilerManagerImpl::HandleEndPIE(const bool bIsSimulating)
{
	bCanQueueCompilation = Private::bEnableForceCompileSynchronouslyInPIESession;
}

void FCompilerManagerImpl::HandleObjectsReinstanced(const FReplacementObjectMap& ObjectMap)
{
	// When a Blueprint used by a StateTree is recompiled, the object is reinstanced. All instances of Blueprint, including those in the compiled StateTree asset.
	// This callback only tells us that some changed and not what changed.
	// There are different scenarios to handle:
	//
	// 1. Blueprint Graph changed: If the StateTree is not running, then there is nothing to do.
	//    If there is a running StateTree instance that uses the Blueprint, the runtime behavior is undefined depending on
	//    what was modified but usually fine. This will be fixed the next time the StateTree starts.
	//
	// 2. Property list changed: The StateTree asset may have a binding to a property that doesn't exist anymore.
	//    Link the asset. The bindings won't be able to link, and it will fail.
	// 
	// 3. Property list changed: The StateTree asset may have a binding to a property that got renamed.
	//    Link the asset. The binding system will attempt to find the new property by its ID; otherwise, it will fail.
	//
	// 4. Memory layout changed: The property memory locations have changed.
	//    Link the asset to get the latest memory locations.
	// 
	// @TODO
	// 5. Property usage changed: A property used to be an Output and is now an Input (or any possible combination).
	//    There is currently no way to detect that. The StateTree will have undefined behavior:
	//    The binding might not behave as expected. It will be fixed the next time the asset is loaded/compiled.
	//    Link the asset to get the latest memory locations.
	// 
	// @TODO
	// 6. Special flags changed: Some Blueprint flags are checked during Link(), and some are checked on node PostLoad()
	//    or Compile() overrides. If a special flag changed on a StateTree node or a Tick event was added to a
	//    StateTreeBlueprintTask, there is currently no way to detect that. The StateTree will have undefined behavior:
	//    it may not tick or may tick incorrectly. This will be fixed the next time the asset is loaded/compiled.

	if (ObjectMap.IsEmpty())
	{
		return;
	}

	TArray<const UObject*, TInlineAllocator<8>> ObjectsToBeReplaced;
	ObjectsToBeReplaced.Reserve(ObjectMap.Num());
	for (TMap<UObject*, UObject*>::TConstIterator It(ObjectMap); It; ++It)
	{
		if (const UObject* ObjectToBeReplaced = It->Value)
		{
			ObjectsToBeReplaced.Add(ObjectToBeReplaced);
		}
	}

	TSet<const UStruct*> StructsToBeReplaced;
	StructsToBeReplaced.Reserve(ObjectsToBeReplaced.Num());
	for (const UObject* ObjectToBeReplaced : ObjectsToBeReplaced)
	{
		// It's a UClass or a UScriptStruct
		if (const UStruct* StructToBeReplaced = Cast<const UStruct>(ObjectToBeReplaced))
		{
			StructsToBeReplaced.Add(StructToBeReplaced);
		}
		else
		{
			StructsToBeReplaced.Add(ObjectToBeReplaced->GetClass());
		}
	}

	UpdateEditorBindingsIfNeeded(StructsToBeReplaced);

	if (bUseDependenciesToTriggerCompilation)
	{
		TArray<TNotNull<UStateTree*>, TInlineAllocator<8>> StateTreeToLink;
		TArray<FObjectKey, TInlineAllocator<8>> StructsToBeReplacedAsKey;
		StructsToBeReplacedAsKey.Reserve(StructsToBeReplaced.Num());

		for (const UStruct* StructToBeReplaced : StructsToBeReplaced)
		{
			const FObjectKey StructToReplacedKey = StructToBeReplaced;
			StructsToBeReplacedAsKey.Add(StructToReplacedKey);

			TArray<TObjectKey<UStateTree>>* Dependencies = DependenciesToStateTree.Find(StructToReplacedKey);
			if (Dependencies)
			{
				for (const TObjectKey<UStateTree>& StateTreeKey : *Dependencies)
				{
					if (UStateTree* StateTree = StateTreeKey.ResolveObjectPtr())
					{
						StateTreeToLink.AddUnique(StateTree);
					}
				}
			}
		}

		for (UStateTree* StateTree : StateTreeToLink)
		{
			if (!StateTree->Link())
			{
				UE_LOGF(LogStateTree, Error, "%ls failed to link after Object reinstantiation. Take a look at the asset for any errors. Asset will not be usable at runtime.", *StateTree->GetPathName());
			}
		}
	}
	else
	{
		for (TObjectIterator<UStateTree> It; It; ++It)
		{
			UStateTree* StateTree = *It;
			check(!StateTree->HasAnyFlags(RF_ClassDefaultObject));

			// If the asset is not linked yet (or has failed), no need to link.
			if (StateTree->CompileStatus != UStateTree::ECompileStatus::Executable
				&& StateTree->CompileStatus != UStateTree::ECompileStatus::Link)
			{
				continue;
			}

			bool bShouldRelink = false;

			// Relink if one of the out of date objects got reinstanced.
			if (StateTree->OutOfDateStructs.Num() > 0)
			{
				for (const FObjectKey& OutOfDateObjectKey : StateTree->OutOfDateStructs)
				{
					if (const UObject* OutOfDateObject = OutOfDateObjectKey.ResolveObjectPtr())
					{
						if (ObjectMap.Contains(OutOfDateObject))
						{
							bShouldRelink = true;
							break;
						}
					}
				}
			}

			// Relink only if the reinstantiated object belongs to this asset,
			// or anything from the property binding refers to the classes of the reinstantiated object.
			if (!bShouldRelink)
			{
				for (const UObject* ObjectToBeReplaced : ObjectsToBeReplaced)
				{
					if (ObjectToBeReplaced->IsInOuter(StateTree))
					{
						bShouldRelink = true;
						break;
					}
				}
			}

			if (!bShouldRelink)
			{
				bShouldRelink |= StateTree->PropertyBindings.ContainsAnyStruct(StructsToBeReplaced);
			}

			if (bShouldRelink)
			{
				if (!StateTree->Link())
				{
					UE_LOGF(LogStateTree, Error, "%ls failed to link after Object reinstantiation. Take a look at the asset for any errors. Asset will not be usable at runtime.", *StateTree->GetPathName());
				}
			}
		}
	}
}

void FCompilerManagerImpl::HandleUserDefinedStructReinstanced(const UUserDefinedStruct& UserDefinedStruct)
{
	// UserDefinedStructs don't significantly affect StateTree compilation semantics. We only need to relink
	// if the memory layout changed to update binding memory locations.
	TSet<const UStruct*> Structs;
	Structs.Add(&UserDefinedStruct);

	UpdateEditorBindingsIfNeeded(Structs);

	if (bUseDependenciesToTriggerCompilation)
	{
		TSet<TNotNull<UStateTree*>> StateTreeToLink;
		const FObjectKey StructToReplacedKey = &UserDefinedStruct;
		TArray<TObjectKey<UStateTree>>* Dependencies = DependenciesToStateTree.Find(StructToReplacedKey);
		if (Dependencies)
		{
			for (const TObjectKey<UStateTree>& StateTreeKey : *Dependencies)
			{
				if (UStateTree* StateTree = StateTreeKey.ResolveObjectPtr())
				{
					StateTreeToLink.Add(StateTree);
				}
			}
		}

		for (UStateTree* StateTree : StateTreeToLink)
		{
			if (!StateTree->Link())
			{
				UE_LOGF(LogStateTree, Error, "%ls failed to link after Object reinstantiation. Take a look at the asset for any errors. Asset will not be usable at runtime.", *StateTree->GetPathName());
			}
		}
	}
	else
	{
		for (TObjectIterator<UStateTree> It; It; ++It)
		{
			UStateTree* StateTree = *It;
			if (StateTree->PropertyBindings.ContainsAnyStruct(Structs))
			{
				if (!StateTree->Link())
				{
					UE_LOGF(LogStateTree, Error, "%ls failed to link after Struct reinstantiation. Take a look at the asset for any errors. Asset will not be usable at runtime.", *StateTree->GetPathName());
				}
			}
		}
	}
}

void FCompilerManagerImpl::PurgeEditorBindingExternalDependencies()
{
	auto PurgeImpl = []<typename MapType>(MapType& InMap)
		requires std::is_same_v<MapType, FEditorBindingDependencies::FDependencyToTreeMap> || std::is_same_v<MapType, FEditorBindingDependencies::FTreeToDependencyMap>
	{
		for (auto MapIt = InMap.CreateIterator(); MapIt; ++MapIt)
		{
			if (!MapIt->Key.ResolveObjectPtr())
			{
				MapIt.RemoveCurrent();
			}
			else
			{
				for (auto SetIt = MapIt->Value.CreateIterator(); SetIt; ++SetIt)
				{
					if (!SetIt->ResolveObjectPtr())
					{
						SetIt.RemoveCurrent();
					}
				}
			}
		}
	};

	PurgeImpl(EditorBindingExternalDependencies.TreeToDependencyMap);
	PurgeImpl(EditorBindingExternalDependencies.DependencyToTreeMap);
}

void FCompilerManagerImpl::GatherDependencies(TNotNull<UStateTree*> StateTree)
{
	// Find the tree in the StateTreeToDependencies
	const TObjectKey<UStateTree> StateTreeKey = StateTree;
	TSharedPtr<FStateTreeDependencies>& FoundDependencies = StateTreeToDependencies.FindOrAdd(StateTreeKey);
	
	// Remove all from DependenciesToStateTree
	if (FoundDependencies)
	{
		for (FStateTreeDependencies::FItem& Item : FoundDependencies->Dependencies)
		{
			TArray<TObjectKey<UStateTree>>* FoundKey = DependenciesToStateTree.Find(Item.Key);
			if (FoundKey)
			{
				FoundKey->RemoveSingleSwap(StateTreeKey);
			}
		}
		FoundDependencies->Dependencies.Reset();
	}
	else
	{
		FoundDependencies = MakeShared<FStateTreeDependencies>();
	}

	auto AddDependency = [this, StateTreeKey, FoundDependencies](TNotNull<const UStruct*> Object, FStateTreeDependencies::EDependencyType DependencyType)
		{
			const FObjectKey ObjectKey = Object;
			DependenciesToStateTree.FindOrAdd(ObjectKey).AddUnique(StateTreeKey);

			if (FStateTreeDependencies::FItem* FoundItem = FoundDependencies->Dependencies.FindByPredicate([ObjectKey](const FStateTreeDependencies::FItem& Other)
				{
					return Other.Key == ObjectKey;
				}))
			{
				FoundItem->Type |= DependencyType;
			}
			else
			{
				FoundDependencies->Dependencies.Add(FStateTreeDependencies::FItem{.Key = ObjectKey, .Type = DependencyType});
			}
		};

	// Gather new inner dependencies
	if (UStateTreeEditorData* EditorData = Cast<UStateTreeEditorData>(StateTree->EditorData))
	{
		// Internal
		{
			FArchiveReferencingProperties DependencyArchive(StateTree);
			EditorData->Serialize(DependencyArchive);
			for (const UStruct* Dependency : DependencyArchive.Dependencies)
			{
				AddDependency(Dependency, FStateTreeDependencies::EDependencyType::DT_Internal);
			}
		}
		// Public
		{
			//The root parameters depend on the editor data type. Serialize it independently in case it is not discoverable by the editor data.
			//It has public dependency because other state tree asset or state tree reference can depends on it.
			FArchiveReferencingProperties DependencyArchive(StateTree);
			const_cast<FInstancedPropertyBag&>(EditorData->GetRootParametersPropertyBag()).Serialize(DependencyArchive);
			for (const UStruct* Dependency : DependencyArchive.Dependencies)
			{
				AddDependency(Dependency, FStateTreeDependencies::EDependencyType::DT_Public);
			}
			if (EditorData->Schema)
			{
				AddDependency(EditorData->Schema->GetClass(), FStateTreeDependencies::EDependencyType::DT_Public);
			}
		}
		// Link
		{
			TMap<FGuid, const FPropertyBindingDataView> AllStructValues;
			EditorData->GetAllStructValues(AllStructValues);
			auto AddBindingPathDependencies = [&AddDependency, &AllStructValues](const FPropertyBindingPath& PropertyPath)
				{
					const FPropertyBindingDataView* FoundStruct = AllStructValues.Find(PropertyPath.GetStructID());
					if (FoundStruct)
					{
						FString Error;
						TArray<FPropertyBindingPathIndirection> Indirections;
						if (PropertyPath.ResolveIndirectionsWithValue(*FoundStruct, Indirections, &Error))
						{
							for (const FPropertyBindingPathIndirection& Indirection : Indirections)
							{
								if (Indirection.GetInstanceStruct())
								{
									AddDependency(Indirection.GetInstanceStruct(), FStateTreeDependencies::EDependencyType::DT_Link);
								}

								if (Indirection.GetContainerStruct())
								{
									AddDependency(Indirection.GetContainerStruct(), FStateTreeDependencies::EDependencyType::DT_Link);
								}
							}
						}
					}
				};

			EditorData->GetEditorPropertyBindings()->VisitBindings([&AddBindingPathDependencies](const FPropertyBindingBinding& Binding)
				{
					AddBindingPathDependencies(Binding.GetSourcePath());
					AddBindingPathDependencies(Binding.GetTargetPath());
					return FPropertyBindingBindingCollection::EVisitResult::Continue;
				});
		}
	}
}

TSet<UStateTree*> FCompilerManagerImpl::GatherStateTreeDependencies_AnyThread(TNotNull<UStateTree*> StateTree) const
{
	FArchiveReferencingStateTree DependencyArchive(StateTree);
	UStateTreeEditorData* EditorData = Cast<UStateTreeEditorData>(StateTree->EditorData);
	if (EditorData)
	{
		EditorData->Serialize(DependencyArchive);
		//The root parameters depend on the editor data type. Serialize it independently in case it is not discoverable by the editor data.
		const_cast<FInstancedPropertyBag&>(EditorData->GetRootParametersPropertyBag()).Serialize(DependencyArchive);
	}
	return DependencyArchive.Dependencies;
}

void FCompilerManagerImpl::LogDependencies(TNotNull<UStateTree*> StateTree) const
{
	TStringBuilder<0> LogString;
	LogString << TEXT("StateTree Dependencies (asset: '");
	StateTree->GetFullName(LogString);
	LogString << TEXT("')\n");
	
	const TObjectKey<UStateTree> StateTreeKey = StateTree;
	const TSharedPtr<FStateTreeDependencies>* FoundDependencies = StateTreeToDependencies.Find(StateTreeKey);
	if (FoundDependencies != nullptr && FoundDependencies->IsValid())
	{
		auto PrintType = [&LogString](FStateTreeDependencies::EDependencyType Type)
			{
				bool bPrinted = false;
				auto PrintSeparator = [&bPrinted, &LogString]()
					{
						if (bPrinted)
						{
							LogString << TEXT(" | ");
						}
						bPrinted = true;
					};
				if (EnumHasAnyFlags(Type, FStateTreeDependencies::EDependencyType::DT_Public))
				{
					PrintSeparator();
					LogString << TEXT("Public");
				}
				if (EnumHasAnyFlags(Type, FStateTreeDependencies::EDependencyType::DT_Internal))
				{
					PrintSeparator();
					LogString << TEXT("Internal");
				}
				if (EnumHasAnyFlags(Type, FStateTreeDependencies::EDependencyType::DT_Link))
				{
					PrintSeparator();
					LogString << TEXT("Link");
				}
			};
		for (const FStateTreeDependencies::FItem& Item : (*FoundDependencies)->Dependencies)
		{
			LogString << TEXT("  ");
			if (const UObject* Object = Item.Key.ResolveObjectPtr())
			{
				Object->GetFullName(LogString);
			}
			else
			{
				LogString << TEXT(" [None]");
			}

			LogString << TEXT(" [");
			PrintType(Item.Type);
			LogString << TEXT("]\n");
		}
	}
	else
	{
		LogString << TEXT("  No Dependency");
	}

	UE_LOGF(LogStateTreeEditor, Log, "%ls", *LogString);
}


} // namespace UE::StateTree::Compiler::Private


namespace UE::StateTree::Compiler
{

void FCompilerManager::Startup()
{
	Private::CompilerManagerImpl = MakeUnique<Private::FCompilerManagerImpl>();
}

void FCompilerManager::Shutdown()
{
	Private::CompilerManagerImpl.Reset();
}

void FCompilerManager::QueueForCompilation(TNotNull<UStateTree*> StateTree)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCompilerManager::QueueForCompilation)
	if (ensureMsgf(Private::CompilerManagerImpl.IsValid(), TEXT("Can't queue the asset when the module is not available.")))
	{
		Private::CompilerManagerImpl->QueueForCompilation(StateTree);
	}
}

void FCompilerManager::FlushCompilationQueue()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCompilerManager::FlushCompilationQueue)
	if (Private::CompilerManagerImpl.IsValid())
	{
		Private::CompilerManagerImpl->FlushCompilationQueue();
	}
}

bool FCompilerManager::CompileSynchronously(TNotNull<UStateTree*> StateTree)
{
	FStateTreeCompilerLog Log;
	return CompileSynchronously(StateTree, Log);
}

bool FCompilerManager::CompileSynchronously(TNotNull<UStateTree*> StateTree, FStateTreeCompilerLog& Log)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCompilerManager::CompileSynchronously)
	if (ensureMsgf(Private::CompilerManagerImpl.IsValid(), TEXT("Can't compile the asset when the module is not available.")))
	{
		return Private::CompilerManagerImpl->CompileInternalSynchronously(StateTree, Log);
	}
	return false;
}


TOptional<bool> FCompilerManager::CompileIfNeededSynchronously(TNotNull<UStateTree*> StateTree)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCompilerManager::CompileIfNeededSynchronously)
	if (ensureMsgf(Private::CompilerManagerImpl.IsValid(), TEXT("Can't compile the asset when the module is not available.")))
	{
		return Private::CompilerManagerImpl->CompileIfNeededSynchronously(StateTree);
	}
	return {};
}

void FCompilerManager::CacheEditorBindingExternalDependencies(TNotNull<UStateTreeEditorData*> EditorData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCompilerManager::CacheEditorBindingExternalDependencies)
	if (ensureMsgf(Private::CompilerManagerImpl.IsValid(), TEXT("Can't compile the asset when the module is not available.")))
	{
		return Private::CompilerManagerImpl->CacheEditorBindingExternalDependencies(EditorData);
	}
}

void FCompilerManager::MarkAsModified(TNotNull<UStateTree*> StateTree)
{
	if (Private::CompilerManagerImpl.IsValid())
	{
		Private::CompilerManagerImpl->MarkAsModified(StateTree);
	}
}
} // UE::StateTree::Compiler

