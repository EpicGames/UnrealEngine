// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeDescriptionHelpers.h"

#include "StateTreeEditorData.h"
#include "StateTreeEditorStyle.h"
#include "StateTreeState.h"
#include "StateTreeTypes.h"
#include "StateTreeNodeBase.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"

namespace UE::StateTree::Editor
{

FText GetStateLinkDesc(const UStateTreeEditorData* EditorData, const FStateTreeStateLink& Link, EStateTreeNodeFormatting Formatting, bool bShowStatePath)
{
	if (!EditorData)
	{
		return FText::GetEmpty();
	}

	if (Link.LinkType == EStateTreeTransitionType::None)
	{
		return Formatting == EStateTreeNodeFormatting::RichText
			? LOCTEXT("TransitionNoneRich", "<i>None</>")
			: LOCTEXT("TransitionNone", "None");
	}
	if (Link.LinkType == EStateTreeTransitionType::Parent)
	{
		return Formatting == EStateTreeNodeFormatting::RichText
			? LOCTEXT("TransitionParentRich", "<i>Parent</>")
			: LOCTEXT("TransitionParent", "Parent");
	}
	if (Link.LinkType == EStateTreeTransitionType::NextState)
	{
		return Formatting == EStateTreeNodeFormatting::RichText
			? LOCTEXT("TransitionNextStateRich", "<i>Next State</>")
			: LOCTEXT("TransitionNextState", "Next State");
	}
	if (Link.LinkType == EStateTreeTransitionType::NextSelectableState)
	{
		return Formatting == EStateTreeNodeFormatting::RichText
			? LOCTEXT("TransitionNextSelectableStateRich", "<i>Next Selectable State</>")
			: LOCTEXT("TransitionNextSelectableState", "Next Selectable State");
	}
	if (Link.LinkType == EStateTreeTransitionType::NextParent)
	{
		return Formatting == EStateTreeNodeFormatting::RichText
			? LOCTEXT("TransitionNextParentRich", "<i>Next Parent</>")
			: LOCTEXT("TransitionNextParent", "Next Parent");
	}
	if (Link.LinkType == EStateTreeTransitionType::NextSelectableParent)
	{
		return Formatting == EStateTreeNodeFormatting::RichText
			? LOCTEXT("TransitionNextSelectableParentRich", "<i>Next Selectable Parent</>")
			: LOCTEXT("TransitionNextSelectableParent", "Next Selectable Parent");
	}
	if (Link.LinkType == EStateTreeTransitionType::Succeeded)
	{
		return Formatting == EStateTreeNodeFormatting::RichText
			? LOCTEXT("TransitionTreeSucceededRich", "<i>Tree Succeeded</>")
			: LOCTEXT("TransitionTreeSucceeded", "Tree Succeeded");
	}
	if (Link.LinkType == EStateTreeTransitionType::Failed)
	{
		return Formatting == EStateTreeNodeFormatting::RichText
			? LOCTEXT("TransitionTreeFailedRich", "<i>Tree Failed</>")
			: LOCTEXT("TransitionTreeFailed", "Tree Failed");
	}
	if (Link.LinkType == EStateTreeTransitionType::GotoState)
	{
		if (const UStateTreeState* State = EditorData->GetStateByID(Link.ID))
		{
			if (bShowStatePath)
			{
				TArray<FText> Path;
				while (State)
				{
					Path.Insert(FText::FromName(State->Name), 0);
					State = State->Parent;
				}
				return FText::Join(FText::FromString(TEXT("/")), Path);
			}
			else
			{
				return FText::FromName(State->Name);
			}
		}
		return FText::FromName(Link.Name);
	}

	return LOCTEXT("TransitionInvalid", "Invalid");
}

const FSlateBrush* GetStateLinkIcon(const UStateTreeEditorData* EditorData, const FStateTreeStateLink& Link)
{
	if (!EditorData)
	{
		return nullptr;
	}

	if (Link.LinkType == EStateTreeTransitionType::None)
	{
		return FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Transition.None");
	}
	if (Link.LinkType == EStateTreeTransitionType::Parent)
	{
		return FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Transition.Parent");
	}
	if (Link.LinkType == EStateTreeTransitionType::NextState)
	{
		return FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Transition.Next");
	}
	if (Link.LinkType == EStateTreeTransitionType::NextSelectableState)
	{
		return FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Transition.Next");
	}
	if (Link.LinkType == EStateTreeTransitionType::NextParent)
	{
		return FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Transition.Parent");
	}
	if (Link.LinkType == EStateTreeTransitionType::NextSelectableParent)
	{
		return FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Transition.Parent");
	}
	if (Link.LinkType == EStateTreeTransitionType::Succeeded)
	{
		return FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Transition.Succeeded");
	}
	if (Link.LinkType == EStateTreeTransitionType::Failed)
	{
		return FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.Transition.Failed");
	}
	if (Link.LinkType == EStateTreeTransitionType::GotoState)
	{
		if (const UStateTreeState* State = EditorData->GetStateByID(Link.ID))
		{
			// Figure out icon.
			if (State->SelectionBehavior == EStateTreeStateSelectionBehavior::None)
			{
				return FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.SelectNone");
			}
			else if (State->SelectionBehavior == EStateTreeStateSelectionBehavior::TryEnterState)
			{
				return FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.TryEnterState");			
			}
			else if (State->SelectionBehavior == EStateTreeStateSelectionBehavior::TrySelectChildrenInOrder)
			{
				if (State->Children.IsEmpty()
					|| State->Type == EStateTreeStateType::Linked
					|| State->Type == EStateTreeStateType::LinkedAsset)
				{
					return FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.TryEnterState");			
				}
				else
				{
					return FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.TrySelectChildrenInOrder");
				}
			}
			else if (State->SelectionBehavior == EStateTreeStateSelectionBehavior::TryFollowTransitions)
			{
				return FStateTreeEditorStyle::Get().GetBrush("StateTreeEditor.TryFollowTransitions");
			}
		}
	}
	
	return nullptr;
}

FSlateColor GetStateLinkColor(const UStateTreeEditorData* EditorData, const FStateTreeStateLink& Link)
{
	if (!EditorData)
	{
		return FLinearColor::White;
	}

	if (Link.LinkType == EStateTreeTransitionType::GotoState)
	{
		if (const UStateTreeState* State = EditorData->GetStateByID(Link.ID))
		{
			FLinearColor Color = FColor(31, 151, 167);
			if (const FStateTreeEditorColor* FoundColor = EditorData->FindColor(State->ColorRef))
			{
				Color = FoundColor->Color;
			}
			return Color;
		}
		
		return FLinearColor(1.f, 1.f, 1.f, 0.25f);
	}
	return FLinearColor::White;
}

FText GetTransitionDesc(const UStateTreeEditorData* EditorData, const FStateTreeTransition& Transition, EStateTreeNodeFormatting Formatting, bool bShowStatePath)
{
	if (!EditorData)
	{
		return FText::GetEmpty();
	}
	
	FText TriggerText;
	if (Transition.Trigger == EStateTreeTransitionTrigger::OnStateCompleted)
	{
		TriggerText = Formatting == EStateTreeNodeFormatting::RichText
			? LOCTEXT("TransitionOnStateCompletedRich", "<b>On State Completed</>")
			: LOCTEXT("TransitionOnStateCompleted", "On State Completed");
	}
	else if (Transition.Trigger == EStateTreeTransitionTrigger::OnStateSucceeded)
	{
		TriggerText = Formatting == EStateTreeNodeFormatting::RichText
			? LOCTEXT("TransitionOnStateSucceededRich", "<b>On State Succeeded</b>")
			: LOCTEXT("TransitionOnStateSucceeded", "On State Succeeded");
	}
	else if (Transition.Trigger == EStateTreeTransitionTrigger::OnStateFailed)
	{
		TriggerText = Formatting == EStateTreeNodeFormatting::RichText
			? LOCTEXT("TransitionOnStateFailedRich", "<b>On State Failed</>")
			: LOCTEXT("TransitionOnStateFailed", "On State Failed");
	}
	else if (Transition.Trigger == EStateTreeTransitionTrigger::OnTick)
	{
		TriggerText = Formatting == EStateTreeNodeFormatting::RichText
			? LOCTEXT("TransitionOnTickRich", "<b>On Tick</b>")
			: LOCTEXT("TransitionOnTick", "On Tick");
	}
	else if (Transition.Trigger == EStateTreeTransitionTrigger::OnEvent)
	{
		TArray<FText> PayloadItems;
		
		if (Transition.RequiredEvent.IsValid())
		{
			if (Transition.RequiredEvent.Tag.IsValid())
			{
				const FText TagFormat = Formatting == EStateTreeNodeFormatting::RichText
					? LOCTEXT("TransitionEventTagRich", "<s>Tag:</> '{0}'")
					: LOCTEXT("TransitionEventTag", "Tag: '{0}'");
				PayloadItems.Add(FText::Format(TagFormat, FText::FromName(Transition.RequiredEvent.Tag.GetTagName())));
			}
			
			if (Transition.RequiredEvent.PayloadStruct)
			{
				const FText PayloadFormat = Formatting == EStateTreeNodeFormatting::RichText
					? LOCTEXT("TransitionEventPayloadRich", "<s>Payload:</> '{0}'")
					: LOCTEXT("TransitionEventPayload", "Payload: '{0}'");
				PayloadItems.Add(FText::Format(PayloadFormat, Transition.RequiredEvent.PayloadStruct->GetDisplayNameText()));
			}
		}
		else
		{
			PayloadItems.Add(LOCTEXT("TransitionInvalidEvent", "Invalid"));
		}

		const FText TransitionFormat = Formatting == EStateTreeNodeFormatting::RichText
			? LOCTEXT("TransitionOnEventRich", "<b>On Event</> ({0})")
			: LOCTEXT("TransitionOnEvent", "On Event ({0})");
		
		TriggerText = FText::Format(TransitionFormat, FText::Join(INVTEXT(", "), PayloadItems));
	}
	else if (Transition.Trigger == EStateTreeTransitionTrigger::OnDelegate)
	{
		FStateTreeBindingLookup BindingLookup(EditorData);

		const FText BoundDelegateText = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(Transition.ID, GET_MEMBER_NAME_CHECKED(FStateTreeTransition, DelegateListener)), Formatting);

		const FText TransitionFormat = Formatting == EStateTreeNodeFormatting::RichText
			? LOCTEXT("TransitionOnDelegateRich", "<b>On Delegate</> ({0})")
			: LOCTEXT("TransitionOnDelegate", "On Delegate ({0})");

		TriggerText = FText::Format(TransitionFormat, BoundDelegateText);
	}

	FText ActionText = Formatting == EStateTreeNodeFormatting::RichText
		? LOCTEXT("ActionGotoRichRich", "<s>go to</>")
		: LOCTEXT("ActionGoto", "go to");
	
	if (Transition.State.LinkType == EStateTreeTransitionType::Succeeded
		|| Transition.State.LinkType == EStateTreeTransitionType::Failed)
	{
		ActionText = Formatting == EStateTreeNodeFormatting::RichText
			? LOCTEXT("ActionReturnRich", "<s>return</>")
			: LOCTEXT("ActionReturn", "return");
	}
	
	FText ReactivateText = Formatting == EStateTreeNodeFormatting::RichText
		? LOCTEXT("ReselectionTextRich", "<s>(Re-Activate)</>")
		: LOCTEXT("ReselectionText", "(Re-Activate)");

	if (Transition.ReactivateTargetState != EStateTreeTransitionChangeTypeRules::ForceChanged)
	{
		ReactivateText = FText::GetEmpty();
	}

	return FText::Format(LOCTEXT("TransitionDesc", "{0} {1} {2} {3}"), TriggerText, ActionText, GetStateLinkDesc(EditorData, Transition.State, Formatting, bShowStatePath), ReactivateText);
}

} // UE::StateTree::Editor

#undef LOCTEXT_NAMESPACE
