// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class UStateTreeEditorData;
struct FStateTreeStateLink;
struct FStateTreeTransition;
enum class EStateTreeNodeFormatting : uint8;
struct FSlateBrush;
struct FSlateColor;
class FText;

namespace UE::StateTree::Editor
{

FText GetStateLinkDesc(const UStateTreeEditorData* EditorData, const FStateTreeStateLink& Link, EStateTreeNodeFormatting Formatting, bool bShowStatePath = false);
const FSlateBrush* GetStateLinkIcon(const UStateTreeEditorData* EditorData, const FStateTreeStateLink& Link);
FSlateColor GetStateLinkColor(const UStateTreeEditorData* EditorData, const FStateTreeStateLink& Link);

FText GetTransitionDesc(const UStateTreeEditorData* EditorData, const FStateTreeTransition& Transition, EStateTreeNodeFormatting Formatting, bool bShowStatePath = false);

} // UE::StateTree::Editor