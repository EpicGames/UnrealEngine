// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeDiffControl.h"
#include "SStateTreeView.h"
#include "StateTree.h"
#include "StateTreeEditorData.h"
#include "StateTreeViewModel.h"
#include "StateTreeDiffHelper.h"
#include "Framework/Commands/UICommandList.h"

#define LOCTEXT_NAMESPACE "SStateTreeDif"

namespace UE::StateTree::Diff
{

FText FDiffControl::RightRevision = LOCTEXT("OlderRevisionIdentifier", "Right Revision");

TSharedRef<SWidget> FDiffControl::GenerateSingleEntryWidget(FSingleDiffEntry DiffEntry, const FText ObjectName)
{
	return SNew(STextBlock)
		.Text(GetStateTreeDiffMessage(DiffEntry, ObjectName, true))
		.ToolTipText(GetStateTreeDiffMessage(DiffEntry, ObjectName))
		.ColorAndOpacity(GetStateTreeDiffMessageColor(DiffEntry));
}

void FDiffControl::OnSelectDiffEntry(const FSingleDiffEntry StateDiff)
{
	OnDiffEntryFocused.ExecuteIfBound();
	OnStateDiffEntryFocused.Broadcast(StateDiff);
}

void FDiffControl::GenerateTreeEntries(TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& OutDifferences)
{
	TArray<FSingleDiffEntry> DifferingProperties;

	for (int32 LeftIndex = 0; LeftIndex < DisplayedAssets.Num() - 1; ++LeftIndex)
	{
		const UStateTree* LeftStateTree = DisplayedAssets[LeftIndex].Get();
		if (!ensure(LeftStateTree))
		{
			continue;
		}
		const TSharedPtr<FAsyncDiff> Diff = StateTreeDifferences[LeftStateTree].Right;
		Diff->FlushQueue(); // make sure differences are fully up-to-date
		Diff->GetStateTreeDifferences(DifferingProperties);
	}

	BindingDiffs.Empty();

	TSet<FString> ExistingEntryPaths;

	for (const FSingleDiffEntry& Difference : DifferingProperties)
	{
		FString DiffPath = Difference.Identifier.ToDisplayName();
		bool bGenerateNewEntry = true;
		if (IsBindingDiff(Difference.DiffType))
		{
			BindingDiffs.Add(Difference);
			bGenerateNewEntry = !ExistingEntryPaths.Contains(DiffPath);
		}

		if (bGenerateNewEntry)
		{
			TSharedPtr<FBlueprintDifferenceTreeEntry> Entry = MakeShared<FBlueprintDifferenceTreeEntry>(
				FOnDiffEntryFocused::CreateSP(AsShared(), &FDiffControl::OnSelectDiffEntry, Difference),
				FGenerateDiffEntryWidget::CreateStatic(&GenerateSingleEntryWidget, Difference, RightRevision));
			OutDifferences.Push(Entry);
			ExistingEntryPaths.Add(Difference.Identifier.ToDisplayName());
		}
	}
}

FDiffControl::FDiffControl(
	const UStateTree* InOldObject,
	const UStateTree* InNewObject,
	const FOnDiffEntryFocused& InSelectionCallback)
	: OnDiffEntryFocused(InSelectionCallback)
{
	if (InOldObject)
	{
		InsertObject(InOldObject);
	}

	if (InNewObject)
	{
		InsertObject(InNewObject);
	}
}

FDiffControl::~FDiffControl()
{
}

TSharedRef<SStateTreeView> FDiffControl::InsertObject(TNotNull<const UStateTree*> StateTree)
{
	const FDiffWidgets DiffWidgets(StateTree);
	TSharedRef<SStateTreeView> TreeView = DiffWidgets.GetStateTreeWidget();

	const int32 Index = DisplayedAssets.Num();
	DisplayedAssets.Insert(TStrongObjectPtr<const UStateTree>(StateTree), Index);

	StateTreeDifferences.Add(StateTree, {});
	StateTreeDiffWidgets.Add(StateTree, DiffWidgets);

	// set up interaction with left panel
	if (DisplayedAssets.IsValidIndex(Index - 1))
	{
		const UStateTree* OtherStateTree = DisplayedAssets[Index - 1].Get();
		const FDiffWidgets& OtherStateTreeDiff = StateTreeDiffWidgets[OtherStateTree];
		const TSharedRef<SStateTreeView> OtherTreeView = OtherStateTreeDiff.GetStateTreeWidget();

		StateTreeDifferences[OtherStateTree].Right = MakeShared<FAsyncDiff>(OtherTreeView, TreeView);
		StateTreeDifferences[StateTree].Left = StateTreeDifferences[OtherStateTree].Right;
	}
	// Set up interaction with right panel
	if (DisplayedAssets.IsValidIndex(Index + 1))
	{
		const UStateTree* OtherStateTree = DisplayedAssets[Index + 1].Get();
		const FDiffWidgets& OtherStateTreeDiff = StateTreeDiffWidgets[OtherStateTree];
		const TSharedRef<SStateTreeView> OtherTreeView = OtherStateTreeDiff.GetStateTreeWidget();

		StateTreeDifferences[OtherStateTree].Left = MakeShared<FAsyncDiff>(TreeView, OtherTreeView);
		StateTreeDifferences[StateTree].Right = StateTreeDifferences[OtherStateTree].Left;
	}

	return TreeView;
}

TSharedRef<SStateTreeView> FDiffControl::GetDetailsWidget(const UStateTree* Object) const
{
	return StateTreeDiffWidgets[Object].GetStateTreeWidget();
}

FDiffWidgets::FDiffWidgets(const UStateTree* InStateTree)
{
	UStateTreeEditorData* EditorData = Cast<UStateTreeEditorData>(InStateTree->EditorData);
	StateTreeViewModel = MakeShareable(new FStateTreeViewModel());
	StateTreeViewModel->Init(EditorData);
	SAssignNew(StateTreeTreeView, SStateTreeView, StateTreeViewModel.ToSharedRef(), MakeShared<FUICommandList>());
}

TSharedRef<SStateTreeView> FDiffWidgets::GetStateTreeWidget() const
{
	return StateTreeTreeView.ToSharedRef();
}

} // UE::StateTree::Diff
#undef LOCTEXT_NAMESPACE
