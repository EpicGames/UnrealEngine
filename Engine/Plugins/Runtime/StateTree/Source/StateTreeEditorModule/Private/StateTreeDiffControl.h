// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AsyncStateTreeDiff.h"
#include "DiffUtils.h"
#include "DiffControl.h"
#include "Internationalization/Text.h"
#include "UObject/ObjectKey.h"
#include "UObject/StrongObjectPtr.h"

#define UE_API STATETREEEDITORMODULE_API

class FBlueprintDifferenceTreeEntry;
class FStateTreeViewModel;
class FUICommandList;
class SLinkableScrollBar;
class SStateTreeView;
class SWidget;
class UStateTree;

namespace UE::StateTree::Diff
{
struct FSingleDiffEntry;

class FDiffWidgets
{
public:
	UE_API explicit FDiffWidgets(const UStateTree* InStateTree);

	/** Returns actual widget that is used to display trees */
	UE_API TSharedRef<SStateTreeView> GetStateTreeWidget() const;

private:
	TSharedPtr<SStateTreeView> StateTreeTreeView;
	TSharedPtr<FStateTreeViewModel> StateTreeViewModel;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnStateDiffEntryFocused, const FSingleDiffEntry&)

class FDiffControl : public TSharedFromThis<FDiffControl>
{
public:
	FDiffControl() = delete;
	FDiffControl(const FDiffControl& Other) = delete;
	FDiffControl(const FDiffControl&& Other) = delete;
	UE_API FDiffControl(const UStateTree* InOldObject, const UStateTree* InNewObject, const FOnDiffEntryFocused& InSelectionCallback);
	UE_API ~FDiffControl();

	UE_API TSharedRef<SStateTreeView> GetDetailsWidget(const UStateTree* Object) const;

	FOnStateDiffEntryFocused& GetOnStateDiffEntryFocused()
	{
		return OnStateDiffEntryFocused;
	};

	UE_API void GenerateTreeEntries(TArray<TSharedPtr<FBlueprintDifferenceTreeEntry>>& OutDifferences);
	TConstArrayView<FSingleDiffEntry> GetBindingDifferences() const
	{
		return BindingDiffs;
	}

protected:
	static UE_API FText RightRevision;
	static UE_API TSharedRef<SWidget> GenerateSingleEntryWidget(FSingleDiffEntry DiffEntry, FText ObjectName);

	UE_API TSharedRef<SStateTreeView> InsertObject(TNotNull<const UStateTree*> StateTree);

	UE_API void OnSelectDiffEntry(const FSingleDiffEntry StateDiff);

	FOnDiffEntryFocused OnDiffEntryFocused;
	FOnStateDiffEntryFocused OnStateDiffEntryFocused;

	TArray<TStrongObjectPtr<const UStateTree>> DisplayedAssets;

	struct FStateTreeTreeDiffPairs
	{
		TSharedPtr<FAsyncDiff> Left;
		TSharedPtr<FAsyncDiff> Right;
	};
	TMap<FObjectKey, FStateTreeTreeDiffPairs> StateTreeDifferences;
	TArray<FSingleDiffEntry> BindingDiffs;
	TMap<FObjectKey, FDiffWidgets> StateTreeDiffWidgets;
};

} // UE::StateTree::Diff

#undef UE_API
