// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeDiffHelper.h"
#include "StateTreeEditorData.h"
#include "StateTreeEditorStyle.h"

#define LOCTEXT_NAMESPACE "StateTreeDiffUtils"

namespace UE::StateTree::Diff
{
bool IsBindingDiff(EStateDiffType DiffType)
{
	return DiffType == EStateDiffType::BindingAddedToA || DiffType == EStateDiffType::BindingAddedToB || DiffType == EStateDiffType::BindingChanged;
}

void FStateSoftPath::PostOrderChain(const UStateTreeState* CurBackwardsState)
{
	if (CurBackwardsState)
	{
		PostOrderChain(CurBackwardsState->Parent);
		StateChain.Push(FChainElement(CurBackwardsState));
	}
}

FStateSoftPath::FChainElement::FChainElement(const UStateTreeState* State)
{
	if (State)
	{
		StateName = State->Name;
		DisplayString = State->Name.ToString();
		ID = State->ID;
	}
}

FStateSoftPath::FStateSoftPath(TNotNull<const UStateTreeState*> InState)
{
	int32 ChainCount = 1;
	const UStateTreeState* CurState = InState;
	while (CurState->Parent)
	{
		ChainCount++;
		CurState = CurState->Parent;
	}
	StateChain.Reserve(ChainCount);
	PostOrderChain(InState);
}

FString FStateSoftPath::ToDisplayName(bool Short) const
{
	if (Short && StateChain.Num() > 0)
	{
		return StateChain.Last().DisplayString;
	}
	FString Ret;
	for (FChainElement Element : StateChain)
	{
		FString PropertyAsString = Element.DisplayString;
		if (Ret.IsEmpty())
		{
			Ret.Append(PropertyAsString);
		}
		else
		{
			Ret.AppendChar('.');
			Ret.Append(PropertyAsString);
		}
	}
	return Ret;
}

UStateTreeState* FStateSoftPath::ResolvePath(TNotNull<const UStateTree*> StateTree) const
{
	if (StateChain.Num() == 0)
	{
		return nullptr;
	}
	return StateTree->EditorData != nullptr ? ResolvePath(Cast<UStateTreeEditorData>(StateTree->EditorData)) : nullptr;

}

UStateTreeState* FStateSoftPath::ResolvePath(TNotNull<const UStateTreeEditorData*> EditorData) const
{
	if (StateChain.Num() == 0)
	{
		return nullptr;
	}

	UStateTreeState* CurState = nullptr;
	for (FChainElement Element : StateChain)
	{
		// First State in the chain should look for States in the SubTrees
		if (CurState == nullptr)
		{
			for (UStateTreeState* RootState : EditorData->SubTrees)
			{
				if (RootState->ID == Element.ID || RootState->Name == Element.StateName)
				{
					CurState = RootState;
					break;
				}
			}

			// Can't find root State, stop parsing the chain
			if (CurState == nullptr)
			{
				break;
			}
			continue;
		}

		bool bHasChild = false;
		for (UStateTreeState* ChildState : CurState->Children)
		{
			if (ChildState->ID == Element.ID || ChildState->Name == Element.StateName)
			{
				CurState = ChildState;
				bHasChild = true;
				break;
			}
		}

		// Can't find child State, stop parsing the chain
		if (!bHasChild)
		{
			CurState = nullptr;
			break;
		}
	}

	return CurState;
}

FText GetStateTreeDiffMessage(const FSingleDiffEntry& Difference, const FText ObjectName, const bool bShort)
{
	FText Message;
	const FString StateName = Difference.Identifier.ToDisplayName(bShort);

	switch (Difference.DiffType)
	{
	case EStateDiffType::StateAddedToA:
		Message = FText::Format(LOCTEXT("StateChange_Removed", "{0} removed from {1}"), FText::FromString(StateName), ObjectName);
		break;
	case EStateDiffType::StateAddedToB:
		Message = FText::Format(LOCTEXT("StateChange_Added", "{0} added to {1}"), FText::FromString(StateName), ObjectName);
		break;
	case EStateDiffType::StateChanged:
	case EStateDiffType::BindingChanged:
	case EStateDiffType::BindingAddedToB:
	case EStateDiffType::BindingAddedToA:
		Message = FText::Format(LOCTEXT("StateChange", "{0} changed values"), FText::FromString(StateName));
		break;
	case EStateDiffType::StateEnabled:
		Message = FText::Format(LOCTEXT("StateEnabled", "{0} enabled in {1}"), FText::FromString(StateName), ObjectName);
		break;
	case EStateDiffType::StateDisabled:
		Message = FText::Format(LOCTEXT("StateDisabled", "{0} disabled in {1}"), FText::FromString(StateName), ObjectName);
		break;
	case EStateDiffType::StateMoved:
		Message = FText::Format(LOCTEXT("StateMoved", "{0} moved"), FText::FromString(StateName));
		break;
	case EStateDiffType::StateTreePropertiesChanged:
		Message = FText::Format(LOCTEXT("StateTreeChange", "State Tree Properties changed value in {0}"), ObjectName);
		break;
	case EStateDiffType::Invalid:
	case EStateDiffType::Identical:
		break;
	}
	return Message;
}

FLinearColor GetStateTreeDiffMessageColor(const FSingleDiffEntry& Difference)
{
	const static FLinearColor ForegroundColor = FAppStyle::GetColor("Graph.ForegroundColor");
	switch (Difference.DiffType)
	{
	case EStateDiffType::StateAddedToA:
		return FStateTreeEditorStyle::Get().GetColor("DiffTools.Removed");
	case EStateDiffType::StateAddedToB:
		return FStateTreeEditorStyle::Get().GetColor("DiffTools.Added");
	case EStateDiffType::StateChanged:
	case EStateDiffType::BindingChanged:
	case EStateDiffType::BindingAddedToB:
	case EStateDiffType::BindingAddedToA:
		return FStateTreeEditorStyle::Get().GetColor("DiffTools.Changed");
	case EStateDiffType::StateMoved:
		return FStateTreeEditorStyle::Get().GetColor("DiffTools.Moved");
	case EStateDiffType::StateEnabled:
		return FStateTreeEditorStyle::Get().GetColor("DiffTools.Enabled");
	case EStateDiffType::StateDisabled:
	return FStateTreeEditorStyle::Get().GetColor("DiffTools.Disabled");
	case EStateDiffType::StateTreePropertiesChanged:
		return FStateTreeEditorStyle::Get().GetColor("DiffTools.Properties");
	default:
		return ForegroundColor;
	}
}

FText GetStateDiffMessage(const FSingleObjectDiffEntry& Difference, FText PropertyName)
{
	switch (Difference.DiffType)
	{
	case EPropertyDiffType::PropertyAddedToA:
		return FText::Format(LOCTEXT("StatePropertyChange_Removed", "{0} removed"), PropertyName);
	case EPropertyDiffType::PropertyAddedToB:
		return FText::Format(LOCTEXT("StatePropertyChange_Added", "{0} added"), PropertyName);
	case EPropertyDiffType::PropertyValueChanged:
		return FText::Format(LOCTEXT("StatePropertyChange", "{0} changed value"), PropertyName);
	default:
		return PropertyName;
	}
}

FLinearColor GetStateDiffMessageColor(const FSingleObjectDiffEntry& Difference)
{
	switch (Difference.DiffType)
	{
	case EPropertyDiffType::PropertyAddedToA:
		return FStateTreeEditorStyle::Get().GetColor("DiffTools.Removed");
	case EPropertyDiffType::PropertyAddedToB:
		return FStateTreeEditorStyle::Get().GetColor("DiffTools.Added");
	case EPropertyDiffType::PropertyValueChanged:
		return FStateTreeEditorStyle::Get().GetColor("DiffTools.Changed");
	default:
		return FStateTreeEditorStyle::Get().GetColor("DiffTools.Properties");
	}
}

} // namespace UE::StateTree::Diff
#undef LOCTEXT_NAMESPACE