// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DiffUtils.h"

class UStateTreeEditorData;
class UStateTree;
class UStateTreeState;

namespace UE::StateTree::Diff
{

enum class EStateDiffType : uint8
{
	Invalid,

	Identical,

	StateAddedToA,
	StateAddedToB,
	StateChanged,

	StateEnabled,
	StateDisabled,

	StateMoved,

	StateTreePropertiesChanged,

	BindingAddedToA,
	BindingAddedToB,
	BindingChanged,
};

struct FStateSoftPath
{
	FStateSoftPath() = default;
	explicit FStateSoftPath(TNotNull<const UStateTreeState*> InState);

	UStateTreeState* ResolvePath(TNotNull<const UStateTree*> StateTree) const;
	UStateTreeState* ResolvePath(TNotNull<const UStateTreeEditorData*> EditorData) const;
	FString ToDisplayName(bool Short = false) const;

	/**
	 * Indicates whether a given path is a base path of the current path.
	 * @param BaseStatePath The path to look for as a base common path
	 * @return Whether the path has the provided path as base path or not
	 */
	bool IsSubStateMatch(const FStateSoftPath& BaseStatePath) const
	{
		if (StateChain.Num() <= BaseStatePath.StateChain.Num())
		{
			return false;
		}

		for (int32 ElementIndex = 0; ElementIndex < BaseStatePath.StateChain.Num(); ElementIndex++)
		{
			if (BaseStatePath.StateChain[ElementIndex] != StateChain[ElementIndex])
			{
				return false;
			}
		}

		return true;
	}

	bool operator==(FStateSoftPath const& RHS) const
	{
		return StateChain == RHS.StateChain;
	}

	bool operator!=(FStateSoftPath const& RHS) const
	{
		return !(*this == RHS);
	}

	explicit operator bool() const
	{
		return StateChain.Num() > 0;
	}

private:
	void PostOrderChain(const UStateTreeState* CurBackwardsState);

	struct FChainElement
	{
		FName StateName;
		FString DisplayString;
		FGuid ID;

		FChainElement() = default;
		explicit FChainElement(const UStateTreeState* State);

		bool operator==(FChainElement const& RHS) const
		{
			return ID == RHS.ID || (StateName == RHS.StateName);
		}

		bool operator!=(FChainElement const& RHS) const
		{
			return !(*this == RHS);
		}
	};
	TArray<FChainElement> StateChain;
};

struct FSingleDiffEntry
{
	FSingleDiffEntry(const FStateSoftPath& InIdentifier, const FStateSoftPath& InSecondaryIdentifier, const EStateDiffType InDiffType, const FPropertySoftPath& InBindingPath)
		: Identifier(InIdentifier)
		, SecondaryIdentifier(InSecondaryIdentifier)
		, BindingPath(InBindingPath)
		, DiffType(InDiffType)
	{
	}

	FSingleDiffEntry(const FStateSoftPath& InIdentifier, const FStateSoftPath& InSecondaryIdentifier, const EStateDiffType InDiffType)
		: FSingleDiffEntry(InIdentifier, InSecondaryIdentifier, InDiffType, FPropertySoftPath())
	{
	}

	FSingleDiffEntry(const FStateSoftPath& InIdentifier, const EStateDiffType InDiffType)
		: FSingleDiffEntry(InIdentifier, FStateSoftPath(), InDiffType)
	{
	}

	FStateSoftPath Identifier;
	FStateSoftPath SecondaryIdentifier;
	FPropertySoftPath BindingPath;
	EStateDiffType DiffType = EStateDiffType::Invalid;
};

FText GetStateTreeDiffMessage(const FSingleDiffEntry& Difference, FText ObjectName, bool bShort = false);

FLinearColor GetStateTreeDiffMessageColor(const FSingleDiffEntry& Difference);

FText GetStateDiffMessage(const FSingleObjectDiffEntry& Difference, FText PropertyName);

FLinearColor GetStateDiffMessageColor(const FSingleObjectDiffEntry& Difference);

bool IsBindingDiff(EStateDiffType DiffType);

} // namespace UE::StateTree::Diff