// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeDragDrop.h"

#include "StateTreeState.h"
#include "StateTreeViewModel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FStateTreeSelectedDragDrop"

TSharedPtr<SWidget> FStateTreeSelectedDragDrop::GetDefaultDecorator() const
{
	// Display all dragged nodes, enable the ones that can be moved into the current target node. 
	const TSharedRef<SVerticalBox> Box = SNew(SVerticalBox);
	
	TWeakPtr<const FStateTreeSelectedDragDrop> Weak = SharedThis(this);
	auto AddTextToBox = [Weak, &Box](const FText& Text)
	{
		Box->AddSlot()
		.Padding(FMargin(4.0f, 2.0f))
		[
			SNew(STextBlock)
			.Text(Text)
			.IsEnabled_Lambda([Weak]()
			{
				TSharedPtr<const FStateTreeSelectedDragDrop> DragDrop = Weak.Pin();
				return DragDrop && DragDrop->bCanDrop;
			})
		];
	};

	if (IsDraggingNode())
	{
		const FStateTreeEditorNode* Node = OriginState->GetNodeByID(NodeID);
		FText NodeDescription;
		if (Node)
		{
			NodeDescription = ViewModel->GetNodeDescription(*Node, EStateTreeNodeFormatting::Text);
		}

		FFormatNamedArguments Args;
		Args.Add(TEXT("ActionName"), bIsControlKeyDown ? LOCTEXT("StateTreeDragDropCopyAction", "Copy") : LOCTEXT("StateTreeDragDropMoveAction", "Move"));
		Args.Add(TEXT("NodeDescription"), NodeDescription);
		FText ActionText = FText::Format(INVTEXT("{ActionName} {NodeDescription}"), Args);
		AddTextToBox(ActionText);
	}
	else if (ViewModel)
	{
		TArray<UStateTreeState*> SelectedStates;
		ViewModel->GetSelectedStates(SelectedStates);

		for (UStateTreeState* State : SelectedStates)
		{
			AddTextToBox(FText::FromName(State->Name));
		}
	}

	return SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		[
			Box
		];
}

#undef LOCTEXT_NAMESPACE