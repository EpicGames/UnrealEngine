// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragAndDrop/DecoratedDragDropOp.h"

struct FStateTreeEditorNode;
class UStateTreeState;
class FStateTreeViewModel;

class FStateTreeSelectedDragDrop : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FActionTreeViewDragDrop, FDecoratedDragDropOp);

	static TSharedRef<FStateTreeSelectedDragDrop> New(TSharedPtr<FStateTreeViewModel> InViewModel)
	{
		TSharedRef<FStateTreeSelectedDragDrop> Operation = MakeShared<FStateTreeSelectedDragDrop>();
		Operation->ViewModel = InViewModel;
		Operation->Construct();

		return Operation;
	}
	
	static TSharedRef<FStateTreeSelectedDragDrop> New(TSharedPtr<FStateTreeViewModel> InViewModel, FGuid NodeID, TWeakObjectPtr<UStateTreeState> OriginState, bool bIsControlKeyDown)
	{
		TSharedRef<FStateTreeSelectedDragDrop> Operation = MakeShared<FStateTreeSelectedDragDrop>();
		Operation->ViewModel = InViewModel;
		Operation->OriginState = OriginState;
		Operation->NodeID = NodeID;
		Operation->bIsControlKeyDown = bIsControlKeyDown;
		Operation->Construct();

		return Operation;
	}

	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;

	void SetCanDrop(const bool bState)
	{
		bCanDrop = bState;
	}
	
	bool IsDraggingNode() const
	{
		return ViewModel.IsValid() && OriginState.IsValid() && NodeID.IsValid();
	}

	TSharedPtr<FStateTreeViewModel> ViewModel;
	TWeakObjectPtr<UStateTreeState> OriginState;
	FGuid NodeID;
	bool bIsControlKeyDown = false;
	bool bCanDrop = false;
};