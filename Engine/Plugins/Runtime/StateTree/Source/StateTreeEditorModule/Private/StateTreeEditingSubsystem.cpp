// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeEditingSubsystem.h"

#include "SStateTreeView.h"
#include "StateTree.h"
#include "StateTreeCompiler.h"
#include "StateTreeCompilerLog.h"
#include "StateTreeCompilerManager.h"
#include "StateTreeDelegates.h"
#include "StateTreeEditorData.h"
#include "StateTreeEditorModule.h"
#include "StateTreeEditorSchema.h"
#include "StateTreeEditorDataExtension.h"
#include "StateTreeObjectHash.h"
#include "StateTreeTaskBase.h"
#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeEditingSubsystem)

namespace UE::StateTree::Editor
{
	bool bUseHashToDetermineIfAssetNeedsCompile = true;
	FAutoConsoleVariableRef CVarUseHashToDetermineIfAssetNeedsCompile(
		TEXT("StateTree.Compiler.UseHashToDetermineIfAssetNeedsCompile"),
		bUseHashToDetermineIfAssetNeedsCompile,
		TEXT("Use the hash to determine if the asset is dirty and requires compilation.\n")
	);
}

UStateTreeEditingSubsystem::UStateTreeEditingSubsystem()
{
	PostGarbageCollectHandle = FCoreUObjectDelegates::GetPostGarbageCollect().AddUObject(this, &UStateTreeEditingSubsystem::HandlePostGarbageCollect);
	PostCompileHandle = UE::StateTree::Delegates::OnPostCompile.AddUObject(this, &UStateTreeEditingSubsystem::HandlePostCompile);
}

void UStateTreeEditingSubsystem::BeginDestroy()
{
	UE::StateTree::Delegates::OnPostCompile.Remove(PostCompileHandle);
	FCoreUObjectDelegates::GetPostGarbageCollect().Remove(PostGarbageCollectHandle);
	Super::BeginDestroy();
}

bool UStateTreeEditingSubsystem::CompileStateTree(TNotNull<UStateTree*> InStateTree, FStateTreeCompilerLog& InOutLog)
{
	return UE::StateTree::Compiler::FCompilerManager::CompileSynchronously(InStateTree, InOutLog);
}

bool UStateTreeEditingSubsystem::NeedsRecompile(TNotNull<const UStateTree*> InStateTree)
{
	bool bIsDirty = InStateTree->IsEditorDataDirty();
	if (!bIsDirty && UE::StateTree::Editor::bUseHashToDetermineIfAssetNeedsCompile)
	{
		const uint32 CurrentHash = CalculateStateTreeHash(InStateTree);
		bIsDirty = InStateTree->LastCompiledEditorDataHash != CurrentHash;
	}
	return bIsDirty;
}

void UStateTreeEditingSubsystem::MarkAsPubliclyModified(TNotNull<UStateTree*> InStateTree)
{
	constexpr bool bPubliclyModified = true;
	InStateTree->MarkAsModified(bPubliclyModified);
}

void UStateTreeEditingSubsystem::MarkAsModified(TNotNull<UStateTree*> InStateTree)
{
	constexpr bool bPubliclyModified = false;
	InStateTree->MarkAsModified(bPubliclyModified);
}

TSharedPtr<FStateTreeViewModel> UStateTreeEditingSubsystem::FindViewModel(TNotNull<const UStateTree*> InStateTree) const
{
	const FObjectKey StateTreeKey = InStateTree;
	TSharedPtr<FStateTreeViewModel> ViewModelPtr = StateTreeViewModels.FindRef(StateTreeKey);
	if (ViewModelPtr)
	{
		// The StateTree could be re-instantiated. Can occur when the object is destroyed and recreated in a pool or when reloaded in editor.
		//The object might have the same pointer value or the same path but it's a new object and all weakptr are now invalid.
		if (ViewModelPtr->GetStateTree() == InStateTree)
		{
			return ViewModelPtr.ToSharedRef();
		}
	}
	return nullptr;
}

TSharedRef<FStateTreeViewModel> UStateTreeEditingSubsystem::FindOrAddViewModel(TNotNull<UStateTree*> InStateTree)
{
	const FObjectKey StateTreeKey = InStateTree;
	TSharedPtr<FStateTreeViewModel> ViewModelPtr = StateTreeViewModels.FindRef(StateTreeKey);
	if (ViewModelPtr)
	{
		// The StateTree could be re-instantiated. Can occur when the object is destroyed and recreated in a pool or when reloaded in editor.
		//The object might have the same pointer value or the same path but it's a new object and all weakptr are now invalid.
		if (ViewModelPtr->GetStateTree() == InStateTree)
		{
			return ViewModelPtr.ToSharedRef();
		}
		else
		{
			StateTreeViewModels.Remove(StateTreeKey);
			ViewModelPtr = nullptr;
		}
	}

	ValidateStateTree(InStateTree);

	TSharedRef<FStateTreeViewModel> SharedModel = StateTreeViewModels.Add(StateTreeKey, MakeShared<FStateTreeViewModel>()).ToSharedRef();
	UStateTreeEditorData* EditorData = Cast<UStateTreeEditorData>(InStateTree->EditorData);
	SharedModel->Init(EditorData);

	return SharedModel;
}

TSharedRef<SWidget> UStateTreeEditingSubsystem::GetStateTreeView(TSharedRef<FStateTreeViewModel> InViewModel, const TSharedRef<FUICommandList>& TreeViewCommandList)
{
	return SNew(SStateTreeView, InViewModel, TreeViewCommandList);
}

void UStateTreeEditingSubsystem::ValidateStateTree(TNotNull<UStateTree*> InStateTree)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UStateTreeEditingSubsystem::ValidateStateTree)

	auto ModifyStateTree = [InStateTree](const bool bPubliclyModified)
		{
			constexpr bool bMarkDirty = false;
			InStateTree->Modify(bMarkDirty);
		};
	auto ModifyState = [&ModifyStateTree](TNotNull<UStateTreeState*> State)
		{
			constexpr bool bMarkDirty = false;
			State->Modify(bMarkDirty);

			constexpr bool bPubliclyModified = false;
			ModifyStateTree(bPubliclyModified);
		};

	auto FixChangedStateLinkName = [&ModifyState](TNotNull<UStateTreeState*> State, FStateTreeStateLink& StateLink, const TMap<FGuid, FName>& IDToName) -> bool
		{
			if (StateLink.ID.IsValid())
			{
				const FName* Name = IDToName.Find(StateLink.ID);
				if (Name == nullptr)
				{
					// Missing link, we'll show these in the UI
					return false;
				}
				if (StateLink.Name != *Name)
				{
					// Name changed, fix!
					ModifyState(State);
					StateLink.Name = *Name;
					return true;
				}
			}
			return false;
		};

	auto ValidateLinkedStates = [&FixChangedStateLinkName](TNotNull<UStateTreeEditorData*> TreeData)
		{
			// Make sure all state links are valid and update the names if needed.

			// Create ID to state name map.
			TMap<FGuid, FName> IDToName;

			TreeData->VisitHierarchy([&IDToName](const UStateTreeState& State, UStateTreeState* /*ParentState*/)
			{
				IDToName.Add(State.ID, State.Name);
				return EStateTreeVisitor::Continue;
			});
		
			// Fix changed names.
			TreeData->VisitHierarchy([&IDToName, FixChangedStateLinkName](UStateTreeState& State, UStateTreeState* /*ParentState*/)
			{
				constexpr bool bMarkDirty = false;
				State.Modify(bMarkDirty);
				if (State.Type == EStateTreeStateType::Linked)
				{
					FixChangedStateLinkName(&State, State.LinkedSubtree, IDToName);
				}
					
				for (FStateTreeTransition& Transition : State.Transitions)
				{
					FixChangedStateLinkName(&State, Transition.State, IDToName);
				}

				return EStateTreeVisitor::Continue;
			});
		};

	auto FixEditorData = [&ModifyStateTree](TNotNull<UStateTree*> StateTree)
	{
		UStateTreeEditorData* EditorData = Cast<UStateTreeEditorData>(StateTree->EditorData);
		// The schema is defined in the EditorData. If we can't find the editor data (probably because the class doesn't exist anymore), then try the compiled schema in the state tree asset.
		TSubclassOf<const UStateTreeSchema> SchemaClass;
		if (EditorData && EditorData->Schema)
		{
			SchemaClass = EditorData->Schema->GetClass();
		}
		else if (StateTree->GetSchema())
		{
			SchemaClass = StateTree->GetSchema()->GetClass();
		}

		if (SchemaClass.Get() == nullptr)
		{
			UE_LOGF(LogStateTreeEditor, Error, "The state tree '%ls' does not have a schema.", *StateTree->GetPathName());
			return;
		}

		TNonNullSubclassOf<UStateTreeEditorData> EditorDataClass = FStateTreeEditorModule::GetModule().GetEditorDataClass(SchemaClass.Get());
		if (EditorData == nullptr)
		{
			EditorData = NewObject<UStateTreeEditorData>(StateTree, EditorDataClass.Get(), FName(), RF_Transactional);
			EditorData->AddRootState();
			EditorData->Schema = NewObject<UStateTreeSchema>(EditorData, SchemaClass.Get());
			EditorData->EditorBindings.SetBindingsOwner(EditorData);

			constexpr bool bPubliclyModified = true;
			ModifyStateTree(bPubliclyModified);
			StateTree->EditorData = EditorData;
		}
		else if (!EditorData->IsA(EditorDataClass.Get()))
		{
			// The current EditorData is not of the correct type. The data needs to be patched by the schema desired editor data subclass.
			UStateTreeEditorData* PreviousEditorData = EditorData;
			EditorData = CastChecked<UStateTreeEditorData>(StaticDuplicateObject(EditorData, StateTree, FName(), RF_Transactional, EditorDataClass.Get()));
			if (EditorData->SubTrees.Num() == 0)
			{
				EditorData->AddRootState();
			}
			if (EditorData->Schema == nullptr || !EditorData->Schema->IsA(SchemaClass.Get()))
			{
				EditorData->Schema = NewObject<UStateTreeSchema>(EditorData, SchemaClass.Get());
			}
			EditorData->EditorBindings.SetBindingsOwner(EditorData);

			constexpr bool bPubliclyModified = true;
			ModifyStateTree(bPubliclyModified);
			StateTree->EditorData = EditorData;

			// Trash the previous EditorData
			const FString TrashEditorDataName = FString::Printf(TEXT("TRASH_%s"), *PreviousEditorData->GetName());
			UE::StateTree::Compiler::RenameObjectToTransientPackage(PreviousEditorData, TrashEditorDataName);
		}
	};

	auto FixEditorSchema = [&ModifyStateTree](TNotNull<UStateTreeEditorData*> EditorData)
		{
			TSubclassOf<const UStateTreeSchema> SchemaClass = EditorData->Schema ? EditorData->Schema->GetClass() : nullptr;
			if (SchemaClass.Get() == nullptr)
			{
				return;
			}

			TNonNullSubclassOf<UStateTreeEditorSchema> EditorSchemaClass = FStateTreeEditorModule::GetModule().GetEditorSchemaClass(SchemaClass.Get());
			if (EditorData->EditorSchema == nullptr)
			{
				constexpr bool bPubliclyModified = false;
				ModifyStateTree(bPubliclyModified);
				EditorData->EditorSchema = NewObject<UStateTreeEditorSchema>(EditorData, EditorSchemaClass.Get(), FName(), RF_Transactional);
			}
			else if (!EditorData->EditorSchema->IsA(EditorSchemaClass.Get()))
			{
				// The current EditorSchema is not of the correct type. The data needs to be patched by the schema desired editor data subclass.
				UStateTreeEditorSchema* PreviousEditorSchema = EditorData->EditorSchema;
				constexpr bool bPubliclyModified = false;
				ModifyStateTree(bPubliclyModified);
				EditorData->EditorSchema = CastChecked<UStateTreeEditorSchema>(StaticDuplicateObject(PreviousEditorSchema, EditorData, FName(), RF_Transactional, EditorSchemaClass.Get()));
				
				// Trash the previous EditorDataSchema
				const FString TrashEditorSchemaName = FString::Printf(TEXT("TRASH_%s"), *PreviousEditorSchema->GetName());
				UE::StateTree::Compiler::RenameObjectToTransientPackage(PreviousEditorSchema, TrashEditorSchemaName );
			}
		};

	auto RemoveUnusedBindings = [](TNotNull<UStateTreeEditorData*> EditorData)
		{
			EditorData->RemoveInvalidBindings();
		};

	auto UpdateLinkedStateParameters = [](TNotNull<UStateTreeEditorData*> TreeData)
		{
			const EStateTreeVisitor Result = TreeData->VisitHierarchy([](UStateTreeState& State, UStateTreeState* /*ParentState*/)
			{
				if (State.Type == EStateTreeStateType::Linked
					|| State.Type == EStateTreeStateType::LinkedAsset)
				{
					constexpr bool bMarkDirty = false;
					State.Modify(bMarkDirty);
					//@todo detect if the tree is dirty and call ModifyStateTree if modified
					State.UpdateParametersFromLinkedSubtree();
				}
				return EStateTreeVisitor::Continue;
			});
		};

	auto UpdateTransactionalFlags = [](TNotNull<UStateTreeEditorData*> EditorData)
		{
			for (UStateTreeState* SubTree : EditorData->SubTrees)
			{
				TArray<UStateTreeState*> Stack;

				Stack.Add(SubTree);
				while (!Stack.IsEmpty())
				{
					if (UStateTreeState* State = Stack.Pop())
					{
						State->SetFlags(RF_Transactional);

						for (UStateTreeState* ChildState : State->Children)
						{
							Stack.Add(ChildState);
						}
					}
				}
			}
		};

	auto RemoveEmptyExtension = [](TNotNull<UStateTreeEditorData*> EditorData)
		{
			EditorData->Extensions.Remove(TObjectPtr<UStateTreeEditorDataExtension>());
		};

	FixEditorData(InStateTree);
	if (InStateTree->EditorData)
	{
		constexpr bool bMarkDirty = false;
		InStateTree->EditorData->Modify(bMarkDirty);

		UStateTreeEditorData* EditorData = CastChecked<UStateTreeEditorData>(InStateTree->EditorData);
		FixEditorSchema(EditorData);

		// Notify the module, schema, and extension. Use that order to go from the less specific to the more specific.
		{
			FStateTreeEditorModule::GetModule().OnPreValidateStateTree().Broadcast(InStateTree);

			if (EditorData->EditorSchema)
			{
				EditorData->EditorSchema->PreValidate(InStateTree);
			}

			TArray<UStateTreeEditorDataExtension*> LocalExtensions = EditorData->Extensions;
			for (UStateTreeEditorDataExtension* Extension : LocalExtensions)
			{
				if (Extension)
				{
					Extension->PreValidate();
				}
			}
		}

		EditorData->ReparentStates();
		EditorData->FixDuplicateIDs();

		RemoveUnusedBindings(EditorData);
		EditorData->UpdateBindings();
		ValidateLinkedStates(EditorData);
		UpdateLinkedStateParameters(EditorData);
		UpdateTransactionalFlags(EditorData);
		RemoveEmptyExtension(EditorData);

		// Notify the module, schema, and extension.
		{
			FStateTreeEditorModule::GetModule().OnValidateStateTree().Broadcast(InStateTree);

			if (EditorData->EditorSchema)
			{
				EditorData->EditorSchema->Validate(InStateTree);
			}

			TArray<UStateTreeEditorDataExtension*> LocalExtensions = EditorData->Extensions;
			for (UStateTreeEditorDataExtension* Extension : LocalExtensions)
			{
				if (Extension)
				{
					Extension->Validate();
				}
			}
		}
		RemoveEmptyExtension(EditorData);
	}
}

uint32 UStateTreeEditingSubsystem::CalculateStateTreeHash(TNotNull<const UStateTree*> InStateTree)
{
	uint32 EditorDataHash = 0;
	if (InStateTree->EditorData != nullptr)
	{
		FStateTreeObjectCRC32 Archive;
		EditorDataHash = Archive.Crc32(InStateTree->EditorData, 0);
	}

	return EditorDataHash;
}

void UStateTreeEditingSubsystem::HandlePostGarbageCollect()
{
	// Remove the stale viewmodels
	for (TMap<FObjectKey, TSharedPtr<FStateTreeViewModel>>::TIterator It(StateTreeViewModels); It; ++It)
	{
		if (!It.Key().ResolveObjectPtr())
		{
			It.RemoveCurrent();
		}
		else if (!It.Value() || !It.Value()->GetStateTree())
		{
			It.RemoveCurrent();
		}
	}
}

void UStateTreeEditingSubsystem::HandlePostCompile(const UStateTree& InStateTree)
{
	// Notify the UI that something changed. Make sure to not request a new viewmodel. That way, we are not creating new viewmodel when cooking/PIE.
	if (TSharedPtr<FStateTreeViewModel> ViewModel = FindViewModel(&InStateTree))
	{
		ViewModel->NotifyAssetChangedExternally();
	}
}
