// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeEditor.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Blueprint/StateTreeTaskBlueprintBase.h"
#include "Blueprint/StateTreeConditionBlueprintBase.h"
#include "Blueprint/StateTreeConsiderationBlueprintBase.h"
#include "ContentBrowserModule.h"
#include "ClassViewerFilter.h"
#include "ContextObjectStore.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Customizations/StateTreeBindingExtension.h"
#include "DetailsViewArgs.h"
#include "EditorModeManager.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "IDetailsView.h"
#include "IContentBrowserSingleton.h"
#include "ISourceCodeAccessModule.h"
#include "ISourceCodeAccessor.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UICommandList.h"
#include "IMessageLogListing.h"
#include "MessageLogModule.h"
#include "Misc/UObjectToken.h"
#include "SStateTreeView.h"
#include "StateTree.h"
#include "StateTreeCompiler.h"
#include "StateTreeCompilerLog.h"
#include "StateTreeDelegates.h"
#include "StateTreeEditorCommands.h"
#include "StateTreeEditorData.h"
#include "StateTreeEditorModule.h"
#include "StateTreeEditorSettings.h"
#include "StateTreeEditorWorkspaceTabHost.h"
#include "StateTreeObjectHash.h"
#include "StateTreeTaskBase.h"
#include "StateTreeViewModel.h"
#include "ToolMenus.h"
#include "ToolMenu.h"
#include "Widgets/Docking/SDockTab.h"
#include "ToolMenuEntry.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "FileHelpers.h"
#include "StateTreeEditorMode.h"
#include "PropertyPath.h"
#include "SStateTreeOutliner.h"
#include "StandaloneStateTreeEditorHost.h"
#include "StateTreeEditingSubsystem.h"
#include "StateTreeEditorUILayer.h"
#include "Debugger/SStateTreeDebuggerView.h"
#include "StateTreeSettings.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"

const FName StateTreeEditorAppName(TEXT("StateTreeEditorApp"));

const FName FStateTreeEditor::SelectionDetailsTabId(TEXT("StateTreeEditor_SelectionDetails"));
const FName FStateTreeEditor::AssetDetailsTabId(TEXT("StateTreeEditor_AssetDetails"));
const FName FStateTreeEditor::StateTreeViewTabId(TEXT("StateTreeEditor_StateTreeView"));
const FName FStateTreeEditor::CompilerResultsTabId(TEXT("StateTreeEditor_CompilerResults"));
const FName FStateTreeEditor::CompilerLogListingName(TEXT("StateTreeCompiler"));
const FName FStateTreeEditor::LayoutLeftStackId("LeftStackId");
const FName FStateTreeEditor::LayoutBottomMiddleStackId("BottomMiddleStackId");

namespace UE::StateTree::Editor
{
bool GbDisplayItemIds = false;

FAutoConsoleVariableRef CVarDisplayItemIds(
	TEXT("statetree.displayitemids"),
	GbDisplayItemIds,
	TEXT("Appends Id to task and state names in the treeview and expose Ids in the details view."));
}

void FStateTreeEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (StateTree != nullptr)
	{
		Collector.AddReferencedObject(StateTree);
	}
}

void FStateTreeEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_StateTreeEditor", "StateTree Editor"));
	auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(SelectionDetailsTabId, FOnSpawnTab::CreateSP(this, &FStateTreeEditor::SpawnTab_SelectionDetails) )
		.SetDisplayName( NSLOCTEXT("StateTreeEditor", "SelectionDetailsTab", "Details" ) )
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(AssetDetailsTabId, FOnSpawnTab::CreateSP(this, &FStateTreeEditor::SpawnTab_AssetDetails))
		.SetDisplayName(NSLOCTEXT("StateTreeEditor", "AssetDetailsTab", "Asset Details"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(StateTreeViewTabId, FOnSpawnTab::CreateSP(this, &FStateTreeEditor::SpawnTab_StateTreeView))
		.SetDisplayName(NSLOCTEXT("StateTreeEditor", "StateTreeViewTab", "States"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner"));
	
	InTabManager->RegisterTabSpawner(CompilerResultsTabId, FOnSpawnTab::CreateSP(this, &FStateTreeEditor::SpawnTab_CompilerResults))
		.SetDisplayName(NSLOCTEXT("StateTreeEditor", "CompilerResultsTab", "Compiler Results"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Log.TabIcon"));

	if (EditorHost)
	{
		using namespace UE::StateTreeEditor;
		TSharedPtr<FWorkspaceTabHost> TabHost = EditorHost->GetTabHost();
		for (const FMinorWorkspaceTabConfig& Config : TabHost->GetTabConfigs())
		{
			FOnSpawnTab Delegate = TabHost->CreateSpawnDelegate(Config.ID);			
			InTabManager->RegisterTabSpawner(Config.ID, Delegate)
				.SetDisplayName(Config.Label)
				.SetGroup(WorkspaceMenuCategoryRef)
				.SetTooltipText(Config.Tooltip)
				.SetIcon(Config.Icon);
		}
	}
}


void FStateTreeEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(SelectionDetailsTabId);
	InTabManager->UnregisterTabSpawner(AssetDetailsTabId);
	InTabManager->UnregisterTabSpawner(StateTreeViewTabId);
	InTabManager->UnregisterTabSpawner(CompilerResultsTabId);

	if (EditorHost)
	{
		using namespace UE::StateTreeEditor;
		TSharedPtr<FWorkspaceTabHost> TabHost = EditorHost->GetTabHost();
		for (const FMinorWorkspaceTabConfig& Config : TabHost->GetTabConfigs())
		{
			InTabManager->UnregisterTabSpawner(Config.ID);
		}
	}
}

void FStateTreeEditor::InitEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UStateTree* InStateTree)
{
	StateTree = InStateTree;
	check(StateTree != NULL);

	UStateTreeEditingSubsystem* StateTreeEditingSubsystem = GEditor->GetEditorSubsystem<UStateTreeEditingSubsystem>();
	check(StateTreeEditingSubsystem);

	EditorHost = MakeShared<FStandaloneStateTreeEditorHost>();
	EditorHost->Init(StaticCastSharedRef<FStateTreeEditor>(AsShared()));

	StateTreeViewModel = StateTreeEditingSubsystem->FindOrAddViewModel(InStateTree);

	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	FMessageLogInitializationOptions LogOptions;
	// Show Pages so that user is never allowed to clear log messages
	LogOptions.bShowPages = false;
	LogOptions.bShowFilters = false;
	LogOptions.bAllowClear = false;
	LogOptions.MaxPageCount = 1;

	MessageLogModule.RegisterLogListing(CompilerLogListingName, FText::FromName(CompilerLogListingName), LogOptions);
	CompilerResultsListing = MessageLogModule.GetLogListing(CompilerLogListingName);
	CompilerResults = MessageLogModule.CreateLogListingWidget(CompilerResultsListing.ToSharedRef());

	TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_StateTree_Layout_v5")
	->AddArea
	(
		FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.2f)
				->SetExtensionId(LayoutLeftStackId)
				->AddTab(AssetDetailsTabId, ETabState::OpenedTab)
				->AddTab(UE::StateTreeEditor::FWorkspaceTabHost::OutlinerTabId.Resolve(), ETabState::ClosedTab)
				->AddTab(UE::StateTreeEditor::FWorkspaceTabHost::StatisticsTabId.Resolve(), ETabState::ClosedTab)
				->SetForegroundTab(AssetDetailsTabId)
			)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
				->SetSizeCoefficient(0.5f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.75f)
					->AddTab(StateTreeViewTabId, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.25f)
					->SetExtensionId(LayoutBottomMiddleStackId)
					->AddTab(CompilerResultsTabId, ETabState::ClosedTab)
					->AddTab(UE::StateTreeEditor::FWorkspaceTabHost::SearchTabId.Resolve(), ETabState::ClosedTab)
					->AddTab(UE::StateTreeEditor::FWorkspaceTabHost::DebuggerTabId.Resolve(), ETabState::ClosedTab)
					->AddTab(UE::StateTreeEditor::FWorkspaceTabHost::BindingTabId.Resolve(), ETabState::ClosedTab)
				)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.3f)
				->AddTab(SelectionDetailsTabId, ETabState::OpenedTab)
				->SetForegroundTab(SelectionDetailsTabId)
			)
		)
	);

	FLayoutExtender LayoutExtender;
	FStateTreeEditorModule& StateTreeEditorModule = FModuleManager::LoadModuleChecked<FStateTreeEditorModule>("StateTreeEditorModule");
	StateTreeEditorModule.OnRegisterLayoutExtensions().Broadcast(LayoutExtender);
	StandaloneDefaultLayout->ProcessExtensions(LayoutExtender);

	CreateEditorModeManager();
	
	constexpr bool bCreateDefaultStandaloneMenu = true;
	constexpr bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, StateTreeEditorAppName, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, StateTree);

	RegisterMenu();
	RegisterToolbar();
	
	AddMenuExtender(StateTreeEditorModule.GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	RegenerateMenusAndToolbars();
}

void FStateTreeEditor::PostInitAssetEditor()
{
	IStateTreeEditor::PostInitAssetEditor();
	
	ModeUILayer = MakeShared<FStateTreeEditorModeUILayer>(ToolkitHost.Pin().Get());
	ModeUILayer->SetModeMenuCategory(WorkspaceMenuCategory);
	ModeUILayer->SetSecondaryModeToolbarName(GetToolMenuToolbarName());
	ToolkitCommands->Append(ModeUILayer->GetModeCommands());

	if (UContextObjectStore* ContextStore = EditorModeManager->GetInteractiveToolsContext()->ContextObjectStore)
	{
		UStateTreeEditorContext* StateTreeEditorContext = ContextStore->FindContext<UStateTreeEditorContext>();
		if (!StateTreeEditorContext)
		{
			StateTreeEditorContext = NewObject<UStateTreeEditorContext>();
			StateTreeEditorContext->EditorHostInterface = EditorHost;
			ContextStore->AddContextObject(StateTreeEditorContext);
		}
	}

	EditorModeManager->SetDefaultMode(UStateTreeEditorMode::EM_StateTree);
	EditorModeManager->ActivateDefaultMode();
}

void FStateTreeEditor::OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit)
{
	ModeUILayer->OnToolkitHostingStarted(Toolkit);
	HostedToolkit = Toolkit;
}

void FStateTreeEditor::InitToolMenuContext(FToolMenuContext& MenuContext)
{
	IStateTreeEditor::InitToolMenuContext(MenuContext);

	UStateTreeEditorContext* EditorContext = NewObject<UStateTreeEditorContext>();
	EditorContext->EditorHostInterface = EditorHost;
	MenuContext.AddObject(EditorContext);
}

void FStateTreeEditor::OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit)
{
	ModeUILayer->OnToolkitHostingFinished(Toolkit);
	HostedToolkit = nullptr;
}

FName FStateTreeEditor::GetToolkitFName() const
{
	return FName("StateTreeEditor");
}

FText FStateTreeEditor::GetBaseToolkitName() const
{
	return NSLOCTEXT("StateTreeEditor", "AppLabel", "State Tree");
}

FString FStateTreeEditor::GetWorldCentricTabPrefix() const
{
	return NSLOCTEXT("StateTreeEditor", "WorldCentricTabPrefix", "State Tree").ToString();
}

FLinearColor FStateTreeEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor( 0.0f, 0.0f, 0.2f, 0.5f );
}

void FStateTreeEditor::OnClose()
{
	if (HostedToolkit.IsValid())
	{
		UToolMenus::UnregisterOwner(&(*HostedToolkit));	
		HostedToolkit = nullptr;
	}
}

TSharedRef<SDockTab> FStateTreeEditor::SpawnTab_StateTreeView(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == StateTreeViewTabId);

	return SNew(SDockTab)
		.Label(NSLOCTEXT("StateTreeEditor", "StateTreeViewTab", "States"))
		.TabColorScale(GetTabColorScale())
		[
			SAssignNew(StateTreeView, SStateTreeView, StateTreeViewModel.ToSharedRef(), TreeViewCommandList)
		];
}

TSharedRef<SDockTab> FStateTreeEditor::SpawnTab_SelectionDetails(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == SelectionDetailsTabId);

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;

	SelectionDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	TArray<UStateTreeState*> SelectedStates;
	StateTreeViewModel->GetSelectedStates(SelectedStates);

	const TArray<UObject*> SelectedObjects = TArray<UObject*>(SelectedStates);
	SelectionDetailsView->SetObjects(SelectedObjects);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(NSLOCTEXT("StateTreeEditor", "SelectionDetailsTab", "Details"))
		[
			SelectionDetailsView.ToSharedRef()
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FStateTreeEditor::SpawnTab_AssetDetails(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == AssetDetailsTabId);

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;

	AssetDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	AssetDetailsView->SetObject(StateTree ? StateTree->EditorData : nullptr);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(NSLOCTEXT("StateTreeEditor", "AssetDetailsTabLabel", "Asset Details"))
		[
			AssetDetailsView.ToSharedRef()
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FStateTreeEditor::SpawnTab_CompilerResults(const FSpawnTabArgs& Args) const
{
	check(Args.GetTabId() == CompilerResultsTabId);
	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("CompilerResultsTitle", "Compiler Results"))
		[
			SNew(SBox)
			[
				CompilerResults.ToSharedRef()
			]
		];
	return SpawnedTab;
}

void FStateTreeEditor::SaveAsset_Execute()
{
	// Remember the treeview expansion state
	if (StateTreeView)
	{
		StateTreeView->SavePersistentExpandedStates();
	}

	// save it
	FAssetEditorToolkit::SaveAsset_Execute();
}

namespace UE::StateTree::Editor::Private
{
void FillDeveloperMenu(UToolMenu* InMenu)
{
	const FStateTreeEditorCommands& Commands = FStateTreeEditorCommands::Get();
	{
		FToolMenuSection& Section = InMenu->AddSection("FileDeveloperCompilerSettings", LOCTEXT("CompileOptionsHeading", "Compiler Settings"));
		Section.AddMenuEntry(Commands.LogCompilationResult);
		Section.AddMenuEntry(Commands.LogDependencies);
	}
	{
		FToolMenuSection& Section = InMenu->AddSection("FileDeveloperSettings", LOCTEXT("DeveloperOptionsHeading", "Settings"));
		Section.AddEntry(FToolMenuEntry::InitMenuEntry(
			"DisplayItemIds",
			LOCTEXT("DisplayItemIds", "Display Nodes IDs"),
			UE::StateTree::Editor::CVarDisplayItemIds->GetDetailedHelp(),
			TAttribute<FSlateIcon>(),
			FUIAction(
				FExecuteAction::CreateLambda([]()
					{
						UE::StateTree::Editor::CVarDisplayItemIds->Set(!UE::StateTree::Editor::CVarDisplayItemIds->GetBool(), ECVF_SetByConsole);
					}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([]()
					{
						return UE::StateTree::Editor::CVarDisplayItemIds->GetBool();
					})
				),
			EUserInterfaceActionType::ToggleButton
			));
	}
}
void FillDynamicDeveloperMenu(FToolMenuSection& Section)
{
	// Only show the developer menu on machines with the solution (assuming they can build it)
	ISourceCodeAccessModule* SourceCodeAccessModule = FModuleManager::GetModulePtr<ISourceCodeAccessModule>("SourceCodeAccess");
	if (SourceCodeAccessModule != nullptr && SourceCodeAccessModule->GetAccessor().CanAccessSourceCode())
	{
		Section.AddSubMenu(
			"DeveloperMenu",
			LOCTEXT("DeveloperMenu", "Developer"),
			LOCTEXT("DeveloperMenu_ToolTip", "Open the developer menu"),
			FNewToolMenuDelegate::CreateStatic(FillDeveloperMenu),
			false);
	}
}
} // UE::StateTree::Editor::Private

void FStateTreeEditor::RegisterMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	const FName FileMenuName = *(GetToolMenuName().ToString() + TEXT(".File"));
	if (!ToolMenus->IsMenuRegistered(FileMenuName))
	{
		const FName ParentFileMenuName = TEXT("MainFrame.MainMenu.File");
		UToolMenus::Get()->RegisterMenu(FileMenuName, ParentFileMenuName, EMultiBoxType::ToolBar);
		{
			UToolMenu* FileMenu = UToolMenus::Get()->RegisterMenu(FileMenuName, ParentFileMenuName);
			const FName FileStateTreeSection = "FileStateTree";
			FToolMenuSection& Section = FileMenu->AddSection("StateTree", LOCTEXT("StateTreeHeading", "State Tree"));
			FToolMenuInsert InsertPosition("FileLoadAndSave", EToolMenuInsertType::After);
			Section.InsertPosition = InsertPosition;

			Section.AddDynamicEntry("FileDeveloper", FNewToolMenuSectionDelegate::CreateStatic(UE::StateTree::Editor::Private::FillDynamicDeveloperMenu));
		}
	}

	const FName EditMenuName = *(GetToolMenuName().ToString() + TEXT(".Edit"));
	if (!UToolMenus::Get()->IsMenuRegistered(EditMenuName))
	{
		const FName ParentEditMenuName = TEXT("MainFrame.MainMenu.Edit");
		UToolMenu* EditMenu = UToolMenus::Get()->RegisterMenu(EditMenuName, ParentEditMenuName);
		FToolMenuSection& Section = EditMenu->AddSection("StateTree", LOCTEXT("StateTreeHeading", "State Tree"));
		FToolMenuInsert InsertPosition("Configuration", EToolMenuInsertType::After);
		Section.InsertPosition = InsertPosition;
	}
}

void FStateTreeEditor::RegisterToolbar()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	FName ParentName;
	const FName MenuName = GetToolMenuToolbarName(ParentName);
	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		UToolMenus::Get()->RegisterMenu(MenuName, ParentName, EMultiBoxType::ToolBar);
	}
}

#undef LOCTEXT_NAMESPACE
