// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "IStateTreeEditor.h"

class FStateTreeEditorModeUILayer;
class FSpawnTabArgs;
class IMessageToken;
class UStateTreeState;
class UStateTreeNodeBlueprintBase;
class IMessageLogListing;
class IDetailsView;
class UStateTree;
class FStateTreeViewModel;
class FStandaloneStateTreeEditorHost;

namespace UE::StateTree::Editor
{
	extern bool GbDisplayItemIds;
} // UE::StateTree::Editor

class FStateTreeEditor : public IStateTreeEditor, public FSelfRegisteringEditorUndoClient, public FGCObject
{
	
private:
	friend class FStandaloneStateTreeEditorHost;	
public:
	static const FName LayoutLeftStackId;
	static const FName LayoutBottomMiddleStackId;
	static const FName CompilerLogListingName;
	static const FName CompilerResultsTabId;
	
	FStateTreeEditor() : TreeViewCommandList(new FUICommandList()) {}

	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	void InitEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UStateTree* StateTree);

	//~ Begin FAssetEditorToolkit Interface
	virtual void PostInitAssetEditor() override;
	virtual void OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit) override;
	virtual void OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit) override;
	virtual void InitToolMenuContext(FToolMenuContext& MenuContext) override;
	//~ End FAssetEditorToolkit Interface

	//~ Begin IToolkit Interface
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual void OnClose() override;
	//~ End IToolkit Interface
	/** FGCObject interface */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FStateTreeEditor");
	}

protected:
	/** Called when "Save" is clicked for this asset */
	virtual void SaveAsset_Execute() override;

private:
	/** Spawns the tab with the update graph inside */
	TSharedRef<SDockTab> SpawnTab_StateTreeView(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_SelectionDetails(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_AssetDetails(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_CompilerResults(const FSpawnTabArgs& Args) const;

	void RegisterMenu();
	void RegisterToolbar();
	
	/** State Tree being edited */
	TObjectPtr<UStateTree> StateTree = nullptr;

	/** The command list used by the tree view. Stored here, so that other windows (e.g. debugger) can add commands to it, even if the tree view is not spawned yet. */
	TSharedRef<FUICommandList> TreeViewCommandList;
	
	/** Selection Property View */
	TSharedPtr<IDetailsView> SelectionDetailsView;

	/** Asset Property View */
	TSharedPtr<IDetailsView> AssetDetailsView;

	/** Tree View */
	TSharedPtr<class SStateTreeView> StateTreeView;

	/** Tree Outliner */
	TSharedPtr<class SStateTreeOutliner> StateTreeOutliner;

	/** Compiler Results log */
	TSharedPtr<SWidget> CompilerResults;
	TSharedPtr<IMessageLogListing> CompilerResultsListing;
	
	TSharedPtr<FStateTreeViewModel> StateTreeViewModel;

	TSharedPtr<FStateTreeEditorModeUILayer> ModeUILayer;
	TSharedPtr<IToolkit> HostedToolkit;
	TSharedPtr<FWorkspaceItem> WorkspaceMenuCategory;
	TSharedPtr<FStandaloneStateTreeEditorHost> EditorHost;

	static const FName StateTreeViewTabId;
	static const FName SelectionDetailsTabId;
	static const FName AssetDetailsTabId;
};
