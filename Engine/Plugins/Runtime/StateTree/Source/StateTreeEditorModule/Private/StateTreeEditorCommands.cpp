// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeEditorCommands.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"

FStateTreeEditorCommands::FStateTreeEditorCommands() 
	: TCommands(
		TEXT("StateTreeEditor"), // Context name for fast lookup
		LOCTEXT("StateTreeEditor", "StateTree Editor"), // Localized context name for displaying
		NAME_None,
		TEXT("StateTreeEditorStyle")
	)
{
}

void FStateTreeEditorCommands::RegisterCommands()
{
	UI_COMMAND(Compile, "Compile", "Compile the current StateTree.", EUserInterfaceActionType::Button, FInputChord(EKeys::F7));
	UI_COMMAND(SaveOnCompile_Never, "Never", "Sets the save-on-compile option to 'Never', meaning that your StateTree will not be saved when they are compiled", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SaveOnCompile_SuccessOnly, "On Success Only", "Sets the save-on-compile option to 'Success Only', meaning that your StateTree will be saved whenever they are successfully compiled", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SaveOnCompile_Always, "Always", "Sets the save-on-compile option to 'Always', meaning that your StateTree will be saved whenever they are compiled (even if there were errors)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(LogCompilationResult, "Log compilation result", "After a StateTree compiles, log the internal content of the StateTree.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(LogDependencies, "Log dependencies", "After a StateTree compiles, log the compilation dependencies of the StateTree.", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(AddSiblingState, "Add Sibling State", "Add a Sibling State", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddChildState, "Add Child State", "Add a Child State", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CutStates, "Cut", "Cut Selected States", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::X));
	UI_COMMAND(CopyStates, "Copy", "Copy Selected States", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::C));
	UI_COMMAND(PasteStatesAsSiblings, "Paste As Siblings", "Paste States as Siblings", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::V));
	UI_COMMAND(PasteStatesAsChildren, "Paste As Children", "Paste States as Children", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::V));
	UI_COMMAND(PasteNodesToSelectedStates, "Paste As Nodes and Transitions", "Append Nodes and Transitions to Selected States", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DuplicateStates, "Duplicate", "Duplicate Selected States", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::D));
	UI_COMMAND(DeleteStates, "Delete", "Delete Selected States", EUserInterfaceActionType::Button, FInputChord(EKeys::Platform_Delete));
	UI_COMMAND(RenameState, "Rename", "Rename Selected State", EUserInterfaceActionType::Button, FInputChord(EKeys::F2));
	UI_COMMAND(EnableStates, "State Enabled", "Enables Selected States", EUserInterfaceActionType::Check, FInputChord());

#if WITH_STATETREE_TRACE_DEBUGGER
	UI_COMMAND(EnableOnEnterStateBreakpoint, "Break on Enter", "Adds or removes a breakpoint when entering the selected state(s). (Debugger Window required)", EUserInterfaceActionType::Check, FInputChord(EKeys::F9));
	UI_COMMAND(EnableOnExitStateBreakpoint, "Break on Exit", "Adds or removes a breakpoint when exiting the selected state(s). (Debugger Window required)", EUserInterfaceActionType::Check, FInputChord(EModifierKey::Shift, EKeys::F9));
#endif //WITH_STATETREE_TRACE_DEBUGGER
}

#undef LOCTEXT_NAMESPACE
