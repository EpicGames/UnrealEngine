// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"


/**
 * StateTree editor command set.
 */
class FStateTreeEditorCommands : public TCommands<FStateTreeEditorCommands>
{
public:
	FStateTreeEditorCommands();

	// TCommands<> overrides
	virtual void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> Compile;
	TSharedPtr<FUICommandInfo> SaveOnCompile_Never;
	TSharedPtr<FUICommandInfo> SaveOnCompile_SuccessOnly;
	TSharedPtr<FUICommandInfo> SaveOnCompile_Always;
	TSharedPtr<FUICommandInfo> LogCompilationResult;
	TSharedPtr<FUICommandInfo> LogDependencies;

	TSharedPtr<FUICommandInfo> AddSiblingState;
	TSharedPtr<FUICommandInfo> AddChildState;
	TSharedPtr<FUICommandInfo> RenameState;
	TSharedPtr<FUICommandInfo> CutStates;
	TSharedPtr<FUICommandInfo> CopyStates;
	TSharedPtr<FUICommandInfo> PasteStatesAsSiblings;
	TSharedPtr<FUICommandInfo> PasteStatesAsChildren;
	TSharedPtr<FUICommandInfo> PasteNodesToSelectedStates;
	TSharedPtr<FUICommandInfo> DuplicateStates;
	TSharedPtr<FUICommandInfo> DeleteStates;
	TSharedPtr<FUICommandInfo> EnableStates;

#if WITH_STATETREE_TRACE_DEBUGGER
	TSharedPtr<FUICommandInfo> EnableOnEnterStateBreakpoint;
	TSharedPtr<FUICommandInfo> EnableOnExitStateBreakpoint;
#endif // WITH_STATETREE_TRACE_DEBUGGER
};