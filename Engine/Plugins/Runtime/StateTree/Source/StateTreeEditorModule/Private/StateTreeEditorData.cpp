// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeEditorData.h"
#include "StateTree.h"
#include "StateTreeConditionBase.h"
#include "StateTreeConsiderationBase.h"
#include "StateTreeDelegates.h"
#include "StateTreeEditingSubsystem.h"
#include "StateTreeEditorDataExtension.h"
#include "StateTreeEditorModule.h"
#include "StateTreeEditorSchema.h"
#include "StateTreeEvaluatorBase.h"
#include "StateTreeNodeClassCache.h"
#include "StateTreePropertyFunctionBase.h"
#include "StateTreePropertyHelpers.h"
#include "StateTreeTaskBase.h"
#include "Algo/LevenshteinDistance.h"
#include "Customizations/StateTreeBindingExtension.h"
#include "Customizations/StateTreeEditorNodeUtils.h"
#include "Modules/ModuleManager.h"
#include "UObject/UE5SpecialProjectStreamObjectVersion.h"

#if WITH_EDITOR
#include "StructUtils/UserDefinedStruct.h"
#endif

#include "StateTreeDelegatesInternal.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeEditorData)

#define LOCTEXT_NAMESPACE "StateTreeEditor"

namespace UE::StateTree::Editor
{
	FAutoConsoleVariable CVarLogEnableBindingSelectionNodeToInstanceData(
		TEXT("StateTree.Compiler.EnableBindingSelectionNodeToInstanceData"),
		true,
		TEXT("Enable binding from enter condition, utility/consideration and state argument to bind to task instance data.\n")
		TEXT("The task instance data is only available once the transition is completed.")
		TEXT("A parent state can enter a child state during state selection (before the transition completes).")
	);

	FAutoConsoleVariable CVarAutoRemoveSourcePathInvalidPropertyBindings(
		TEXT("StateTree.Editor.AutoRemoveSourcePathInvalidEditorBindings"),
		false,
		TEXT("If set to true, EditorBindings whose source path does not resolve anymore,")
		TEXT("will be automatically removed when the asset is loaded, when the users authors in the editor,")
		TEXT("when state tree gets compiled.\n")
		TEXT("If set to false, the compilation will fail and the user will be notified and be responsible for which binding is invalid and fixing them.\n")
		TEXT("Note that missing target struct or failed to resolve target binding path bindings will always be removed automatically.")
		TEXT("Note that missing source struct bindings will never be removed automatically.")
	);

	const FString GlobalStateName(TEXT("Global"));
	const FString PropertyFunctionStateName(TEXT("Property Functions"));
	const FName ParametersNodeName(TEXT("Parameters"));

	bool IsPropertyFunctionOwnedByNode(FGuid NodeID, FGuid PropertyFuncID, const FStateTreeEditorPropertyBindings& EditorBindings)
	{
		for (const FPropertyBindingBinding& Binding : EditorBindings.GetBindings())
		{
			const FGuid TargetID = Binding.GetTargetPath().GetStructID();
			if (TargetID == NodeID)
			{
				return true;
			}

			FConstStructView NodeView = Binding.GetPropertyFunctionNode();
			if (const FStateTreeEditorNode* Node = NodeView.GetPtr<const FStateTreeEditorNode>())
			{
				if (Node->ID == PropertyFuncID)
				{
					PropertyFuncID = TargetID;
				}
			}
		}

		return false;
	}

	FStateTreeEditorColor CreateDefaultColor()
	{
		FStateTreeEditorColor DefaultColor;
		DefaultColor.ColorRef = FStateTreeEditorColorRef();
		DefaultColor.Color = FLinearColor(FColor(31, 151, 167));
		DefaultColor.DisplayName = TEXT("Default Color");
		return DefaultColor;
	}
}

UStateTreeEditorData::UStateTreeEditorData()
{
	Colors.Add(UE::StateTree::Editor::CreateDefaultColor());

	EditorBindings.SetBindingsOwner(this);
}

/*static*/ UStateTreeEditorData* UStateTreeEditorData::GetEditorData(UStateTree* StateTree)
{
	if (StateTree == nullptr)
	{
		return nullptr;
	}
	return Cast<UStateTreeEditorData>(StateTree->EditorData);
}

void UStateTreeEditorData::OnNodeAdded(FStateTreeEditorNode& EditorNode)
{
	if (UStateTree* StateTree = GetTypedOuter<UStateTree>())
	{
		UStateTreeEditingSubsystem::MarkAsModified(StateTree);
	}
}

void UStateTreeEditorData::PostInitProperties()
{
	Super::PostInitProperties();

	if(!IsTemplate())
	{
		RootParametersGuid = FGuid::NewGuid();
	}

#if WITH_EDITOR
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		OnParametersChangedHandle = UE::StateTree::Delegates::OnParametersChanged.AddUObject(this, &UStateTreeEditorData::OnParametersChanged);
		OnStateParametersChangedHandle = UE::StateTree::Delegates::OnStateParametersChanged.AddUObject(this, &UStateTreeEditorData::OnStateParametersChanged);
	}
	
#endif
}

void UStateTreeEditorData::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);	
	Ar.UsingCustomVersion(FUE5SpecialProjectStreamObjectVersion::GUID);
}

#if WITH_EDITOR

void UStateTreeEditorData::BeginDestroy()
{
	if (OnParametersChangedHandle.IsValid())
	{
		UE::StateTree::Delegates::OnParametersChanged.Remove(OnParametersChangedHandle);
		OnParametersChangedHandle.Reset();
	}
	if (OnStateParametersChangedHandle.IsValid())
	{
		UE::StateTree::Delegates::OnStateParametersChanged.Remove(OnStateParametersChangedHandle);
		OnStateParametersChangedHandle.Reset();
	}
	
	Super::BeginDestroy();
}

void UStateTreeEditorData::OnParametersChanged(const UStateTree& StateTree)
{
	if (const UStateTree* OwnerStateTree = GetTypedOuter<UStateTree>())
	{
		if (OwnerStateTree == &StateTree)
		{
			UpdateBindings();
		}
	}
}

void UStateTreeEditorData::OnStateParametersChanged(const UStateTree& StateTree, const FGuid StateID)
{
	if (const UStateTree* OwnerStateTree = GetTypedOuter<UStateTree>())
	{
		if (OwnerStateTree == &StateTree)
		{
			if (UStateTreeState* State = GetMutableStateByID(StateID))
			{
				UpdateBindings();
			}
		}
	}
}

void UStateTreeEditorData::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerCustomVersion(FUE5SpecialProjectStreamObjectVersion::GUID) < FUE5SpecialProjectStreamObjectVersion::StateTreeGlobalParameterChanges)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		RootParameterPropertyBag = RootParameters.Parameters;
		RootParametersGuid = RootParameters.ID;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	// Ensure the schema and states have had their PostLoad() fixed applied as we may need them in the later calls (or StateTree compile which might be calling this).
	if (Schema)
	{
		Schema->ConditionalPostLoad();
	}
	if (EditorSchema)
	{
		EditorSchema->ConditionalPostLoad();
	}

	for (UStateTreeEditorDataExtension* Extension : Extensions)
	{
		if (Extension)
		{
			Extension->ConditionalPostLoad();
		}
	}

	VisitHierarchy([](UStateTreeState& State, UStateTreeState* ParentState) mutable 
	{
		State.ConditionalPostLoad();
		return EStateTreeVisitor::Continue;
	});
	CallPostLoadOnNodes();

	ReparentStates();
	FixObjectNodes();
	FixDuplicateIDs();

	RemoveInvalidBindings();

	UpdateBindings();
}

void UStateTreeEditorData::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	FProperty* Property = PropertyChangedEvent.Property;
	FProperty* MemberProperty = nullptr;
	if (PropertyChangedEvent.PropertyChain.GetActiveMemberNode())
	{
		MemberProperty = PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue();
	}

	if (MemberProperty && Property)
	{
		UStateTree* StateTree = GetTypedOuter<UStateTree>();
		checkf(StateTree, TEXT("UStateTreeEditorData should only be allocated within a UStateTree"));
		
		const FName MemberName = MemberProperty->GetFName();
		if (MemberName == GET_MEMBER_NAME_CHECKED(UStateTreeEditorData, Schema)
			|| MemberName == GET_MEMBER_NAME_CHECKED(UStateTreeEditorData, EditorSchema))
		{
			if (EditorSchema && !EditorSchema->AllowExtensions())
			{
				Extensions.Reset();
			}
			UE::StateTree::Delegates::OnSchemaChanged.Broadcast(*StateTree);
			UStateTreeEditingSubsystem::MarkAsPubliclyModified(StateTree);
		}
		else if (MemberName == GET_MEMBER_NAME_CHECKED(UStateTreeEditorData, RootParameterPropertyBag))
		{
			UE::StateTree::Delegates::OnParametersChanged.Broadcast(*StateTree);
			UStateTreeEditingSubsystem::MarkAsPubliclyModified(StateTree);
		}
		// Notify that the color data has changed and fix existing data
		else if (MemberName == GET_MEMBER_NAME_CHECKED(UStateTreeEditorData, Colors))
		{
			if (Colors.IsEmpty())
			{
				// Add default color
				Colors.Add(UE::StateTree::Editor::CreateDefaultColor());
			}
			VisitHierarchy([Self = this](UStateTreeState& State, UStateTreeState* ParentState)
				{
					if (!Self->FindColor(State.ColorRef))
					{
						State.Modify();
						State.ColorRef = FStateTreeEditorColorRef();
					}
					return EStateTreeVisitor::Continue;
				});

			UE::StateTree::Delegates::OnVisualThemeChanged.Broadcast(*StateTree);
		}
		else
		{
			UStateTreeEditingSubsystem::MarkAsModified(StateTree);
		}

		// Ensure unique ID on duplicated items.
		if (PropertyChangedEvent.ChangeType == EPropertyChangeType::Duplicate)
		{
			if (MemberName == GET_MEMBER_NAME_CHECKED(UStateTreeEditorData, Evaluators))
			{
				const int32 ArrayIndex = PropertyChangedEvent.GetArrayIndex(MemberProperty->GetFName().ToString());
				if (Evaluators.IsValidIndex(ArrayIndex))
				{
					const FGuid OldStructID = Evaluators[ArrayIndex].ID;
					Evaluators[ArrayIndex].ID = FGuid::NewGuid();
					EditorBindings.CopyBindings(OldStructID, Evaluators[ArrayIndex].ID);
				}
			}
			else if (MemberName == GET_MEMBER_NAME_CHECKED(UStateTreeEditorData, GlobalTasks))
			{
				const int32 ArrayIndex = PropertyChangedEvent.GetArrayIndex(MemberProperty->GetFName().ToString());
				if (GlobalTasks.IsValidIndex(ArrayIndex))
				{
					const FGuid OldStructID = GlobalTasks[ArrayIndex].ID;
					GlobalTasks[ArrayIndex].ID = FGuid::NewGuid();
					EditorBindings.CopyBindings(OldStructID, GlobalTasks[ArrayIndex].ID);
				}
			}
		}
		else if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayRemove)
		{
			if (MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UStateTreeEditorData, Evaluators)
				|| MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UStateTreeEditorData, GlobalTasks))
			{
				RemoveInvalidBindings();
			}
		}

		// Notify that the global data changed (will need to update binding widgets, etc)
		if (MemberName == GET_MEMBER_NAME_CHECKED(UStateTreeEditorData, Evaluators)
			|| MemberName == GET_MEMBER_NAME_CHECKED(UStateTreeEditorData, GlobalTasks))
		{
			UE::StateTree::Delegates::OnGlobalDataChanged.Broadcast(*StateTree);
		}
	}

	UE::StateTree::PropertyHelpers::DispatchPostEditToNodes(*this, PropertyChangedEvent, *this);
}

void UStateTreeEditorData::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

	EditorBindings.SetBindingsOwner(this);
	DuplicateIDs();
	UStateTreeEditingSubsystem::MarkAsPubliclyModified(GetTypedOuter<UStateTree>());
}
#endif // WITH_EDITOR

void UStateTreeEditorData::GetBindableStructs(const FGuid TargetStructID, TArray<TInstancedStruct<FPropertyBindingBindableStructDescriptor>>& OutStructDescs) const
{
	// Find the states that are updated before the current state.
	TArray<const UStateTreeState*> Path;
	const UStateTreeState* State = GetStateByStructID(TargetStructID);
	while (State != nullptr)
	{
		Path.Insert(State, 0);

		// Stop at subtree root.
		if (State->Type == EStateTreeStateType::Subtree)
		{
			break;
		}

		State = State->Parent;
	}
	
	GetAccessibleStructsInExecutionPath(Path, TargetStructID, OutStructDescs);
}

void UStateTreeEditorData::GetAccessibleStructsInExecutionPath(const TConstArrayView<const UStateTreeState*> Path, const FGuid TargetStructID, TArray<TInstancedStruct<FPropertyBindingBindableStructDescriptor>>& OutStructDescs) const
{
	const UStateTree* StateTree = GetTypedOuter<UStateTree>();
	checkf(StateTree, TEXT("UStateTreeEditorData should only be allocated within a UStateTree"));

	bool bAcceptTaskInstanceData = true;
	TInstancedStruct<FPropertyBindingBindableStructDescriptor> TargetStructDesc;
	bool bIsTargetPropertyFunction = false;
	if (GetBindableStructByID(TargetStructID, TargetStructDesc))
	{
		bIsTargetPropertyFunction = TargetStructDesc.Get<FStateTreeBindableStructDesc>().DataSource == EStateTreeBindableStructSource::PropertyFunction;
		if (!UE::StateTree::Editor::CVarLogEnableBindingSelectionNodeToInstanceData->GetBool())
		{
			bAcceptTaskInstanceData = UE::StateTree::AcceptTaskInstanceData(TargetStructDesc.Get<FStateTreeBindableStructDesc>().DataSource);
		}
	}

	EStateTreeVisitor BaseProgress = VisitGlobalNodes([&OutStructDescs, TargetStructID]
		(const UStateTreeState* State, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)
	{
		if (Desc.ID == TargetStructID)
		{
			return EStateTreeVisitor::Break;
		}
		
		OutStructDescs.Add(TInstancedStruct<FStateTreeBindableStructDesc>::Make(Desc));
		
		return EStateTreeVisitor::Continue;
	});

	if (BaseProgress == EStateTreeVisitor::Continue)
	{
		TArray<TInstancedStruct<FStateTreeBindableStructDesc>, TInlineAllocator<32>> BindableDescs;

		for (const UStateTreeState* State : Path)
		{
			if (State == nullptr)
			{
				continue;
			}
			
			const EStateTreeVisitor StateProgress = VisitStateNodes(*State,
				[&OutStructDescs, &BindableDescs, &Path, TargetStructID, bIsTargetPropertyFunction, bAcceptTaskInstanceData, this]
				(const UStateTreeState* State, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)
				{
					// Stop iterating as soon as we find the target node.
					if (Desc.ID == TargetStructID)
					{
						OutStructDescs.Append(BindableDescs);
						return EStateTreeVisitor::Break;
					}

					// Not at target yet, collect all bindable source accessible so far.
					switch (Desc.DataSource)
					{
						case EStateTreeBindableStructSource::StateParameter:
						case EStateTreeBindableStructSource::StateEvent:
							BindableDescs.Add(TInstancedStruct<FStateTreeBindableStructDesc>::Make(Desc));
							break;

						case EStateTreeBindableStructSource::Task:
							if (bAcceptTaskInstanceData)
							{
								BindableDescs.Add(TInstancedStruct<FStateTreeBindableStructDesc>::Make(Desc));
							}
							break;

						case EStateTreeBindableStructSource::TransitionEvent:
						{
							// Checking if BindableStruct's owning Transition contains the Target.
							if (State == Path.Last())
							{
								for (const FStateTreeTransition& Transition : State->Transitions)
								{
									bool bFoundOwningTransition = false;
									for (const FStateTreeEditorNode& ConditionNode : Transition.Conditions)
									{
										if (ConditionNode.ID == TargetStructID
											|| (bIsTargetPropertyFunction && UE::StateTree::Editor::IsPropertyFunctionOwnedByNode(ConditionNode.ID, TargetStructID, EditorBindings)))
										{
											if (Transition.GetEventID() == Desc.ID)
											{
												BindableDescs.Add(TInstancedStruct<FStateTreeBindableStructDesc>::Make(Desc));
											}

											bFoundOwningTransition = true;
											break;
										}
									}

									if (bFoundOwningTransition)
									{
										break;
									}
								}
							}
							break;
						}

						case EStateTreeBindableStructSource::PropertyFunction:
						{
							if (State == Path.Last())
							{
								if (UE::StateTree::Editor::IsPropertyFunctionOwnedByNode(TargetStructID, Desc.ID, EditorBindings))
								{
									BindableDescs.Add(TInstancedStruct<FStateTreeBindableStructDesc>::Make(Desc));
								}
							}
						}
					}
							
					return EStateTreeVisitor::Continue;
				});
			
			if (StateProgress == EStateTreeVisitor::Break)
			{
				break;
			}
		}
	}
}

UStateTreeEditorDataExtension* UStateTreeEditorData::K2_GetExtension(TSubclassOf<UStateTreeEditorDataExtension> InExtensionType)
{
	for (UStateTreeEditorDataExtension* Extension : Extensions)
	{
		if (Extension && Extension->IsA(InExtensionType))
		{
			return Extension;
		}
	}
	return nullptr;
}

FStateTreeBindableStructDesc UStateTreeEditorData::FindContextData(const UStruct* ObjectType, const FString ObjectNameHint) const
{
	if (Schema == nullptr)
	{
		return FStateTreeBindableStructDesc();
	}

	// Find candidates based on type.
	TArray<FStateTreeBindableStructDesc> Candidates;
	for (const FStateTreeExternalDataDesc& Desc : Schema->GetContextDataDescs())
	{
		if (Desc.Struct != nullptr
			&& Desc.Struct->IsChildOf(ObjectType))
		{
			Candidates.Emplace(UE::StateTree::Editor::GlobalStateName, Desc.Name, Desc.Struct, FStateTreeDataHandle(), EStateTreeBindableStructSource::Context, Desc.ID);
		}
	}

	// Handle trivial cases.
	if (Candidates.IsEmpty())
	{
		return FStateTreeBindableStructDesc();
	}

	if (Candidates.Num() == 1)
	{
		return Candidates[0];
	}
	
	check(!Candidates.IsEmpty());
	
	// Multiple candidates, pick one that is closest match based on name.
	auto CalculateScore = [](const FString& Name, const FString& CandidateName)
	{
		if (CandidateName.IsEmpty())
		{
			return 1.0f;
		}
		const float WorstCase = static_cast<float>(Name.Len() + CandidateName.Len());
		return 1.0f - (static_cast<float>(Algo::LevenshteinDistance(Name, CandidateName)) / WorstCase);
	};
	
	const FString ObjectNameLowerCase = ObjectNameHint.ToLower();
	
	int32 HighestScoreIndex = 0;
	float HighestScore = CalculateScore(ObjectNameLowerCase, Candidates[0].Name.ToString().ToLower());
	
	for (int32 Index = 1; Index < Candidates.Num(); Index++)
	{
		const float Score = CalculateScore(ObjectNameLowerCase, Candidates[Index].Name.ToString().ToLower());
		if (Score > HighestScore)
		{
			HighestScore = Score;
			HighestScoreIndex = Index;
		}
	}
	
	return Candidates[HighestScoreIndex];
}

EStateTreeVisitor UStateTreeEditorData::EnumerateBindablePropertyFunctionNodes(TFunctionRef<EStateTreeVisitor(const UScriptStruct* NodeStruct, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)> InFunc) const
{
	if (Schema == nullptr)
	{
		return EStateTreeVisitor::Continue;
	}

	FStateTreeEditorModule& EditorModule = FModuleManager::GetModuleChecked<FStateTreeEditorModule>(TEXT("StateTreeEditorModule"));
	FStateTreeNodeClassCache* ClassCache = EditorModule.GetNodeClassCache().Get();
	check(ClassCache);

	TArray<TSharedPtr<FStateTreeNodeClassData>> StructNodes;
	ClassCache->GetStructs(FStateTreePropertyFunctionBase::StaticStruct(), StructNodes);
	for (const TSharedPtr<FStateTreeNodeClassData>& NodeClassData : StructNodes)
	{
		if (const UScriptStruct* NodeStruct = NodeClassData->GetScriptStruct())
		{
			if (NodeStruct == FStateTreePropertyFunctionBase::StaticStruct() || NodeStruct->HasMetaData(TEXT("Hidden")))
			{
				continue;
			}

			if (Schema->IsStructAllowed(NodeStruct))
			{
				if (const UStruct* InstanceDataStruct = NodeClassData->GetInstanceDataStruct())
				{
					FStateTreeBindableStructDesc Desc;
					Desc.Struct = InstanceDataStruct;
					Desc.ID = FGuid::NewDeterministicGuid(NodeStruct->GetName());
					Desc.DataSource = EStateTreeBindableStructSource::PropertyFunction;
					Desc.Name = FName(NodeStruct->GetDisplayNameText().ToString());
					Desc.StatePath = UE::StateTree::Editor::PropertyFunctionStateName;
					Desc.Category = NodeStruct->GetMetaData(TEXT("Category"));

					if (InFunc(NodeStruct, Desc, FStateTreeDataView(InstanceDataStruct, nullptr)) == EStateTreeVisitor::Break)
					{
						return EStateTreeVisitor::Break;
					}
				}
			}
		}
	}

	return EStateTreeVisitor::Continue;
}

void UStateTreeEditorData::AppendBindablePropertyFunctionStructs(TArray<TInstancedStruct<FPropertyBindingBindableStructDescriptor>>& InOutStructs) const
{
	EnumerateBindablePropertyFunctionNodes([&InOutStructs](const UScriptStruct*, const FStateTreeBindableStructDesc& Desc, const FPropertyBindingDataView)
		{
			InOutStructs.Add(TInstancedStruct<FStateTreeBindableStructDesc>::Make(Desc));
			return EStateTreeVisitor::Continue;
		});
}

bool UStateTreeEditorData::CanCreateParameter(const FGuid StructID) const
{
	if (RootParametersGuid == StructID)
	{
		return true;
	}

	bool bFoundStructID = false;

	VisitHierarchy([&StructID, &bFoundStructID](UStateTreeState& State, UStateTreeState* ParentState)->EStateTreeVisitor
	{
		if (State.Type != EStateTreeStateType::Linked && State.Type != EStateTreeStateType::LinkedAsset && State.Parameters.ID == StructID)
		{
			// Linked State parameter is fixed. No promotion is allowed
			bFoundStructID = true;
			return EStateTreeVisitor::Break;
		}
		return EStateTreeVisitor::Continue;
	});

	return bFoundStructID;
}

void UStateTreeEditorData::CreateParametersForStruct(const FGuid StructID, TArrayView<UE::PropertyBinding::FPropertyCreationDescriptor> InOutCreationDescs)
{
	if (InOutCreationDescs.IsEmpty())
	{
		return;
	}

	UStateTree* StateTree = GetTypedOuter<UStateTree>();
	checkf(StateTree, TEXT("UStateTreeEditorData should only be allocated within a UStateTree"));

	if (RootParametersGuid == StructID)
	{
		CreateRootProperties(InOutCreationDescs);
		UE::StateTree::Delegates::OnParametersChanged.Broadcast(*StateTree);
		UStateTreeEditingSubsystem::MarkAsPubliclyModified(StateTree);
		return;
	}

	VisitHierarchy([&StructID, StateTree, &InOutCreationDescs](UStateTreeState& State, UStateTreeState* ParentState)->EStateTreeVisitor
	{
		if (State.Parameters.ID == StructID)
		{
			UE::PropertyBinding::CreateUniquelyNamedPropertiesInPropertyBag(InOutCreationDescs,State.Parameters.Parameters);
			UE::StateTree::Delegates::OnStateParametersChanged.Broadcast(*StateTree, State.ID);
			UStateTreeEditingSubsystem::MarkAsModified(StateTree);
			return EStateTreeVisitor::Break;
		}
		return EStateTreeVisitor::Continue;
	});
}

void UStateTreeEditorData::OnPropertyBindingChanged(const FPropertyBindingPath& InSourcePath, const FPropertyBindingPath& InTargetPath)
{
	UE::StateTree::PropertyBinding::OnStateTreePropertyBindingChanged.Broadcast(InSourcePath, InTargetPath);
}

bool UStateTreeEditorData::GetBindableStructByID(const FGuid StructID, TInstancedStruct<FPropertyBindingBindableStructDescriptor>& OutStructDesc) const
{
	VisitAllNodes([&OutStructDesc, StructID](const UStateTreeState* State, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)
	{
		if (Desc.ID == StructID)
		{
			OutStructDesc = TInstancedStruct<FStateTreeBindableStructDesc>::Make(Desc);
			return EStateTreeVisitor::Break;
		}
		return EStateTreeVisitor::Continue;
	});
	
	return OutStructDesc.IsValid();
}

bool UStateTreeEditorData::GetBindingDataViewByID(const FGuid StructID, FPropertyBindingDataView& OutDataView) const
{
	bool bFound = false;
	VisitAllNodes([&OutDataView, &bFound, StructID](const UStateTreeState* State, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)
	{
		if (Desc.ID == StructID)
		{
			bFound = true;
			OutDataView = Value;
			return EStateTreeVisitor::Break;
		}
		return EStateTreeVisitor::Continue;
	});

	return bFound;
}

const UStateTreeState* UStateTreeEditorData::GetStateByStructID(const FGuid TargetStructID) const
{
	const UStateTreeState* Result = nullptr;

	VisitHierarchyNodes([&Result, TargetStructID](const UStateTreeState* State, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)
		{
			if (Desc.ID == TargetStructID)
			{
				Result = State;
				return EStateTreeVisitor::Break;
			}
			return EStateTreeVisitor::Continue;
			
		});

	return Result;
}

const UStateTreeState* UStateTreeEditorData::GetStateByID(const FGuid StateID) const
{
	return const_cast<UStateTreeEditorData*>(this)->GetMutableStateByID(StateID);
}

UStateTreeState* UStateTreeEditorData::GetMutableStateByID(const FGuid StateID)
{
	UStateTreeState* Result = nullptr;
	
	VisitHierarchy([&Result, &StateID](UStateTreeState& State, UStateTreeState* /*ParentState*/)
	{
		if (State.ID == StateID)
		{
			Result = &State;
			return EStateTreeVisitor::Break;
		}

		return EStateTreeVisitor::Continue;
	});

	return Result;
}

void UStateTreeEditorData::GetAllStructValues(TMap<FGuid, const FPropertyBindingDataView>& OutAllValues) const
{
	OutAllValues.Reset();

	const UStateTree* StateTree = GetTypedOuter<UStateTree>();
	checkf(StateTree, TEXT("UStateTreeEditorData should only be allocated within a UStateTree"));

	VisitAllNodes([&OutAllValues](const UStateTreeState* State, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)
		{
			OutAllValues.Emplace(Desc.ID, Value);
			return EStateTreeVisitor::Continue;
		});
}

void UStateTreeEditorData::GetAllStructValues(TMap<FGuid, const FStateTreeDataView>& OutAllValues) const
{
	OutAllValues.Reset();

	const UStateTree* StateTree = GetTypedOuter<UStateTree>();
	checkf(StateTree, TEXT("UStateTreeEditorData should only be allocated within a UStateTree"));

	VisitAllNodes([&OutAllValues](const UStateTreeState* State, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)
		{
			OutAllValues.Emplace(Desc.ID, Value);
			return EStateTreeVisitor::Continue;
		});
}

void UStateTreeEditorData::ReparentStates()
{
	VisitHierarchy([TreeData = this](UStateTreeState& State, UStateTreeState* ParentState) mutable 
	{
		UObject* ExpectedOuter = ParentState ? Cast<UObject>(ParentState) : Cast<UObject>(TreeData);
		if (State.GetOuter() != ExpectedOuter)
		{
			UE_LOGF(LogStateTreeEditor, Log, "%ls: Fixing outer on state %ls.", *TreeData->GetFullName(), *GetNameSafe(&State));
			State.Rename(nullptr, ExpectedOuter, REN_DontCreateRedirectors | REN_DoNotDirty | REN_AllowPackageLinkerMismatch);
		}

		if (State.Parent != ParentState)
		{
			UStateTreeEditingSubsystem::MarkAsModified(TreeData->GetTypedOuter<UStateTree>());
			State.Parent = ParentState;
		}
		
		return EStateTreeVisitor::Continue;
	});
}

void UStateTreeEditorData::CreateRootProperties(TArrayView<UE::PropertyBinding::FPropertyCreationDescriptor> InOutCreationDescs)
{
	UE::PropertyBinding::CreateUniquelyNamedPropertiesInPropertyBag(InOutCreationDescs, RootParameterPropertyBag);
	UStateTreeEditingSubsystem::MarkAsPubliclyModified(GetTypedOuter<UStateTree>());
}

void UStateTreeEditorData::FixObjectNodes()
{
	// Older version of State Trees had all instances outered to the editor data. This causes issues with State copy/paste.
	// Instance data does not get duplicated but the copied state will reference the object on the source state instead.
	//
	// Ensure that all node objects are parented to their states, and make duplicated instances unique.

	TSet<UObject*> SeenObjects;
	bool bDirty = false;
	VisitHierarchy([&SeenObjects, &bDirty](UStateTreeState& State, UStateTreeState* ParentState) mutable
	{

		// Enter conditions
		for (FStateTreeEditorNode& Node : State.EnterConditions)
		{
			if (Node.FixObjectInstances(SeenObjects, &State))
			{
				bDirty = true;
			}
		}

		// Consideration Utilities
		for (FStateTreeEditorNode& Node : State.Considerations)
		{
			if (Node.FixObjectInstances(SeenObjects, &State))
			{
				bDirty = true;
			}
		}

		// Tasks
		for (FStateTreeEditorNode& Node : State.Tasks)
		{
			if (Node.FixObjectInstances(SeenObjects, &State))
			{
				bDirty = true;
			}
		}

		if (State.SingleTask.FixObjectInstances(SeenObjects, &State))
		{
			bDirty = true;
		}

		// Transitions
		for (FStateTreeTransition& Transition : State.Transitions)
		{
			for (FStateTreeEditorNode& Node : Transition.Conditions)
			{
				if (Node.FixObjectInstances(SeenObjects, &State))
				{
					bDirty = true;
				}
			}
		}

		return EStateTreeVisitor::Continue;
	});

	for (FStateTreeEditorNode& Node : Evaluators)
	{
		if (Node.FixObjectInstances(SeenObjects, this))
		{
			bDirty = true;
		}
	}

	for (FStateTreeEditorNode& Node : GlobalTasks)
	{
		if (Node.FixObjectInstances(SeenObjects, this))
		{
			bDirty = true;
		}
	}

	if (bDirty)
	{
		UStateTreeEditingSubsystem::MarkAsModified(GetTypedOuter<UStateTree>());
	}
}

FText UStateTreeEditorData::GetNodeDescription(const FStateTreeEditorNode& Node, const EStateTreeNodeFormatting Formatting) const
{
	if (const FStateTreeNodeBase* NodePtr = Node.GetNode().GetPtr<FStateTreeNodeBase>())
	{
		// If the node has name override, return it.
		if (!NodePtr->Name.IsNone())
		{
			return FText::FromName(NodePtr->Name);
		}

		// If the node has automatic description, return it.
		const FStateTreeBindingLookup BindingLookup(this);
		const FStateTreeDataView InstanceData = Node.GetInstance();
		if (InstanceData.IsValid())
		{
			
			const FText Description = NodePtr->GetDescription(Node.ID, InstanceData, BindingLookup, Formatting);
			if (!Description.IsEmpty())
			{
				return Description;
			}
		}

		// As last resort, return node's display name.
		check(Node.GetNode().GetScriptStruct());
		return Node.GetNode().GetScriptStruct()->GetDisplayNameText();
	}

	// The node is not initialized.
	return LOCTEXT("EmptyNode", "None");
}

void UStateTreeEditorData::FixDuplicateIDs()
{
	// Around version 5.1-5.3 we had issue that copy/paste or some duplication methods could create nodes with duplicate IDs.
	// This code tries to fix that, it looks for duplicates, makes them unique, and duplicates the bindings when ID changes.
	TSet<FGuid> FoundNodeIDs;
	bool bDirty = false;

	// Evaluators
	for (int32 Index = 0; Index < Evaluators.Num(); Index++)
	{
		FStateTreeEditorNode& Node = Evaluators[Index];
		if (const FStateTreeEvaluatorBase* Evaluator = Node.GetNode().GetPtr<FStateTreeEvaluatorBase>())
		{
			const FGuid OldID = Node.ID; 
			if (FoundNodeIDs.Contains(Node.ID))
			{
				Node.ID = UE::StateTree::PropertyHelpers::MakeDeterministicID(*this, TEXT("Evaluators"), Index);
				
				UE_LOGF(LogStateTreeEditor, Log, "%ls: Found Evaluator '%ls' with duplicate ID, changing ID:%ls to ID:%ls.",
					*GetFullName(), *Node.GetName().ToString(), *OldID.ToString(), *Node.ID.ToString());
				EditorBindings.CopyBindings(OldID, Node.ID);
				bDirty = true;
			}
			FoundNodeIDs.Add(Node.ID);
		}
	}
	
	// Global Tasks
	for (int32 Index = 0; Index < GlobalTasks.Num(); Index++)
	{
		FStateTreeEditorNode& Node = GlobalTasks[Index];
		if (const FStateTreeTaskBase* Task = Node.GetNode().GetPtr<FStateTreeTaskBase>())
		{
			const FGuid OldID = Node.ID;
			if (FoundNodeIDs.Contains(Node.ID))
			{
				Node.ID = UE::StateTree::PropertyHelpers::MakeDeterministicID(*this, TEXT("GlobalTasks"), Index);
				
				UE_LOGF(LogStateTreeEditor, Log, "%ls: Found GlobalTask '%ls' with duplicate ID, changing ID:%ls to ID:%ls.",
					*GetFullName(), *Node.GetName().ToString(), *OldID.ToString(), *Node.ID.ToString());
				EditorBindings.CopyBindings(OldID, Node.ID);
				bDirty = true;
			}
			FoundNodeIDs.Add(Node.ID);
		}
	}
	
	VisitHierarchy([&bDirty, &FoundNodeIDs, &EditorBindings = EditorBindings, &Self = *this](UStateTreeState& State, UStateTreeState* ParentState)
	{
		// Enter conditions
		for (int32 Index = 0; Index < State.EnterConditions.Num(); Index++)
		{
			FStateTreeEditorNode& Node = State.EnterConditions[Index];
			if (const FStateTreeConditionBase* Cond = Node.GetNode().GetPtr<FStateTreeConditionBase>())
			{
				const FGuid OldID = Node.ID;

				bool bIsAlreadyInSet = false;
				FoundNodeIDs.Add(Node.ID, &bIsAlreadyInSet);
				if (bIsAlreadyInSet)
				{
					Node.ID = UE::StateTree::PropertyHelpers::MakeDeterministicID(State, TEXT("EnterConditions"), Index);
					
					UE_LOGF(LogStateTreeEditor, Log, "%ls: Found Enter Condition '%ls' with duplicate ID on state '%ls', changing ID:%ls to ID:%ls.",
						*Self.GetFullName(), *Node.GetName().ToString(), *GetNameSafe(&State), *OldID.ToString(), *Node.ID.ToString());
					EditorBindings.CopyBindings(OldID, Node.ID);
					bDirty = true;
				}
			}
		}

		// Consideration Utilities
		for (int32 Index = 0; Index < State.Considerations.Num(); Index++)
		{
			FStateTreeEditorNode& Node = State.Considerations[Index];
			if (const FStateTreeConsiderationBase* Cond = Node.GetNode().GetPtr<FStateTreeConsiderationBase>())
			{
				const FGuid OldID = Node.ID;
				
				bool bIsAlreadyInSet = false;
				FoundNodeIDs.Add(Node.ID, &bIsAlreadyInSet);
				if (bIsAlreadyInSet)
				{
					Node.ID = UE::StateTree::PropertyHelpers::MakeDeterministicID(State, TEXT("Consideration"), Index);
					
					UE_LOGF(LogStateTreeEditor, Log, "%ls: Found Consideration '%ls' with duplicate ID on state '%ls', changing ID:%ls to ID:%ls.",
						*Self.GetFullName(), *Node.GetName().ToString(), *GetNameSafe(&State), *OldID.ToString(), *Node.ID.ToString());
					EditorBindings.CopyBindings(OldID, Node.ID);
					bDirty = true;
				}
			}
		}

		// Tasks
		for (int32 Index = 0; Index < State.Tasks.Num(); Index++)
		{
			FStateTreeEditorNode& Node = State.Tasks[Index];
			if (const FStateTreeTaskBase* Task = Node.GetNode().GetPtr<FStateTreeTaskBase>())
			{
				const FGuid OldID = Node.ID;

				bool bIsAlreadyInSet = false;
				FoundNodeIDs.Add(Node.ID, &bIsAlreadyInSet);
				if (bIsAlreadyInSet)
				{
					Node.ID = UE::StateTree::PropertyHelpers::MakeDeterministicID(State, TEXT("Tasks"), Index);

					UE_LOGF(LogStateTreeEditor, Log, "%ls: Found Task '%ls' with duplicate ID on state '%ls', changing ID:%ls to ID:%ls.",
						*Self.GetFullName(), *Node.GetName().ToString(), *GetNameSafe(&State), *OldID.ToString(), *Node.ID.ToString());
					EditorBindings.CopyBindings(OldID, Node.ID);
					bDirty = true;
				}
			}
		}

		if (FStateTreeTaskBase* Task = State.SingleTask.GetNode().GetPtr<FStateTreeTaskBase>())
		{
			const FGuid OldID = State.SingleTask.ID;

			bool bIsAlreadyInSet = false;
			FoundNodeIDs.Add(State.SingleTask.ID, &bIsAlreadyInSet);
			if (bIsAlreadyInSet)
			{
				State.SingleTask.ID = UE::StateTree::PropertyHelpers::MakeDeterministicID(State, TEXT("SingleTask"), 0);

				UE_LOGF(LogStateTreeEditor, Log, "%ls: Found enter condition '%ls' with duplicate ID on state '%ls', changing ID:%ls to ID:%ls.",
					*Self.GetFullName(), *State.SingleTask.GetName().ToString(), *GetNameSafe(&State), *OldID.ToString(), *State.SingleTask.ID.ToString());
				EditorBindings.CopyBindings(OldID, State.SingleTask.ID);
				bDirty = true;
			}
		}

		// Transitions
		for (int32 TransitionIndex = 0; TransitionIndex < State.Transitions.Num(); TransitionIndex++)
		{
			FStateTreeTransition& Transition = State.Transitions[TransitionIndex];
			for (int32 Index = 0; Index < Transition.Conditions.Num(); Index++)
			{
				FStateTreeEditorNode& Node = Transition.Conditions[Index];
				if (const FStateTreeConditionBase* Cond = Node.GetNode().GetPtr<FStateTreeConditionBase>())
				{
					const FGuid OldID = Node.ID;
					bool bIsAlreadyInSet = false;
					FoundNodeIDs.Add(Node.ID, &bIsAlreadyInSet);
					if (bIsAlreadyInSet)
					{
						Node.ID = UE::StateTree::PropertyHelpers::MakeDeterministicID(State, TEXT("TransitionConditions"), ((uint64)TransitionIndex << 32) | (uint64)Index);

						UE_LOGF(LogStateTreeEditor, Log, "%ls: Found transition condition '%ls' with duplicate ID on state '%ls', changing ID:%ls to ID:%ls.",
							*Self.GetFullName(), *Node.GetName().ToString(), *GetNameSafe(&State), *OldID.ToString(), *Node.ID.ToString());
						EditorBindings.CopyBindings(OldID, Node.ID);
						bDirty = true;
					}
				}
			}
		}
		
		return EStateTreeVisitor::Continue;
	});

	if (bDirty)
	{
		UStateTreeEditingSubsystem::MarkAsModified(GetTypedOuter<UStateTree>());
	}
}

void UStateTreeEditorData::DuplicateIDs()
{
	TMap<FGuid, FGuid> OldToNewIDs;

	// Visit and create new ids
	{
		auto AddId = [&OldToNewIDs](const FGuid& OldID, bool bTestIfContains)
		{
			ensureAlwaysMsgf(bTestIfContains == false || !OldToNewIDs.Contains(OldID), TEXT("The id is duplicated and FixDuplicateIDs failed to fix it."));

			if (OldID.IsValid())
			{
				const FGuid NewID = FGuid::NewGuid();
				OldToNewIDs.Add(OldID, NewID);
			}
		};

		VisitAllNodes([&AddId](const UStateTreeState* State, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)
			{
				if (State)
				{
					AddId(State->ID, false);
					AddId(State->GetEventID(), false);
				}

				// Schema should not be included in OldToNewIDs
				if(Desc.DataSource != EStateTreeBindableStructSource::Context)
				{
					AddId(Desc.ID, true);
				}

				return EStateTreeVisitor::Continue;
			});
	}

	// Remap ids properties to the new generated ids
	{
		bool bDirty = false;
		TArray<const UObject*> ObjectToSearch;
		TSet<const UObject*> ObjectSearched;
		ObjectToSearch.Add(this);
		ObjectSearched.Add(this);
		while (ObjectToSearch.Num())
		{
			const UObject* CurrentObject = ObjectToSearch.Pop();
			for (TPropertyValueIterator<FProperty> It(CurrentObject->GetClass(), CurrentObject, EPropertyValueIteratorFlags::FullRecursion, EFieldIteratorFlags::ExcludeDeprecated); It; ++It)
			{
				const FProperty* Property = It.Key();
				if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
				{
					if (StructProperty->Struct == TBaseStructure<FGuid>::Get())
					{
						// Skip the guid properties.
						It.SkipRecursiveProperty();

						// Modify the value if needed.
						// We const_cast because we don't change the layout and the iterator advance is still going to work.
						FGuid* GuidValue = static_cast<FGuid*>(const_cast<void*>(It.Value()));
						if (FGuid* NewGuidValue = OldToNewIDs.Find(*GuidValue))
						{
							if (*GuidValue != *NewGuidValue)
							{
								*GuidValue = *NewGuidValue;
								bDirty = true;
							}
						}
					}
				}
				else if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
				{
					if (const UObject* ObjectValue = ObjectProperty->GetPropertyValue(It.Value()))
					{
						if (!ObjectSearched.Contains(ObjectValue))
						{
							// Add the inner properties of this instanced object.
							bool bAddObject = false;
							if (Property->HasAllPropertyFlags(CPF_ExportObject) && ObjectValue->GetClass()->HasAllClassFlags(CLASS_EditInlineNew))
							{
								bAddObject = true;
							}
							else if (Property->HasAllPropertyFlags(CPF_InstancedReference))
							{
								bAddObject = true;
							}

							if (bAddObject)
							{
								ObjectSearched.Add(ObjectValue);
								ObjectToSearch.Add(ObjectValue);
							}
						}
					}
				}
			}
		}

		if (bDirty)
		{
			UStateTreeEditingSubsystem::MarkAsModified(GetTypedOuter<UStateTree>());
		}
	}
}

void UStateTreeEditorData::UpdateBindings()
{
	TMap<FGuid, const FStateTreeDataView> AllValues;
	GetAllStructValues(AllValues);

	TArray<TNotNull<const UStruct*>, TInlineAllocator<32>> BindingsDependentStructs;

	TArray<FPropertyBindingPathIndirection, TInlineAllocator<16>> Indirections;

	auto UpdateBindingPathAndAddDependencies = [&AllValues, &Indirections, &BindingsDependentStructs](FPropertyBindingPath& InBindingPath)
		{
			constexpr bool bHandleRedirects = true;
			constexpr FString* Error = nullptr;

			// For failed to resolve bindings, we need to log dependency on structs found so far
			constexpr bool bResetIndirectionsIfFailed = false;

			if (AllValues.Contains(InBindingPath.GetStructID()))
			{
				// Will reset Indirection 
				if (InBindingPath.ResolveIndirectionsWithValue(AllValues[InBindingPath.GetStructID()], Indirections, Error, bHandleRedirects, bResetIndirectionsIfFailed))
				{
					InBindingPath.UpdateSegmentsFromIndirections(Indirections);
				}

				for (const FPropertyBindingPathIndirection& Indirection : Indirections)
				{
					if (const UStruct* InstanceStruct = Indirection.GetInstanceStruct())
					{
						BindingsDependentStructs.Emplace(InstanceStruct);
					}

					if (const UStruct* ContainerStruct = Indirection.GetContainerStruct())
					{
						BindingsDependentStructs.Emplace(ContainerStruct);
					}
				}
			}
		};

	for (FPropertyBindingBinding& Binding : EditorBindings.GetMutableBindings())
	{
		UpdateBindingPathAndAddDependencies(Binding.GetMutableSourcePath());
		UpdateBindingPathAndAddDependencies(Binding.GetMutableTargetPath());
	}

	EditorBindings.CacheDependency(BindingsDependentStructs);

	UE::StateTree::Delegates::Private::OnStateTreeEditorBindingUpdated.ExecuteIfBound(this);
}

void UStateTreeEditorData::RemoveInvalidBindings()
{
	TMap<FGuid, const FPropertyBindingDataView> AllStructValues;
	GetAllStructValues(AllStructValues);

	FPropertyBindingBindingCollection::EInvalidBindingCriteria Criteria = 
		FPropertyBindingBindingCollection::EInvalidBindingCriteria::RemoveMissingTargetStruct
		| FPropertyBindingBindingCollection::EInvalidBindingCriteria::RemoveFailedToResolveTargetPath;

	if (UE::StateTree::Editor::CVarAutoRemoveSourcePathInvalidPropertyBindings->GetBool())
	{
		Criteria |= FPropertyBindingBindingCollection::EInvalidBindingCriteria::RemoveFailedToResolveSourcePath;
	}

	EditorBindings.RemoveInvalidBindings(AllStructValues, Criteria);
}

void UStateTreeEditorData::CallPostLoadOnNodes()
{
	for (FStateTreeEditorNode& EvaluatorEditorNode : Evaluators)
	{
		if (FStateTreeNodeBase* EvaluatorNode = EvaluatorEditorNode.GetNode().GetPtr<FStateTreeNodeBase>())
		{
			UE::StateTreeEditor::EditorNodeUtils::ConditionalUpdateNodeInstanceData(EvaluatorEditorNode, *this);
			EvaluatorNode->PostLoad(EvaluatorEditorNode.GetInstance());
		}
	}

	for (FStateTreeEditorNode& GlobalTaskEditorNode : GlobalTasks)
	{
		if (FStateTreeNodeBase* GlobalTaskNode = GlobalTaskEditorNode.GetNode().GetPtr<FStateTreeNodeBase>())
		{
			UE::StateTreeEditor::EditorNodeUtils::ConditionalUpdateNodeInstanceData(GlobalTaskEditorNode, *this);
			GlobalTaskNode->PostLoad(GlobalTaskEditorNode.GetInstance());
		}
	}
}

EStateTreeVisitor UStateTreeEditorData::VisitStateNodes(const UStateTreeState& State, TFunctionRef<EStateTreeVisitor(const UStateTreeState* State, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)> InFunc) const
{
	auto VisitFuncNodes = [&InFunc, &State, this](const FGuid StructID, const FName NodeName)
	{
		const FString StatePath = FString::Printf(TEXT("%s/%s"), *State.GetPath(), *NodeName.ToString());
		return VisitStructBoundPropertyFunctions(StructID, StatePath, [&InFunc, &State](const FStateTreeEditorNode& EditorNode, const FStateTreeBindableStructDesc& Desc, FStateTreeDataView Value)
		{
			return InFunc(&State, Desc, Value);
		});
	};

	bool bContinue = true;

	const FString StatePath = State.GetPath();
	
	if (bContinue)
	{
		check(State.Parameters.ID.IsValid());
		// Bindable state parameters
		if (VisitFuncNodes(State.Parameters.ID, UE::StateTree::Editor::ParametersNodeName) == EStateTreeVisitor::Break)
		{
			bContinue = false;
		}
		else
		{
			FStateTreeBindableStructDesc Desc;
			Desc.StatePath = StatePath;
			Desc.Struct = State.Parameters.Parameters.GetPropertyBagStruct();
			Desc.Name = FName("Parameters");
			Desc.ID = State.Parameters.ID;
			Desc.DataSource = EStateTreeBindableStructSource::StateParameter;

			if (InFunc(&State, Desc, FStateTreeDataView(const_cast<FInstancedPropertyBag&>(State.Parameters.Parameters).GetMutableValue())) == EStateTreeVisitor::Break)
			{
				bContinue = false;
			}
		}
	}

	const FString StatePathWithConditions = StatePath + TEXT("/EnterConditions");
	
	if (bContinue)
	{
		if (State.bHasRequiredEventToEnter )
		{
			FStateTreeBindableStructDesc Desc;
			Desc.StatePath = StatePathWithConditions;
			Desc.Struct = FStateTreeEvent::StaticStruct();
			Desc.Name = FName("Enter Event");
			Desc.ID = State.GetEventID();
			Desc.DataSource = EStateTreeBindableStructSource::StateEvent;

			if (InFunc(&State, Desc, FStateTreeDataView(FStructView::Make(const_cast<UStateTreeState&>(State).RequiredEventToEnter.GetTemporaryEvent()))) == EStateTreeVisitor::Break)
			{
				bContinue = false;
			}
		}
	}

	if (bContinue)
	{
		// Enter conditions
		for (const FStateTreeEditorNode& Node : State.EnterConditions)
		{
			if (VisitFuncNodes(Node.ID, Node.GetName()) == EStateTreeVisitor::Break)
			{
				bContinue = false;
				break;
			}
			else if (const FStateTreeConditionBase* Cond = Node.GetNode().GetPtr<FStateTreeConditionBase>())
			{
				FStateTreeBindableStructDesc Desc;
				Desc.StatePath = StatePathWithConditions;
				Desc.Struct = Cond->GetInstanceDataType();
				Desc.Name = Node.GetName();
				Desc.ID = Node.ID;
				Desc.DataSource = EStateTreeBindableStructSource::Condition;

				if (InFunc(&State, Desc, Node.GetInstance()) == EStateTreeVisitor::Break)
				{
					bContinue = false;
					break;
				}

				FStateTreeBindableStructDesc NodeDesc;
				NodeDesc.StatePath = StatePathWithConditions;
				NodeDesc.Struct = Node.GetNode().GetScriptStruct();
				NodeDesc.Name = Node.GetName();
				NodeDesc.ID = Node.GetNodeID();
				NodeDesc.DataSource = EStateTreeBindableStructSource::Condition;

				if (InFunc(&State, NodeDesc, Node.GetNode()) == EStateTreeVisitor::Break)
				{
					bContinue = false;
					break;
				}
			}
		}
	}
	if (bContinue)
	{
		const FString StatePathWithConsiderations = StatePath + TEXT("/Considerations");
		// Utility Considerations
		for (const FStateTreeEditorNode& Node : State.Considerations)
		{
			if (VisitFuncNodes(Node.ID, Node.GetName()) == EStateTreeVisitor::Break)
			{
				bContinue = false;
				break;
			}
			else if (const FStateTreeConsiderationBase* Consideration = Node.GetNode().GetPtr<FStateTreeConsiderationBase>())
			{
				FStateTreeBindableStructDesc Desc;
				Desc.StatePath = StatePathWithConsiderations;
				Desc.Struct = Consideration->GetInstanceDataType();
				Desc.Name = Node.GetName();
				Desc.ID = Node.ID;
				Desc.DataSource = EStateTreeBindableStructSource::Consideration;

				if (InFunc(&State, Desc, Node.GetInstance()) == EStateTreeVisitor::Break)
				{
					bContinue = false;
					break;
				}

				FStateTreeBindableStructDesc NodeDesc;
				NodeDesc.StatePath = StatePathWithConsiderations;
				NodeDesc.Struct = Node.GetNode().GetScriptStruct();
				NodeDesc.Name = Node.GetName();
				NodeDesc.ID = Node.GetNodeID();
				NodeDesc.DataSource = EStateTreeBindableStructSource::Consideration;

				if (InFunc(&State, NodeDesc, Node.GetNode()) == EStateTreeVisitor::Break)
				{
					bContinue = false;
					break;
				}
			}
		}
	}
	if (bContinue)
	{
		// Tasks
		for (const FStateTreeEditorNode& Node : State.Tasks)
		{
			if (VisitFuncNodes(Node.ID, Node.GetName()) == EStateTreeVisitor::Break)
			{
				bContinue = false;
				break;
			}
			else if (const FStateTreeTaskBase* Task = Node.GetNode().GetPtr<FStateTreeTaskBase>())
			{
				FStateTreeBindableStructDesc Desc;
				Desc.StatePath = StatePath;
				Desc.Struct = Task->GetInstanceDataType();
				Desc.Name = Node.GetName();
				Desc.ID = Node.ID;
				Desc.DataSource = EStateTreeBindableStructSource::Task;

				if (InFunc(&State, Desc, Node.GetInstance()) == EStateTreeVisitor::Break)
				{
					bContinue = false;
					break;
				}

				FStateTreeBindableStructDesc NodeDesc;
				NodeDesc.StatePath = StatePath;
				NodeDesc.Struct = Node.GetNode().GetScriptStruct();
				NodeDesc.Name = Node.GetName();
				NodeDesc.ID = Node.GetNodeID();
				NodeDesc.DataSource = EStateTreeBindableStructSource::Task;

				if (InFunc(&State, NodeDesc, Node.GetNode()) == EStateTreeVisitor::Break)
				{
					bContinue = false;
					break;
				}
			}
		}
	}
	if (bContinue)
	{
		if (const FStateTreeTaskBase* Task = State.SingleTask.GetNode().GetPtr<FStateTreeTaskBase>())
		{
			if (VisitFuncNodes(State.SingleTask.ID, State.SingleTask.GetName()) == EStateTreeVisitor::Break)
			{
				bContinue = false;
			}
			else
			{
				FStateTreeBindableStructDesc Desc;
				Desc.StatePath = StatePath;
				Desc.Struct = Task->GetInstanceDataType();
				Desc.Name = State.SingleTask.GetName();
				Desc.ID = State.SingleTask.ID;
				Desc.DataSource = EStateTreeBindableStructSource::Task;

				if (InFunc(&State, Desc, State.SingleTask.GetInstance()) == EStateTreeVisitor::Break)
				{
					bContinue = false;
				}

				FStateTreeBindableStructDesc NodeDesc;
				NodeDesc.StatePath = StatePath;
				NodeDesc.Struct = State.SingleTask.GetNode().GetScriptStruct();
				NodeDesc.Name = State.SingleTask.GetName();
				NodeDesc.ID = State.SingleTask.GetNodeID();
				NodeDesc.DataSource = EStateTreeBindableStructSource::Task;

				if (InFunc(&State, NodeDesc, State.SingleTask.GetNode()) == EStateTreeVisitor::Break)
				{
					bContinue = false;
				}
			}
		}

	}
	if (bContinue)
	{
		// Transitions
		for (int32 TransitionIndex = 0; TransitionIndex < State.Transitions.Num(); TransitionIndex++)
		{
			const FStateTreeTransition& Transition = State.Transitions[TransitionIndex];
			const FString StatePathWithTransition = StatePath + FString::Printf(TEXT("/Transition[%d]"), TransitionIndex);

			{
				FStateTreeBindableStructDesc Desc;
				Desc.StatePath = StatePathWithTransition;
				Desc.Struct = FStateTreeTransition::StaticStruct();
				Desc.Name = FName(TEXT("Transition"));
				Desc.ID = Transition.ID;
				Desc.DataSource = EStateTreeBindableStructSource::Transition;

				if (InFunc(&State, Desc, FStateTreeDataView(FStructView::Make(const_cast<FStateTreeTransition&>(Transition)))) == EStateTreeVisitor::Break)
				{
					bContinue = false;
					break;
				}
			}

			if (Transition.Trigger == EStateTreeTransitionTrigger::OnEvent)
			{
				FStateTreeBindableStructDesc Desc;
				Desc.StatePath = StatePathWithTransition;
				Desc.Struct = FStateTreeEvent::StaticStruct();
				Desc.Name = FName(TEXT("Transition Event"));
				Desc.ID = Transition.GetEventID();
				Desc.DataSource = EStateTreeBindableStructSource::TransitionEvent;

				if (InFunc(&State, Desc, FStateTreeDataView(FStructView::Make(const_cast<FStateTreeTransition&>(Transition).RequiredEvent.GetTemporaryEvent()))) == EStateTreeVisitor::Break)
				{
					bContinue = false;
					break;
				}
			}

			for (const FStateTreeEditorNode& Node : Transition.Conditions)
			{
				if (VisitFuncNodes(Node.ID, Node.GetName()) == EStateTreeVisitor::Break)
				{
					bContinue = false;
					break;
				}
				else if (const FStateTreeConditionBase* Cond = Node.GetNode().GetPtr<FStateTreeConditionBase>())
				{
					FStateTreeBindableStructDesc Desc;
					Desc.StatePath = StatePathWithTransition;
					Desc.Struct = Cond->GetInstanceDataType();
					Desc.Name = Node.GetName();
					Desc.ID = Node.ID;
					Desc.DataSource = EStateTreeBindableStructSource::Condition;

					if (InFunc(&State, Desc, Node.GetInstance()) == EStateTreeVisitor::Break)
					{
						bContinue = false;
						break;
					}

					FStateTreeBindableStructDesc NodeDesc;
					NodeDesc.StatePath = StatePathWithTransition;
					NodeDesc.Struct = Node.GetNode().GetScriptStruct();
					NodeDesc.Name = Node.GetName();
					NodeDesc.ID = Node.GetNodeID();
					NodeDesc.DataSource = EStateTreeBindableStructSource::Condition;

					if (InFunc(&State, NodeDesc, Node.GetNode()) == EStateTreeVisitor::Break)
					{
						bContinue = false;
						break;
					}
				}
			}
		}
	}

	return bContinue ? EStateTreeVisitor::Continue : EStateTreeVisitor::Break;
}

EStateTreeVisitor UStateTreeEditorData::VisitStructBoundPropertyFunctions(FGuid StructID, const FString& StatePath, TFunctionRef<EStateTreeVisitor(const FStateTreeEditorNode& EditorNode, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)> InFunc) const
{
	TArray<const FPropertyBindingBinding*> Bindings;
	EditorBindings.FPropertyBindingBindingCollection::GetBindingsFor(StructID, Bindings);

	for (const FPropertyBindingBinding* Binding : Bindings)
	{
		const FConstStructView FunctionNodeView = Binding->GetPropertyFunctionNode();
		if (const FStateTreeEditorNode* FunctionNode = FunctionNodeView.GetPtr<const FStateTreeEditorNode>())
		{
			if (VisitStructBoundPropertyFunctions(FunctionNode->ID, StatePath, InFunc) == EStateTreeVisitor::Break)
			{
				return EStateTreeVisitor::Break;
			}

			if (const UStruct* NodeStruct = FunctionNode->GetNode().GetScriptStruct())
			{
				FStateTreeBindableStructDesc Desc;
				Desc.Struct = FunctionNode->GetInstance().GetStruct();
				Desc.ID = FunctionNode->ID;
				Desc.DataSource = EStateTreeBindableStructSource::PropertyFunction;
				Desc.Name = FName(NodeStruct->GetDisplayNameText().ToString());
				Desc.StatePath = FString::Printf(TEXT("%s/%s"), *StatePath, *UE::StateTree::Editor::PropertyFunctionStateName);

				if (InFunc(*FunctionNode, Desc, FunctionNode->GetInstance()) == EStateTreeVisitor::Break)
				{
					return EStateTreeVisitor::Break;
				}

				FStateTreeBindableStructDesc NodeDesc;
				NodeDesc.Struct = FunctionNode->GetNode().GetScriptStruct();
				NodeDesc.ID = FunctionNode->GetNodeID();
				NodeDesc.DataSource = EStateTreeBindableStructSource::PropertyFunction;
				NodeDesc.Name = FName(NodeStruct->GetDisplayNameText().ToString());
				NodeDesc.StatePath = FString::Printf(TEXT("%s/%s"), *StatePath, *UE::StateTree::Editor::PropertyFunctionStateName);

				if (InFunc(*FunctionNode, NodeDesc, FunctionNode->GetNode()) == EStateTreeVisitor::Break)
				{
					return EStateTreeVisitor::Break;
				}
			}
		}
	}

	return EStateTreeVisitor::Continue;
}

EStateTreeVisitor UStateTreeEditorData::VisitStateBindings(TNotNull<const UStateTreeState*> State,
	TFunctionRef<EStateTreeVisitor(TNotNull<const UStateTreeState*> State, const FStateTreePropertyPathBinding& Binding, const
	FStateTreeBindableStructDesc& BindingTargetNodeDesc, const FStateTreeDataView BindingTargetNodeValue)> InFunc) const
{
	return VisitStateNodes(*State, [&InFunc, Self = this](const UStateTreeState* State, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)
		{
			check(State);
			EStateTreeVisitor VisitResult = EStateTreeVisitor::Continue;

			Self->EditorBindings.VisitBindings([&InFunc, &Desc, &Value, State, &VisitResult](const FPropertyBindingBinding& Binding)
				{
					const FStateTreePropertyPathBinding& StateTreeBinding = static_cast<const FStateTreePropertyPathBinding&>(Binding);
					if (Desc.ID == StateTreeBinding.GetTargetPath().GetStructID())
					{
						if (InFunc(State, static_cast<const FStateTreePropertyPathBinding&>(Binding), Desc, Value) == EStateTreeVisitor::Break)
						{
							VisitResult = EStateTreeVisitor::Break;
							return FPropertyBindingBindingCollection::EVisitResult::Break;
						}
					}

					return FPropertyBindingBindingCollection::EVisitResult::Continue;
				});

			return VisitResult;
		});
}

EStateTreeVisitor UStateTreeEditorData::VisitHierarchy(TFunctionRef<EStateTreeVisitor(UStateTreeState& State, UStateTreeState* ParentState)> InFunc) const
{
	using FStatePair = TTuple<UStateTreeState*, UStateTreeState*>; 
	TArray<FStatePair> Stack;
	bool bContinue = true;

	for (UStateTreeState* SubTree : SubTrees)
	{
		if (!SubTree)
		{
			continue;
		}

		Stack.Add( FStatePair(nullptr, SubTree));

		while (!Stack.IsEmpty() && bContinue)
		{
			FStatePair Current = Stack[0];
			UStateTreeState* ParentState = Current.Get<0>();
			UStateTreeState* State = Current.Get<1>();
			check(State);

			Stack.RemoveAt(0);

			bContinue = InFunc(*State, ParentState) == EStateTreeVisitor::Continue;
			
			if (bContinue)
			{
				// Children
				for (UStateTreeState* ChildState : State->Children)
				{
					Stack.Add(FStatePair(State, ChildState));
				}
			}
		}
		
		if (!bContinue)
		{
			break;
		}
	}

	return EStateTreeVisitor::Continue;
}

EStateTreeVisitor UStateTreeEditorData::VisitGlobalNodes(TFunctionRef<EStateTreeVisitor(const UStateTreeState* State, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)> InFunc) const
{
	// Root parameters
	{
		FStateTreeBindableStructDesc Desc;
		Desc.StatePath = UE::StateTree::Editor::GlobalStateName;
		Desc.Struct = GetRootParametersPropertyBag().GetPropertyBagStruct();
		Desc.Name = FName(TEXT("Parameters"));
		Desc.ID = RootParametersGuid;
		Desc.DataSource = EStateTreeBindableStructSource::Parameter;
		
		if (InFunc(nullptr, Desc, FStateTreeDataView(const_cast<FInstancedPropertyBag&>(GetRootParametersPropertyBag()).GetMutableValue())) == EStateTreeVisitor::Break)
		{
			return EStateTreeVisitor::Break;
		}
	}

	// All named external data items declared by the schema
	if (Schema != nullptr)
	{
		for (const FStateTreeExternalDataDesc& ContextDesc : Schema->GetContextDataDescs())
		{
			if (ContextDesc.Struct)
			{
				FStateTreeBindableStructDesc Desc;
				Desc.StatePath = UE::StateTree::Editor::GlobalStateName;
				Desc.Struct = ContextDesc.Struct;
				Desc.Name = ContextDesc.Name;
				Desc.ID = ContextDesc.ID;
				Desc.DataSource = EStateTreeBindableStructSource::Context;

				// We don't have value for the external objects, but return the type and null value so that users of GetAllStructValues() can use the type.
				if (InFunc(nullptr, Desc, FStateTreeDataView(Desc.Struct, nullptr)) == EStateTreeVisitor::Break)
				{
					return EStateTreeVisitor::Break;
				}
			}
		}
	}

	auto VisitFuncNodesFunc = [&InFunc, this](const FStateTreeEditorNode& Node)
	{
		const FString StatePath = FString::Printf(TEXT("%s/%s"), *UE::StateTree::Editor::GlobalStateName, *Node.GetName().ToString());
		return VisitStructBoundPropertyFunctions(Node.ID, StatePath, [&InFunc](const FStateTreeEditorNode& EditorNode, const FStateTreeBindableStructDesc& Desc, FStateTreeDataView Value)
		{
			return InFunc(nullptr, Desc, Value);
		});
	};

	// Evaluators
	for (const FStateTreeEditorNode& Node : Evaluators)
	{
		if (VisitFuncNodesFunc(Node) == EStateTreeVisitor::Break)
		{
			return EStateTreeVisitor::Break;
		}

		if (const FStateTreeEvaluatorBase* Evaluator = Node.GetNode().GetPtr<FStateTreeEvaluatorBase>())
		{
			FStateTreeBindableStructDesc Desc;
			Desc.StatePath = UE::StateTree::Editor::GlobalStateName;
			Desc.Struct = Evaluator->GetInstanceDataType();
			Desc.Name = Node.GetName();
			Desc.ID = Node.ID;
			Desc.DataSource = EStateTreeBindableStructSource::Evaluator;

			if (InFunc(nullptr, Desc, Node.GetInstance()) == EStateTreeVisitor::Break)
			{
				return EStateTreeVisitor::Break;
			}

			FStateTreeBindableStructDesc NodeDesc;
			NodeDesc.StatePath = UE::StateTree::Editor::GlobalStateName;
			NodeDesc.Struct = Node.GetNode().GetScriptStruct();
			NodeDesc.Name = Node.GetName();
			NodeDesc.ID = Node.GetNodeID();
			NodeDesc.DataSource = EStateTreeBindableStructSource::Evaluator;

			if (InFunc(nullptr, NodeDesc, Node.GetNode()) == EStateTreeVisitor::Break)
			{
				return EStateTreeVisitor::Break;
			}
		}
	}

	// Global tasks
	for (const FStateTreeEditorNode& Node : GlobalTasks)
	{
		if (VisitFuncNodesFunc(Node) == EStateTreeVisitor::Break)
		{
			return EStateTreeVisitor::Break;
		}

		if (const FStateTreeTaskBase* Task = Node.GetNode().GetPtr<FStateTreeTaskBase>())
		{
			FStateTreeBindableStructDesc Desc;
			Desc.StatePath = UE::StateTree::Editor::GlobalStateName;
			Desc.Struct = Task->GetInstanceDataType();
			Desc.Name = Node.GetName();
			Desc.ID = Node.ID;
			Desc.DataSource = EStateTreeBindableStructSource::GlobalTask;

			if (InFunc(nullptr, Desc, Node.GetInstance()) == EStateTreeVisitor::Break)
			{
				return EStateTreeVisitor::Break;
			}

			FStateTreeBindableStructDesc NodeDesc;
			NodeDesc.StatePath = UE::StateTree::Editor::GlobalStateName;
			NodeDesc.Struct = Node.GetNode().GetScriptStruct();
			NodeDesc.Name = Node.GetName();
			NodeDesc.ID = Node.GetNodeID();
			NodeDesc.DataSource = EStateTreeBindableStructSource::GlobalTask;

			if (InFunc(nullptr, NodeDesc, Node.GetNode()) == EStateTreeVisitor::Break)
			{
				return EStateTreeVisitor::Break;
			}
		}
	}

	return  EStateTreeVisitor::Continue;
}

EStateTreeVisitor UStateTreeEditorData::VisitHierarchyNodes(TFunctionRef<EStateTreeVisitor(const UStateTreeState* State, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)> InFunc) const
{
	return VisitHierarchy([this, &InFunc](const UStateTreeState& State, UStateTreeState* /*ParentState*/)
	{
		return VisitStateNodes(State, InFunc);
	});
}

EStateTreeVisitor UStateTreeEditorData::VisitAllNodes(TFunctionRef<EStateTreeVisitor(const UStateTreeState* State, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)> InFunc) const
{
	if (VisitGlobalNodes(InFunc) == EStateTreeVisitor::Break)
	{
		return EStateTreeVisitor::Break;
	}

	if (VisitHierarchyNodes(InFunc) == EStateTreeVisitor::Break)
	{
		return EStateTreeVisitor::Break;
	}

	return EStateTreeVisitor::Continue;
}

#if WITH_STATETREE_TRACE_DEBUGGER
bool UStateTreeEditorData::HasAnyBreakpoint(const FGuid ID) const
{
	return Breakpoints.ContainsByPredicate([ID](const FStateTreeEditorBreakpoint& Breakpoint) { return Breakpoint.ID == ID; });
}

bool UStateTreeEditorData::HasBreakpoint(const FGuid ID, const EStateTreeBreakpointType BreakpointType) const
{
	return GetBreakpoint(ID, BreakpointType) != nullptr;
}

const FStateTreeEditorBreakpoint* UStateTreeEditorData::GetBreakpoint(const FGuid ID, const EStateTreeBreakpointType BreakpointType) const
{
	return Breakpoints.FindByPredicate([ID, BreakpointType](const FStateTreeEditorBreakpoint& Breakpoint)
		{
			return Breakpoint.ID == ID && Breakpoint.BreakpointType == BreakpointType;
		});
}

void UStateTreeEditorData::AddBreakpoint(const FGuid ID, const EStateTreeBreakpointType BreakpointType)
{
	Breakpoints.Emplace(ID, BreakpointType);

	const UStateTree* StateTree = GetTypedOuter<UStateTree>();
	checkf(StateTree, TEXT("UStateTreeEditorData should only be allocated within a UStateTree"));
	UE::StateTree::Delegates::OnBreakpointsChanged.Broadcast(*StateTree);
}

bool UStateTreeEditorData::RemoveBreakpoint(const FGuid ID, const EStateTreeBreakpointType BreakpointType)
{
	const int32 Index = Breakpoints.IndexOfByPredicate([ID, BreakpointType](const FStateTreeEditorBreakpoint& Breakpoint)
		{
			return Breakpoint.ID == ID && Breakpoint.BreakpointType == BreakpointType;
		});
		
	if (Index != INDEX_NONE)
	{
		Breakpoints.RemoveAtSwap(Index);
		
		const UStateTree* StateTree = GetTypedOuter<UStateTree>();
		checkf(StateTree, TEXT("UStateTreeEditorData should only be allocated within a UStateTree"));
		UE::StateTree::Delegates::OnBreakpointsChanged.Broadcast(*StateTree);
	}

	return Index != INDEX_NONE;
}

void UStateTreeEditorData::RemoveAllBreakpoints()
{
	Breakpoints.Reset();

	const UStateTree* StateTree = GetTypedOuter<UStateTree>();
	checkf(StateTree, TEXT("UStateTreeEditorData should only be allocated within a UStateTree"));
	UE::StateTree::Delegates::OnBreakpointsChanged.Broadcast(*StateTree);
}

#endif // WITH_STATETREE_TRACE_DEBUGGER

#undef LOCTEXT_NAMESPACE