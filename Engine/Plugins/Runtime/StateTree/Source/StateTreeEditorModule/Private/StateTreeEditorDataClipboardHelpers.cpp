// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeEditorDataClipboardHelpers.h"

#include "Customizations/StateTreeEditorNodeUtils.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "StateTreeConditionBase.h"
#include "StateTreeConsiderationBase.h"
#include "StateTreeEditorData.h"
#include "StateTreeTaskBase.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"

namespace UE::StateTreeEditor
{

/** Helper class to detect if there were issues when calling ImportText() */
class FDefaultValueImportErrorContext : public FOutputDevice
{
public:

	int32 NumErrors = 0;

	FDefaultValueImportErrorContext() = default;

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
	{
		++NumErrors;
	}
};

void ExportTextAsClipboardEditorData(const FClipboardEditorData& InClipboardEditorData)
{
	FString Value;

	// Use PPF_Copy so that all properties get copied.
	constexpr EPropertyPortFlags PortFlags = PPF_Copy;
	TBaseStructure<FClipboardEditorData>::Get()->ExportText(Value, &InClipboardEditorData, nullptr, nullptr, PortFlags, nullptr);

	FPlatformApplicationMisc::ClipboardCopy(*Value);
}

bool ImportTextAsClipboardEditorData(const UScriptStruct* InTargetType, TNotNull<UStateTreeEditorData*> InTargetTree, TNotNull<UObject*> InOwner,
	FClipboardEditorData& OutClipboardEditorData, bool bProcessBuffer /*true*/)
{
	OutClipboardEditorData.Reset();

	FString PastedText;

	FPlatformApplicationMisc::ClipboardPaste(PastedText);

	if (PastedText.IsEmpty())
	{
		return false;
	}

	const UScriptStruct* ScriptStruct = TBaseStructure<FClipboardEditorData>::Get();
	FDefaultValueImportErrorContext ErrorPipe;
	ScriptStruct->ImportText(*PastedText, &OutClipboardEditorData, nullptr, PPF_None, &ErrorPipe, ScriptStruct->GetName());

	if (bProcessBuffer)
	{
		OutClipboardEditorData.ProcessBuffer(InTargetType, InTargetTree, InOwner);
	}

	return !bProcessBuffer || OutClipboardEditorData.IsValid();
}

void AddErrorNotification(const FText& InText, float InExpiredDuration)
{
	FNotificationInfo NotificationInfo(FText::GetEmpty());
	NotificationInfo.Text = InText;
	NotificationInfo.ExpireDuration = InExpiredDuration;
	FSlateNotificationManager::Get().AddNotification(NotificationInfo);
}

struct FScopedEditorDataFixer
{
	explicit FScopedEditorDataFixer(TNotNull<UStateTreeEditorData*> InEditorData, TNotNull<UObject*> InOwner, bool InbShouldRegenerateGUID = true,
		bool InbShouldReinstantiateInstanceData = true)
		: EditorData(InEditorData)
		, Owner(InOwner)
		, bShouldRegenerateGUID(InbShouldRegenerateGUID)
		, bShouldReinstantiateInstanceData(InbShouldReinstantiateInstanceData)
	{
	}

	explicit FScopedEditorDataFixer(TNotNull<UStateTreeEditorData*> InEditorData, TNotNull<UObject*> InOwner, FClipboardEditorData& InClipboardEditorData)
		: FScopedEditorDataFixer(InEditorData, InOwner)
	{

		EditorNodesToFix = InClipboardEditorData.GetEditorNodesInBuffer();
		TransitionsToFix = InClipboardEditorData.GetTransitionsInBuffer();
		BindingsToFix = InClipboardEditorData.GetBindingsInBuffer();
	}

	~FScopedEditorDataFixer()
	{
		// OldID -> NewID
		TMap<FGuid, FGuid> IDsMap;

		auto UpdateBindings = [Self = this, &IDsMap]()
			{
				for (FStateTreePropertyPathBinding& Binding : Self->BindingsToFix)
				{
					if (FGuid* NewSourceID = IDsMap.Find(Binding.GetSourcePath().GetStructID()))
					{
						Binding.GetMutableSourcePath().SetStructID(*NewSourceID);
					}

					if (FGuid* NewTargetID = IDsMap.Find(Binding.GetTargetPath().GetStructID()))
					{
						Binding.GetMutableTargetPath().SetStructID(*NewTargetID);
					}
				}
			};

		auto ReinstantiateEditorNodeInstanceData = [Self = this](FStateTreeEditorNode& EditorNode)
			{
				EditorNodeUtils::InstantiateStructSubobjects(*Self->Owner, EditorNode.GetNode());
				if (EditorNode.InstanceObject)
				{
					EditorNode.InstanceObject = DuplicateObject(EditorNode.InstanceObject, Self->Owner);
				}
				else
				{
					EditorNodeUtils::InstantiateStructSubobjects(*Self->Owner, EditorNode.Instance);
				}
			};

		auto FixEditorNode = [Self = this, &IDsMap, &ReinstantiateEditorNodeInstanceData](FStateTreeEditorNode& EditorNode)
			{
				FGuid OldInstanceID = EditorNode.ID;
				FGuid OldTemplateID = EditorNode.GetNodeID();
				if (Self->bShouldRegenerateGUID)
				{
					EditorNode.ID = FGuid::NewGuid();

					IDsMap.Add(OldInstanceID, EditorNode.ID);
					IDsMap.Add(OldTemplateID, EditorNode.GetNodeID());
				}

				if (Self->bShouldReinstantiateInstanceData)
				{
					ReinstantiateEditorNodeInstanceData(EditorNode);
				}
			};

		for (FStateTreeEditorNode& EditorNode : EditorNodesToFix)
		{
			FixEditorNode(EditorNode);
		}

		for (FStateTreeTransition& Transition : TransitionsToFix)
		{
			if (bShouldRegenerateGUID)
			{
				FGuid OldTransitionID = Transition.ID;
				Transition.ID = FGuid::NewGuid();
				IDsMap.Add(OldTransitionID, Transition.ID);
			}

			for (FStateTreeEditorNode& CondNode : Transition.Conditions)
			{
				FixEditorNode(CondNode);
			}
		}

		for (FStateTreePropertyPathBinding& Binding : BindingsToFix)
		{
			FStructView PropertyFunctionView = Binding.GetMutablePropertyFunctionNode();
			if (FStateTreeEditorNode* PropertyFunctionNode = PropertyFunctionView.GetPtr<FStateTreeEditorNode>())
			{
				FixEditorNode(*PropertyFunctionNode);
			}
		}

		UpdateBindings();
	}

private:
	TArrayView<FStateTreeEditorNode> EditorNodesToFix;
	TArrayView<FStateTreeTransition> TransitionsToFix;
	TArrayView<FStateTreePropertyPathBinding> BindingsToFix;

	TNotNull<UStateTreeEditorData*> EditorData;
	TNotNull<UObject*> Owner;

	uint8 bShouldRegenerateGUID : 1;
	uint8 bShouldReinstantiateInstanceData : 1;
};

void FClipboardEditorData::Append(TNotNull<const UStateTreeEditorData*> InStateTree, TConstArrayView<FStateTreeEditorNode> InEditorNodes)
{
	bBufferProcessed = false;
	EditorNodesBuffer.Append(InEditorNodes);
	
	CollectBindingsForEditorNodes(InStateTree, InEditorNodes);
}

void FClipboardEditorData::Append(TNotNull<const UStateTreeEditorData*> InStateTree, TConstArrayView<FStateTreeTransition> InTransitions)
{
	bBufferProcessed = false;
	TransitionsBuffer.Append(InTransitions);

	CollectBindingsForTransitions(InStateTree, InTransitions);
	for (const FStateTreeTransition& Transition : InTransitions)
	{
		CollectBindingsForEditorNodes(InStateTree, Transition.Conditions);
	}
}

void FClipboardEditorData::Append(TNotNull<const UStateTreeEditorData*> InStateTree, TConstArrayView<const FPropertyBindingBinding*> InBindingPtrs)
{
	bBufferProcessed = false;
	for (const FPropertyBindingBinding* BindingPtr : InBindingPtrs)
	{
		if (BindingsBuffer.ContainsByPredicate([BindingPtr](const FStateTreePropertyPathBinding& InBinding)
			{
				return BindingPtr && BindingPtr->GetSourcePath() == InBinding.GetSourcePath() && BindingPtr->GetTargetPath() == InBinding.GetTargetPath();
			}))
		{
			continue;
		}

		BindingsBuffer.Add(*static_cast<const FStateTreePropertyPathBinding*>(BindingPtr));
		const FConstStructView FunctionNodeView = BindingPtr->GetPropertyFunctionNode();
		if (const FStateTreeEditorNode* FunctionNode = FunctionNodeView.GetPtr<const FStateTreeEditorNode>())
		{
			CollectBindingsForEditorNodes(InStateTree, { FunctionNode, 1 });
		}
	}
}

void FClipboardEditorData::Reset()
{
	EditorNodesBuffer.Reset();
	TransitionsBuffer.Reset();
	BindingsBuffer.Reset();
	bBufferProcessed = false;
}

void FClipboardEditorData::CollectBindingsForEditorNodes(TNotNull<const UStateTreeEditorData*> InStateTree, TConstArrayView<FStateTreeEditorNode> InEditorNodes)
{
	TArray<const FPropertyBindingBinding*> TempBindingPtrs;
	
	for (const FStateTreeEditorNode& EditorNode : InEditorNodes)
	{
		InStateTree->GetPropertyEditorBindings()->GetBindingsFor(EditorNode.ID, TempBindingPtrs);

		// recursively collect property function node bindings
		constexpr auto EmptyStatePath = TEXT("");
		InStateTree->VisitStructBoundPropertyFunctions(EditorNode.ID, EmptyStatePath, 
			[InStateTree, &TempBindingPtrs](const FStateTreeEditorNode& EditorNode, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)
			{
				TArray<const FPropertyBindingBinding*> StructBindingsPtr;
				InStateTree->GetPropertyEditorBindings()->GetBindingsFor(Desc.ID, StructBindingsPtr);
				TempBindingPtrs.Append(MoveTemp(StructBindingsPtr));

				return EStateTreeVisitor::Continue;
			});
	}

	Algo::Transform(TempBindingPtrs, BindingsBuffer, [](const FPropertyBindingBinding* BindingPtr) { return *static_cast<const FStateTreePropertyPathBinding*>(BindingPtr); });
}

void FClipboardEditorData::CollectBindingsForTransitions(TNotNull<const UStateTreeEditorData*> InStateTree, TConstArrayView<FStateTreeTransition> InTransitions)
{
	TArray<const FPropertyBindingBinding*> TempBindingPtrs;
	
	for (const FStateTreeTransition& Transition : InTransitions)
	{
		InStateTree->GetPropertyEditorBindings()->GetBindingsFor(Transition.ID, TempBindingPtrs);

		// recursively collect property function node bindings
		constexpr auto EmptyStatePath = TEXT("");
		InStateTree->VisitStructBoundPropertyFunctions(Transition.ID, EmptyStatePath,
			[InStateTree, &TempBindingPtrs](const FStateTreeEditorNode& EditorNode, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)
			{
				TArray<const FPropertyBindingBinding*> StructBindingsPtr;
				InStateTree->GetPropertyEditorBindings()->GetBindingsFor(Desc.ID, StructBindingsPtr);
				TempBindingPtrs.Append(MoveTemp(StructBindingsPtr));

				return EStateTreeVisitor::Continue;
			});
	}

	Algo::Transform(TempBindingPtrs, BindingsBuffer, [](const FPropertyBindingBinding* BindingPtr) { return *static_cast<const FStateTreePropertyPathBinding*>(BindingPtr); });
}

void FClipboardEditorData::ProcessBuffer(const UScriptStruct* InTargetType, TNotNull<UStateTreeEditorData*> InEditorData, TNotNull<UObject*> InTargetOwner)
{
	bBufferProcessed = false;
	check(!InTargetType || InTargetType->IsChildOf(TBaseStructure<FStateTreeNodeBase>::Get()) || InTargetType->IsChildOf(TBaseStructure<FStateTreeTransition>::Get()));
	check(InTargetOwner->IsA<UStateTreeState>() || InTargetOwner->IsA<UStateTreeEditorData>());

	auto ValidateEditorNodes = [&InEditorData](TConstArrayView<FStateTreeEditorNode> InEditorNodes, const UScriptStruct* InRequiredType)
		{
			for (const FStateTreeEditorNode& EditorNode : InEditorNodes)
			{
				FStructView NodeView = EditorNode.GetNode();
				FStateTreeDataView InstanceView = EditorNode.GetInstance();
				if (!NodeView.IsValid() || !InstanceView.IsValid())
				{
					AddErrorNotification(LOCTEXT("MalformedNode", "Clipboard text contains invalid data."));
					return false;
				}

				if (InRequiredType && !NodeView.GetScriptStruct()->IsChildOf(InRequiredType))
				{
					AddErrorNotification(FText::Format(LOCTEXT("NotSupportedByType", "This property only accepts nodes of type {0}."), InRequiredType->GetDisplayNameText()));
					return false;
				}

				if (const UStateTreeSchema* TargetSchema = InEditorData->Schema)
				{
					bool bIsNodeAllowed = true;
					const UStruct* InstanceType = InstanceView.GetStruct();
					if (const UScriptStruct* InstanceTypeStruct = Cast<UScriptStruct>(InstanceType))
					{
						if (!TargetSchema->IsStructAllowed(NodeView.GetScriptStruct()))
						{
							bIsNodeAllowed = false;
						}
					}
					else if (const UClass* InstanceTypeClass = Cast<UClass>(InstanceType))
					{
						if (!TargetSchema->IsClassAllowed(InstanceTypeClass))
						{
							bIsNodeAllowed = false;
						}
					}

					if (!bIsNodeAllowed)
					{
						AddErrorNotification(FText::Format(LOCTEXT("NotSupportedBySchema", "Node {0} is not supported by {1} schema."),
							NodeView.GetScriptStruct()->GetDisplayNameText(), TargetSchema->GetClass()->GetDisplayNameText()));

						return false;
					}
				}
			}

			return true;
		};

	auto ValidateTransitions = [Self = this, &ValidateEditorNodes](TConstArrayView<FStateTreeTransition> InTransitions, const UScriptStruct* InRequiredType)
		{
			if (Self->TransitionsBuffer.Num() && InRequiredType && InRequiredType != TBaseStructure<FStateTreeTransition>::Get())
			{
				AddErrorNotification(FText::Format(LOCTEXT("NotSupportedByType", "This property only accepts nodes of type {0}."), InRequiredType->GetDisplayNameText()));
				return false;
			}

			for (const FStateTreeTransition& Transition : InTransitions)
			{
				static const UScriptStruct* ConditionRequiredType = TBaseStructure<FStateTreeConditionBase>::Get();
				if (!ValidateEditorNodes(Transition.Conditions, ConditionRequiredType))
				{
					return false;
				}
			}

			return true;
		};

	if (ValidateEditorNodes(EditorNodesBuffer, InTargetType) && ValidateTransitions(TransitionsBuffer, InTargetType))
	{
		FScopedEditorDataFixer Fixer(InEditorData, InTargetOwner, *this);
		bBufferProcessed = true;
	}
	else
	{
		Reset();
	}
}
}
#undef LOCTEXT_NAMESPACE
