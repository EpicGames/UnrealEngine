// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeEditorDataClipboardHelpers.generated.h"

namespace UE::StateTreeEditor
{
struct FClipboardEditorData;
}

struct FPropertyBindingBinding;
struct FStateTreeTransition;
struct FStateTreeEditorNode;
struct FStateTreePropertyPathBinding;

class UStateTreeEditorData;

namespace UE::StateTreeEditor
{
void ExportTextAsClipboardEditorData(const FClipboardEditorData& InClipboardEditorData);

bool ImportTextAsClipboardEditorData(const UScriptStruct* InTargetType, TNotNull<UStateTreeEditorData*> InTargetTree, TNotNull<UObject*> InOwner,
                                     FClipboardEditorData& OutClipboardEditorData, bool bProcessBuffer = true);

void AddErrorNotification(const FText& InText, float InExpiredDuration = 5.f);

USTRUCT()
struct FClipboardEditorData
{
	GENERATED_BODY()

	void ProcessBuffer(const UScriptStruct* InTargetType, TNotNull<UStateTreeEditorData*> InEditorData, TNotNull<UObject*> InOwner);

	bool IsValid() const
	{
		return bBufferProcessed;
	}

	void Append(TNotNull<const UStateTreeEditorData*> InStateTree, TConstArrayView<FStateTreeEditorNode> InEditorNodes);
	void Append(TNotNull<const UStateTreeEditorData*> InStateTree, TConstArrayView<FStateTreeTransition> InTransitions);
	void Append(TNotNull<const UStateTreeEditorData*> InStateTree, TConstArrayView<const FPropertyBindingBinding*> InBindingPtrs);

	void Reset();

	TArrayView<FStateTreeEditorNode> GetEditorNodesInBuffer()
	{
		return EditorNodesBuffer;
	}

	TConstArrayView<FStateTreeEditorNode> GetEditorNodesInBuffer() const
	{
		return EditorNodesBuffer;
	}

	TArrayView<FStateTreeTransition> GetTransitionsInBuffer()
	{
		return TransitionsBuffer;
	}

	TConstArrayView<FStateTreeTransition> GetTransitionsInBuffer() const
	{
		return TransitionsBuffer;
	}

	TArrayView<FStateTreePropertyPathBinding> GetBindingsInBuffer()
	{
		return BindingsBuffer;
	}

	TConstArrayView<FStateTreePropertyPathBinding> GetBindingsInBuffer() const
	{
		return BindingsBuffer;
	}

private:
	void CollectBindingsForEditorNodes(TNotNull<const UStateTreeEditorData*> InStateTree, TConstArrayView<FStateTreeEditorNode> InEditorNodes);
	void CollectBindingsForTransitions(TNotNull<const UStateTreeEditorData*> InStateTree, TConstArrayView<FStateTreeTransition> InEditorNodes);

	UPROPERTY()
	TArray<FStateTreeEditorNode> EditorNodesBuffer;

	UPROPERTY()
	TArray<FStateTreeTransition> TransitionsBuffer;

	UPROPERTY()
	TArray<FStateTreePropertyPathBinding> BindingsBuffer;

	bool bBufferProcessed = false;
};
}
