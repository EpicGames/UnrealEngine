// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeEditorWorkspaceTabHost.h"
#include "Toolkits/AssetEditorModeUILayer.h"

#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "WorkspaceTabHost"

class SDockTab;

namespace UE::StateTreeEditor
{
const FLazyName FWorkspaceTabHost::BindingTabId = "StateTreeEditor_Binding";
const FLazyName FWorkspaceTabHost::DebuggerTabId = "StateTreeEditor_Debugger";
const FLazyName FWorkspaceTabHost::OutlinerTabId = "StateTreeEditor_StateTreeOutliner";
const FLazyName FWorkspaceTabHost::SearchTabId = "StateTreeEditor_StateTreeSearch";
const FLazyName FWorkspaceTabHost::StatisticsTabId = "StateTreeEditor_StateTreeStatistics";

TConstArrayView<FMinorWorkspaceTabConfig> FWorkspaceTabHost::GetTabConfigs() const
{
	auto BuildTabConfigs = []()
		{
			TArray<FMinorWorkspaceTabConfig> TabConfigs;
			TabConfigs.Emplace(BindingTabId.Resolve(), LOCTEXT("StateTreeBindingTab", "Bindings"), FText(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Link"), UAssetEditorUISubsystem::BottomRightTabID);
			TabConfigs.Emplace(OutlinerTabId.Resolve(), LOCTEXT("StateTreeOutlinerTab", "Outliner"), FText(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner"), UAssetEditorUISubsystem::TopLeftTabID);
			TabConfigs.Emplace(SearchTabId.Resolve(), LOCTEXT("StateTreeSearchTab", "Find"), FText(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlueprintEditor.FindInBlueprints.MenuIcon"), UAssetEditorUISubsystem::BottomLeftTabID);
			TabConfigs.Emplace(StatisticsTabId.Resolve(), LOCTEXT("StatisticsTitle", "Statistics"), FText(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "AssetEditor.ToggleStats"), UAssetEditorUISubsystem::BottomRightTabID);
#if WITH_STATETREE_TRACE_DEBUGGER
			TabConfigs.Emplace(DebuggerTabId.Resolve(), LOCTEXT("DebuggerTab", "Debugger"), FText(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Debug"), UAssetEditorUISubsystem::BottomRightTabID);
#endif
			return TabConfigs;
		};
	static TArray<FMinorWorkspaceTabConfig> TabConfigs = BuildTabConfigs();
	return TabConfigs;
}

FOnSpawnTab FWorkspaceTabHost::CreateSpawnDelegate(FName TabID)
{
	return ::FOnSpawnTab::CreateSP(this, &FWorkspaceTabHost::HandleSpawnDelegate, TabID);
}

TSharedRef<SDockTab> FWorkspaceTabHost::HandleSpawnDelegate(const FSpawnTabArgs& Args, FName TabID)
{
	return Spawn(TabID);
}

TSharedRef<SDockTab> FWorkspaceTabHost::Spawn(FName TabID)
{
	const FMinorWorkspaceTabConfig* Found = GetTabConfigs().FindByPredicate([TabID](const FMinorWorkspaceTabConfig& Config)
		{
			return Config.ID == TabID;
		});

	if (Found)
	{
		TSharedRef<SDockTab> Result = SNew(SDockTab)
			.Label(Found->Label);
		Result->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateSP(this, &FWorkspaceTabHost::HandleTabClosed));

		FSpawnedWorkspaceTab* SpawnedTab = SpawnedTabs.FindByPredicate([TabID](const FSpawnedWorkspaceTab& SpawnedTab)
			{
				return SpawnedTab.TabID == TabID;
			});
		if (SpawnedTab)
		{
			ensureMsgf(false, TEXT("The tab is already spawned."));
			SpawnedTab->DockTab = Result;
		}
		else
		{
			SpawnedTabs.Emplace(TabID, TWeakPtr<SDockTab>(Result));
		}

		OnTabSpawned.Broadcast(FSpawnedWorkspaceTab{ TabID, Result });
		return Result;
	}
	else
	{
		return SNew(SDockTab)
			.Label(LOCTEXT("StateTreeErrorNotFoundDockTab", "<NOT FOUND>"));
	}
}

void FWorkspaceTabHost::HandleTabClosed(TSharedRef<SDockTab> Tab)
{
	// Broadcast before removing
	for (const FSpawnedWorkspaceTab& SpawnedTab : SpawnedTabs)
	{
		if (SpawnedTab.DockTab == Tab)
		{
			OnTabClosed.Broadcast(SpawnedTab);
		}
	}

	SpawnedTabs.RemoveAllSwap([Tab](const FSpawnedWorkspaceTab& SpawnedTab)
		{
			return SpawnedTab.DockTab == Tab;
		});
}

} // namespace UE::StateTreeEditor

#undef LOCTEXT_NAMESPACE
