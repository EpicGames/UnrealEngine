// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeEditorMode.h"

#include "ContextObjectStore.h"
#include "EditorModeManager.h"
#include "FileHelpers.h"
#include "HAL/IConsoleManager.h"

#include "StateTreeEditorModeToolkit.h"
#include "StateTreeEditorSettings.h"
#include "StateTreeCompilerLog.h"
#include "StateTreeDelegates.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "IMessageLogListing.h"
#include "InteractiveToolManager.h"
#include "PropertyPath.h"
#include "StateTreeEditorCommands.h"
#include "Misc/UObjectToken.h"
#include "Toolkits/ToolkitManager.h"
#include "Modules/ModuleManager.h"

#include "IStateTreeEditorHost.h"
#include "MessageLogModule.h"
#include "StateTreeCompiler.h"
#include "StateTreeEditingSubsystem.h"
#include "Customizations/StateTreeBindingExtension.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeEditorMode)

#define LOCTEXT_NAMESPACE "UStateTreeEditorMode"

class FUObjectToken;
const FEditorModeID UStateTreeEditorMode::EM_StateTree("StateTreeEditorMode");

UStateTreeEditorMode::UStateTreeEditorMode()
{
	Info = FEditorModeInfo(UStateTreeEditorMode::EM_StateTree,
		LOCTEXT("StateTreeEditorModeName", "StateTreeEditorMode"),
		FSlateIcon(),
		false);
}

void UStateTreeEditorMode::Enter()
{
	Super::Enter();
	
	DetailsViewExtensionHandler = MakeShared<FStateTreeBindingExtension>();
	DetailsViewChildrenCustomizationHandler = MakeShared<FStateTreeBindingsChildrenCustomization>();
	
	if (const UContextObjectStore* ContextObjectStore = GetToolManager()->GetContextObjectStore())
	{
		if (const UStateTreeEditorContext* Context = ContextObjectStore->FindContext<UStateTreeEditorContext>())
		{
			TSharedRef<IStateTreeEditorHost> Host = Context->EditorHostInterface.ToSharedRef();
			Host->OnStateTreeChanged().AddUObject(this, &UStateTreeEditorMode::OnStateTreeChanged);

			if (TSharedPtr<IMessageLogListing> MessageLogListing = GetMessageLogListing())
			{
				MessageLogListing->OnMessageTokenClicked().AddUObject(this, &UStateTreeEditorMode::HandleMessageTokenClicked);
			}

			if (TSharedPtr<IDetailsView> DetailsView = GetDetailsView())
			{
				DetailsView->OnFinishedChangingProperties().AddUObject(this, &UStateTreeEditorMode::OnSelectionFinishedChangingProperties);

				DetailsView->SetExtensionHandler(DetailsViewExtensionHandler);
				DetailsView->SetChildrenCustomizationHandler(DetailsViewChildrenCustomizationHandler);
			}
			
			if (TSharedPtr<IDetailsView> AssetDetailsView = GetAssetDetailsView())
			{
				AssetDetailsView->OnFinishedChangingProperties().AddUObject(this, &UStateTreeEditorMode::OnAssetFinishedChangingProperties);
				
				AssetDetailsView->SetExtensionHandler(DetailsViewExtensionHandler);
				AssetDetailsView->SetChildrenCustomizationHandler(DetailsViewChildrenCustomizationHandler);
				bForceAssetDetailViewToRefresh = true;
			}
		}
	}

	UE::StateTree::Delegates::OnIdentifierChanged.AddUObject(this, &UStateTreeEditorMode::OnIdentifierChanged);
	UE::StateTree::Delegates::OnSchemaChanged.AddUObject(this, &UStateTreeEditorMode::OnSchemaChanged);
	UE::StateTree::Delegates::OnParametersChanged.AddUObject(this, &UStateTreeEditorMode::OnRefreshDetailsView);
	UE::StateTree::Delegates::OnGlobalDataChanged.AddUObject(this, &UStateTreeEditorMode::OnRefreshDetailsView);
	UE::StateTree::Delegates::OnStateParametersChanged.AddUObject(this, &UStateTreeEditorMode::OnStateParametersChanged);
	UE::StateTree::PropertyBinding::OnStateTreePropertyBindingChanged.AddUObject(this, &UStateTreeEditorMode::OnPropertyBindingChanged);
	OnStateTreeChanged();
}


void UStateTreeEditorMode::OnIdentifierChanged(const UStateTree& InStateTree)
{
	if (GetStateTree() == &InStateTree)
	{
		UpdateAsset();
	}
}

void UStateTreeEditorMode::OnSchemaChanged(const UStateTree& InStateTree)
{
	if (GetStateTree() == &InStateTree)
	{
		UpdateAsset();

		if(UStateTreeEditingSubsystem* StateTreeEditingSubsystem = GEditor->GetEditorSubsystem<UStateTreeEditingSubsystem>())
		{
			TSharedRef<FStateTreeViewModel> ViewModel = StateTreeEditingSubsystem->FindOrAddViewModel(GetStateTree());
			ViewModel->NotifyAssetChangedExternally();
		}
		
		ForceRefreshDetailsView();
	}
}

void UStateTreeEditorMode::ForceRefreshDetailsView() const
{
	if (TSharedPtr<IDetailsView> DetailsView = GetDetailsView())
	{
		if (!GEditor->GetTimerManager()->IsTimerActive(SetObjectTimerHandle))
		{
			DetailsView->ForceRefresh();
		}
	}
}

void UStateTreeEditorMode::OnRefreshDetailsView(const UStateTree& InStateTree) const
{
	if (GetStateTree() == &InStateTree)
	{
		// Accessible structs might be different after modifying parameters so forcing refresh
		// so the FStateTreeBindingExtension can rebuild the list of bindable structs
		ForceRefreshDetailsView();
	}
}

void UStateTreeEditorMode::OnStateParametersChanged(const UStateTree& InStateTree, const FGuid ChangedStateID) const
{
	UStateTree* StateTree = GetStateTree(); 
	if (StateTree == &InStateTree)
	{
		if (const UStateTreeEditorData* TreeData = Cast<UStateTreeEditorData>(StateTree->EditorData))
		{
			TreeData->VisitHierarchy([&ChangedStateID](UStateTreeState& State, UStateTreeState* /*ParentState*/)
			{
				if (State.Type == EStateTreeStateType::Linked && State.LinkedSubtree.ID == ChangedStateID)
				{
					State.UpdateParametersFromLinkedSubtree();
				}
				return EStateTreeVisitor::Continue;
			});
		}

		// Accessible structs might be different after modifying parameters so forcing refresh
		// so the FStateTreeBindingExtension can rebuild the list of bindable structs
		ForceRefreshDetailsView();
	}
}

void UStateTreeEditorMode::HandleMessageTokenClicked(const TSharedRef<IMessageToken>& InMessageToken) const
{
	if (InMessageToken->GetType() == EMessageToken::Object)
	{
		const TSharedRef<FUObjectToken> ObjectToken = StaticCastSharedRef<FUObjectToken>(InMessageToken);

		if (UStateTreeState* State = Cast<UStateTreeState>(ObjectToken->GetObject().Get()))
		{
			if(UStateTreeEditingSubsystem* StateTreeEditingSubsystem = GEditor->GetEditorSubsystem<UStateTreeEditingSubsystem>())
			{
				StateTreeEditingSubsystem->FindOrAddViewModel(GetStateTree())->SetSelection(State);
			}
			
		}
	}
}

void UStateTreeEditorMode::Exit()
{
	if (Toolkit.IsValid())
	{
		FToolkitManager::Get().CloseToolkit(Toolkit.ToSharedRef());
		Toolkit.Reset();
	}
	
	if (UContextObjectStore* ContextObjectStore = GetToolManager()->GetContextObjectStore())
	{
		if (const UStateTreeEditorContext* Context = ContextObjectStore->FindContext<UStateTreeEditorContext>())
		{
			Context->EditorHostInterface->OnStateTreeChanged().RemoveAll(this);
			
			if (TSharedPtr<IMessageLogListing> MessageLogListing = GetMessageLogListing())
			{
				MessageLogListing->OnMessageTokenClicked().RemoveAll(this);
			}
			
			if (TSharedPtr<IDetailsView> DetailsView = GetDetailsView())
			{
				DetailsView->OnFinishedChangingProperties().RemoveAll(this);
				DetailsView->SetExtensionHandler(nullptr);
				DetailsView->SetChildrenCustomizationHandler(nullptr);
			}
			
			if (TSharedPtr<IDetailsView> AssetDetailsView = GetAssetDetailsView())
			{
				AssetDetailsView->OnFinishedChangingProperties().RemoveAll(this);
				AssetDetailsView->SetExtensionHandler(nullptr);
				AssetDetailsView->SetChildrenCustomizationHandler(nullptr);
				bForceAssetDetailViewToRefresh = true;
			}
		}
	}

	if (CachedStateTree.IsValid())
	{
		if (UStateTreeEditingSubsystem* StateTreeEditingSubsystem = GEditor->GetEditorSubsystem<UStateTreeEditingSubsystem>())
		{		
			TSharedRef<FStateTreeViewModel> ViewModel = StateTreeEditingSubsystem->FindOrAddViewModel(CachedStateTree.Get());
			{
				ViewModel->GetOnAssetChanged().RemoveAll(this);
				ViewModel->GetOnStateAdded().RemoveAll(this);
				ViewModel->GetOnStatesRemoved().RemoveAll(this);
				ViewModel->GetOnStatesMoved().RemoveAll(this);
				ViewModel->GetOnStateNodesChanged().RemoveAll(this);
				ViewModel->GetOnSelectionChanged().RemoveAll(this);
				ViewModel->GetOnBringNodeToFocus().RemoveAll(this);
				ViewModel->GetOnBringBindingPathToFocus().RemoveAll(this);
			}
		}
	}

	UE::StateTree::Delegates::OnIdentifierChanged.RemoveAll(this);
	UE::StateTree::Delegates::OnSchemaChanged.RemoveAll(this);
	UE::StateTree::Delegates::OnParametersChanged.RemoveAll(this);
	UE::StateTree::Delegates::OnGlobalDataChanged.RemoveAll(this);
	UE::StateTree::Delegates::OnStateParametersChanged.RemoveAll(this);
	UE::StateTree::PropertyBinding::OnStateTreePropertyBindingChanged.RemoveAll(this);
	Super::Exit();
}

void UStateTreeEditorMode::CreateToolkit()
{
	Toolkit = MakeShareable(new FStateTreeEditorModeToolkit(this));
}

void UStateTreeEditorMode::OnStateTreeChanged()
{
	UContextObjectStore* ContextStore = GetInteractiveToolsContext()->ToolManager->GetContextObjectStore();
	if (const UStateTreeEditorContext* Context = ContextStore->FindContext<UStateTreeEditorContext>())
	{
		if (UStateTreeEditingSubsystem* StateTreeEditingSubsystem = GEditor->GetEditorSubsystem<UStateTreeEditingSubsystem>())
		{
			if (CachedStateTree.IsValid())
			{
				TSharedRef<FStateTreeViewModel> OldViewModel = StateTreeEditingSubsystem->FindOrAddViewModel(CachedStateTree.Get());
				{
					OldViewModel->GetOnAssetChanged().RemoveAll(this);
					OldViewModel->GetOnStateAdded().RemoveAll(this);
					OldViewModel->GetOnStatesRemoved().RemoveAll(this);
					OldViewModel->GetOnStatesMoved().RemoveAll(this);
					OldViewModel->GetOnStateNodesChanged().RemoveAll(this);
					OldViewModel->GetOnSelectionChanged().RemoveAll(this);
					OldViewModel->GetOnBringNodeToFocus().RemoveAll(this);
					OldViewModel->GetOnBringBindingPathToFocus().RemoveAll(this);
				}
			}
		}

		UStateTree* StateTree = Context->EditorHostInterface->GetStateTree();
		CachedStateTree = StateTree;
		UpdateAsset();

		if (TSharedPtr<IDetailsView> AssetDetailsView = GetAssetDetailsView())
		{
			AssetDetailsView->SetObject(StateTree ? StateTree->EditorData : nullptr, bForceAssetDetailViewToRefresh);
			bForceAssetDetailViewToRefresh = false;
		}

		if (StateTree)
		{
			if (UStateTreeEditingSubsystem* StateTreeEditingSubsystem = GEditor->GetEditorSubsystem<UStateTreeEditingSubsystem>())
			{				
				TSharedRef<FStateTreeViewModel> NewViewModel = StateTreeEditingSubsystem->FindOrAddViewModel(StateTree);
				{
					NewViewModel->GetOnAssetChanged().AddUObject(this, &UStateTreeEditorMode::HandleModelAssetChanged);
					NewViewModel->GetOnStateAdded().AddUObject(this, &UStateTreeEditorMode::HandleStateAdded);
					NewViewModel->GetOnStatesRemoved().AddUObject(this, &UStateTreeEditorMode::HandleStatesRemoved);
					NewViewModel->GetOnStatesMoved().AddUObject(this, &UStateTreeEditorMode::HandleOnStatesMoved);
					NewViewModel->GetOnStateNodesChanged().AddUObject(this, &UStateTreeEditorMode::HandleOnStateNodesChanged);
					NewViewModel->GetOnSelectionChanged().AddUObject(this, &UStateTreeEditorMode::HandleModelSelectionChanged);
					NewViewModel->GetOnBringNodeToFocus().AddUObject(this, &UStateTreeEditorMode::HandleModelBringNodeToFocus);
					NewViewModel->GetOnBringBindingPathToFocus().AddUObject(this, &UStateTreeEditorMode::HandleModelBringBindingPathToFocus);
				}
			}
		}
	}

	if (Toolkit)
	{
		StaticCastSharedPtr<FStateTreeEditorModeToolkit>(Toolkit)->OnStateTreeChanged();
	}
}


namespace UE::StateTree::Editor::Internal
{
static void SetSaveOnCompileSetting(const EStateTreeSaveOnCompile NewSetting)
{
	UStateTreeEditorSettings* Settings = GetMutableDefault<UStateTreeEditorSettings>();
	Settings->SaveOnCompile = NewSetting;
	Settings->SaveConfig();
}

static bool IsSaveOnCompileOptionSet(const EStateTreeSaveOnCompile Option)
{
	const UStateTreeEditorSettings* Settings = GetDefault<UStateTreeEditorSettings>();
	return (Settings->SaveOnCompile == Option);
}

static IConsoleVariable* GetLogCompilationResultCVar()
{
	static IConsoleVariable* FoundVariable = IConsoleManager::Get().FindConsoleVariable(TEXT("StateTree.Compiler.LogResultOnCompilationCompleted"));
	return FoundVariable;
}

static void ToggleLogCompilationResult()
{
	IConsoleVariable* LogResultCVar = GetLogCompilationResultCVar();
	if (ensure(LogResultCVar))
	{
		LogResultCVar->Set(!LogResultCVar->GetBool(), ECVF_SetByConsole);
	}
}

static bool IsLogCompilationResult()
{
	IConsoleVariable* LogResultCVar = GetLogCompilationResultCVar();
	return LogResultCVar ? LogResultCVar->GetBool() : false;
}

static IConsoleVariable* GetLogDependenciesCVar()
{
	static IConsoleVariable* FoundVariable = IConsoleManager::Get().FindConsoleVariable(TEXT("StateTree.Compiler.LogDependenciesOnCompilation"));
	return FoundVariable;
}

static void ToggleLogDependencies()
{
	IConsoleVariable* LogResultCVar = GetLogDependenciesCVar();
	if (ensure(LogResultCVar))
	{
		LogResultCVar->Set(!LogResultCVar->GetBool(), ECVF_SetByConsole);
	}
}

static bool IsLogDependencies()
{
	IConsoleVariable* LogResultCVar = GetLogDependenciesCVar();
	return LogResultCVar ? LogResultCVar->GetBool() : false;
}
} // namespace UE::StateTree::Editor::Internal

void UStateTreeEditorMode::BindToolkitCommands(const TSharedRef<FUICommandList>& ToolkitCommands)
{
	FStateTreeEditorCommands::Register();
	const FStateTreeEditorCommands& Commands = FStateTreeEditorCommands::Get();

	ToolkitCommands->MapAction(
		Commands.Compile,
		FExecuteAction::CreateUObject(this, &UStateTreeEditorMode::Compile),
		FCanExecuteAction::CreateUObject(this, &UStateTreeEditorMode::CanCompile),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateUObject(this, &UStateTreeEditorMode::IsCompileVisible));

	ToolkitCommands->MapAction(
		FStateTreeEditorCommands::Get().SaveOnCompile_Never,
		FExecuteAction::CreateStatic(&UE::StateTree::Editor::Internal::SetSaveOnCompileSetting, EStateTreeSaveOnCompile::Never),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic(&UE::StateTree::Editor::Internal::IsSaveOnCompileOptionSet, EStateTreeSaveOnCompile::Never),
		FIsActionButtonVisible::CreateUObject(this, &UStateTreeEditorMode::HasValidStateTree)
	);
	ToolkitCommands->MapAction(
		FStateTreeEditorCommands::Get().SaveOnCompile_SuccessOnly,
		FExecuteAction::CreateStatic(&UE::StateTree::Editor::Internal::SetSaveOnCompileSetting, EStateTreeSaveOnCompile::SuccessOnly),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic(&UE::StateTree::Editor::Internal::IsSaveOnCompileOptionSet,  EStateTreeSaveOnCompile::SuccessOnly),
		FIsActionButtonVisible::CreateUObject(this, &UStateTreeEditorMode::HasValidStateTree)
	);
	ToolkitCommands->MapAction(
		FStateTreeEditorCommands::Get().SaveOnCompile_Always,
		FExecuteAction::CreateStatic(&UE::StateTree::Editor::Internal::SetSaveOnCompileSetting, EStateTreeSaveOnCompile::Always),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic(&UE::StateTree::Editor::Internal::IsSaveOnCompileOptionSet,  EStateTreeSaveOnCompile::Always),
		FIsActionButtonVisible::CreateUObject(this, &UStateTreeEditorMode::HasValidStateTree)
	);
	ToolkitCommands->MapAction(
		FStateTreeEditorCommands::Get().LogCompilationResult,
		FExecuteAction::CreateStatic(&UE::StateTree::Editor::Internal::ToggleLogCompilationResult),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic(&UE::StateTree::Editor::Internal::IsLogCompilationResult)
	);
	ToolkitCommands->MapAction(
		FStateTreeEditorCommands::Get().LogDependencies,
		FExecuteAction::CreateStatic(&UE::StateTree::Editor::Internal::ToggleLogDependencies),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic(&UE::StateTree::Editor::Internal::IsLogDependencies)
	);
}

void UStateTreeEditorMode::OnPropertyBindingChanged(const FPropertyBindingPath& SourcePath, const FPropertyBindingPath& TargetPath)
{
	UpdateAsset();
}

void UStateTreeEditorMode::BindCommands()
{
	UEdMode::BindCommands();
	const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();
	BindToolkitCommands(CommandList);
}

void UStateTreeEditorMode::Compile()
{
	UStateTree* StateTree = GetStateTree();

	if (!StateTree)
	{
		return;
	}

	UpdateAsset();

	if (TSharedPtr<IMessageLogListing> Listing = GetMessageLogListing())
	{
		Listing->ClearMessages();
	}
	
	FStateTreeCompilerLog Log;
	const bool bCompileSucceeded = UStateTreeEditingSubsystem::CompileStateTree(StateTree, Log);

	if (bCompileSucceeded)
	{
		// We always want to check outers when compiling from the editor to catch invalid references early.
		// The cvar bEnableCheckOutersOnCompilationSucceeded is an optimization that runs the same check
		// inside the compiler pipeline itself. If it is already enabled, skip the check here to avoid
		// running it twice.
		const IConsoleVariable* CVarCheckOuters = IConsoleManager::Get().FindConsoleVariable(TEXT("StateTree.Compiler.bEnableCheckOutersOnCompilationSucceeded"));
		if (!CVarCheckOuters || !CVarCheckOuters->GetBool())
		{
			FStateTreeCompiler::CheckCompiledStateTreeOuters(StateTree, Log);
		}
	}

	if (TSharedPtr<IMessageLogListing> Listing = GetMessageLogListing())
	{
		Log.AppendToLog(Listing.Get());

		if (!bCompileSucceeded)
		{
			// Show log
			ShowCompilerTab();
		}
	}
	

	const UStateTreeEditorSettings* Settings = GetMutableDefault<UStateTreeEditorSettings>();
	const bool bShouldSaveOnCompile = ((Settings->SaveOnCompile == EStateTreeSaveOnCompile::Always)
									|| ((Settings->SaveOnCompile == EStateTreeSaveOnCompile::SuccessOnly) && bCompileSucceeded));

	if (bShouldSaveOnCompile)
	{
		const TArray<UPackage*> PackagesToSave { StateTree->GetOutermost() };
		FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, /*bCheckDirty =*/true, /*bPromptToSave =*/false);
	}
}

bool UStateTreeEditorMode::CanCompile() const
{
	if (GetStateTree() == nullptr)
	{
		return false;
	}

	// We can't recompile while in PIE
	if (GEditor->IsPlaySessionInProgress())
	{
		return false;
	}

	return true;
}

bool UStateTreeEditorMode::IsCompileVisible() const
{
	const UInteractiveToolManager* ToolManager = GetToolManager();
	if(!HasValidStateTree() || ToolManager == nullptr)
	{
		return false;
	}

	if (const UContextObjectStore* ContextObjectStore = ToolManager->GetContextObjectStore())
	{
		if (const UStateTreeEditorContext* Context = ContextObjectStore->FindContext<UStateTreeEditorContext>())
		{
			return Context->EditorHostInterface->ShouldShowCompileButton();
		}
	}

	return true;
}

bool UStateTreeEditorMode::HasValidStateTree() const
{
	return GetStateTree() != nullptr;
}

void UStateTreeEditorMode::HandleModelAssetChanged()
{
	UpdateAsset();
}

void UStateTreeEditorMode::HandleModelSelectionChanged(const TArray<TWeakObjectPtr<UStateTreeState>>& InSelectedStates) const
{
	if (TSharedPtr<IDetailsView> DetailsView = GetDetailsView())
	{
		TArray<UObject*> Selected;
		Selected.Reserve(InSelectedStates.Num());
		for (const TWeakObjectPtr<UStateTreeState>& WeakState : InSelectedStates)
		{
			if (UStateTreeState* State = WeakState.Get())
			{
				Selected.Add(State);
			}
		}
		DetailsView->SetObjects(Selected);
	}
}

void UStateTreeEditorMode::HandleModelBringNodeToFocus(const UStateTreeState* State, const FGuid NodeID) const
{
	const FPropertyBindingPath& BindingPathToNode = FPropertyBindingPath(NodeID);
	return HandleModelBringBindingPathToFocus(State, BindingPathToNode);
}

void UStateTreeEditorMode::HandleModelBringBindingPathToFocus(const UStateTreeState* State, const FPropertyBindingPath& InBindingPath) const
{
	auto BringToFocus = [Self = this](const FPropertyPath& HighlightPath, TSharedPtr<IDetailsView> DetailsView)
		{
			if (Self->HighlightTimerHandle.IsValid())
			{
				if (TSharedPtr<IDetailsView> AssetDetailsView = Self->GetAssetDetailsView())
				{
					AssetDetailsView->HighlightProperty({});
				}

				if (TSharedPtr<IDetailsView> StateDetailsView = Self->GetDetailsView())
				{
					StateDetailsView->HighlightProperty({});
				}

				GEditor->GetTimerManager()->ClearTimer(Self->HighlightTimerHandle);
			}

			constexpr bool bExpandProperty = true;
			DetailsView->ScrollPropertyIntoView(HighlightPath, bExpandProperty);
			DetailsView->HighlightProperty(HighlightPath);

			constexpr bool bLoop = false;
			GEditor->GetTimerManager()->SetTimer(
				Self->HighlightTimerHandle,
				FTimerDelegate::CreateLambda([WeakEditorMode = TWeakObjectPtr<const UStateTreeEditorMode>(Self), WeakSelectionDetailsView = DetailsView.ToWeakPtr()]()
					{
						if (TSharedPtr<IDetailsView> SelectionDetailsView = WeakSelectionDetailsView.Pin())
						{
							SelectionDetailsView->HighlightProperty({});
						}

						if (const UStateTreeEditorMode* EditorModePtr = WeakEditorMode.Get())
						{
							EditorModePtr->HighlightTimerHandle.Invalidate();
						}
					}),
				1.0f,
				bLoop);
		};

	auto MakeHighlightPathFromIndirections = [](FStateTreeDataView InValue, const FPropertyBindingPath& InPropertyBindingPath, FPropertyPath& OutHighlightPath, int32 InStartIndex = 0)
		{
			check(InStartIndex >= 0);

			// We want it to be in sync with what we show
			constexpr bool bHandleRedirects = false;
			// We want to have partial path to find the most match one
			constexpr bool bResetIndirectionsIfFailed = false;
			TArray<FPropertyBindingPathIndirection> Indirections;

			InPropertyBindingPath.ResolveIndirectionsWithValue(InValue, Indirections, nullptr, bHandleRedirects, bResetIndirectionsIfFailed);
			for (int32 Idx = InStartIndex; Idx < Indirections.Num(); ++Idx)
			{
				const FPropertyBindingPathIndirection& Indirection = Indirections[Idx];
				if (FProperty* Property = const_cast<FProperty*>(Indirection.GetProperty()))
				{
					OutHighlightPath.AddProperty(FPropertyInfo(Property, Indirection.GetArrayIndex()));
				}
			}
		};

	auto MakeHighlightPathFromEvent = [&MakeHighlightPathFromIndirections](FProperty* InEventProperty, const FStateTreeBindableStructDesc& InDesc, FStateTreeDataView InValue, const FPropertyBindingPath& InPropertyBindingPath, FPropertyPath& OutHighlightPath)
		{
			// @todo: see UE-356039. We can only track up to RequiredEvent level because StateTree Event populates "fake" property row not associated with any PropertyNode.
			// Once the customization is moved to BindingExtension, StateTree Event should just use InstancedStruct to populate rows, and we will be able to track property paths.

			check(InDesc.DataSource == EStateTreeBindableStructSource::StateEvent || InDesc.DataSource == EStateTreeBindableStructSource::TransitionEvent);

			OutHighlightPath.AddProperty(FPropertyInfo(InEventProperty));
			OutHighlightPath.AddProperty(FPropertyInfo(FStateTreeEventDesc::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FStateTreeEventDesc, PayloadStruct))));

			// Skip InstanceStruct
			constexpr int32 IndirectionStartIndex = 1;
			MakeHighlightPathFromIndirections(InValue, InPropertyBindingPath, OutHighlightPath, IndirectionStartIndex);
		};

	auto MakeHighlightPathFromTransition = [&MakeHighlightPathFromEvent, &MakeHighlightPathFromIndirections](TNotNull<const UStateTreeState*> InState, const FStateTreeBindableStructDesc& InDesc, FStateTreeDataView InValue, const FPropertyBindingPath& InPropertyBindingPath, FPropertyPath& OutHighlightPath)
		{
			check(InDesc.DataSource == EStateTreeBindableStructSource::TransitionEvent || InDesc.DataSource == EStateTreeBindableStructSource::Transition);

			const FGuid& StructID = InDesc.ID;

			if (FArrayProperty* TransitionArrayProperty = CastFieldChecked<FArrayProperty>(UStateTreeState::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UStateTreeState, Transitions))))
			{
				const int32 Index = InState->Transitions.IndexOfByPredicate([&StructID, &InDesc](const FStateTreeTransition& Element)
					{
						if (InDesc.DataSource == EStateTreeBindableStructSource::TransitionEvent)
						{
							return Element.GetEventID() == StructID;
						}

						return Element.ID == StructID;
					});

				if (Index != INDEX_NONE)
				{
					OutHighlightPath.AddProperty(FPropertyInfo(TransitionArrayProperty));
					OutHighlightPath.AddProperty(FPropertyInfo(TransitionArrayProperty->Inner, Index));

					if (InDesc.DataSource == EStateTreeBindableStructSource::TransitionEvent)
					{
						FProperty* RequiredEventProperty = FStateTreeTransition::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FStateTreeTransition, RequiredEvent));
						MakeHighlightPathFromEvent(RequiredEventProperty, InDesc, InValue, InPropertyBindingPath, OutHighlightPath);
					}
					else
					{
						MakeHighlightPathFromIndirections(InValue, InPropertyBindingPath, OutHighlightPath);
					}
				}
			}
		};

	auto MakeHighlightPathFromNode = [&MakeHighlightPathFromIndirections]<typename NodeOwnerType>(
		FName InNodePropertyName, 
		const TConstArrayView<FStateTreeEditorNode> InNodes, 
		const FStateTreeBindableStructDesc& InDesc,
		FStateTreeDataView InValue,
		const FPropertyBindingPath& InPropertyBindingPath,
		FPropertyPath& OutHighlightPath) requires
		(TIsDerivedFrom<NodeOwnerType, UStateTreeState>::Value 
		|| TIsDerivedFrom<NodeOwnerType, UStateTreeEditorData>::Value 
		|| TIsDerivedFrom<NodeOwnerType, FStateTreeTransition>::Value
		|| TIsDerivedFrom<NodeOwnerType, FStateTreePropertyPathBinding>::Value)
		{
			const FGuid& StructID = InDesc.ID;

			const int32 Index = InNodes.IndexOfByPredicate([&StructID](const FStateTreeEditorNode& Element)
				{
					// Binding could be static or dynamic
					return Element.ID == StructID || Element.GetNodeID() == StructID;
				});

			if (Index != INDEX_NONE)
			{
				// For Property Functions, we directly append the property function node instance to the bound property
				if constexpr (!TIsDerivedFrom<NodeOwnerType, FStateTreePropertyPathBinding>::Value)
				{
					FProperty* NodeProperty;
					if constexpr (TIsDerivedFrom<NodeOwnerType, UObject>::Value)
					{
						NodeProperty = NodeOwnerType::StaticClass()->FindPropertyByName(InNodePropertyName);
					}
					else
					{
						NodeProperty = NodeOwnerType::StaticStruct()->FindPropertyByName(InNodePropertyName);
					}

					OutHighlightPath.AddProperty(FPropertyInfo(NodeProperty));
					if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(NodeProperty))
					{
						OutHighlightPath.AddProperty(FPropertyInfo(ArrayProperty->Inner, Index));
					}
				}

				check(InDesc.Struct);

				FName InnerPropertyName;
				if (InDesc.Struct->IsChildOf<UObject>())
				{
					InnerPropertyName = GET_MEMBER_NAME_CHECKED(FStateTreeEditorNode, InstanceObject);
				}
				else if (InDesc.Struct->IsChildOf<FStateTreeNodeBase>())
				{
					InnerPropertyName = GET_MEMBER_NAME_CHECKED(FStateTreeEditorNode, Node);
				}
				else
				{
					InnerPropertyName = GET_MEMBER_NAME_CHECKED(FStateTreeEditorNode, Instance);
				}

				OutHighlightPath.AddProperty(FPropertyInfo(FStateTreeEditorNode::StaticStruct()->FindPropertyByName(InnerPropertyName)));

				MakeHighlightPathFromIndirections(InValue, InPropertyBindingPath, OutHighlightPath);
			}
		};

	auto MakeHighlightPathFromParameters = [&MakeHighlightPathFromIndirections](const FStateTreeBindableStructDesc & InDesc, FStateTreeDataView InValue, const FPropertyBindingPath& InPropertyBindingPath, FPropertyPath& OutHighlightPath)
		{
			check(InDesc.DataSource == EStateTreeBindableStructSource::Parameter || InDesc.DataSource == EStateTreeBindableStructSource::StateParameter);

			if (InDesc.DataSource == EStateTreeBindableStructSource::Parameter)
			{
				// Properties are private, cant go through GET_MEMBER_NAME
				OutHighlightPath.AddProperty(FPropertyInfo(UStateTreeEditorData::StaticClass()->FindPropertyByName(TEXT("RootParameterPropertyBag"))));
			}
			else
			{
				OutHighlightPath.AddProperty(FPropertyInfo(UStateTreeState::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UStateTreeState, Parameters))));
				OutHighlightPath.AddProperty(FPropertyInfo(FStateTreeStateParameters::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FStateTreeStateParameters, Parameters))));
			}

			OutHighlightPath.AddProperty(FPropertyInfo(FInstancedPropertyBag::StaticStruct()->FindPropertyByName(TEXT("Value"))));

			MakeHighlightPathFromIndirections(InValue, InPropertyBindingPath, OutHighlightPath);
		};

	auto MakeHighlightPath = [State, &MakeHighlightPathFromNode, &MakeHighlightPathFromTransition, &MakeHighlightPathFromEvent, &MakeHighlightPathFromParameters](
		auto&& Self, 
		const FStateTreeBindableStructDesc& InDesc, 
		const FPropertyBindingPath& InPropertyBindingPath, 
		const FStateTreeDataView InValue, 
		TNotNull<const UStateTreeEditorData*> InTreeData, 
		FPropertyPath& OutHighlightPath)
	->void
		{
			if (InDesc.DataSource == EStateTreeBindableStructSource::PropertyFunction)
			{
				if (const FStateTreeEditorPropertyBindings* Bindings = InTreeData->GetPropertyEditorBindings())
				{
					for (const FStateTreePropertyPathBinding& Binding : Bindings->GetBindings())
					{
						if (Binding.GetSourcePath().GetStructID() == InDesc.ID)
						{
							InTreeData->VisitAllNodes([&Binding, &Self, InTreeData, &OutHighlightPath](const UStateTreeState* State, const FStateTreeBindableStructDesc& InTargetStructDesc, const FStateTreeDataView InTargetStructValue)
								{
									if (Binding.GetTargetPath().GetStructID() == InTargetStructDesc.ID)
									{
										Self(Self, InTargetStructDesc, Binding.GetTargetPath(), InTargetStructValue, InTreeData, OutHighlightPath);
										return EStateTreeVisitor::Break;
									}

									return EStateTreeVisitor::Continue;
								});

							constexpr FName EmptyName;
							MakeHighlightPathFromNode.operator()<FStateTreePropertyPathBinding>(
								EmptyName,
								TConstArrayView<FStateTreeEditorNode>(Binding.GetPropertyFunctionNode().GetPtr<FStateTreeEditorNode>(), 1),
								InDesc,
								InValue,
								InPropertyBindingPath,
								OutHighlightPath
								);

							break;
						}
					}
				}
			}
			else if (InDesc.DataSource == EStateTreeBindableStructSource::GlobalTask)
			{
				MakeHighlightPathFromNode.operator()<UStateTreeEditorData>(
					GET_MEMBER_NAME_CHECKED(UStateTreeEditorData, GlobalTasks), 
					InTreeData->GlobalTasks,
					InDesc,
					InValue,
					InPropertyBindingPath,
					OutHighlightPath
					);
			}
			else if (InDesc.DataSource == EStateTreeBindableStructSource::Evaluator)
			{
				MakeHighlightPathFromNode.operator()<UStateTreeEditorData>(
					GET_MEMBER_NAME_CHECKED(UStateTreeEditorData, Evaluators), 
					InTreeData->Evaluators,
					InDesc,
					InValue,
					InPropertyBindingPath,
					OutHighlightPath);
			}
			else if (InDesc.DataSource == EStateTreeBindableStructSource::Parameter)
			{
				MakeHighlightPathFromParameters(InDesc, InValue, InPropertyBindingPath, OutHighlightPath);
			}
			else if (InDesc.DataSource == EStateTreeBindableStructSource::Context)
			{
				// Do nothing. We cannot highlight a category node and there is no property nodes associated with context
			}
			else if (State)
			{
				if (InDesc.DataSource == EStateTreeBindableStructSource::StateParameter)
				{
					MakeHighlightPathFromParameters(InDesc, InValue, InPropertyBindingPath, OutHighlightPath);
				}
				else if (InDesc.DataSource == EStateTreeBindableStructSource::Task)
				{
					MakeHighlightPathFromNode.operator()<UStateTreeState>(
						GET_MEMBER_NAME_CHECKED(UStateTreeState, Tasks), 
						State->Tasks,
						InDesc,
						InValue,
						InPropertyBindingPath,
						OutHighlightPath);
					if (!OutHighlightPath.IsValid())
					{
						MakeHighlightPathFromNode.operator()<UStateTreeState>(
							GET_MEMBER_NAME_CHECKED(UStateTreeState, SingleTask), 
							TConstArrayView<FStateTreeEditorNode>(&State->SingleTask, 1),
							InDesc,
							InValue,
							InPropertyBindingPath,
							OutHighlightPath);
					}
				}
				else if (InDesc.DataSource == EStateTreeBindableStructSource::Condition)
				{
					MakeHighlightPathFromNode.operator()<UStateTreeState>(
						GET_MEMBER_NAME_CHECKED(UStateTreeState, EnterConditions), 
						State->EnterConditions,
						InDesc,
						InValue,
						InPropertyBindingPath,
						OutHighlightPath);

					if (!OutHighlightPath.IsValid())
					{
						FPropertyPath TempHighlightPath;
						for (int32 Idx = 0; Idx < State->Transitions.Num(); ++Idx)
						{
							const FStateTreeTransition& Transition = State->Transitions[Idx];

							MakeHighlightPathFromNode.operator()<FStateTreeTransition>(
								GET_MEMBER_NAME_CHECKED(FStateTreeTransition, Conditions),
								Transition.Conditions,
								InDesc,
								InValue,
								InPropertyBindingPath,
								TempHighlightPath);

							if (TempHighlightPath.IsValid())
							{
								FArrayProperty* TransitionsProperty = CastFieldChecked<FArrayProperty>(UStateTreeState::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UStateTreeState, Transitions)));
								check(TransitionsProperty);

								OutHighlightPath.AddProperty(FPropertyInfo(TransitionsProperty));
								OutHighlightPath.AddProperty(FPropertyInfo(TransitionsProperty->Inner, Idx));

								for (int32 PropertyInfoIdx = 0; PropertyInfoIdx < TempHighlightPath.GetNumProperties(); ++PropertyInfoIdx)
								{
									OutHighlightPath.AddProperty(TempHighlightPath.GetPropertyInfo(PropertyInfoIdx));
								}

								break;
							}
						}
					}
				}
				else if (InDesc.DataSource == EStateTreeBindableStructSource::Consideration)
				{
					MakeHighlightPathFromNode.operator()<UStateTreeState>(
						GET_MEMBER_NAME_CHECKED(UStateTreeState, Considerations), 
						State->Considerations,
						InDesc,
						InValue,
						InPropertyBindingPath,
						OutHighlightPath);
				}
				else if (InDesc.DataSource == EStateTreeBindableStructSource::Transition)
				{
					MakeHighlightPathFromTransition(
						State,
						InDesc,
						InValue,
						InPropertyBindingPath,
						OutHighlightPath);
				}
				else if (InDesc.DataSource == EStateTreeBindableStructSource::TransitionEvent)
				{
					MakeHighlightPathFromTransition(
						State,
						InDesc,
						InValue,
						InPropertyBindingPath,
						OutHighlightPath);
				}
				else if (InDesc.DataSource == EStateTreeBindableStructSource::StateEvent)
				{
					MakeHighlightPathFromEvent(
						UStateTreeState::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UStateTreeState, RequiredEventToEnter)),
						InDesc,
						InValue,
						InPropertyBindingPath,
						OutHighlightPath
					);
				}
				
			}
		};

	UStateTree* StateTree = GetStateTree();
	if (StateTree == nullptr)
	{
		return;
	}
	const UStateTreeEditorData* TreeData = Cast<UStateTreeEditorData>(StateTree->EditorData);
	if (TreeData == nullptr)
	{
		return;
	}

	TSharedPtr<IDetailsView> DetailsView = State ? GetDetailsView() : GetAssetDetailsView();
	if (!DetailsView.IsValid())
	{
		return;
	}

	const FGuid& StructID = InBindingPath.GetStructID();

	FPropertyPath HighlightPath;

	if (State)
	{
		TreeData->VisitStateNodes(*State, [&StructID, TreeData, &HighlightPath, &InBindingPath, &MakeHighlightPath](const UStateTreeState* State, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)
			{
				if (Desc.ID == StructID)
				{
					MakeHighlightPath(MakeHighlightPath, Desc, InBindingPath, Value, TreeData, HighlightPath);
					return EStateTreeVisitor::Break;
				}
				return EStateTreeVisitor::Continue;
			});
	}
	else
	{
		TreeData->VisitGlobalNodes([&StructID, TreeData, &HighlightPath, &InBindingPath, &MakeHighlightPath](const UStateTreeState* State, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)
			{
				if (Desc.ID == StructID)
				{
					MakeHighlightPath(MakeHighlightPath, Desc, InBindingPath, Value, TreeData, HighlightPath);
					return EStateTreeVisitor::Break;
				}

				return EStateTreeVisitor::Continue;
			});
	}

	BringToFocus(HighlightPath, DetailsView);
}

void UStateTreeEditorMode::UpdateAsset()
{
	UStateTree* StateTree = GetStateTree();
	if (!StateTree)
	{
		return;
	}

	UStateTreeEditingSubsystem::ValidateStateTree(StateTree);
	const uint32 CurrentEditorDataHash = UStateTreeEditingSubsystem::CalculateStateTreeHash(StateTree);
	if (CurrentEditorDataHash != StateTree->LastCompiledEditorDataHash)
	{
		UStateTreeEditingSubsystem::MarkAsModified(StateTree);
	}
}

TSharedPtr<IDetailsView> UStateTreeEditorMode::GetDetailsView() const
{
	if (const UContextObjectStore* ContextObjectStore = GetToolManager()->GetContextObjectStore())
	{
		if (const UStateTreeEditorContext* Context = ContextObjectStore->FindContext<UStateTreeEditorContext>())
		{
			return Context->EditorHostInterface->GetDetailsView();
		}
	}

	return nullptr;
}

TSharedPtr<IDetailsView> UStateTreeEditorMode::GetAssetDetailsView() const
{
	if (UContextObjectStore* ContextObjectStore = GetToolManager()->GetContextObjectStore())
	{
		if (const UStateTreeEditorContext* Context = ContextObjectStore->FindContext<UStateTreeEditorContext>())
		{
			return Context->EditorHostInterface->GetAssetDetailsView();
		}
	}

	return nullptr;
}

TSharedPtr<IMessageLogListing> UStateTreeEditorMode::GetMessageLogListing() const
{
	if (UContextObjectStore* ContextObjectStore = GetToolManager()->GetContextObjectStore())
	{
		if (const UStateTreeEditorContext* Context = ContextObjectStore->FindContext<UStateTreeEditorContext>())
		{
			FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
			return MessageLogModule.GetLogListing(Context->EditorHostInterface->GetCompilerLogName());
		}
	}

	return nullptr;
}

void UStateTreeEditorMode::ShowCompilerTab() const
{
	if (UContextObjectStore* ContextObjectStore = GetToolManager()->GetContextObjectStore())
	{
		if (const UStateTreeEditorContext* Context = ContextObjectStore->FindContext<UStateTreeEditorContext>())
		{
			if(TSharedPtr<FTabManager> TabManager = GetModeManager()->GetToolkitHost()->GetTabManager())
			{
				TabManager->TryInvokeTab(Context->EditorHostInterface->GetCompilerTabName());
			}
		}
	}
}

UStateTree* UStateTreeEditorMode::GetStateTree() const
{
	return CachedStateTree.Get();
}

void UStateTreeEditorMode::OnAssetFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent) const
{
	// Make sure nodes get updates when properties are changed.
	if(UStateTreeEditingSubsystem* StateTreeEditingSubsystem = GEditor->GetEditorSubsystem<UStateTreeEditingSubsystem>())
	{
		if (TSharedPtr<IDetailsView> AssetDetailsView = GetAssetDetailsView())
		{
			// From the path FPropertyHandleBase::NotifyFinishedChangingProperties(), UObject info is not included in PropertyChangedEvent
			// So we fetch it from DetailsView
			const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = AssetDetailsView->GetSelectedObjects();
			for (const TWeakObjectPtr<UObject>& WeakObject : SelectedObjects)
			{
				if (const UStateTreeEditorData* EditorData = Cast<UStateTreeEditorData>(WeakObject.Get()))
				{
					if (const UStateTree* StateTree = Cast<UStateTree>(EditorData->GetOuter()))
					{
						if (GetStateTree() == StateTree)
						{
							StateTreeEditingSubsystem->FindOrAddViewModel(GetStateTree())->NotifyAssetChangedExternally();
							break;
						}
					}
				}
			}
		}
	}
}

void UStateTreeEditorMode::OnSelectionFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent)
{
	// Make sure nodes get updates when properties are changed.
	if(UStateTreeEditingSubsystem* StateTreeEditingSubsystem = GEditor->GetEditorSubsystem<UStateTreeEditingSubsystem>())
	{
		if (TSharedPtr<IDetailsView> DetailsView = GetDetailsView())
		{
			const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = DetailsView->GetSelectedObjects();
			TSet<UStateTreeState*> ChangedStates;
			for (const TWeakObjectPtr<UObject>& WeakObject : SelectedObjects)
			{
				if (UObject* Object = WeakObject.Get())
				{
					if (UStateTreeState* State = Cast<UStateTreeState>(Object))
					{
						ChangedStates.Add(State);
					}
				}
			}
			if (ChangedStates.Num() > 0)
			{
				StateTreeEditingSubsystem->FindOrAddViewModel(GetStateTree())->NotifyStatesChangedExternally(ChangedStates, PropertyChangedEvent);
				UpdateAsset();
				UStateTreeEditingSubsystem::MarkAsModified(GetStateTree());
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE // "UStateTreeEditorMode"
