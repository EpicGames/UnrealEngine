// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeEditorModeToolkit.h"

#include "ClassViewerModule.h"
#include "ContextObjectStore.h"
#include "InteractiveToolManager.h"
#include "StateTreeEditorCommands.h"
#include "StateTreeEditorWorkspaceTabHost.h"
#include "Blueprint/StateTreeConditionBlueprintBase.h"
#include "Blueprint/StateTreeConsiderationBlueprintBase.h"
#include "Blueprint/StateTreeNodeBlueprintBase.h"
#include "Blueprint/StateTreeTaskBlueprintBase.h"

#include "ClassViewerFilter.h"
#include "ContentBrowserModule.h"
#include "EditorModeManager.h"
#include "FindTools/SStateTreeFind.h"
#include "IContentBrowserSingleton.h"
#include "SStateTreeOutliner.h"
#include "StateTreeEditingSubsystem.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Debugger/SStateTreeDebuggerView.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Toolkits/AssetEditorModeUILayer.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SPropertyBindingView.h"
#include "ToolMenus.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Framework/Application/SlateApplication.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"

#define LOCTEXT_NAMESPACE "StateTreeModeToolkit"

FStateTreeEditorModeToolkit::FStateTreeEditorModeToolkit(UStateTreeEditorMode* InEditorMode)
	: WeakEditorMode(InEditorMode)
{
}

void FStateTreeEditorModeToolkit::RequestModeUITabs()
{
	using namespace UE::StateTreeEditor;
	TSharedPtr<FWorkspaceTabHost> TabHost = EditorHost->GetTabHost();
	check(TabHost);

	if (EditorHost->CanToolkitSpawnWorkspaceTab())
	{
		FModeToolkit::RequestModeUITabs();
	}

	if (ModeUILayer.IsValid())
	{
		if (EditorHost && EditorHost->CanToolkitSpawnWorkspaceTab())
		{
			TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
			TSharedPtr<FWorkspaceItem> MenuGroup = ModeUILayerPtr->GetModeMenuCategory();
			if (!MenuGroup)
			{
				return;
			}

			for (const FMinorWorkspaceTabConfig& Config : TabHost->GetTabConfigs())
			{
				FMinorTabConfig TabInfo;
				TabInfo.TabId = Config.ID;
				TabInfo.TabLabel = Config.Label;
				TabInfo.TabTooltip = Config.Tooltip;
				TabInfo.TabIcon = Config.Icon;
				TabInfo.WorkspaceGroup = MenuGroup;
				TabInfo.OnSpawnTab = TabHost->CreateSpawnDelegate(Config.ID);
				ModeUILayerPtr->SetModePanelInfo(Config.UISystemID, TabInfo);
			}
		}

		for (const FSpawnedWorkspaceTab& SpawnedTab : TabHost->GetSpawnedTabs())
		{
			HandleTabSpawned(SpawnedTab);
		}
		TabHost->OnTabSpawned.AddSP(this, &FStateTreeEditorModeToolkit::HandleTabSpawned);
		TabHost->OnTabClosed.AddSP(this, &FStateTreeEditorModeToolkit::HandleTabClosed);
	}
}

void FStateTreeEditorModeToolkit::Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode)
{
	FModeToolkit::Init(InitToolkitHost, InOwningMode);

	if (UContextObjectStore* ContextObjectStore = InOwningMode->GetToolManager()->GetContextObjectStore())
	{
		if (UStateTreeEditorContext* Context = ContextObjectStore->FindContext<UStateTreeEditorContext>())
		{
			EditorHost = Context->EditorHostInterface;
		}
	}
}

void FStateTreeEditorModeToolkit::InvokeUI()
{
	if (ModeUILayer.IsValid() && EditorHost && EditorHost->CanToolkitSpawnWorkspaceTab())
	{
		TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
		ModeUILayerPtr->GetTabManager()->TryInvokeTab(UAssetEditorUISubsystem::TopLeftTabID);
		ModeUILayerPtr->GetTabManager()->TryInvokeTab(UAssetEditorUISubsystem::BottomRightTabID);
	}
}

void FStateTreeEditorModeToolkit::HandleTabSpawned(UE::StateTreeEditor::FSpawnedWorkspaceTab SpawnedTab)
{
	if (SpawnedTab.TabID == UE::StateTreeEditor::FWorkspaceTabHost::BindingTabId)
	{
		if (TSharedPtr<SDockTab> DockTab = SpawnedTab.DockTab.Pin())
		{
			const UStateTreeEditorData* EditorData = nullptr;
			if (UStateTreeEditorMode* EditorMode = WeakEditorMode.Get())
			{
				if (UStateTree* StateTree = EditorMode->GetStateTree())
				{
					if (UStateTreeEditingSubsystem* Subsystem = GEditor->GetEditorSubsystem<UStateTreeEditingSubsystem>())
					{
						EditorData = Subsystem->FindOrAddViewModel(StateTree)->GetStateTreeEditorData();
					}
				}
			}

			DockTab->SetContent(
				SNew(UE::PropertyBinding::SBindingView)
				.CollectionOwner(TScriptInterface<const IPropertyBindingBindingCollectionOwner>(EditorData))
				.OnBindingClicked_Lambda([WeakEditorHost = EditorHost.ToWeakPtr()](bool bInIsStruct, const FPropertyBindingPath& InBindingPath)
					{
						if (const TSharedPtr<const IStateTreeEditorHost> EditorHost = WeakEditorHost.Pin())
						{
							if (UStateTree* StateTree = EditorHost->GetStateTree())
							{
								if (UStateTreeEditingSubsystem* StateTreeEditingSubsystem = GEditor->GetEditorSubsystem<UStateTreeEditingSubsystem>())
								{
									// Make an empty binding path only containing struct ID if it is a struct
									const FPropertyBindingPath& BindingPath = bInIsStruct ? FPropertyBindingPath(InBindingPath.GetStructID()) : InBindingPath;

									const TSharedRef<FStateTreeViewModel> ViewModel = StateTreeEditingSubsystem->FindOrAddViewModel(StateTree);
									ViewModel->BringBindingPathToFocus(BindingPath);
								}
							}
						}
					})
			);
		}
	}
	else if (SpawnedTab.TabID == UE::StateTreeEditor::FWorkspaceTabHost::OutlinerTabId)
	{
		WeakOutlinerTab = SpawnedTab.DockTab;
		UpdateStateTreeOutliner();
	}
	else if (SpawnedTab.TabID == UE::StateTreeEditor::FWorkspaceTabHost::SearchTabId)
	{
		if (TSharedPtr<SDockTab> DockTab = SpawnedTab.DockTab.Pin())
		{
			DockTab->SetContent(
				SNew(UE::StateTreeEditor::SFindInAsset, EditorHost)
				.bShowSearchBar(true)
			);
		}
	}
	else if (SpawnedTab.TabID == UE::StateTreeEditor::FWorkspaceTabHost::StatisticsTabId)
	{
		if (TSharedPtr<SDockTab> DockTab = SpawnedTab.DockTab.Pin())
		{
			DockTab->SetContent(
				SNew(SMultiLineEditableTextBox)
				.Padding(10.0f)
				.Style(FAppStyle::Get(), "Log.TextBox")
				.Font(FCoreStyle::GetDefaultFontStyle("Mono", 9))
				.ForegroundColor(FLinearColor::Gray)
				.IsReadOnly(true)
				.Text(this, &FStateTreeEditorModeToolkit::GetStatisticsText)
			);
		}
	}
	else if (SpawnedTab.TabID == UE::StateTreeEditor::FWorkspaceTabHost::DebuggerTabId)
	{
#if WITH_STATETREE_TRACE_DEBUGGER
		WeakDebuggerTab = SpawnedTab.DockTab;
		UpdateDebuggerView();
#endif // WITH_STATETREE_TRACE_DEBUGGER
	}
}

void FStateTreeEditorModeToolkit::HandleTabClosed(UE::StateTreeEditor::FSpawnedWorkspaceTab SpawnedTab)
{
	if (TSharedPtr<SDockTab> DockTab = WeakDebuggerTab.Pin())
	{
		// Destroy the inner widget.
		DockTab->SetContent(SNullWidget::NullWidget);
	}
}

FName FStateTreeEditorModeToolkit::GetToolkitFName() const
{
	return FName("StateTreeMode");
}

FText FStateTreeEditorModeToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("ToolkitName", "State Tree Mode");
}


FSlateIcon FStateTreeEditorModeToolkit::GetCompileStatusImage() const
{
	static const FName CompileStatusBackground("Blueprint.CompileStatus.Background");
	static const FName CompileStatusUnknown("Blueprint.CompileStatus.Overlay.Unknown");
	static const FName CompileStatusError("Blueprint.CompileStatus.Overlay.Error");
	static const FName CompileStatusGood("Blueprint.CompileStatus.Overlay.Good");

	if (UStateTreeEditorMode* EditorMode = WeakEditorMode.Get())
	{
		UStateTree* StateTree = EditorMode->GetStateTree();
		if (StateTree)
		{
			if (StateTree->IsEditorDataDirty())
			{
				return FSlateIcon(FAppStyle::GetAppStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusUnknown);
			}
			else if (!StateTree->IsReadyToRun())
			{
				return FSlateIcon(FAppStyle::GetAppStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusError);
			}

			return FSlateIcon(FAppStyle::GetAppStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusGood);
		}
	}

	return FSlateIcon(FAppStyle::GetAppStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusUnknown);
}


namespace UE::StateTree::Editor::Internal
{
static void MakeSaveOnCompileSubMenu(UToolMenu* InMenu)
{
	FToolMenuSection& Section = InMenu->AddSection("Section");
	const FStateTreeEditorCommands& Commands = FStateTreeEditorCommands::Get();
	Section.AddMenuEntry(Commands.SaveOnCompile_Never);
	Section.AddMenuEntry(Commands.SaveOnCompile_SuccessOnly);
	Section.AddMenuEntry(Commands.SaveOnCompile_Always);
}

static void GenerateCompileOptionsMenu(UToolMenu* InMenu)
{
	FToolMenuSection& Section = InMenu->AddSection("Section");

	// @TODO: disable the menu and change up the tooltip when all sub items are disabled
	Section.AddSubMenu(
		"SaveOnCompile",
		LOCTEXT("SaveOnCompileSubMenu", "Save on Compile"),
		LOCTEXT("SaveOnCompileSubMenu_ToolTip", "Determines how the StateTree is saved whenever you compile it."),
		FNewToolMenuDelegate::CreateStatic(&MakeSaveOnCompileSubMenu));
}
}

void FStateTreeEditorModeToolkit::ExtendSecondaryModeToolbar(UToolMenu* ToolBar)
{
	ToolBar->Context.AppendCommandList(ToolkitCommands);

	const FStateTreeEditorCommands& Commands = FStateTreeEditorCommands::Get();
	ensure(ToolBar->Context.GetActionForCommand(Commands.Compile));

	static const FToolMenuInsert InsertLast(NAME_None, EToolMenuInsertType::Last);

	FToolMenuSection& CompileSection = ToolBar->AddSection("Compile", FText(), InsertLast);

	auto GetToolkitFromAssetEditorContext = [](const UAssetEditorToolkitMenuContext* InContext) -> TSharedPtr<FStateTreeEditorModeToolkit>
	{
		if (InContext)
		{
			if (TSharedPtr<FAssetEditorToolkit> SharedToolkit = InContext->Toolkit.Pin())
			{
				if(UStateTreeEditorMode* Mode = Cast<UStateTreeEditorMode>(SharedToolkit->GetEditorModeManager().GetActiveScriptableMode(UStateTreeEditorMode::EM_StateTree)))
				{
					if (TSharedPtr<FModeToolkit> Toolkit = Mode->GetToolkit().Pin())
					{
						return StaticCastSharedPtr<FStateTreeEditorModeToolkit>(Toolkit);
					}
				}
			}
		}

		return nullptr;
	};

	CompileSection.AddDynamicEntry("CompileCommands", FNewToolMenuSectionDelegate::CreateLambda([GetToolkitFromAssetEditorContext](FToolMenuSection& InSection)
	{
		if (UAssetEditorToolkitMenuContext* ToolkitContextObject = InSection.FindContext<UAssetEditorToolkitMenuContext>())
		{
			const FStateTreeEditorCommands& Commands = FStateTreeEditorCommands::Get();
			FToolMenuEntry& CompileButton = InSection.AddEntry(FToolMenuEntry::InitToolBarButton(
				Commands.Compile,
				TAttribute<FText>(),
				TAttribute<FText>(),
				TAttribute<FSlateIcon>::CreateLambda([GetToolkitFromAssetEditorContext, ToolkitContextObject]() -> FSlateIcon
				{
					if(TSharedPtr<FStateTreeEditorModeToolkit> SharedToolkit = GetToolkitFromAssetEditorContext(ToolkitContextObject))
					{
						return SharedToolkit->GetCompileStatusImage();
					}

					return FSlateIcon();
				}))
			);
			CompileButton.Name = "StateTreeCompile";
			CompileButton.StyleNameOverride = "CalloutToolbar";

			FToolMenuEntry& CompileOptions = InSection.AddEntry(FToolMenuEntry::InitComboButton(
				"CompileComboButton",
				FUIAction(),
				FNewToolMenuDelegate::CreateStatic(&UE::StateTree::Editor::Internal::GenerateCompileOptionsMenu),
				LOCTEXT("CompileOptions_ToolbarTooltip", "Options to customize how State Trees compile")
			));
			CompileOptions.StyleNameOverride = "CalloutToolbar";
			CompileOptions.ToolBarData.bSimpleComboBox = true;
		}
	}));

	static const FToolMenuInsert InsertAfterCompileSection("Compile", EToolMenuInsertType::After);

	FToolMenuSection& CreateNewNodeSection = ToolBar->AddSection("CreateNewNodes", TAttribute<FText>(), InsertAfterCompileSection);
	CreateNewNodeSection.AddDynamicEntry("CreateNewNodes", FNewToolMenuSectionDelegate::CreateLambda([GetToolkitFromAssetEditorContext](FToolMenuSection& InSection)
	{
		if (UAssetEditorToolkitMenuContext* ToolkitContextObject = InSection.FindContext<UAssetEditorToolkitMenuContext>())
		{
			InSection.AddEntry(FToolMenuEntry::InitComboButton(
				 "CreateNewTaskComboButton",
				 FUIAction(FExecuteAction(), FCanExecuteAction(), FIsActionChecked(),
				 	FIsActionButtonVisible::CreateLambda([ToolkitContextObject, GetToolkitFromAssetEditorContext]() -> bool
					{
				 		if(TSharedPtr<FStateTreeEditorModeToolkit> SharedToolkit = GetToolkitFromAssetEditorContext(ToolkitContextObject))
					 	{
							 if(UStateTree* StateTree = SharedToolkit->EditorHost->GetStateTree())
							 {
								 if (const UStateTreeEditorData* EditorData = Cast<UStateTreeEditorData>(StateTree->EditorData))
								 {
									 if (const UStateTreeSchema* Schema = EditorData->Schema.Get())
									 {
										 return Schema->IsClassAllowed(UStateTreeTaskBlueprintBase::StaticClass());
									 }
								 }
							 }
						}

						return false;
					})),
				 FOnGetContent::CreateLambda([ToolkitContextObject, GetToolkitFromAssetEditorContext]() -> TSharedRef<SWidget>
				 {
				 	if(TSharedPtr<FStateTreeEditorModeToolkit> SharedToolkit = GetToolkitFromAssetEditorContext(ToolkitContextObject))
					{
						return SharedToolkit->GenerateTaskBPBaseClassesMenu();
					}

					return SNullWidget::NullWidget;
				 }),
				 LOCTEXT("CreateNewTask_Title", "New Task"),
				 LOCTEXT("CreateNewTask_ToolbarTooltip", "Create a new Blueprint State Tree Task"),
				 GetNewTaskButtonImage()
			 ));

			InSection.AddEntry(FToolMenuEntry::InitComboButton(
				 "CreateNewConditionComboButton",
				 FUIAction(FExecuteAction(), FCanExecuteAction(), FIsActionChecked(),
					 FIsActionButtonVisible::CreateLambda([ToolkitContextObject, GetToolkitFromAssetEditorContext]() -> bool
					{
						if(TSharedPtr<FStateTreeEditorModeToolkit> SharedToolkit = GetToolkitFromAssetEditorContext(ToolkitContextObject))
						{
							if(UStateTree* StateTree = SharedToolkit->EditorHost->GetStateTree())
							{
								if (const UStateTreeEditorData* EditorData = Cast<UStateTreeEditorData>(StateTree->EditorData))
								{
									if (const UStateTreeSchema* Schema = EditorData->Schema.Get())
									{
										return Schema->IsClassAllowed(UStateTreeConditionBlueprintBase::StaticClass());
									}
								}
							}
						}

						return false;
					})),
				FOnGetContent::CreateLambda([ToolkitContextObject, GetToolkitFromAssetEditorContext]() -> TSharedRef<SWidget>
				{
					if(TSharedPtr<FStateTreeEditorModeToolkit> SharedToolkit = GetToolkitFromAssetEditorContext(ToolkitContextObject))
					{
					   return SharedToolkit->GenerateConditionBPBaseClassesMenu();
					}

					return SNullWidget::NullWidget;
				}),
				 LOCTEXT("CreateNewCondition_Title", "New Condition"),
				 LOCTEXT("CreateNewCondition_ToolbarTooltip", "Create a new Blueprint State Tree Condition"),
				 GetNewConditionButtonImage()
			 ));

			 InSection.AddEntry(FToolMenuEntry::InitComboButton(
				 "CreateNewConsiderationComboButton",
				 FUIAction(FExecuteAction(), FCanExecuteAction(), FIsActionChecked(),
					 FIsActionButtonVisible::CreateLambda([ToolkitContextObject, GetToolkitFromAssetEditorContext]() -> bool
					{
					 	if(TSharedPtr<FStateTreeEditorModeToolkit> SharedToolkit = GetToolkitFromAssetEditorContext(ToolkitContextObject))
					 	{
							 if(UStateTree* StateTree = SharedToolkit->EditorHost->GetStateTree())
							 {
								 if (const UStateTreeEditorData* EditorData = Cast<UStateTreeEditorData>(StateTree->EditorData))
								 {
									 if (const UStateTreeSchema* Schema = EditorData->Schema.Get())
									 {
										 return Schema->IsClassAllowed(UStateTreeConsiderationBlueprintBase::StaticClass());
									 }
								 }
							 }
						}

						return false;
					})),
				 FOnGetContent::CreateLambda([ToolkitContextObject, GetToolkitFromAssetEditorContext]() -> TSharedRef<SWidget>
				{
				 	if(TSharedPtr<FStateTreeEditorModeToolkit> SharedToolkit = GetToolkitFromAssetEditorContext(ToolkitContextObject))
				 	{
						return SharedToolkit->GenerateConsiderationBPBaseClassesMenu();
					}

					return SNullWidget::NullWidget;
				}),
				 LOCTEXT("CreateNewConsideration_Title", "New Consideration"),
				 LOCTEXT("CreateNewConsideration_ToolbarTooltip", "Create a new Blueprint State Tree Utility Consideration"),
				 GetNewConsiderationButtonImage()
			 ));
		}
	}));

	const FName StateTreeEditModeProfile = TEXT("StateTreeEditModeDisabledProfile");
	FToolMenuProfile* ToolbarProfile = UToolMenus::Get()->AddRuntimeMenuProfile(ToolBar->GetMenuName(), StateTreeEditModeProfile);
	{
		ToolbarProfile->MenuPermissions.AddDenyListItem("CompileCommands", "StateTreeCompile");
		ToolbarProfile->MenuPermissions.AddDenyListItem("CreateNewNodes", "CreateNewTaskComboButton");
		ToolbarProfile->MenuPermissions.AddDenyListItem("CreateNewNodes", "CreateNewConditionComboButton");
		ToolbarProfile->MenuPermissions.AddDenyListItem("CreateNewNodes", "CreateNewConsiderationComboButton");
	}
}

UStateTree* FStateTreeEditorModeToolkit::GetStateTree() const
{
	return EditorHost ? EditorHost->GetStateTree() : nullptr;
}

void FStateTreeEditorModeToolkit::OnStateTreeChanged()
{
	// Update underlying state tree state
	UpdateStateTreeOutliner();

#if WITH_STATETREE_TRACE_DEBUGGER
	UpdateDebuggerView();
#endif // WITH_STATETREE_TRACE_DEBUGGER
}


namespace UE::StateTree::Editor
{
template <typename ClassType, typename = typename TEnableIf<TIsDerivedFrom<ClassType, UStateTreeNodeBlueprintBase>::Value>::Type>
class FEditorNodeClassFilter : public IClassViewerFilter
{
public:
	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		check(InClass);
		return InClass->IsChildOf(ClassType::StaticClass());
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return InUnloadedClassData->IsChildOf(ClassType::StaticClass());
	}
};

using FStateTreeTaskBPClassFilter = FEditorNodeClassFilter<UStateTreeTaskBlueprintBase>;
using FStateTreeConditionBPClassFilter = FEditorNodeClassFilter<UStateTreeConditionBlueprintBase>;
using FStateTreeConsiderationBPClassFilter = FEditorNodeClassFilter<UStateTreeConsiderationBlueprintBase>;
}; // UE::StateTree::Editor

FSlateIcon FStateTreeEditorModeToolkit::GetNewTaskButtonImage()
{
	return FSlateIcon("StateTreeEditorStyle", "StateTreeEditor.Tasks.Large");
}

TSharedRef<SWidget> FStateTreeEditorModeToolkit::GenerateTaskBPBaseClassesMenu() const
{
	FClassViewerInitializationOptions Options;
	Options.NameTypeToDisplay = EClassViewerNameTypeToDisplay::DisplayName;
	Options.ClassFilters.Add(MakeShareable(new UE::StateTree::Editor::FStateTreeTaskBPClassFilter));

	FOnClassPicked OnPicked(FOnClassPicked::CreateSP(this, &FStateTreeEditorModeToolkit::OnNodeBPBaseClassPicked));

	return FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateClassViewer(Options, OnPicked);
}

FSlateIcon FStateTreeEditorModeToolkit::GetNewConditionButtonImage()
{
	return FSlateIcon("StateTreeEditorStyle", "StateTreeEditor.Conditions.Large");
}

TSharedRef<SWidget> FStateTreeEditorModeToolkit::GenerateConditionBPBaseClassesMenu() const
{
	FClassViewerInitializationOptions Options;
	Options.NameTypeToDisplay = EClassViewerNameTypeToDisplay::DisplayName;
	Options.ClassFilters.Add(MakeShareable(new UE::StateTree::Editor::FStateTreeConditionBPClassFilter));

	FOnClassPicked OnPicked(FOnClassPicked::CreateSP(this, &FStateTreeEditorModeToolkit::OnNodeBPBaseClassPicked));

	return FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateClassViewer(Options, OnPicked);
}

FSlateIcon FStateTreeEditorModeToolkit::GetNewConsiderationButtonImage()
{
    return FSlateIcon("StateTreeEditorStyle", "StateTreeEditor.Utility.Large");
}

TSharedRef<SWidget> FStateTreeEditorModeToolkit::GenerateConsiderationBPBaseClassesMenu() const
{
	FClassViewerInitializationOptions Options;
    Options.NameTypeToDisplay = EClassViewerNameTypeToDisplay::DisplayName;
    Options.ClassFilters.Add(MakeShareable(new UE::StateTree::Editor::FStateTreeConsiderationBPClassFilter));

    FOnClassPicked OnPicked(FOnClassPicked::CreateSP(this, &FStateTreeEditorModeToolkit::OnNodeBPBaseClassPicked));

    return FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateClassViewer(Options, OnPicked);
}

void FStateTreeEditorModeToolkit::OnNodeBPBaseClassPicked(UClass* NodeClass) const
{
	check(NodeClass);

	UStateTreeEditorMode* EditorMode = WeakEditorMode.Get();
	if (EditorMode == nullptr)
	{
		return;
	}
	UStateTree* StateTree = EditorMode->GetStateTree();
	if (StateTree == nullptr)
	{
		return;
	}

	const FString ClassName = FBlueprintEditorUtils::GetClassNameWithoutSuffix(NodeClass);
	const FString PathName = FPaths::GetPath(StateTree->GetOutermost()->GetPathName());

	// Now that we've generated some reasonable default locations/names for the package, allow the user to have the final say
	// before we create the package and initialize the blueprint inside of it.
	FSaveAssetDialogConfig SaveAssetDialogConfig;
	SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveAssetDialogTitle", "Save Asset As");
	SaveAssetDialogConfig.DefaultPath = PathName;
	SaveAssetDialogConfig.DefaultAssetName = ClassName + TEXT("_New");
	SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::Disallow;

	const FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	const FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);
	if (!SaveObjectPath.IsEmpty())
	{
		const FString SavePackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);
		const FString SavePackagePath = FPaths::GetPath(SavePackageName);
		const FString SaveAssetName = FPaths::GetBaseFilename(SavePackageName);

		if (UPackage* Package = CreatePackage(*SavePackageName))
		{
			// Create and init a new Blueprint
			if (UBlueprint* NewBP = FKismetEditorUtilities::CreateBlueprint(NodeClass, Package, FName(*SaveAssetName), BPTYPE_Normal, UBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass()))
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(NewBP);

				// Notify the asset registry
				FAssetRegistryModule::AssetCreated(NewBP);

				Package->MarkPackageDirty();
			}
		}
	}

	FSlateApplication::Get().DismissAllMenus();
}

FText FStateTreeEditorModeToolkit::GetStatisticsText() const
{
	UStateTreeEditorMode* EditorMode = WeakEditorMode.Get();
	if (EditorMode == nullptr)
	{
		return FText::GetEmpty();
	}

	UStateTree* StateTree = EditorMode->GetStateTree();
	if (StateTree == nullptr)
	{
		return FText::GetEmpty();
	}


   TArray<FStateTreeMemoryUsage> MemoryUsages = StateTree->CalculateEstimatedMemoryUsage();
   if (MemoryUsages.IsEmpty())
   {
	return FText::GetEmpty();
   }

	TArray<FText> Rows;
	for (const FStateTreeMemoryUsage& Usage : MemoryUsages)
	{
		const FText SizeText = FText::AsMemory(Usage.EstimatedMemoryUsage);
		const FText NumNodesText = FText::AsNumber(Usage.NodeCount);
		Rows.Add(FText::Format(LOCTEXT("UsageRow", "{0}: {1}, {2} nodes"), FText::FromString(Usage.Name), SizeText, NumNodesText));
	}

	return FText::Join(FText::FromString(TEXT("\n")), Rows);
}

const FPropertyBindingBindingCollection* FStateTreeEditorModeToolkit::GetBindingCollection() const
{
	if (TStrongObjectPtr<UStateTreeEditorMode> EditorMode = WeakEditorMode.Pin())
	{
		if (UStateTree* StateTree = EditorMode->GetStateTree())
		{
			if (UStateTreeEditingSubsystem* Subsystem = GEditor->GetEditorSubsystem<UStateTreeEditingSubsystem>())
			{
				if (const UStateTreeEditorData* EditorData = Subsystem->FindOrAddViewModel(StateTree)->GetStateTreeEditorData())
				{
					return EditorData->GetPropertyEditorBindings();
				}
			}
		}
	}
	return nullptr;
}

void FStateTreeEditorModeToolkit::UpdateStateTreeOutliner()
{
	StateTreeOutliner = SNullWidget::NullWidget;
	if (UStateTreeEditorMode* EditorMode = WeakEditorMode.Get())
	{
		if (EditorMode && EditorMode->GetStateTree())
		{
			if (UStateTreeEditingSubsystem* Subsystem = GEditor->GetEditorSubsystem<UStateTreeEditingSubsystem>())
			{
				StateTreeOutliner = SNew(SStateTreeOutliner, Subsystem->FindOrAddViewModel(EditorMode->GetStateTree()), GetToolkitCommands());
			}
		}
	}

	if(TSharedPtr<SDockTab> SharedOutlinerTab = WeakOutlinerTab.Pin())
	{
		SharedOutlinerTab->SetContent(StateTreeOutliner.ToSharedRef());
	}
}

#if WITH_STATETREE_TRACE_DEBUGGER
void FStateTreeEditorModeToolkit::UpdateDebuggerView()
{
	TSharedRef<SWidget> DebuggerView = SNullWidget::NullWidget;
	TSharedPtr<SDockTab> SharedDebuggerTab = WeakDebuggerTab.Pin();

	if (SharedDebuggerTab)
	{
		// Clear any references the previous tab might have to a previous debugger view.
		// The view will clear the shared debugger's bindings on dtor.
		// We don't want it to clear newly created bindings from our new view.
		SharedDebuggerTab->SetContent(DebuggerView);
	}

	if (UStateTreeEditorMode* EditorMode = WeakEditorMode.Get())
	{
		if (UStateTree* StateTree = EditorMode->GetStateTree())
		{
			if (UStateTreeEditingSubsystem* Subsystem = GEditor->GetEditorSubsystem<UStateTreeEditingSubsystem>())
			{
				DebuggerView = SNew(SStateTreeDebuggerView, StateTree, Subsystem->FindOrAddViewModel(StateTree), GetToolkitCommands());
			}
		}
	}

	if (SharedDebuggerTab)
	{
		SharedDebuggerTab->SetContent(DebuggerView);
	}
}
#endif // WITH_STATETREE_TRACE_DEBUGGER

#undef LOCTEXT_NAMESPACE // "StateTreeModeToolkit"
