// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeEditorModule.h"
#include "Blueprint/StateTreeConditionBlueprintBase.h"
#include "Blueprint/StateTreeConsiderationBlueprintBase.h"
#include "Blueprint/StateTreeEvaluatorBlueprintBase.h"
#include "Blueprint/StateTreeTaskBlueprintBase.h"
#include "Customizations/StateTreeAnyEnumDetails.h"
#include "Customizations/StateTreeBindingExtension.h"
#include "Customizations/StateTreeBlueprintPropertyRefDetails.h"
#include "Customizations/StateTreeEditorColorDetails.h"
#include "Customizations/StateTreeEditorDataDetails.h"
#include "Customizations/StateTreeEditorNodeDetails.h"
#include "Customizations/StateTreeEnumValueScorePairsDetails.h"
#include "Customizations/StateTreeEventDescDetails.h"
#include "Customizations/StateTreeReferenceDetails.h"
#include "Customizations/StateTreeReferenceOverridesDetails.h"
#include "Customizations/StateTreeStateDetails.h"
#include "Customizations/StateTreeStateLinkDetails.h"
#include "Customizations/StateTreeStateParametersDetails.h"
#include "Customizations/StateTreeTransitionDetails.h"
#include "Debugger/StateTreeDebuggerCommands.h"
#include "Debugger/StateTreeRewindDebuggerExtensions.h"
#include "Debugger/StateTreeRewindDebuggerTrack.h"
#include "IRewindDebuggerExtension.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "StateTree.h"
#include "StateTreeCompilerLog.h"
#include "StateTreeCompilerManager.h"
#include "StateTreeDelegates.h"
#include "StateTreeDelegatesInternal.h"
#include "StateTreeEditingSubsystem.h"
#include "StateTreeEditor.h"
#include "StateTreeEditorCommands.h"
#include "StateTreeEditorData.h"
#include "StateTreeEditorDataExtension.h"
#include "StateTreeEditorSchema.h"
#include "StateTreeEditorStyle.h"
#include "StateTreeModule.h"
#include "StateTreeNodeClassCache.h"
#include "StateTreePropertyFunctionBase.h"
#include "Templates/GuardValueAccessors.h"
#include "UObject/AssetRegistryTagsContext.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"

DEFINE_LOG_CATEGORY(LogStateTreeEditor);

IMPLEMENT_MODULE(FStateTreeEditorModule, StateTreeEditorModule)

namespace UE::StateTree::Editor
{
	/**
	 * If disabled, the dependencies won't be taken into consideration.
	 * The PostLoad calls ConditionalPostLoad on the editor data. That calls ConditionalPostLoad on all editor nodes.
	 * The editor node call ConditionalPostLoad on the other state tree assets.
	 * That ensures that dependent state tree assets are compiled before being used.
	 * When compiling in the editor (once the asset are loaded), you might have a dependency bug.
	 */
	bool bEnableQueuedCompilationOnAssetLoad = false;
	FAutoConsoleVariableRef CVarEnableQueuedCompilationOnAssetLoad(
		TEXT("StateTree.Compiler.EnableQueuedCompilationOnAssetLoad"),
		bEnableQueuedCompilationOnAssetLoad,
		TEXT("On state tree asset load, queue the asset compilation. The asset will be compiled when needed.\n")
		TEXT("If false, then then asset is compiled when loaded.")
	);

	static TSharedRef<FStateTreeNodeClassCache> InitNodeClassCache()
	{
		TSharedRef<FStateTreeNodeClassCache> NodeClassCache = MakeShareable(new FStateTreeNodeClassCache());
		NodeClassCache->AddRootScriptStruct(FStateTreeEvaluatorBase::StaticStruct());
		NodeClassCache->AddRootScriptStruct(FStateTreeTaskBase::StaticStruct());
		NodeClassCache->AddRootScriptStruct(FStateTreeConditionBase::StaticStruct());
		NodeClassCache->AddRootScriptStruct(FStateTreeConsiderationBase::StaticStruct());
		NodeClassCache->AddRootScriptStruct(FStateTreePropertyFunctionBase::StaticStruct());
		NodeClassCache->AddRootClass(UStateTreeEvaluatorBlueprintBase::StaticClass());
		NodeClassCache->AddRootClass(UStateTreeTaskBlueprintBase::StaticClass());
		NodeClassCache->AddRootClass(UStateTreeConditionBlueprintBase::StaticClass());
		NodeClassCache->AddRootClass(UStateTreeConsiderationBlueprintBase::StaticClass());
		NodeClassCache->AddRootClass(UStateTreeSchema::StaticClass());
		return NodeClassCache;
	}

}; // UE::StateTree::Editor


namespace UE::StateTree::Editor::Private
{

#if WITH_STATETREE_TRACE_DEBUGGER
	class FStateTreeEditorDebugInfoProvider : public IStateTreeDebugInfoProvider
	{
		virtual FText GetNodeDescription(TNotNull<const UStateTree*> StateTree, FStateTreeIndex16 NodeIndex) const override
		{
			const FStateTreeNodeBase* Node = StateTree->GetNode(NodeIndex.Get()).GetPtr<FStateTreeNodeBase>();

			if (Node)
			{
				if (UStateTreeEditorData* EditorData = Cast<UStateTreeEditorData>(StateTree->EditorData))
				{
					FStateTreeDataView NodeDataView;

					auto SetNodeDataView = [&NodeDataView, Node](auto& DataContainer)
					{
						// const casts needed as FStateTreeDataView doesn't accept const UObject* & FConstStructView
						if (Node->InstanceDataHandle.IsObjectSource())
						{
							NodeDataView = FStateTreeDataView(const_cast<UObject*>(DataContainer.GetObject(Node->InstanceTemplateIndex.Get())));
						}
						else
						{
							FConstStructView StructView = DataContainer.GetStruct(Node->InstanceTemplateIndex.Get());
							NodeDataView = FStateTreeDataView(StructView.GetScriptStruct(), const_cast<uint8*>(StructView.GetMemory()));
						}
					};

					if (Node->InstanceDataHandle.GetSource() == EStateTreeDataSourceType::EvaluationScopeInstanceData
						|| Node->InstanceDataHandle.GetSource() == EStateTreeDataSourceType::EvaluationScopeInstanceDataObject)
					{
						SetNodeDataView(StateTree->GetDefaultEvaluationScopeInstanceData());
					}
					else if (Node->InstanceDataHandle.GetSource() == EStateTreeDataSourceType::SharedInstanceData
						|| Node->InstanceDataHandle.GetSource() == EStateTreeDataSourceType::SharedInstanceDataObject)
					{
						SetNodeDataView(*StateTree->GetSharedInstanceData());
					}
					else
					{
						SetNodeDataView(StateTree->GetDefaultInstanceData());
					}

					if (NodeDataView.IsValid())
					{
						const FGuid NodeID = StateTree->GetNodeIdFromIndex(NodeIndex);
						const FStateTreeBindingLookup BindingLookup(EditorData);

						return Node->GetDescription(NodeID, NodeDataView, BindingLookup, EStateTreeNodeFormatting::Text);
					}
				}
			}

			return FText::FromString(FString::Printf(TEXT("%s"), Node ? *Node->Name.ToString() : *("Invalid Node Index: " + LexToString(NodeIndex.Get()))));
		}
	};

	bool bEnableEditorDebugInfoProvider = true;

	static void HandleEnableEditorDebugInfoProvider(IConsoleVariable*)
	{
		if (bEnableEditorDebugInfoProvider)
		{
			IStateTreeModule::Get().SetDebugInfoProvider(MakeShared<UE::StateTree::Editor::Private::FStateTreeEditorDebugInfoProvider>());
		}
		else
		{
			IStateTreeModule::Get().SetDebugInfoProvider(nullptr);
		}
	};

	FAutoConsoleVariableRef CVarEnableEditorDebugInfoProvider(
		TEXT("StateTree.Debugger.EnableEditorDebugInfoProvider"),
		bEnableEditorDebugInfoProvider,
		TEXT("True means we will use editor info for runtime debugging views."),
		FConsoleVariableDelegate::CreateStatic(HandleEnableEditorDebugInfoProvider)
	);
#endif // WITH_STATETREE_TRACE_DEBUGGER

} // namespace UE::StateTree::Editor::Private


FStateTreeEditorModule& FStateTreeEditorModule::GetModule()
{
	return FModuleManager::LoadModuleChecked<FStateTreeEditorModule>("StateTreeEditorModule");
}

FStateTreeEditorModule* FStateTreeEditorModule::GetModulePtr()
{
	return FModuleManager::GetModulePtr<FStateTreeEditorModule>("StateTreeEditorModule");
}

void FStateTreeEditorModule::StartupModule()
{
	UE::StateTree::Compiler::FCompilerManager::Startup();

	UE::StateTree::Delegates::OnRequestEditorHash.BindRaw(this, &FStateTreeEditorModule::HandleRequestEditorHash);
	UE::StateTree::Delegates::Private::OnCompileIfChanged.BindRaw(this, &FStateTreeEditorModule::HandleCompileIfChanged);
	UE::StateTree::Delegates::Private::OnStateTreeMarkedAsModified.BindStatic(&UE::StateTree::Compiler::FCompilerManager::MarkAsModified);
	UE::StateTree::Delegates::Private::OnRequestAssetRegistryTags.BindRaw(this, &FStateTreeEditorModule::HandleRequestAssetRegistryTags);
	UE::StateTree::Delegates::Private::OnStateTreeAssetLoaded.BindRaw(this, &FStateTreeEditorModule::HandleStateTreeAssetLoaded);
	UE::StateTree::Delegates::Private::OnStateTreeEditorBindingUpdated.BindRaw(this, &FStateTreeEditorModule::HandleStateTreeEditorBindingUpdated);
	UE::StateTree::Delegates::Private::OnAppendToClassSchema.BindRaw(this, &FStateTreeEditorModule::HandleAppendToStateTreeClassSchema);

	// class cache isn't needed in non-editor runs
	if (GIsEditor)
	{
		OnPostEngineInitHandle = FCoreDelegates::GetOnPostEngineInit().AddLambda([this]()
			{
				NodeClassCache = UE::StateTree::Editor::InitNodeClassCache();

				FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>("Kismet");
				{
					UE::BlueprintEditor::FAdditionalCategory ContextCategory;
					ContextCategory.Category = LOCTEXT("ContextCategory", "Context");
					ContextCategory.ClassFilters.Add(FSoftClassPath(UStateTreeNodeBlueprintBase::StaticClass()));
					ContextCategory.Placeholders.Emplace(
						UE::BlueprintEditor::FAdditionalCategory::FPlaceholder{
							.MenuDescription = LOCTEXT("ContextCategoryMenu", "Category for context bindings."),
							.Tooltip = LOCTEXT("ContextCategoryTooltip", "The property is connected to one of the context objects, or can be overridden with property binding."),
							.SectionID = NodeSectionID::VARIABLE
						});
					ContextCategoryHandle = BlueprintEditorModule.RegisterAdditionalBlueprintCategory(ContextCategory);
				}
				{
					UE::BlueprintEditor::FAdditionalCategory InputCategory;
					InputCategory.Category = LOCTEXT("InputCategory", "Input");
					InputCategory.ClassFilters.Add(FSoftClassPath(UStateTreeNodeBlueprintBase::StaticClass()));
					InputCategory.Placeholders.Emplace(
						UE::BlueprintEditor::FAdditionalCategory::FPlaceholder{
							.MenuDescription = LOCTEXT("InputCategoryMenu", "Category for input bindings."),
							.Tooltip = LOCTEXT("InputCategoryTooltip", "The property is always expected to be bound to some other property."),
							.SectionID = NodeSectionID::VARIABLE
						});
					InputCategoryHandle = BlueprintEditorModule.RegisterAdditionalBlueprintCategory(InputCategory);
				}
				{
					UE::BlueprintEditor::FAdditionalCategory OutputCategory;
					OutputCategory.Category = LOCTEXT("OutputCategory", "Output");
					OutputCategory.ClassFilters.Add(FSoftClassPath(UStateTreeTaskBlueprintBase::StaticClass()));
					OutputCategory.ClassFilters.Add(FSoftClassPath(UStateTreeEvaluatorBlueprintBase::StaticClass()));
					OutputCategory.Placeholders.Emplace(
						UE::BlueprintEditor::FAdditionalCategory::FPlaceholder{
							.MenuDescription = LOCTEXT("OutputCategoryMenu", "Category for output bindings."),
							.Tooltip = LOCTEXT("OutputCategoryTooltip", "The node will always set its value. The property can bind to another property to push the value. Other nodes can also bind to it to fetch the value."),
							.SectionID = NodeSectionID::VARIABLE
						});
					OutputCategoryHandle = BlueprintEditorModule.RegisterAdditionalBlueprintCategory(OutputCategory);
				}
			});
	}

#if WITH_STATETREE_TRACE_DEBUGGER
	FStateTreeDebuggerCommands::Register();

	RewindDebuggerPlaybackExtension = MakePimpl<UE::StateTreeDebugger::FRewindDebuggerPlaybackExtension>();
	IModularFeatures::Get().RegisterModularFeature(IRewindDebuggerExtension::ModularFeatureName, RewindDebuggerPlaybackExtension.Get());

	RewindDebuggerTrackCreator = MakePimpl<UE::StateTreeDebugger::FRewindDebuggerTrackCreator>();
	IModularFeatures::Get().RegisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, RewindDebuggerTrackCreator.Get());

	if (UE::StateTree::Editor::Private::bEnableEditorDebugInfoProvider)
	{
		IStateTreeModule::Get().SetDebugInfoProvider(MakeShared<UE::StateTree::Editor::Private::FStateTreeEditorDebugInfoProvider>());
	}
#endif // WITH_STATETREE_TRACE_DEBUGGER

	MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
	ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);

	FStateTreeEditorStyle::Register();
	FStateTreeEditorCommands::Register();

	// Register the details customizer
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomPropertyTypeLayout("StateTreeTransition", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FStateTreeTransitionDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("StateTreeEventDesc", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FStateTreeEventDescDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("StateTreeStateLink", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FStateTreeStateLinkDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("StateTreeEditorNode", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FStateTreeEditorNodeDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("StateTreeStateParameters", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FStateTreeStateParametersDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("StateTreeAnyEnum", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FStateTreeAnyEnumDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("StateTreeReference", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FStateTreeReferenceDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("StateTreeReferenceOverrides", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FStateTreeReferenceOverridesDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("StateTreeEditorColorRef", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FStateTreeEditorColorRefDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("StateTreeEditorColor", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FStateTreeEditorColorDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("StateTreeBlueprintPropertyRef", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FStateTreeBlueprintPropertyRefDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("StateTreeEnumValueScorePairs", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FStateTreeEnumValueScorePairsDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("StateTreeState", FOnGetDetailCustomizationInstance::CreateStatic(&FStateTreeStateDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("StateTreeEditorData", FOnGetDetailCustomizationInstance::CreateStatic(&FStateTreeEditorDataDetails::MakeInstance));

	PropertyModule.NotifyCustomizationModuleChanged();
}

void FStateTreeEditorModule::ShutdownModule()
{
	// Unregister the details customization
	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomPropertyTypeLayout("StateTreeTransition");
		PropertyModule.UnregisterCustomPropertyTypeLayout("StateTreeEventDesc");
		PropertyModule.UnregisterCustomPropertyTypeLayout("StateTreeStateLink");
		PropertyModule.UnregisterCustomPropertyTypeLayout("StateTreeEditorNode");
		PropertyModule.UnregisterCustomPropertyTypeLayout("StateTreeStateParameters");
		PropertyModule.UnregisterCustomPropertyTypeLayout("StateTreeAnyEnum");
		PropertyModule.UnregisterCustomPropertyTypeLayout("StateTreeReference");
		PropertyModule.UnregisterCustomPropertyTypeLayout("StateTreeReferenceOverrides");
		PropertyModule.UnregisterCustomPropertyTypeLayout("StateTreeEditorColorRef");
		PropertyModule.UnregisterCustomPropertyTypeLayout("StateTreeEditorColor");
		PropertyModule.UnregisterCustomPropertyTypeLayout("StateTreeBlueprintPropertyRef");
		PropertyModule.UnregisterCustomClassLayout("StateTreeState");
		PropertyModule.UnregisterCustomClassLayout("StateTreeEditorData");
		PropertyModule.NotifyCustomizationModuleChanged();
	}

	FStateTreeEditorStyle::Unregister();
	FStateTreeEditorCommands::Unregister();

	MenuExtensibilityManager.Reset();
	ToolBarExtensibilityManager.Reset();

#if WITH_STATETREE_TRACE_DEBUGGER
	IModularFeatures::Get().UnregisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, RewindDebuggerTrackCreator.Get());
	IModularFeatures::Get().UnregisterModularFeature(IRewindDebuggerExtension::ModularFeatureName, RewindDebuggerPlaybackExtension.Get());
	FStateTreeDebuggerCommands::Unregister();
#endif // WITH_STATETREE_TRACE_DEBUGGER

	if (OnPostEngineInitHandle.IsValid())
	{
		FCoreDelegates::GetOnPostEngineInit().Remove(OnPostEngineInitHandle);
	}
	NodeClassCache.Reset();

	if (FBlueprintEditorModule* BlueprintEditorModule = FModuleManager::GetModulePtr<FBlueprintEditorModule>("Kismet"))
	{
		BlueprintEditorModule->UnregisterAdditionalBlueprintCategory(InputCategoryHandle);
		BlueprintEditorModule->UnregisterAdditionalBlueprintCategory(ContextCategoryHandle);
		BlueprintEditorModule->UnregisterAdditionalBlueprintCategory(OutputCategoryHandle);
	}

	UE::StateTree::Compiler::FCompilerManager::Shutdown();

	UE::StateTree::Delegates::Private::OnAppendToClassSchema.Unbind();
	UE::StateTree::Delegates::Private::OnStateTreeAssetLoaded.Unbind();
	UE::StateTree::Delegates::Private::OnRequestAssetRegistryTags.Unbind();
	UE::StateTree::Delegates::Private::OnStateTreeMarkedAsModified.Unbind();
	UE::StateTree::Delegates::Private::OnCompileIfChanged.Unbind();
	UE::StateTree::Delegates::OnRequestEditorHash.Unbind();
	UE::StateTree::Delegates::Private::OnStateTreeEditorBindingUpdated.Unbind();
}

TSharedRef<IStateTreeEditor> FStateTreeEditorModule::CreateStateTreeEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UStateTree* StateTree)
{
	TSharedRef<FStateTreeEditor> NewEditor(new FStateTreeEditor());
	NewEditor->InitEditor(Mode, InitToolkitHost, StateTree);
	if (StateTree->LastCompiledEditorDataHash == 0)
	{
		UE::StateTree::Compiler::FCompilerManager::CompileSynchronously(StateTree);
	}
	return NewEditor;
}

void FStateTreeEditorModule::SetDetailPropertyHandlers(IDetailsView& DetailsView)
{
	DetailsView.SetExtensionHandler(MakeShared<FStateTreeBindingExtension>());
	DetailsView.SetChildrenCustomizationHandler(MakeShared<FStateTreeBindingsChildrenCustomization>());
}

TSharedPtr<FStateTreeNodeClassCache> FStateTreeEditorModule::GetNodeClassCache()
{
	check(NodeClassCache.IsValid());
	return NodeClassCache;
}

uint32 FStateTreeEditorModule::HandleRequestEditorHash(const UStateTree& InStateTree)
{
	return UStateTreeEditingSubsystem::CalculateStateTreeHash(&InStateTree);
}

void FStateTreeEditorModule::HandleCompileIfChanged(const TNotNull<UStateTree*> InStateTree)
{
	UE::StateTree::Compiler::FCompilerManager::CompileIfNeededSynchronously(InStateTree);
}

void FStateTreeEditorModule::HandleRequestAssetRegistryTags(TNotNull<const UStateTree*> InStateTree, FAssetRegistryTagsContext& Context)
{
	// Notify the module, schema, and extension. Use that order to go from the less specific to the more specific.
	OnAssetRegistryTags().Broadcast(InStateTree, Context);

	if (UStateTreeEditorData* EditorData = Cast<UStateTreeEditorData>(InStateTree->EditorData))
	{
		if (EditorData->Schema)
		{
			EditorData->Schema->GetAssetRegistryTags(Context);
		}
		for (const UStateTreeEditorDataExtension* Extension : EditorData->Extensions)
		{
			if (Extension)
			{
				Extension->GetAssetRegistryTags(Context);
			}
		}

		// Add the schema tag (override previously set value).
		const FString SchemaClassName = EditorData->Schema ? EditorData->Schema->GetClass()->GetPathName() : TEXT("");
		Context.AddTag(UObject::FAssetRegistryTag(UE::StateTree::SchemaTag, SchemaClassName, UObject::FAssetRegistryTag::TT_Alphabetical));
	}
}

void FStateTreeEditorModule::HandleStateTreeAssetLoaded(TNotNull<UStateTree*> InStateTree)
{
	TGuardValueAccessors<bool> IsEditorLoadingPackageGuard(UE::GetIsEditorLoadingPackage, UE::SetIsEditorLoadingPackage, true);
	if (UE::StateTree::Editor::bEnableQueuedCompilationOnAssetLoad)
	{
		UE::StateTree::Compiler::FCompilerManager::QueueForCompilation(InStateTree);
	}
	else
	{
		FStateTreeCompilerLog Log;
		UE::StateTree::Compiler::FCompilerManager::CompileSynchronously(InStateTree, Log);
	}
}

void FStateTreeEditorModule::HandleStateTreeEditorBindingUpdated(TNotNull<UStateTreeEditorData*> InStateTreeEditorData)
{
	UE::StateTree::Compiler::FCompilerManager::CacheEditorBindingExternalDependencies(InStateTreeEditorData);
}

void FStateTreeEditorModule::HandleAppendToStateTreeClassSchema(FAppendToClassSchemaContext& Context)
{
	FStateTreeCompiler::AppendToStateTreeClassSchema(Context);
}

bool FStateTreeEditorModule::FEditorTypes::HasData() const
{
	return EditorData.IsValid() || EditorSchema.IsValid();
}

namespace UE::StateTreeEditor::Private
{
	TOptional<int32> GetDepth(TNotNull<const UStruct*> Struct, TNotNull<const UStruct*> MatchingParent)
	{
		int32 Depth = 0;
		if (Struct == MatchingParent)
		{
			return Depth;
		}

		for (const UStruct* TempStruct : Struct->GetSuperStructIterator())
		{
			++Depth;
			if (TempStruct == MatchingParent)
			{
				return Depth;
			}
		}

		return {};
	}
}

void FStateTreeEditorModule::RegisterEditorDataClass(TNonNullSubclassOf<const UStateTreeSchema> Schema, TNonNullSubclassOf<const UStateTreeEditorData> EditorData)
{
	FEditorTypes* EditorType = EditorTypes.FindByPredicate(
		[Schema](const FEditorTypes& Other)
		{
			return Other.Schema == Schema.Get();
		});
	if (EditorType)
	{
		ensureMsgf(EditorType->EditorData.IsExplicitlyNull(), TEXT("The type %s is already registered."), *Schema.Get()->GetName());
		EditorType->EditorData = EditorData.Get();
	}
	else
	{
		FEditorTypes& NewEditorType = EditorTypes.AddDefaulted_GetRef();
		NewEditorType.Schema = Schema.Get();
		NewEditorType.EditorData = EditorData.Get();
	}
}

void FStateTreeEditorModule::UnregisterEditorDataClass(TNonNullSubclassOf<const UStateTreeSchema> Schema)
{
	const int32 FoundIndex = EditorTypes.IndexOfByPredicate(
		[Schema](const FEditorTypes& Other)
		{
			return Other.Schema == Schema.Get();
		});
	if (FoundIndex != INDEX_NONE)
	{
		EditorTypes[FoundIndex].EditorData.Reset();
		if (!EditorTypes[FoundIndex].HasData())
		{
			EditorTypes.RemoveAtSwap(FoundIndex);
		}
	}
}

TNonNullSubclassOf<UStateTreeEditorData> FStateTreeEditorModule::GetEditorDataClass(TNonNullSubclassOf<const UStateTreeSchema> Schema) const
{
	int32 BestDepth = INT_MAX;
	const FEditorTypes* BestEditorType = nullptr;
	for (const FEditorTypes& EditorType : EditorTypes)
	{
		const UClass* OtherSchema = EditorType.Schema.Get();
		TOptional<int32> Depth = UE::StateTreeEditor::Private::GetDepth(OtherSchema, Schema.Get());
		if (Depth.IsSet() && Depth.GetValue() < BestDepth)
		{
			BestDepth = Depth.GetValue();
			BestEditorType = &EditorType;
		}
	}

	const UClass* Result = BestEditorType && BestEditorType->EditorData.Get() ? BestEditorType->EditorData.Get() : UStateTreeEditorData::StaticClass();
	return const_cast<UClass*>(Result); // NewObject wants none const UClass
}

void FStateTreeEditorModule::RegisterEditorSchemaClass(TNonNullSubclassOf<const UStateTreeSchema> Schema, TNonNullSubclassOf<const UStateTreeEditorSchema> EditorSchema)
{
	FEditorTypes* EditorType = EditorTypes.FindByPredicate(
		[Schema](const FEditorTypes& Other)
		{
			return Other.Schema == Schema.Get();
		});
	if (EditorType)
	{
		ensureMsgf(EditorType->EditorSchema.IsExplicitlyNull(), TEXT("The type %s is already registered."), *Schema.Get()->GetName());
		EditorType->EditorSchema = EditorSchema.Get();
	}
	else
	{
		FEditorTypes& NewEditorType = EditorTypes.AddDefaulted_GetRef();
		NewEditorType.Schema = Schema.Get();
		NewEditorType.EditorSchema = EditorSchema.Get();
	}
}

void FStateTreeEditorModule::UnregisterEditorSchemaClass(TNonNullSubclassOf<const UStateTreeSchema> Schema)
{
	const int32 FoundIndex = EditorTypes.IndexOfByPredicate(
		[Schema](const FEditorTypes& Other)
		{
			return Other.Schema == Schema.Get();
		});
	if (FoundIndex != INDEX_NONE)
	{
		EditorTypes[FoundIndex].EditorSchema.Reset();
		if (!EditorTypes[FoundIndex].HasData())
		{
			EditorTypes.RemoveAtSwap(FoundIndex);
		}
	}
}

TNonNullSubclassOf<UStateTreeEditorSchema> FStateTreeEditorModule::GetEditorSchemaClass(TNonNullSubclassOf<const UStateTreeSchema> Schema) const
{
	int32 BestDepth = INT_MAX;
	const FEditorTypes* BestEditorType = nullptr;
	for (const FEditorTypes& OtherEditorType : EditorTypes)
	{
		const UClass* OtherSchema = OtherEditorType.Schema.Get();
		TOptional<int32> Depth = UE::StateTreeEditor::Private::GetDepth(OtherSchema, Schema.Get());
		if (Depth.IsSet() && Depth.GetValue() < BestDepth)
		{
			BestDepth = Depth.GetValue();
			BestEditorType = &OtherEditorType;
		}
	}

	const UClass* Result = BestEditorType && BestEditorType->EditorSchema.Get() ? BestEditorType->EditorSchema.Get() : UStateTreeEditorSchema::StaticClass();
	return const_cast<UClass*>(Result); // NewObject wants none const UClass
}

#undef LOCTEXT_NAMESPACE
