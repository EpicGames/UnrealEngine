// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeEditorNode.h"

#include "StateTreeEditorModule.h"
#include "Blueprint/StateTreeConditionBlueprintBase.h"
#include "Blueprint/StateTreeConsiderationBlueprintBase.h"
#include "Blueprint/StateTreeEvaluatorBlueprintBase.h"
#include "Blueprint/StateTreeTaskBlueprintBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeEditorNode)

void FStateTreeEditorNode::Reset()
{
	Node.Reset();
	Instance.Reset();
	InstanceObject = nullptr;
	ExecutionRuntimeData.Reset();
	ExecutionRuntimeDataObject = nullptr;
	ID = FGuid();
	CachedNodeName.Reset();
	// don't init the ExpressionIndent and ExpressionOperand
}

void FStateTreeEditorNode::InitializeAs(TNotNull<UObject*> InOuter, TNotNull<const UScriptStruct*> InNodeStruct)
{
	Reset();
	Node.InitializeAs(InNodeStruct);
	ID = FGuid::NewGuid();
	InitializeInstanceData(InOuter);
}

void FStateTreeEditorNode::ReallocInstanceData(TNotNull<UObject*> InOuter)
{
	Instance.Reset();
	InstanceObject = nullptr;
	ExecutionRuntimeData.Reset();
	ExecutionRuntimeDataObject = nullptr;
	CachedNodeName.Reset();
	InitializeInstanceData(InOuter);
}

void FStateTreeEditorNode::InitializeInstanceData(TNotNull<UObject*> InOuter)
{
	const FStateTreeNodeBase& NodeBase = GetNode().Get<FStateTreeNodeBase>();
	if (const UScriptStruct* InstanceType = Cast<const UScriptStruct>(NodeBase.GetInstanceDataType()))
	{
		Instance.InitializeAs(InstanceType);
	}
	else if (const UClass* InstanceClass = Cast<const UClass>(NodeBase.GetInstanceDataType()))
	{
		InstanceObject = NewObject<UObject>(InOuter, InstanceClass, FName(), RF_Transactional);
	}

	if (const UScriptStruct* RuntimeType = Cast<const UScriptStruct>(NodeBase.GetExecutionRuntimeDataType()))
	{
		ExecutionRuntimeData.InitializeAs(RuntimeType);
	}
	else if (const UClass* RuntimeClass = Cast<const UClass>(NodeBase.GetExecutionRuntimeDataType()))
	{
		ExecutionRuntimeDataObject = NewObject<UObject>(InOuter, RuntimeClass, FName(), RF_Transactional);
	}
}

bool FStateTreeEditorNode::FixObjectInstances(TSet<UObject*>& SeenObjects, TNotNull<UObject*> Outer)
{
	auto NewInstances = [this, Outer](const UStruct* Struct, const UStruct* ExecutionRuntimeStruct) -> bool
		{
			bool bDirty = false;
			if (Struct && !GetInstance().IsValid())
			{
				if (const UScriptStruct* InstanceType = Cast<const UScriptStruct>(Struct))
				{
					Instance.InitializeAs(InstanceType);
					bDirty = true;
				}
				else if (const UClass* InstanceClass = Cast<const UClass>(Struct))
				{
					InstanceObject = NewObject<UObject>(Outer, InstanceClass, FName(), RF_Transactional);
					bDirty = true;
				}
			}
			if (ExecutionRuntimeStruct && !GetExecutionRuntimeData().IsValid())
			{
				if (const UScriptStruct* InstanceType = Cast<const UScriptStruct>(ExecutionRuntimeStruct))
				{
					ExecutionRuntimeData.InitializeAs(InstanceType);
					bDirty = true;
				}
				else if (const UClass* InstanceClass = Cast<const UClass>(ExecutionRuntimeStruct))
				{
					ExecutionRuntimeDataObject = NewObject<UObject>(Outer, InstanceClass, FName(), RF_Transactional);
					bDirty = true;
				}
			}
			return bDirty;
		};
	auto FixInstance = [](FInstancedStruct& InInstance, const UStruct* Struct) -> bool
		{
			bool bDirty = false;
			if (InInstance.IsValid())
			{
				if (InInstance.GetScriptStruct() != Struct)
				{
					InInstance.Reset();
					bDirty = true;
				}
			}
			return bDirty;
		};
	auto FixInstanceObject = [&SeenObjects, Outer](TObjectPtr<UObject>& InInstanceObject, const UStruct* Struct) -> bool
		{
			bool bDirty = false;
			if (InInstanceObject)
			{
				if (InInstanceObject->GetClass() != Struct)
				{
					InInstanceObject = nullptr;
					bDirty = true;
				}
				// Found a duplicate reference to an object, make unique copy.
				else if (SeenObjects.Contains(InInstanceObject))
				{
					UE_LOGF(LogStateTreeEditor, Log, "%ls: Making duplicate node instance %ls unique.", *Outer->GetFullName(), *GetNameSafe(InInstanceObject));
					InInstanceObject = DuplicateObject(InInstanceObject, Outer);
					bDirty = true;
				}
				else
				{
					// Make sure the instance object is properly outered.
					if (InInstanceObject->GetOuter() != Outer)
					{
						UE_LOGF(LogStateTreeEditor, Log, "%ls: Fixing outer on node instance %ls.", *Outer->GetFullName(), *GetNameSafe(InInstanceObject));
						InInstanceObject->Rename(nullptr, Outer, REN_DontCreateRedirectors | REN_DoNotDirty | REN_AllowPackageLinkerMismatch);
						bDirty = true;
					}
				}

				// Make sure state has transactional flags to make it work with undo (to fix a bug instance object created without this flag).
				InInstanceObject->SetFlags(RF_Transactional);

				SeenObjects.Add(InInstanceObject);
			}
			return bDirty;
		};

	const FStateTreeNodeBase* NodeBase = GetNode().GetPtr<FStateTreeNodeBase>();
	bool bDirty = false;
	if (NodeBase)
	{
		if (FixInstance(Instance, NodeBase->GetInstanceDataType()))
		{
			bDirty = true;
		}
		if (FixInstanceObject(InstanceObject, NodeBase->GetInstanceDataType()))
		{
			bDirty = true;
		}
		if (FixInstance(ExecutionRuntimeData, NodeBase->GetExecutionRuntimeDataType()))
		{
			bDirty = true;
		}
		if (FixInstanceObject(ExecutionRuntimeDataObject, NodeBase->GetExecutionRuntimeDataType()))
		{
			bDirty = true;
		}
		if (NewInstances(NodeBase->GetInstanceDataType(), NodeBase->GetExecutionRuntimeDataType()))
		{
			bDirty = true;
		}
	}
	else
	{
		if (GetInstance().IsValid())
		{
			Instance.Reset();
			InstanceObject = nullptr;
			bDirty = true;
		}
		if (GetExecutionRuntimeData().IsValid())
		{
			ExecutionRuntimeData.Reset();
			ExecutionRuntimeDataObject = nullptr;
			bDirty = true;
		}
	}

	return bDirty;
}

void FStateTreeEditorNode::SetNodeName(FName NewName)
{
	FStateTreeNodeBase* NodePtr = Node.GetMutablePtr<FStateTreeNodeBase>();
	if (NodePtr && NodePtr->Name != NewName)
	{
		NodePtr->Name = NewName;
		CachedNodeName.Reset();
	}
}

FName FStateTreeEditorNode::GetName() const
{
	if (!CachedNodeName.IsSet())
	{
		auto MakeName = [this]() -> FName
			{
				const UScriptStruct* NodeType = Node.GetScriptStruct();
				if (!NodeType)
				{
					return FName();
				}

				if (const FStateTreeNodeBase* NodePtr = Node.GetPtr<FStateTreeNodeBase>())
				{
					if (NodePtr->Name.IsNone())
					{
						if (InstanceObject &&
							(NodeType->IsChildOf(TBaseStructure<FStateTreeBlueprintTaskWrapper>::Get())
								|| NodeType->IsChildOf(TBaseStructure<FStateTreeBlueprintEvaluatorWrapper>::Get())
								|| NodeType->IsChildOf(TBaseStructure<FStateTreeBlueprintConditionWrapper>::Get())
								|| NodeType->IsChildOf(TBaseStructure<FStateTreeBlueprintConsiderationWrapper>::Get())))
						{
							return FName(InstanceObject->GetClass()->GetDisplayNameText().ToString());
						}
						return FName(NodeType->GetDisplayNameText().ToString());
					}
					return NodePtr->Name;
				}
				return FName();
			};
		CachedNodeName = MakeName();
	}
	return CachedNodeName.GetValue();
}
