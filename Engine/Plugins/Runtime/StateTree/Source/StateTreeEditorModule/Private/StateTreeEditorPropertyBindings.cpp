// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeEditorPropertyBindings.h"

#include "Algo/Copy.h"
#include "StateTreeEditingSubsystem.h"
#include "StateTreeEditorNode.h"
#include "StateTreePropertyBindingCompiler.h"
#include "StateTreeNodeBase.h"
#include "StateTreePropertyFunctionBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeEditorPropertyBindings)

#define LOCTEXT_NAMESPACE "StateTreeEditor"

UStateTreeEditorPropertyBindingsOwner::UStateTreeEditorPropertyBindingsOwner(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

//////////////////////////////////////////////////////////////////////////

FPropertyBindingPath FStateTreeEditorPropertyBindings::AddFunctionBinding(const UScriptStruct* InPropertyFunctionNodeStruct, TConstArrayView<FPropertyBindingPathSegment> InSourcePathSegments, const FPropertyBindingPath& TargetPath)
{
	check(InPropertyFunctionNodeStruct->IsChildOf<FStateTreePropertyFunctionBase>());
	UObject* OwnerObject = GetBindingsOwnerObject();
	check(OwnerObject);

	FStateTreeEditorNode PropertyFunction;
	PropertyFunction.InitializeAs(OwnerObject, InPropertyFunctionNodeStruct);

	Super::RemoveBindings(TargetPath, ESearchMode::Exact);

	FConstStructView PropertyFunctionView = FConstStructView::Make(PropertyFunction);
	FPropertyBindingPath SourcePath = FPropertyBindingPath(PropertyFunction.GetInstanceDataID(), InSourcePathSegments);
	PropertyBindings.Emplace(PropertyFunctionView, SourcePath, TargetPath);

	UpdateSegmentsForNewlyAddedBinding(PropertyBindings.Last());

	if (UStateTree* StateTree = OwnerObject->GetTypedOuter<UStateTree>())
	{
		UStateTreeEditingSubsystem::MarkAsModified(StateTree);
	}

	return SourcePath;
}

const FStateTreePropertyPathBinding* FStateTreeEditorPropertyBindings::AddOutputBinding(const FPropertyBindingPath& InSourcePath,
	const FPropertyBindingPath& InTargetPath)
{
	if (FStateTreePropertyPathBinding* AddedBinding = static_cast<FStateTreePropertyPathBinding*>(AddBindingInternal(InSourcePath, InTargetPath)))
	{
		AddedBinding->SetIsOutputBinding(true);

		UpdateSegmentsForNewlyAddedBinding(*AddedBinding);

		return AddedBinding;
	}

	return nullptr;
}

const FStateTreePropertyPathBinding* FStateTreeEditorPropertyBindings::AddTaskCompletionBinding(const FGuid& InSourceTask
	, const FPropertyBindingPath& InTargetPath
	, const UE::StateTree::ETaskCompletionCondition Completion)
{
	if (FStateTreePropertyPathBinding* AddedBinding = static_cast<FStateTreePropertyPathBinding*>(AddBindingInternal(FPropertyBindingPath(InSourceTask), InTargetPath)))
	{
		AddedBinding->SetCompletionBinding(Completion);

		UpdateSegmentsForNewlyAddedBinding(*AddedBinding);

		return AddedBinding;
	}

	return nullptr;
}

bool FStateTreeEditorPropertyBindings::IsDependentOn(TConstArrayView<UStruct*> InStructsDependedOn) const
{
	for (const UStruct* Struct : InStructsDependedOn)
	{
		if (CachedDependencies.Contains(Struct))
		{
			return true;
		}
	}

	return false;
}

TArray<TNotNull<const UStruct*>> FStateTreeEditorPropertyBindings::GatherDependencies() const
{
	TArray<TNotNull<const UStruct*>> DependentStructs;
	DependentStructs.Reserve(CachedDependencies.Num());

	for (const TObjectKey<const UStruct> StructObjectKey : CachedDependencies)
	{
		if (const UStruct* Struct = StructObjectKey.ResolveObjectPtr())
		{
			DependentStructs.Emplace(Struct);
		}
	}

	return DependentStructs;
}

void FStateTreeEditorPropertyBindings::CacheDependency(const TConstArrayView<TNotNull<const UStruct*>> InDependentStructs)
{
	CachedDependencies.Reset();
	CachedDependencies.Reserve(InDependentStructs.Num());

	Algo::Copy(InDependentStructs, CachedDependencies, Algo::NoRef);
}

FPropertyBindingBinding* FStateTreeEditorPropertyBindings::AddBindingInternal(const FPropertyBindingPath& InSourcePath
	, const FPropertyBindingPath& InTargetPath)
{
	constexpr bool bIsOutputBinding = false;
	return &PropertyBindings.Emplace_GetRef(InSourcePath, InTargetPath, bIsOutputBinding);
}

void FStateTreeEditorPropertyBindings::CopyBindingsInternal(const FGuid InFromStructID, const FGuid InToStructID)
{
	// Find the StructID and copy the binding. If we find functions, then copy the function and copy the functions binding recursively.
	TArray<TTuple<FGuid, FGuid>, TInlineAllocator<8>> TargetIDs;
	TargetIDs.Emplace(InFromStructID, InToStructID);

	for (int32 Index = 0; Index < TargetIDs.Num(); ++Index)
	{
		const FGuid& FromStructID = TargetIDs[Index].Get<0>();
		const FGuid& ToStructID = TargetIDs[Index].Get<1>();

		TArray<TTuple<const UScriptStruct*, FPropertyBindingPath, FPropertyBindingPath>, TInlineAllocator<8>> FunctionBindingsToCopy;
		CopyBindingsImplementation(FromStructID, ToStructID, [ToStructID, &FunctionBindingsToCopy](const FPropertyBindingBinding& Binding)
			{
				const bool bIsFunctionBinding = Binding.GetPropertyFunctionNode().IsValid();
				if (bIsFunctionBinding)
				{
					const FStateTreeEditorNode* EditorNode = Binding.GetPropertyFunctionNode().GetPtr<const FStateTreeEditorNode>();
					if (ensure(EditorNode))
					{
						FunctionBindingsToCopy.Emplace(EditorNode->GetNode().GetScriptStruct(), Binding.GetSourcePath(), FPropertyBindingPath(ToStructID, Binding.GetTargetPath().GetSegments()));
					}
				}
				return !bIsFunctionBinding;
			});

		// Copy all functions bindings that target "FromStructID" and retarget them to "ToStructID".
		for (const TTuple<const UScriptStruct*, FPropertyBindingPath, FPropertyBindingPath>& FunctionBindingToCopy : FunctionBindingsToCopy)
		{
			const FGuid NewID = AddFunctionBinding(FunctionBindingToCopy.Get<0>(), FunctionBindingToCopy.Get<1>().GetSegments(), FunctionBindingToCopy.Get<2>()).GetStructID();
			// Adds the new function binding to the list of copy bindings to look for.
			TargetIDs.AddUnique(TTuple<FGuid, FGuid>{FunctionBindingToCopy.Get<1>().GetStructID(), NewID});
		}
	}
}

void FStateTreeEditorPropertyBindings::RemoveBindingsInternal(TFunctionRef<bool(FPropertyBindingBinding&)> InPredicate)
{
	PropertyBindings.RemoveAllSwap(InPredicate);
}

bool FStateTreeEditorPropertyBindings::HasBindingInternal(TFunctionRef<bool(const FPropertyBindingBinding&)> InPredicate) const
{
	return PropertyBindings.ContainsByPredicate(InPredicate);
}

const FPropertyBindingBinding* FStateTreeEditorPropertyBindings::FindBindingInternal(TFunctionRef<bool(const FPropertyBindingBinding&)> InPredicate) const
{
	return PropertyBindings.FindByPredicate(InPredicate);
}

void FStateTreeEditorPropertyBindings::ForEachBinding(TFunctionRef<void(const FPropertyBindingBinding& Binding)> InFunction) const
{
	for (const FStateTreePropertyPathBinding& Binding : PropertyBindings)
	{
		InFunction(Binding);
	}
}

void FStateTreeEditorPropertyBindings::ForEachBinding(const FPropertyBindingIndex16 InBegin, const FPropertyBindingIndex16 InEnd
	, const TFunctionRef<void(const FPropertyBindingBinding& Binding, const int32 BindingIndex)> InFunction) const
{
	checkf(InBegin.IsValid() && InEnd.IsValid(), TEXT("%hs expects valid indices."), __FUNCTION__);

	for (int32 BindingIndex = InBegin.Get(); BindingIndex < InEnd.Get(); ++BindingIndex)
	{
		InFunction(PropertyBindings[BindingIndex], BindingIndex);
	}
}

void FStateTreeEditorPropertyBindings::ForEachMutableBinding(TFunctionRef<void(FPropertyBindingBinding& Binding)> InFunction)
{
	for (FStateTreePropertyPathBinding& Binding : PropertyBindings)
	{
		InFunction(Binding);
	}
}

void FStateTreeEditorPropertyBindings::VisitBindings(TFunctionRef<EVisitResult(const FPropertyBindingBinding& Binding)> InFunction) const
{
	for (const FStateTreePropertyPathBinding& Binding : PropertyBindings)
	{
		if (InFunction(Binding) == EVisitResult::Break)
		{
			break;
		}
	}
}

void FStateTreeEditorPropertyBindings::VisitBindings(FPropertyBindingIndex16 InBegin, FPropertyBindingIndex16 InEnd,
	TFunctionRef<EVisitResult(const FPropertyBindingBinding& Binding, const int32 BindingIndex)> InFunction) const
{
	checkf(InBegin.IsValid() && InEnd.IsValid(), TEXT("%hs expects valid indices."), __FUNCTION__);

	for (int32 BindingIndex = InBegin.Get(); BindingIndex < InEnd.Get(); ++BindingIndex)
	{
		if (InFunction(PropertyBindings[BindingIndex], BindingIndex) == EVisitResult::Break)
		{
			break;
		}
	}
}

void FStateTreeEditorPropertyBindings::VisitMutableBindings(TFunctionRef<EVisitResult(FPropertyBindingBinding& Binding)> InFunction)
{
	for (FStateTreePropertyPathBinding& Binding : PropertyBindings)
	{
		if (InFunction(Binding) == EVisitResult::Break)
		{
			break;
		}
	}
}

const FPropertyBindingBindableStructDescriptor* FStateTreeEditorPropertyBindings::GetBindableStructDescriptorFromHandle(
	FConstStructView InSourceHandleView) const
{
	// This is not used for Editor operation and handled at runtime in FStateTreePropertyBindings
	return nullptr;
}

int32 FStateTreeEditorPropertyBindings::GetNumBindableStructDescriptors() const
{
	// This is not used for Editor operation and handled at runtime in FStateTreePropertyBindings
	return 0;
}

int32 FStateTreeEditorPropertyBindings::GetNumBindings() const
{
	return PropertyBindings.Num();
}

//////////////////////////////////////////////////////////////////////////

FStateTreeBindingLookup::FStateTreeBindingLookup(const IStateTreeEditorPropertyBindingsOwner* InBindingOwner)
	: BindingOwner(InBindingOwner)
{
	check(BindingOwner);
}

const FPropertyBindingPath* FStateTreeBindingLookup::GetPropertyBindingSource(const FPropertyBindingPath& InTargetPath) const
{
	check(BindingOwner);
	const FStateTreeEditorPropertyBindings* EditorBindings = BindingOwner->GetPropertyEditorBindings();
	check(EditorBindings);

	return static_cast<const FPropertyBindingPath*>(EditorBindings->FPropertyBindingBindingCollection::GetBindingSource(InTargetPath));
}

FText FStateTreeBindingLookup::GetPropertyPathDisplayName(const FPropertyBindingPath& InPath, EStateTreeNodeFormatting Formatting) const
{
	check(BindingOwner);
	const FStateTreeEditorPropertyBindings* EditorBindings = BindingOwner->GetPropertyEditorBindings();
	check(EditorBindings);

	FString StructName;
	int32 FirstSegmentToStringify = 0;

	// If path's struct is a PropertyFunction, let it override a display name.
	{
		const FPropertyBindingBinding* BindingToPath = EditorBindings->GetBindings().FindByPredicate([&InPath](const FPropertyBindingBinding& Binding)
		{
			return Binding.GetSourcePath() == InPath;
		});

		if (BindingToPath && BindingToPath->GetPropertyFunctionNode().IsValid())
		{
			const FConstStructView PropertyFuncEditorNodeView = BindingToPath->GetPropertyFunctionNode();
			const FStateTreeEditorNode& EditorNode = PropertyFuncEditorNodeView.Get<const FStateTreeEditorNode>();

			if (!EditorNode.GetNode().IsValid())
			{
				return LOCTEXT("Unlinked", "???");
			}

			const FStateTreeNodeBase& Node = EditorNode.GetNode().Get<FStateTreeNodeBase>();

			// Skipping an output property if there's only one of them.
			if (UE::StateTree::GetStructSingleOutputProperty(*Node.GetInstanceDataType()))
			{
				FirstSegmentToStringify = 1;
			}

			const FText Description = Node.GetDescription(BindingToPath->GetSourcePath().GetStructID(), EditorNode.GetInstance(), *this, Formatting);
			if (!Description.IsEmpty())
			{
				StructName = Description.ToString();
			}
		}
	
		if (StructName.IsEmpty())
		{
			TInstancedStruct<FPropertyBindingBindableStructDescriptor> Struct;
			if (static_cast<const IPropertyBindingBindingCollectionOwner*>(BindingOwner)->GetBindableStructByID(InPath.GetStructID(), Struct))
			{
				StructName = Struct.Get().Name.ToString();
			}
		}
	}

	FString Result = MoveTemp(StructName);
	if (InPath.NumSegments() > FirstSegmentToStringify)
	{
		Result += TEXT(".") + InPath.ToString(/*HighlightedSegment*/ INDEX_NONE, /*HighlightPrefix*/ nullptr, /*HighlightPostfix*/ nullptr, /*bOutputInstances*/ false, FirstSegmentToStringify);
	}

	return FText::FromString(Result);
}

FText FStateTreeBindingLookup::GetBindingSourceDisplayName(const FPropertyBindingPath& InTargetPath, EStateTreeNodeFormatting Formatting) const
{
	check(BindingOwner);
	const FStateTreeEditorPropertyBindings* EditorBindings = BindingOwner->GetPropertyEditorBindings();
	check(EditorBindings);

	// Check if the target property is bound, if so, return binding description.
	if (const FPropertyBindingPath* SourcePath = GetPropertyBindingSource(InTargetPath))
	{
		return GetPropertyPathDisplayName(*SourcePath, Formatting);
	}

	// Check if it's bound to context data.
	const UStruct* TargetStruct = nullptr;
	const FProperty* TargetProperty = nullptr;
	EStateTreePropertyUsage Usage = EStateTreePropertyUsage::Invalid;
	
	TInstancedStruct<FPropertyBindingBindableStructDescriptor> TargetStructDesc;
	if (static_cast<const IPropertyBindingBindingCollectionOwner*>(BindingOwner)->GetBindableStructByID(InTargetPath.GetStructID(), TargetStructDesc))
	{
		TArray<FPropertyBindingPathIndirection> Indirection;
		if (InTargetPath.ResolveIndirections(TargetStructDesc.Get().Struct, Indirection)
			&& Indirection.Num() > 0)
		{
			const FPropertyBindingPathIndirection& Leaf = Indirection.Last();
			TargetProperty = Leaf.GetProperty();
			if (TargetProperty)
			{
				Usage = UE::StateTree::GetUsageFromMetaData(TargetProperty);
			}
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(TargetProperty))
			{
				TargetStruct = StructProperty->Struct;
			}
			if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(TargetProperty))
			{
				TargetStruct = ObjectProperty->PropertyClass;
			}
		}
	}

	if (Usage == EStateTreePropertyUsage::Context)
	{
		if (TargetStruct)
		{
			const FStateTreeBindableStructDesc Desc = BindingOwner->FindContextData(TargetStruct, TargetProperty->GetName());
			if (Desc.IsValid())
			{
				// Connected
				return FText::FromName(Desc.Name);
			}
		}
		return LOCTEXT("Unlinked", "???");
	}

	// Not a binding nor context data.
	return FText::GetEmpty();
}

const FProperty* FStateTreeBindingLookup::GetPropertyPathLeafProperty(const FPropertyBindingPath& InPath) const
{
	check(BindingOwner);
	const FStateTreeEditorPropertyBindings* EditorBindings = BindingOwner->GetPropertyEditorBindings();
	check(EditorBindings);

	const FProperty* Result = nullptr;
	TInstancedStruct<FPropertyBindingBindableStructDescriptor> Struct;
	if (static_cast<const IPropertyBindingBindingCollectionOwner*>(BindingOwner)->GetBindableStructByID(InPath.GetStructID(), Struct))
	{
		TArray<FPropertyBindingPathIndirection> Indirection;
		if (InPath.ResolveIndirections(Struct.Get().Struct, Indirection) && Indirection.Num() > 0)
		{
			return Indirection.Last().GetProperty();
		}
	}

	return Result;
}

#undef LOCTEXT_NAMESPACE