// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeEditorSchema.h"
#include "StateTree.h"
#include "StateTreeEditorData.h"
#include "StateTreeEditorModule.h"
#include "StateTreeSchema.h"
#include "StateTreeState.h"
#include "StateTreeTaskBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeEditorSchema)

void UStateTreeEditorSchema::Validate(TNotNull<UStateTree*> StateTree)
{
	UStateTreeEditorData* TreeData = Cast<UStateTreeEditorData>(StateTree->EditorData);
	if (!TreeData)
	{
		return;
	}

	const UStateTreeSchema* Schema = TreeData->Schema;
	if (!Schema)
	{
		return;
	}

	// Clear evaluators if not allowed.
	if (Schema->AllowEvaluators() == false && TreeData->Evaluators.Num() > 0)
	{
		UE_LOGF(LogStateTreeEditor, Warning, "%ls: Resetting Evaluators due to current schema restrictions.", *StateTree->GetName());
		TreeData->Evaluators.Reset();
	}

	TreeData->VisitHierarchy(
		[&StateTree, Schema](UStateTreeState& State, UStateTreeState* /*ParentState*/)
		{
			constexpr bool bMarkDirty = false;
			State.Modify(bMarkDirty);

			// Clear enter conditions if not allowed.
			if (Schema->AllowEnterConditions() == false && State.EnterConditions.Num() > 0)
			{
				UE_LOGF(LogStateTreeEditor, Warning, "%ls: Resetting Enter Conditions in state %ls due to current schema restrictions.", *StateTree->GetName(), *State.GetName());
				State.EnterConditions.Reset();
			}

			// Clear Utility if not allowed
			if (Schema->AllowUtilityConsiderations() == false && State.Considerations.Num() > 0)
			{
				UE_LOGF(LogStateTreeEditor, Warning, "%ls: Resetting Utility Considerations in state %ls due to current schema restrictions.", *StateTree->GetName(), *State.GetName());
				State.Considerations.Reset();
			}

			// Keep single and many tasks based on what is allowed.
			if (Schema->AllowMultipleTasks() == false)
			{
				if (State.Tasks.Num() > 0)
				{
					State.Tasks.Reset();
					UE_LOGF(LogStateTreeEditor, Warning, "%ls: Resetting Tasks in state %ls due to current schema restrictions.", *StateTree->GetName(), *State.GetName());
				}

				// Task name is the same as state name.
				if (FStateTreeTaskBase* Task = State.SingleTask.GetNode().GetPtr<FStateTreeTaskBase>())
				{
					Task->Name = State.Name;
				}
			}
			else
			{
				if (State.SingleTask.GetNode().IsValid())
				{
					State.SingleTask.Reset();
					UE_LOGF(LogStateTreeEditor, Warning, "%ls: Resetting Single Task in state %ls due to current schema restrictions.", *StateTree->GetName(), *State.GetName());
				}
			}

			return EStateTreeVisitor::Continue;
		});
}
