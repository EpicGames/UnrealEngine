// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeEditorSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeEditorSettings)

UStateTreeEditorSettings::UStateTreeEditorSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR

FText UStateTreeEditorSettings::GetSectionText() const
{
	return NSLOCTEXT("StateTreeEditor", "StateTreeEditorSettingsName", "StateTree Editor");
}

FText UStateTreeEditorSettings::GetSectionDescription() const
{
	return NSLOCTEXT("StateTreeEditor", "StateTreeEditorSettingsDescription", "Configure options for the StateTree Editor.");
}

#endif // WITH_EDITOR

FName UStateTreeEditorSettings::GetCategoryName() const
{
	return FName(TEXT("Plugins"));
}
