// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeEditorStyle.h"
#include "Brushes/SlateBoxBrush.h"
#include "Styling/SlateStyleRegistry.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/CoreStyle.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Styling/SlateTypes.h"
#include "Misc/Paths.h"
#include "Styling/StyleColors.h"
#include "StateTreeTypes.h"
#include "Styling/SlateStyleMacros.h"

FStateTreeEditorStyle::FStateTreeEditorStyle()
	: FStateTreeStyle(TEXT("StateTreeEditorStyle"))
{
	const FString EngineEditorSlateContentDir = FPaths::EngineContentDir() / TEXT("Editor/Slate");

	const FScrollBarStyle ScrollBar = FAppStyle::GetWidgetStyle<FScrollBarStyle>("ScrollBar");
	const FTextBlockStyle& NormalText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");

	// State
	{
		const FEditableTextBoxStyle StateTitleEditableText = FEditableTextBoxStyle()
			.SetTextStyle(NormalText)
			.SetFont(DEFAULT_FONT("Bold", 12))
			.SetBackgroundImageNormal(CORE_BOX_BRUSH("Common/TextBox", FMargin(4.0f / 16.0f)))
			.SetBackgroundImageHovered(CORE_BOX_BRUSH("Common/TextBox_Hovered", FMargin(4.0f / 16.0f)))
			.SetBackgroundImageFocused(CORE_BOX_BRUSH("Common/TextBox_Hovered", FMargin(4.0f / 16.0f)))
			.SetBackgroundImageReadOnly(CORE_BOX_BRUSH("Common/TextBox_ReadOnly", FMargin(4.0f / 16.0f)))
			.SetBackgroundColor(FLinearColor(0.0f, 0.0f, 0.0f, 0.1f))
			.SetPadding(FMargin(0))
			.SetScrollBarStyle(ScrollBar);
		Set("StateTree.State.TitleEditableText", StateTitleEditableText);

		Set("StateTree.State.TitleInlineEditableText", FInlineEditableTextBlockStyle()
			.SetTextStyle(GetWidgetStyle<FTextBlockStyle>(StateTitleTextStyleName))
			.SetEditableTextBoxStyle(StateTitleEditableText));
	}

	// Details
	{
		const FTextBlockStyle Details = FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 10))
			.SetColorAndOpacity(FLinearColor(230.0f / 255.0f, 230.0f / 255.0f, 230.0f / 255.0f, 0.75f));
		Set("StateTree.Details", Details);

		Set("StateTree.Node.Label", new FSlateRoundedBoxBrush(FStyleColors::AccentGray, 6.f));

		// For multi selection with mixed values for a given property
		const FLinearColor Color = FStyleColors::Hover.GetSpecifiedColor();
		const FLinearColor HollowColor = Color.CopyWithNewOpacity(0.0);
		Set("StateTree.Node.Label.Mixed", new FSlateRoundedBoxBrush(HollowColor, 6.0f, Color, 1.0f));

		const FTextBlockStyle DetailsCategory = FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Bold", 8));
		Set("StateTree.Category", DetailsCategory);
	}

	// Task
	{
		const FLinearColor ForegroundCol = FStyleColors::Foreground.GetSpecifiedColor();

		Set("StateTree.Task.Title", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 10))
			.SetColorAndOpacity(ForegroundCol.CopyWithNewOpacity(0.8f)));

		Set("StateTree.Task.Title.Bold", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Bold", 10))
			.SetColorAndOpacity(ForegroundCol.CopyWithNewOpacity(0.8f)));

		Set("StateTree.Task.Title.Subdued", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 10))
			.SetColorAndOpacity(ForegroundCol.CopyWithNewOpacity(0.4f)));

		// Tasks to be show up a bit darker than the state
		Set("StateTree.Task.Rect", new FSlateColorBrush(FLinearColor(FVector3f(0.67f))));
	}

	// Details rich text
	{
		Set("Details.Normal", FTextBlockStyle(NormalText)
			.SetFont(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont"))));

		Set("Details.Bold", FTextBlockStyle(NormalText)
			.SetFont(FAppStyle::GetFontStyle(TEXT("PropertyWindow.BoldFont"))));

		Set("Details.Italic", FTextBlockStyle(NormalText)
			.SetFont(FAppStyle::GetFontStyle(TEXT("PropertyWindow.ItalicFont"))));

		Set("Details.Subdued", FTextBlockStyle(NormalText)
			.SetColorAndOpacity(FSlateColor::UseSubduedForeground())
			.SetFont(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont"))));
	}

	// Transition rich text
	{
		const FLinearColor ForegroundCol = FStyleColors::White.GetSpecifiedColor();
		Set("Transition.Normal", FTextBlockStyle(NormalText)
			.SetColorAndOpacity(ForegroundCol.CopyWithNewOpacity(0.9f))
			.SetFont(DEFAULT_FONT("Regular", 11)));

		Set("Transition.Bold", FTextBlockStyle(NormalText)
			.SetColorAndOpacity(ForegroundCol.CopyWithNewOpacity(0.9f))
			.SetFont(DEFAULT_FONT("Bold", 11)));

		Set("Transition.Italic", FTextBlockStyle(NormalText)
			.SetColorAndOpacity(ForegroundCol.CopyWithNewOpacity(0.9f))
			.SetFont(DEFAULT_FONT("Italic", 11)));

		Set("Transition.Subdued", FTextBlockStyle(NormalText)
			.SetColorAndOpacity(ForegroundCol.CopyWithNewOpacity(0.5f))
			.SetFont(DEFAULT_FONT("Regular", 11)));
	}

	// Diff tool
	{
		Set("DiffTools.Added", FLinearColor(0.3f, 1.f, 0.3f)); // green
		Set("DiffTools.Removed", FLinearColor(1.0f, 0.2f, 0.3f)); // red
		Set("DiffTools.Changed", FLinearColor(0.85f, 0.71f, 0.25f)); // yellow
		Set("DiffTools.Moved", FLinearColor(0.5f, 0.8f, 1.f)); // light blue
		Set("DiffTools.Enabled", FLinearColor(0.7f, 1.f, 0.7f)); // light green
		Set("DiffTools.Disabled", FLinearColor(1.0f, 0.6f, 0.5f)); // light red
		Set("DiffTools.Properties", FLinearColor(0.2f, 0.4f, 1.f)); // blue
	}

	const FLinearColor SelectionColor = FColor(0, 0, 0, 32);
	const FTableRowStyle& NormalTableRowStyle = FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row");
	Set("StateTree.Selection",
		FTableRowStyle(NormalTableRowStyle)
		.SetActiveBrush(CORE_IMAGE_BRUSH("Common/Selection", CoreStyleConstants::Icon8x8, SelectionColor))
		.SetActiveHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", CoreStyleConstants::Icon8x8, SelectionColor))
		.SetInactiveBrush(CORE_IMAGE_BRUSH("Common/Selection", CoreStyleConstants::Icon8x8, SelectionColor))
		.SetInactiveHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", CoreStyleConstants::Icon8x8, SelectionColor))
		.SetSelectorFocusedBrush(CORE_IMAGE_BRUSH("Common/Selection", CoreStyleConstants::Icon8x8, SelectionColor))
	);

	const FComboButtonStyle& ComboButtonStyle = FCoreStyle::Get().GetWidgetStyle<FComboButtonStyle>("ComboButton");

	// Expression Operand combo button
	const FButtonStyle OperandButton = FButtonStyle()
		.SetNormal(FSlateRoundedBoxBrush(FStyleColors::AccentGreen.GetSpecifiedColor().Desaturate(0.3f), 4.0f))
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::AccentGreen.GetSpecifiedColor().Desaturate(0.2f), 4.0f))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::AccentGreen.GetSpecifiedColor().Desaturate(0.1f), 4.0f))
		.SetNormalForeground(FStyleColors::Foreground)
		.SetHoveredForeground(FStyleColors::ForegroundHover)
		.SetPressedForeground(FStyleColors::ForegroundHover)
		.SetDisabledForeground(FStyleColors::ForegroundHover)
		.SetNormalPadding(FMargin(2, 2, 2, 2))
		.SetPressedPadding(FMargin(2, 3, 2, 1));

	Set("StateTree.Node.Operand.ComboBox", FComboButtonStyle(ComboButtonStyle).SetButtonStyle(OperandButton));

	Set("StateTree.Node.Operand", FTextBlockStyle(NormalText)
		.SetFont(FAppStyle::GetFontStyle(TEXT("PropertyWindow.BoldFont")))
		.SetFontSize(8));

	Set("StateTree.Node.Parens", FTextBlockStyle(NormalText)
		.SetFont(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		.SetFontSize(12));

	// Parameter labels
	Set("StateTree.Param.Label", FTextBlockStyle(NormalText)
		.SetFont(FAppStyle::GetFontStyle(TEXT("PropertyWindow.BoldFont")))
		.SetFontSize(7));

	Set("StateTree.Param.Background", new FSlateRoundedBoxBrush(FStyleColors::Hover, 6.f));

	// Expression Indent combo button
	const FButtonStyle IndentButton = FButtonStyle()
		.SetNormal(FSlateRoundedBoxBrush(FLinearColor::Transparent, 2.0f))
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Background, 2.0f, FStyleColors::InputOutline, 1.0f))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Background, 2.0f, FStyleColors::Hover, 1.0f))
		.SetNormalForeground(FStyleColors::Transparent)
		.SetHoveredForeground(FStyleColors::Hover)
		.SetPressedForeground(FStyleColors::Foreground)
		.SetNormalPadding(FMargin(2, 2, 2, 2))
		.SetPressedPadding(FMargin(2, 3, 2, 1));

	Set("StateTree.Node.Indent.ComboBox", FComboButtonStyle(ComboButtonStyle).SetButtonStyle(IndentButton));

	// Node text styles
	{
		FEditableTextStyle EditableTextStyle = FEditableTextStyle(FAppStyle::GetWidgetStyle<FEditableTextStyle>("NormalEditableText"));
		EditableTextStyle.Font = FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont"));
		EditableTextStyle.Font.Size = 10.0f;
		Set("StateTree.Node.Editable", EditableTextStyle);

		FEditableTextBoxStyle EditableTextBlockStyle = FEditableTextBoxStyle(FAppStyle::GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox"));
		EditableTextStyle.Font = FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont"));
		EditableTextStyle.Font.Size = 10.0f;
		Set("StateTree.Node.EditableTextBlock", EditableTextBlockStyle);

		const FTextBlockStyle StateNodeNormalText = FTextBlockStyle(NormalText)
			.SetFont(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			.SetFontSize(10);
		Set("StateTree.Node.Normal", StateNodeNormalText);

		Set("StateTree.Node.Bold", FTextBlockStyle(NormalText)
			.SetFont(FAppStyle::GetFontStyle(TEXT("PropertyWindow.BoldFont")))
			.SetFontSize(10));

		Set("StateTree.Node.Subdued", FTextBlockStyle(NormalText)
			.SetColorAndOpacity(FSlateColor::UseSubduedForeground())
			.SetFont(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			.SetFontSize(10));

		Set("StateTree.Node.TitleInlineEditableText", FInlineEditableTextBlockStyle()
			.SetTextStyle(StateNodeNormalText)
			.SetEditableTextBoxStyle(EditableTextBlockStyle));
	}

	// State Centric Styling
	{
		// Note: AccentBlue below used as placeholder for state color, final color will be determined by state. See 'FStateCentricViewUtils'

		// Parent styles
		{
			// How do we get a half-rounded rectangle border w/ no bottom in slate? There is no built in way, 
			// so we set crazy high curve edges to cause the bottom corners to cull (90000.0f).
			// To use correctly will need to negative pad to extend the half outline (Depends, -100 is safe).
			Set("StateTree.CentricView.Parent.HalfTopOutline", new FSlateRoundedBoxBrush(
				FStyleColors::AccentBlue.GetSpecifiedColor().CopyWithNewOpacity(0.0f)
				, FVector4{ 10.0, 10.0, 90000.0, 90000.0 }
				, FStyleColors::AccentBlue
				, 2.0f));

			Set("StateTree.CentricView.Parent.Outline", new FSlateRoundedBoxBrush(
				FStyleColors::AccentBlue.GetSpecifiedColor().CopyWithNewOpacity(0.0f)
				, FVector4{ 10.0, 10.0, 10.0, 10.0 }
				, FStyleColors::AccentBlue
				, 2.0f));

			Set("StateTree.CentricView.Parent.Header.Background", new FSlateRoundedBoxBrush(
				FStyleColors::AccentBlue
				, FVector4{ 10.0, 10.0, 0.0, 0.0 }
				, FStyleColors::AccentBlue
				, 0.0f
				, FVector2f{ 32.0f, 22.0f }));

			Set("StateTree.CentricView.Parent.Header.Background.Muted", new FSlateRoundedBoxBrush(
				FStyleColors::Header
				, FVector4{ 10.0, 10.0, 0.0, 0.0 }
				, FStyleColors::Header
				, 0.0f
				, FVector2f{ 32.0f, 22.0f }));

			Set("StateTree.CentricView.Parent.Background.Muted.Outlined", new FSlateRoundedBoxBrush(
				FStyleColors::AccentBlack
				, FVector4{ 10.0, 10.0, 10.0, 10.0 }
				, FStyleColors::AccentBlue
				, 2.0f
				, FVector2f{ 32.0f, 22.0f }));

			// Same hack used to get a correctly culling bottom rounded corner (90000.0f). Also needs negative padding for correct spacing (-10)
			Set("StateTree.CentricView.Parent.Footnote.Background", new FSlateRoundedBoxBrush(
				FStyleColors::AccentBlack
				, FVector4{ 90000.0, 90000.0, 10.0, 10.0 }
				, FStyleColors::AccentBlack
				, 0.0f
				, FVector2f{ 32.0f, 20.0f }));

			Set("StateTree.CentricView.Parent.Subnode.Background.Outlined", new FSlateRoundedBoxBrush(
				FStyleColors::AccentBlue
				, FVector4{ 5.0, 5.0, 5.0, 5.0 }
				, FStyleColors::AccentBlue // Note: Will have bright state color outline
				, 1.0f
				, FVector2f{ 32.0f, 18.0f }));

			Set("StateTree.CentricView.Parent.SubNode.Background", new FSlateRoundedBoxBrush(
				FStyleColors::AccentBlue
				, FVector4{ 5.0, 5.0, 5.0, 5.0 }
				, FStyleColors::AccentBlue
				, 0.0f
				, FVector2f{ 32.0f, 18.0f }));
		}

		// Leaf styles
		{
			Set("StateTree.CentricView.Leaf.Outline", new FSlateRoundedBoxBrush(
				FStyleColors::AccentBlue.GetSpecifiedColor().CopyWithNewOpacity(0.0f)
				, FVector4{ 0.0 }
				, FStyleColors::AccentBlue
				, 2.0f));

			Set("StateTree.CentricView.Leaf.Header.Background", new FSlateRoundedBoxBrush(
				FStyleColors::AccentBlue
				, FVector4{ 0.0 }
				, FStyleColors::AccentBlue
				, 0.0f
				, FVector2f{ 32.0f, 22.0f }));

			Set("StateTree.CentricView.Leaf.Footnote.Background", new FSlateRoundedBoxBrush(
				FStyleColors::AccentBlack
				, FVector4{ 0.0 }
				, FStyleColors::AccentBlack
				, 0.0f
				, FVector2f{ 32.0f, 10.0f }));

			Set("StateTree.CentricView.Leaf.Background.Outlined", new FSlateRoundedBoxBrush(
				FStyleColors::AccentBlack
				, FVector4{ 10.0, 10.0, 10.0, 10.0 }
				, FStyleColors::AccentBlue
				, 0.0f
				, FVector2f{ 32.0f, 22.0f }));

			Set("StateTree.CentricView.Leaf.Subnode.Background.Outlined", new FSlateRoundedBoxBrush(
				FStyleColors::AccentBlue
				, FVector4{ 0.0 }
				, FStyleColors::AccentBlue // Note: Will have bright state color outline
				, 1.0f
				, FVector2f{ 32.0f, 18.0f }));

			Set("StateTree.CentricView.Leaf.SubNode.Background", new FSlateRoundedBoxBrush(
				FStyleColors::AccentBlue
				, FVector4{ 0.0 }
				, FStyleColors::AccentBlue
				, 0.0f
				, FVector2f{ 32.0f, 18.0f }));
		}

		// General node styling
		{
			Set("StateTree.CentricView.SubNode.Background", new FSlateRoundedBoxBrush(
				FStyleColors::AccentBlack
				, FVector4{ 5.0, 5.0, 5.0, 5.0 }
				, FStyleColors::AccentBlack
				, 0.0f));

			// Same hack used to get a correctly culling bottom rounded corner (90000.0f). Also needs negative padding for correct spacing (-5)
			Set("StateTree.CentricView.SubNode.Footnote.Background", new FSlateRoundedBoxBrush(
				FStyleColors::AccentBlack
				, FVector4{ 90000.0, 90000.0, 5.0, 5.0 }
				, FStyleColors::AccentBlack
				, 0.0f
				, FVector2f{ 32.0f, 10.0f }));
		}

		// State centric text, these values don't exist elsewhere
		{
			Set("StateTree.CentricView.StateTitle.Bold",
				FTextBlockStyle(NormalText)
				.SetFont(DEFAULT_FONT("Bold", 12))
				.SetColorAndOpacity(FStyleColors::White));

			Set("StateTree.CentricView.StateTitle.Regular",
				FTextBlockStyle(NormalText)
				.SetFont(DEFAULT_FONT("Regular", 12))
				.SetColorAndOpacity(FStyleColors::White));
		}

		// State Centric Button Styles
		{
			const FButtonStyle& SimpleButtonStyle = FAppStyle::GetWidgetStyle<FButtonStyle>("SimpleButton");

			FButtonStyle HoverHighlight = FButtonStyle(SimpleButtonStyle)
				.SetHovered(FSlateRoundedBoxBrush(FStyleColors::White.GetSpecifiedColor().CopyWithNewOpacity(0.2f), 5.0f))
				.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Black.GetSpecifiedColor().CopyWithNewOpacity(0.1f), 5.0f));

			Set("StateTree.CentricView.HoverHighlight", HoverHighlight);
		}
	}

	// Command icons
	{
		// From generic Engine
		FContentRootScope Scope(this, EngineSlateContentDir);
		Set("StateTreeEditor.CutStates", new IMAGE_BRUSH_SVG("Starship/Common/Cut", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.CopyStates", new IMAGE_BRUSH_SVG("Starship/Common/Copy", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.DuplicateStates", new IMAGE_BRUSH_SVG("Starship/Common/Duplicate", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.DeleteStates", new IMAGE_BRUSH_SVG("Starship/Common/Delete", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.RenameState", new IMAGE_BRUSH_SVG("Starship/Common/Rename", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.AutoScroll", new IMAGE_BRUSH_SVG("Starship/Insights/AutoScrollRight_20", CoreStyleConstants::Icon16x16));

		Set("StateTreeEditor.Debugger.ResetTracks", new IMAGE_BRUSH_SVG("Starship/Common/Delete", CoreStyleConstants::Icon16x16));

		Set("StateTreeEditor.Debugger.Task.Enter", new CORE_IMAGE_BRUSH_SVG("Starship/Common/arrow-right", CoreStyleConstants::Icon16x16, FStyleColors::Foreground));
		Set("StateTreeEditor.Debugger.Task.Exit", new CORE_IMAGE_BRUSH_SVG("Starship/Common/arrow-left", CoreStyleConstants::Icon16x16, FStyleColors::Foreground));

		Set("StateTreeEditor.Debugger.Unset", new CORE_IMAGE_BRUSH_SVG("Starship/Common/help", CoreStyleConstants::Icon16x16, FStyleColors::AccentBlack));

		// Common Node Icons
		Set("Node.EnableDisable", new CORE_IMAGE_BRUSH_SVG("Starship/Common/check-circle", CoreStyleConstants::Icon16x16));
		Set("Node.Time", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Recent", CoreStyleConstants::Icon16x16));
		Set("Node.Sync", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Update", CoreStyleConstants::Icon16x16));
	}

	{
		// From RewindDebugger
		FContentRootScope Scope(this, FPaths::EngineDir() / TEXT("Plugins/Animation/GameplayInsights/Content"));
		Set("StateTreeEditor.Debugger.OpenRewindDebugger", new IMAGE_BRUSH("Rewind_24x", CoreStyleConstants::Icon16x16));
	}

	{
		// From generic Engine Editor
		FContentRootScope Scope(this, EngineEditorSlateContentDir);
		Set("StateTreeEditor.Debugger.StartRewindDebuggerRecording", new IMAGE_BRUSH("Sequencer/Transport_Bar/Record_24x", CoreStyleConstants::Icon16x16));

		Set("StateTreeEditor.Debugger.StartRecording", new IMAGE_BRUSH("Sequencer/Transport_Bar/Record_24x", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.Debugger.StopRecording", new IMAGE_BRUSH("Sequencer/Transport_Bar/Recording_24x", CoreStyleConstants::Icon16x16));

		Set("StateTreeEditor.Debugger.PreviousFrameWithStateChange", new IMAGE_BRUSH("Sequencer/Transport_Bar/Go_To_Front_24x", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.Debugger.PreviousFrameWithEvents", new IMAGE_BRUSH("Sequencer/Transport_Bar/Step_Backwards_24x", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.Debugger.NextFrameWithEvents", new IMAGE_BRUSH("Sequencer/Transport_Bar/Step_Forward_24x", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.Debugger.NextFrameWithStateChange", new IMAGE_BRUSH("Sequencer/Transport_Bar/Go_To_End_24x", CoreStyleConstants::Icon16x16));

		Set("StateTreeEditor.Debugger.ToggleOnEnterStateBreakpoint", new IMAGE_BRUSH_SVG("Starship/Blueprints/Breakpoint_Valid", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.Debugger.EnableOnEnterStateBreakpoint", new IMAGE_BRUSH_SVG("Starship/Blueprints/Breakpoint_Valid", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.Debugger.EnableOnExitStateBreakpoint", new IMAGE_BRUSH_SVG("Starship/Blueprints/Breakpoint_Valid", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.DebugOptions", new IMAGE_BRUSH_SVG("Starship/Common/Bug", CoreStyleConstants::Icon16x16));

		Set("StateTreeEditor.Debugger.OwnerTrack", new IMAGE_BRUSH_SVG("Starship/AssetIcons/AIController_64", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.Debugger.InstanceTrack", new IMAGE_BRUSH_SVG("Starship/AssetIcons/AnimInstance_64", CoreStyleConstants::Icon16x16));

		Set("StateTreeEditor.EnableStates", new IMAGE_BRUSH("Icons/Empty_16x", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.Debugger.Breakpoint.EnabledAndValid", new IMAGE_BRUSH_SVG("Starship/Blueprints/Breakpoint_Valid", CoreStyleConstants::Icon16x16, FStyleColors::AccentRed));
		Set("StateTreeEditor.Debugger.ResumeDebuggerAnalysis", new IMAGE_BRUSH_SVG("Starship/Common/Timeline", CoreStyleConstants::Icon16x16));

		Set("StateTreeEditor.Transition.None", new CORE_IMAGE_BRUSH_SVG("Starship/Common/x-circle", CoreStyleConstants::Icon16x16, FSlateColor::UseSubduedForeground()));
		Set("StateTreeEditor.Transition.Succeeded", new CORE_IMAGE_BRUSH_SVG("Starship/Common/check", CoreStyleConstants::Icon16x16, FStyleColors::AccentGreen));
		Set("StateTreeEditor.Transition.Failed", new CORE_IMAGE_BRUSH_SVG("Starship/Common/close-small", CoreStyleConstants::Icon16x16, FStyleColors::AccentRed));

		Set("StateTreeEditor.Transition.Succeeded", new CORE_IMAGE_BRUSH_SVG("Starship/Common/check", CoreStyleConstants::Icon16x16, FStyleColors::AccentGreen));
		Set("StateTreeEditor.Transition.Failed", new CORE_IMAGE_BRUSH_SVG("Starship/Common/close-small", CoreStyleConstants::Icon16x16, FStyleColors::AccentRed));

		// Common Node Icons
		Set("Node.Navigation", new IMAGE_BRUSH_SVG("Starship/Common/Navigation", CoreStyleConstants::Icon16x16));
		Set("Node.Event", new IMAGE_BRUSH_SVG("Starship/Common/Event", CoreStyleConstants::Icon16x16));
		Set("Node.Animation", new IMAGE_BRUSH_SVG("Starship/Common/Animation", CoreStyleConstants::Icon16x16));
		Set("Node.Debug", new IMAGE_BRUSH_SVG("Starship/Common/Debug", CoreStyleConstants::Icon16x16));
		Set("Node.Find", new IMAGE_BRUSH_SVG("Starship/Common/Find", CoreStyleConstants::Icon16x16));
	}

	{
		// From plugin
		Set("ClassThumbnail.StateTree", new IMAGE_BRUSH_SVG("Icons/StateTree_64", CoreStyleConstants::Icon64x64));
		Set("ClassIcon.StateTree", new IMAGE_BRUSH_SVG("Icons/StateTree", CoreStyleConstants::Icon16x16));

		Set("StateTreeEditor.AddSiblingState", new IMAGE_BRUSH_SVG("Icons/Sibling_State", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.AddChildState", new IMAGE_BRUSH_SVG("Icons/Child_State", CoreStyleConstants::Icon16x16));

		Set("StateTreeEditor.PasteStatesAsSiblings", new IMAGE_BRUSH_SVG("Icons/Sibling_State", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.PasteStatesAsChildren", new IMAGE_BRUSH_SVG("Icons/Child_State", CoreStyleConstants::Icon16x16));

		Set("StateTreeEditor.StateConditions", new IMAGE_BRUSH_SVG("Icons/State_Conditions", CoreStyleConstants::Icon16x16));

		Set("StateTreeEditor.Conditions", new IMAGE_BRUSH_SVG("Icons/Conditions", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.Conditions.Large", new IMAGE_BRUSH_SVG("Icons/Conditions", CoreStyleConstants::Icon24x24));
		Set("StateTreeEditor.Evaluators", new IMAGE_BRUSH_SVG("Icons/Evaluators", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.Parameters", new IMAGE_BRUSH_SVG("Icons/Parameters", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.Utility", new IMAGE_BRUSH_SVG("Icons/Utility", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.Utility.Large", new IMAGE_BRUSH_SVG("Icons/Utility", CoreStyleConstants::Icon24x24));
		Set("StateTreeEditor.Tasks", new IMAGE_BRUSH_SVG("Icons/Tasks", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.Tasks.Large", new IMAGE_BRUSH_SVG("Icons/Tasks", CoreStyleConstants::Icon24x24));
		Set("StateTreeEditor.Transitions", new IMAGE_BRUSH_SVG("Icons/Transitions", CoreStyleConstants::Icon16x16));

		Set("StateTreeEditor.TasksCompletion.Enabled", new IMAGE_BRUSH_SVG("Icons/ConsiderTask", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.TasksCompletion.Disabled", new IMAGE_BRUSH_SVG("Icons/NotConsiderTask", CoreStyleConstants::Icon16x16));

		Set("StateTreeEditor.StateSubtree", new IMAGE_BRUSH_SVG("Icons/State_Subtree", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.StateLinked", new IMAGE_BRUSH_SVG("Icons/State_Linked", CoreStyleConstants::Icon16x16));

		Set("StateTreeEditor.Transition.Dash", new IMAGE_BRUSH_SVG("Icons/Transition_Dash", CoreStyleConstants::Icon16x16, FStyleColors::Foreground));
		Set("StateTreeEditor.Transition.Goto", new IMAGE_BRUSH_SVG("Icons/Transition_Goto", CoreStyleConstants::Icon16x16, FStyleColors::Foreground));
		Set("StateTreeEditor.Transition.Next", new IMAGE_BRUSH_SVG("Icons/Transition_Next", CoreStyleConstants::Icon16x16, FStyleColors::Foreground));
		Set("StateTreeEditor.Transition.Parent", new IMAGE_BRUSH_SVG("Icons/Transition_Parent", CoreStyleConstants::Icon16x16, FStyleColors::Foreground));

		Set("StateTreeEditor.Transition.Condition", new IMAGE_BRUSH_SVG("Icons/State_Conditions", CoreStyleConstants::Icon16x16, FStyleColors::AccentGray));

		// Common Node Icons
		Set("Node.Movement", new IMAGE_BRUSH_SVG("Icons/Movement", CoreStyleConstants::Icon16x16));
		Set("Node.Tag", new IMAGE_BRUSH_SVG("Icons/Tag", CoreStyleConstants::Icon16x16));
		Set("Node.RunParallel", new IMAGE_BRUSH_SVG("Icons/RunParallel", CoreStyleConstants::Icon16x16));
		Set("Node.Task", new IMAGE_BRUSH_SVG("Icons/Task", CoreStyleConstants::Icon16x16));
		Set("Node.Text", new IMAGE_BRUSH_SVG("Icons/Text", CoreStyleConstants::Icon16x16));
		Set("Node.Function", new IMAGE_BRUSH_SVG("Icons/Function", CoreStyleConstants::Icon16x16));

		// Runtime flag
		Set("StateTreeEditor.Flags.Tick", new IMAGE_BRUSH_SVG("Icons/Tick", CoreStyleConstants::Icon16x16));
		Set("StateTreeEditor.Flags.TickOnEvent", new IMAGE_BRUSH_SVG("Icons/TickEvent", CoreStyleConstants::Icon16x16));
	}
	{
		Set("Colors.StateLinkingIn", FLinearColor::Yellow);
		Set("Colors.StateLinkedOut", FLinearColor::Green);
		Set("Colors.StateSelection", FLinearColor(FColor(236, 134, 39)));
		Set("Colors.Diff.MissingFromTree1", FLinearColor(0.f, 1.f, 0.f, .7f));
		Set("Colors.Diff.MissingFromTree2", FLinearColor(1.f, 0.f, 0.f, .7f));
		Set("Colors.Diff.Changed", FLinearColor(1.f, 1.f, 0.f, .7f));
	}
}

void FStateTreeEditorStyle::Register()
{
	FSlateStyleRegistry::RegisterSlateStyle(Get());
}

void FStateTreeEditorStyle::Unregister()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(Get());
}

FStateTreeEditorStyle& FStateTreeEditorStyle::Get()
{
	static FStateTreeEditorStyle Instance;
	return Instance;
}

const FSlateBrush* FStateTreeEditorStyle::GetBrushForSelectionBehaviorType(const EStateTreeStateSelectionBehavior InSelectionBehavior, const bool bInHasChildren, const EStateTreeStateType InStateType)
{
	// Simply redirecting to the base class.
	// Keeping the method in the derived class otherwise other modules that were already calling it will have a missing module dependency to StateTreeDeveloper otherwise.
	return FStateTreeStyle::GetBrushForSelectionBehaviorType(InSelectionBehavior, bInHasChildren, InStateType);
}
