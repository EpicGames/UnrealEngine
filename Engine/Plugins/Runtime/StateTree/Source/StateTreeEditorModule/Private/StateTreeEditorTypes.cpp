// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeEditorTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeEditorTypes)

bool FStateTreeEditorColor::ExportTextItem(FString& OutValueString, const FStateTreeEditorColor& DefaultValue, UObject* OwnerObject, int32 PortFlags, UObject* ExportRootScope) const
{
#if WITH_EDITORONLY_DATA
	// Only go through this Export path if Copy or Duplicate 
	if ((PortFlags & (PPF_Copy | PPF_Duplicate)) == 0)
	{
		return false;
	}

	uint32 Count = 0;

	for (FProperty* Property : TFieldRange<FProperty>(FStateTreeEditorColor::StaticStruct()))
	{
		if (!Property->ShouldPort(PortFlags) || Property->HasMetaData(TEXT("StructExportTransient")))
		{
			continue;
		}

		for (int32 Index = 0; Index < Property->ArrayDim; Index++)
		{
			FString InnerValue;
			if (Property->ExportText_InContainer(Index, InnerValue, this, &DefaultValue, OwnerObject, PPF_Delimited | PortFlags, ExportRootScope))
			{
				OutValueString += Count++ == 0 ? TEXT('(') : TEXT(',');

				if (Property->ArrayDim == 1)
				{
					OutValueString += FString::Printf(TEXT("%s="), *Property->GetName());
				}
				else
				{
					OutValueString += FString::Printf(TEXT("%s[%i]="), *Property->GetName(), Index);
				}

				OutValueString += InnerValue;
			}
		}
	}

	if (Count > 0)
	{
		OutValueString += TEXT(")");
	}
	else
	{
		OutValueString += TEXT("()");
	}
	return true;
#else
	return false;
#endif
}
