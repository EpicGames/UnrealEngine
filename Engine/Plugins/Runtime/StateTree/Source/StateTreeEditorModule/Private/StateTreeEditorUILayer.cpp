// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeEditorUILayer.h"
#include "StateTreeEditor.h"
#include "StateTreeEditorModule.h"
#include "Toolkits/IToolkit.h"
#include "ToolMenus.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeEditorUILayer)

FStateTreeEditorModeUILayer::FStateTreeEditorModeUILayer(const IToolkitHost* InToolkitHost) : FAssetEditorModeUILayer(InToolkitHost)
{
}

void FStateTreeEditorModeUILayer::OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit)
{
	if (!Toolkit->IsAssetEditor())
	{
		FAssetEditorModeUILayer::OnToolkitHostingStarted(Toolkit);
		HostedToolkit = Toolkit;
		Toolkit->SetModeUILayer(SharedThis(this));
		Toolkit->RegisterTabSpawners(ToolkitHost->GetTabManager().ToSharedRef());
		RegisterModeTabSpawners();	

		OnToolkitHostReadyForUI.Execute();

		// Set up an owner for the current scope so that we can cleanly clean up the toolbar extension on hosting finish
		UToolMenu* SecondaryModeToolbar = UToolMenus::Get()->ExtendMenu(GetSecondaryModeToolbarName());
		OnRegisterSecondaryModeToolbarExtension.ExecuteIfBound(SecondaryModeToolbar);
	}
}

void FStateTreeEditorModeUILayer::OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit)
{	
	if (HostedToolkit.IsValid() && HostedToolkit.Pin() == Toolkit)
	{
		FAssetEditorModeUILayer::OnToolkitHostingFinished(Toolkit);
	}
}

void FStateTreeEditorModeUILayer::SetModeMenuCategory(const TSharedPtr<FWorkspaceItem>& MenuCategoryIn)
{
	MenuCategory = MenuCategoryIn;
}

TSharedPtr<FWorkspaceItem> FStateTreeEditorModeUILayer::GetModeMenuCategory() const
{
	check(MenuCategory);
	return MenuCategory;
}

void UStateTreeEditorUISubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	FStateTreeEditorModule& StateTreeEditorModule = FModuleManager::LoadModuleChecked<FStateTreeEditorModule>("StateTreeEditorModule");
	StateTreeEditorModule.OnRegisterLayoutExtensions().AddUObject(this, &UStateTreeEditorUISubsystem::RegisterLayoutExtensions);
}

void UStateTreeEditorUISubsystem::Deinitialize()
{
	FStateTreeEditorModule& StateTreeEditorModule = FModuleManager::LoadModuleChecked<FStateTreeEditorModule>("StateTreeEditorModule");
	StateTreeEditorModule.OnRegisterLayoutExtensions().RemoveAll(this);
}

void UStateTreeEditorUISubsystem::RegisterLayoutExtensions(FLayoutExtender& Extender)
{	
	Extender.ExtendStack(FStateTreeEditor::LayoutLeftStackId, ELayoutExtensionPosition::After, FTabManager::FTab(UAssetEditorUISubsystem::TopLeftTabID, ETabState::ClosedTab));
	Extender.ExtendStack(FStateTreeEditor::LayoutLeftStackId, ELayoutExtensionPosition::After, FTabManager::FTab(UAssetEditorUISubsystem::BottomRightTabID, ETabState::ClosedTab));

#if WITH_STATETREE_TRACE_DEBUGGER
	Extender.ExtendStack(FStateTreeEditor::LayoutBottomMiddleStackId, ELayoutExtensionPosition::After, FTabManager::FTab(UAssetEditorUISubsystem::TopRightTabID, ETabState::ClosedTab));
#endif // WITH_STATETREE_TRACE_DEBUGGER
}
