// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/AssetEditorModeUILayer.h"

#include "StateTreeEditorUILayer.generated.h"

class FStateTreeEditorModeUILayer : public FAssetEditorModeUILayer
{
public:
	FStateTreeEditorModeUILayer(const IToolkitHost* InToolkitHost);
	virtual void OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit) override;
	virtual void OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit) override;
	virtual TSharedPtr<FWorkspaceItem> GetModeMenuCategory() const override;
	
	void SetModeMenuCategory(const TSharedPtr<FWorkspaceItem>& MenuCategoryIn);
protected:
	TSharedPtr<FWorkspaceItem> MenuCategory;
};

UCLASS()
class UStateTreeEditorUISubsystem : public UAssetEditorUISubsystem
{
	GENERATED_BODY()

public:
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	virtual void RegisterLayoutExtensions(FLayoutExtender& Extender) override;
};