// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeEditorUserSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeEditorUserSettings)

void UStateTreeEditorUserSettings::SetStatesViewDisplayNodeType(EStateTreeEditorUserSettingsNodeType Value)
{
	if (StatesViewDisplayNodeType != Value)
	{
		StatesViewDisplayNodeType = Value;
		OnSettingsChanged.Broadcast();
	}
}

void UStateTreeEditorUserSettings::SetDetailsViewDisplayCount(EStateTreeEditorUserSettingsNodeType Value)
{
	if (DetailsViewDisplayCount != Value)
	{
		DetailsViewDisplayCount = Value;
		OnSettingsChanged.Broadcast();
	}
}

void UStateTreeEditorUserSettings::SetDisplayMigrationToRewindDebuggerWarning(bool bNewValue)
{
	if (bDisplayMigrationToRewindDebuggerWarning != bNewValue)
	{
		bDisplayMigrationToRewindDebuggerWarning = bNewValue;
		SaveConfig();
	}
}

void UStateTreeEditorUserSettings::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	FName PropertyName = PropertyChangedEvent.Property->GetFName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UStateTreeEditorUserSettings, StatesViewDisplayNodeType)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UStateTreeEditorUserSettings, StatesViewStateRowHeight)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UStateTreeEditorUserSettings, StatesViewNodeRowHeight)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UStateTreeEditorUserSettings, DetailsViewDisplayCount))
	{
		OnSettingsChanged.Broadcast();
		SaveConfig();
	}

	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}
