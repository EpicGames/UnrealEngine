// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h" //DeveloperSettings
#include "StateTreeEditorUserSettings.generated.h"

UENUM(meta=(Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EStateTreeEditorUserSettingsNodeType : uint8
{
	None = 0 << 0 UMETA(Hidden),
	Condition = 1 << 0,
	Task = 1 << 1,
	Transition = 1 << 2,
	Flag = 1 << 3,
	Parameter = 1 << 4,
	Utility = 1 << 5,
	Evaluator = 1 << 6,
	All = (Condition | Task | Transition | Flag | Parameter | Utility | Evaluator) UMETA(Hidden),
};
ENUM_CLASS_FLAGS(EStateTreeEditorUserSettingsNodeType);

/** User settings for the StateTree editor */
UCLASS(MinimalAPI, config=EditorPerProjectUserSettings, DisplayName="StateTree User Settings")
class UStateTreeEditorUserSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UStateTreeEditorUserSettings() = default;

	EStateTreeEditorUserSettingsNodeType GetStatesViewDisplayNodeType() const
	{
		return StatesViewDisplayNodeType;
	}
	void SetStatesViewDisplayNodeType(EStateTreeEditorUserSettingsNodeType Value);

	EStateTreeEditorUserSettingsNodeType GetDetailsViewDisplayCount() const
	{
		return DetailsViewDisplayCount;
	}
	void SetDetailsViewDisplayCount(EStateTreeEditorUserSettingsNodeType Value);

	float GetStatesViewStateRowHeight() const
	{
		return StatesViewStateRowHeight;
	}
	
	float GetStatesViewNodeRowHeight() const
	{
		return StatesViewNodeRowHeight;
	}

	bool GetDisplayMigrationToRewindDebuggerWarning() const
	{
		return bDisplayMigrationToRewindDebuggerWarning;
	}

	void SetDisplayMigrationToRewindDebuggerWarning(bool bNewValue);

	/** Broadcast when a setting changes. */
	FSimpleMulticastDelegate OnSettingsChanged;

public:
	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;

private:
	/** Which additional node type to display in the States View. */
	UPROPERTY(config, EditAnywhere, Category = "State View", meta = (DisplayName = "Display Node"))
	EStateTreeEditorUserSettingsNodeType StatesViewDisplayNodeType = EStateTreeEditorUserSettingsNodeType::All;
	
	/** Height of a state in the States View. */
	UPROPERTY(config, EditAnywhere, Category = "State View", meta = (DisplayName = "State Height", ClampMin = 8.0f))
	float StatesViewStateRowHeight = 32.0f;
	
	/** Height of a node in the States View. */
	UPROPERTY(config, EditAnywhere, Category = "State View", meta = (DisplayName = "Node Height", ClampMin = 8.0f))
	float StatesViewNodeRowHeight = 16.0f;

	/** True if we should show the count for that node in details. */
	UPROPERTY(config, EditAnywhere, Category = "Details View", meta = (DisplayName = "Display Count"))
	EStateTreeEditorUserSettingsNodeType DetailsViewDisplayCount = EStateTreeEditorUserSettingsNodeType::All;

	/** Display the debugger migration warning. */
	UPROPERTY(config)
	bool bDisplayMigrationToRewindDebuggerWarning = true;
};
