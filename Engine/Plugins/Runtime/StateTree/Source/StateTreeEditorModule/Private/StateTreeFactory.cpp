// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeFactory.h"
#include "StateTree.h"
#include "Kismet2/SClassPickerDialog.h"
#include "ClassViewerFilter.h"
#include "StateTreeCompilerLog.h"
#include "StateTreeCompiler.h"
#include "StateTreeDelegates.h"
#include "StateTreeEditor.h"
#include "StateTreeEditorData.h"
#include "StateTreeEditorModule.h"
#include "StateTreeEditorSchema.h"
#include "StateTreeEditingSubsystem.h"
#include "StateTreeNodeClassCache.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeFactory)

#define LOCTEXT_NAMESPACE "StateTreeEditor"

/////////////////////////////////////////////////////
// FStateTreeClassFilter

class FStateTreeClassFilter : public IClassViewerFilter
{
public:
	/** All children of these classes will be included unless filtered out by another setting. */
	TSet<const UClass*> AllowedChildrenOfClasses;

	/** Disallowed class flags. */
	EClassFlags DisallowedClassFlags = CLASS_None;

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return !InClass->HasAnyClassFlags(DisallowedClassFlags)
			&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InClass) != EFilterReturn::Failed;
	}
	
	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return !InUnloadedClassData->HasAnyClassFlags(DisallowedClassFlags)
			&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Failed;
	}

};

/////////////////////////////////////////////////////
// UStateTreeFactory

UStateTreeFactory::UStateTreeFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UStateTree::StaticClass();
}

bool UStateTreeFactory::ConfigureProperties()
{
	// If it is an overridden factory and the schema class has been specified by caller
	if (StateTreeSchemaClass)
	{
		return true;
	}

	// Load the classviewer module to display a class picker
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	// Load StateTree class cache to get all schema classes
	FStateTreeEditorModule& EditorModule = FModuleManager::GetModuleChecked<FStateTreeEditorModule>(TEXT("StateTreeEditorModule"));
	FStateTreeNodeClassCache* ClassCache = EditorModule.GetNodeClassCache().Get();
	check(ClassCache);

	// Fill in options
	FClassViewerInitializationOptions Options;
	Options.DisplayMode = EClassViewerDisplayMode::Type::TreeView;
	Options.Mode = EClassViewerMode::ClassPicker;
	Options.bShowNoneOption = false;
	Options.bExpandAllNodes = true;

	TSharedPtr<FStateTreeClassFilter> Filter = MakeShareable(new FStateTreeClassFilter);
	Options.ClassFilters.Add(Filter.ToSharedRef());

	// Add all schemas which are tagged as "CommonSchema" to the common section.
	TArray<TSharedPtr<FStateTreeNodeClassData>> AvailableClasses;
	ClassCache->GetClasses(UStateTreeSchema::StaticClass(), AvailableClasses);
	for (const TSharedPtr<FStateTreeNodeClassData>& ClassData : AvailableClasses)
	{
		if (FStateTreeNodeClassData* Data = ClassData.Get())
		{
			if (UClass* Class = Data->GetClass())
			{
				if (Class->HasMetaData("CommonSchema"))
				{
					Options.ExtraPickerCommonClasses.Add(Class);					
				}
			}
		}
	}

	Filter->DisallowedClassFlags = CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_Abstract | CLASS_HideDropDown;
	Filter->AllowedChildrenOfClasses.Add(UStateTreeSchema::StaticClass());

	const FText TitleText = LOCTEXT("CreateStateTree", "Pick Schema for State Tree");

	UClass* ChosenClass = nullptr;
	const bool bResult = SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, UStateTreeSchema::StaticClass());
	StateTreeSchemaClass = ChosenClass;

	return bResult;
}

void UStateTreeFactory::SetSchemaClass(const TObjectPtr<UClass>& InSchemaClass)
{
	StateTreeSchemaClass = InSchemaClass;
}

UObject* UStateTreeFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (StateTreeSchemaClass == nullptr)
	{
		return nullptr;
	}

	// Create new asset
	UStateTree* NewStateTree = NewObject<UStateTree>(InParent, Class, Name, Flags);

	// Create and init new editor data.
	TNonNullSubclassOf<UStateTreeEditorData> EditorDataClass = FStateTreeEditorModule::GetModule().GetEditorDataClass(StateTreeSchemaClass.Get());
	UStateTreeEditorData* EditorData = NewObject<UStateTreeEditorData>(NewStateTree, EditorDataClass, FName(), RF_Transactional);
	NewStateTree->EditorData = EditorData;

	EditorData->Schema = NewObject<UStateTreeSchema>(EditorData, StateTreeSchemaClass, FName(), RF_Transactional);
	EditorData->AddRootState();

	TNonNullSubclassOf<UStateTreeEditorSchema> EditorSchemaClass = FStateTreeEditorModule::GetModule().GetEditorSchemaClass(StateTreeSchemaClass.Get());
	EditorData->EditorSchema = NewObject<UStateTreeEditorSchema>(EditorData, EditorSchemaClass, FName(), RF_Transactional);

	FStateTreeCompilerLog Log;
	const bool bSuccess = UStateTreeEditingSubsystem::CompileStateTree(NewStateTree, Log);

	if (!bSuccess)
	{
		NewStateTree = nullptr;
	}
	
	return NewStateTree;
}

#undef LOCTEXT_NAMESPACE

