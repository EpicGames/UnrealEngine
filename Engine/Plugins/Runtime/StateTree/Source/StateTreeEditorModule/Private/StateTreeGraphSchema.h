// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphSchema_K2.h"

#include "StateTreeGraphSchema.generated.h"

namespace UE::StateTree::Editor
{
	UCLASS()
	class UEdGraphSchema_StateTree : public UEdGraphSchema_K2
	{
		GENERATED_BODY()
	};
}
