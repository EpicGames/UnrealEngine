// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeNodeClassCache.h"
#include "Blueprint/BlueprintSupport.h"
#include "Misc/FeedbackContext.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectIterator.h"
#include "Misc/PackageName.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/Blueprint.h"
#include "Editor.h"
#include "AssetRegistry/ARFilter.h"
#include "Logging/MessageLog.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/EnumerateRange.h"
#include "StateTreeNodeBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeNodeClassCache)

#define LOCTEXT_NAMESPACE "StateTreeEditor"

FStateTreeNodeClassData::FStateTreeNodeClassData(TNotNull<UStruct*> InStruct)
	: StructAssetPath(InStruct->GetStructPathName())
	, CachedStruct(InStruct)
{
}

// deprecated
FStateTreeNodeClassData::FStateTreeNodeClassData(const FString& InClassAssetName, const FString& InClassPackageName, const FName InStructName, UStruct* InStruct)
	: StructAssetPath(*InClassPackageName, InStructName)
	, CachedStruct(InStruct)
{
}

FStateTreeNodeClassData::FStateTreeNodeClassData(const FTopLevelAssetPath& AssetPath)
	: StructAssetPath(AssetPath)
{
}



UStruct* FStateTreeNodeClassData::GetStruct(bool bSilent)
{
	UStruct* Result = CachedStruct.Get();
	
	if (Result == nullptr && StructAssetPath.IsValid())
	{
		GWarn->BeginSlowTask(LOCTEXT("LoadPackage", "Loading Package..."), true);

		UObject* Object = StaticLoadAsset(nullptr, StructAssetPath);
		if (Object)
		{
			if (UBlueprint* BlueprintOb = Cast<UBlueprint>(Object))
			{
				Result = *BlueprintOb->GeneratedClass;
			}
			else if (UStruct* Struct = Cast<UStruct>(Object))
			{
				Result = Struct;
			}

			CachedStruct = Result;
		}
		else
		{
			if (!bSilent)
			{
				FMessageLog EditorErrors("EditorErrors");
				EditorErrors.Error(LOCTEXT("PackageLoadFail", "Package Load Failed"));
				EditorErrors.Info(FText::FromName(StructAssetPath.GetPackageName()));
				EditorErrors.Notify(LOCTEXT("PackageLoadFail", "Package Load Failed"));
			}
		}

		GWarn->EndSlowTask();
	}

	return Result;
}

const UStruct* FStateTreeNodeClassData::GetInstanceDataStruct(bool bSilent /*= false*/)
{
	const UStruct* Result = CachedInstanceDataStruct.Get();
	if (Result == nullptr && StructAssetPath.IsValid())
	{
		if (const UScriptStruct* ScriptStruct = GetScriptStruct(bSilent))
		{
			TInstancedStruct<FStateTreeNodeBase> NodeInstance;
			NodeInstance.InitializeAsScriptStruct(ScriptStruct);

			Result = NodeInstance.Get().GetInstanceDataType();
			CachedInstanceDataStruct = Result;
		}
	}

	return Result;
}

//----------------------------------------------------------------------//
//  FStateTreeNodeClassCache
//----------------------------------------------------------------------//
FStateTreeNodeClassCache::FStateTreeNodeClassCache()
{
	// Register with the Asset Registry to be informed when it is done loading up files.
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().OnFilesLoaded().AddRaw(this, &FStateTreeNodeClassCache::InvalidateCache);
	AssetRegistryModule.Get().OnAssetAdded().AddRaw(this, &FStateTreeNodeClassCache::OnAssetAdded);
	AssetRegistryModule.Get().OnAssetRemoved().AddRaw(this, &FStateTreeNodeClassCache::OnAssetRemoved);

	// Register to have Populate called when doing a Reload.
	FCoreUObjectDelegates::ReloadCompleteDelegate.AddRaw(this, &FStateTreeNodeClassCache::OnReloadComplete);

	// Register to have Populate called when a Blueprint is compiled.
	if (GIsEditor)
	{
		check(GEditor);
		GEditor->OnBlueprintCompiled().AddRaw(this, &FStateTreeNodeClassCache::InvalidateCache);
		GEditor->OnClassPackageLoadedOrUnloaded().AddRaw(this, &FStateTreeNodeClassCache::InvalidateCache);
	}
}

FStateTreeNodeClassCache::~FStateTreeNodeClassCache()
{
	// Unregister with the Asset Registry to be informed when it is done loading up files.
	if (FModuleManager::Get().IsModuleLoaded(TEXT("AssetRegistry")))
	{
		IAssetRegistry* AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).TryGet();
		if (AssetRegistry)
		{
			AssetRegistry->OnFilesLoaded().RemoveAll(this);
			AssetRegistry->OnAssetAdded().RemoveAll(this);
			AssetRegistry->OnAssetRemoved().RemoveAll(this);
		}

		// Unregister to have Populate called when doing a Reload.
		FCoreUObjectDelegates::ReloadCompleteDelegate.RemoveAll(this);

		// Unregister to have Populate called when a Blueprint is compiled.
		if (GEditor != nullptr)
		{
			// GEditor can't have been destructed before we call this or we'll crash.
			GEditor->OnBlueprintCompiled().RemoveAll(this);
			GEditor->OnClassPackageLoadedOrUnloaded().RemoveAll(this);
		}
	}
}

void FStateTreeNodeClassCache::AddRootStruct(TNotNull<UStruct*> RootStruct)
{
	if (RootClasses.ContainsByPredicate([RootStruct](const FRootClassContainer& RootClass)
		{
			return RootClass.BaseStruct == RootStruct;
		}))
	{
		return;
	}
	
	RootClasses.Emplace(RootStruct, RootClasses.Num());

	InvalidateCache();
}

void FStateTreeNodeClassCache::GetStructs(TNotNull<const UStruct*> BaseStruct, TArray<TSharedPtr<FStateTreeNodeClassData>>& AvailableClasses)
{
	AvailableClasses.Reset();
	
	const int32 RootClassIndex = RootClasses.IndexOfByPredicate([BaseStruct](const FRootClassContainer& RootClass)
		{
			return RootClass.BaseStruct == BaseStruct;
		});
	if (RootClassIndex != INDEX_NONE)
	{
		FRootClassContainer& RootClass = RootClasses[RootClassIndex];
		if (!RootClass.bUpdated)
		{
			CacheClasses(RootClass);
		}

		AvailableClasses.Append(RootClass.ClassData);
	}
}

void FStateTreeNodeClassCache::OnAssetAdded(const FAssetData& AssetData)
{
	UpdateBlueprintClass(AssetData);
}

void FStateTreeNodeClassCache::OnAssetRemoved(const FAssetData& AssetData)
{
	FString AssetClassName;
	FString AssetNativeParentClassName;
	if (AssetData.GetTagValue(FBlueprintTags::GeneratedClassPath, AssetClassName) && AssetData.GetTagValue(FBlueprintTags::NativeParentClassPath, AssetNativeParentClassName))
	{
		if (const int32* RootClassIndex = ClassNameToRootIndex.Find(AssetNativeParentClassName))
		{
			FRootClassContainer& RootClass = RootClasses[*RootClassIndex];
			const FTopLevelAssetPath StructAssetPath = FTopLevelAssetPath(AssetClassName);
			RootClass.ClassData.RemoveAll([&StructAssetPath](const TSharedPtr<FStateTreeNodeClassData>& ClassData)
			{
				return ClassData->GetStructPath() == StructAssetPath;
			});
		}
	}
}

void FStateTreeNodeClassCache::InvalidateCache()
{
	for (FRootClassContainer& RootClass : RootClasses)
	{
		RootClass.ClassData.Reset();
		RootClass.bUpdated = false;
	}
	ClassNameToRootIndex.Reset();
}

void FStateTreeNodeClassCache::OnReloadComplete(EReloadCompleteReason Reason)
{
	InvalidateCache();
}

void FStateTreeNodeClassCache::UpdateBlueprintClass(const FAssetData& AssetData)
{
	FString AssetClassName;
	FString AssetNativeParentClassName;
	if (AssetData.GetTagValue(FBlueprintTags::GeneratedClassPath, AssetClassName) && AssetData.GetTagValue(FBlueprintTags::NativeParentClassPath, AssetNativeParentClassName))
	{
		ConstructorHelpers::StripObjectClass(AssetNativeParentClassName);
		if (const int32* RootClassIndex = ClassNameToRootIndex.Find(AssetNativeParentClassName))
		{
			const FTopLevelAssetPath StructAssetPath = FTopLevelAssetPath(AssetClassName);
			if (StructAssetPath.IsValid())
			{
				FRootClassContainer& RootClass = RootClasses[*RootClassIndex];
				int32 ClassDataIndex = RootClass.ClassData.IndexOfByPredicate([&StructAssetPath](const TSharedPtr<FStateTreeNodeClassData>& ClassData)
					{
						return ClassData->GetStructPath() == StructAssetPath;
					});

				UObject* AssetOb = AssetData.IsAssetLoaded() ? AssetData.GetAsset() : nullptr;
				UBlueprint* AssetBP = AssetOb ? Cast<UBlueprint>(AssetOb) : nullptr;
				UClass* AssetClass = AssetBP ? *AssetBP->GeneratedClass : AssetOb ? AssetOb->GetClass() : nullptr;
				if (ClassDataIndex == INDEX_NONE && AssetClass != nullptr)
				{
					// The path can change if the BP is renamed.
					const TWeakObjectPtr<UStruct> WeakAsset = AssetClass;
					ClassDataIndex = RootClass.ClassData.IndexOfByPredicate([WeakAsset](const TSharedPtr<FStateTreeNodeClassData>& ClassData)
						{
							return ClassData->GetWeakStruct() == WeakAsset;
						});
				}

				if (ClassDataIndex == INDEX_NONE)
				{
					if (AssetClass)
					{
						TSharedRef<FStateTreeNodeClassData> NewData = MakeShared<FStateTreeNodeClassData>(AssetClass);
						RootClass.ClassData.Add(NewData);
					}
					else
					{
						TSharedRef<FStateTreeNodeClassData> NewData = MakeShared<FStateTreeNodeClassData>(AssetData.GetSoftObjectPath().GetAssetPath());
						RootClass.ClassData.Add(NewData);
					}
				}
				else
				{
					// Update existing entry. That can occur if the asset got renamed.
					if (AssetClass)
					{
						TSharedRef<FStateTreeNodeClassData> NewData = MakeShared<FStateTreeNodeClassData>(AssetClass);
						RootClass.ClassData[ClassDataIndex] = NewData;
					}
				}
			}
		}
	}
}

void FStateTreeNodeClassCache::CacheClasses()
{
	for (TEnumerateRef<FRootClassContainer> RootClass : EnumerateRange(RootClasses))
	{
		CacheClasses(*RootClass);
	}
}

void FStateTreeNodeClassCache::CacheClasses(FRootClassContainer& RootClass)
{
	RootClass.ClassData.Reset();
	RootClass.bUpdated = true;

	const UStruct* BaseStruct = RootClass.BaseStruct.Get();

	// gather all native classes
	if (const UClass* Class = Cast<UClass>(BaseStruct))
	{
		for (TObjectIterator<UClass> It; It; ++It)
		{
			UClass* TestClass = *It;
			if (TestClass->HasAnyClassFlags(CLASS_Native) && TestClass->IsChildOf(Class))
			{
				RootClass.ClassData.Add(MakeShared<FStateTreeNodeClassData>(TestClass));
				ClassNameToRootIndex.Add(TestClass->GetPathName(), RootClass.Index);
			}
		}
	}
	else if (const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(BaseStruct))
	{
		for (TObjectIterator<UScriptStruct> It; It; ++It)
		{
			UScriptStruct* TestStruct = *It;
			if (TestStruct->IsChildOf(ScriptStruct))
			{
				RootClass.ClassData.Add(MakeShared<FStateTreeNodeClassData>(TestStruct));
				ClassNameToRootIndex.Add(TestStruct->GetPathName(), RootClass.Index);
			}
		}
	}

	// Gather all blueprints
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	TArray<FAssetData> BlueprintList;

	FARFilter Filter;
	Filter.ClassPaths.Add(UBlueprint::StaticClass()->GetClassPathName());
	AssetRegistryModule.Get().GetAssets(Filter, BlueprintList);

	for (const FAssetData& AssetData : BlueprintList)
	{
		UpdateBlueprintClass(AssetData);
	}
}

#undef LOCTEXT_NAMESPACE

