// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeObjectHash.h"
#include "UObject/UnrealType.h"

#if WITH_EDITORONLY_DATA

bool FStateTreeObjectCRC32::ShouldSkipProperty(const FProperty* InProperty) const
{
	if (InProperty == nullptr)
	{
		return false;
	}
	
	static const FName ExcludeFromHashName(TEXT("ExcludeFromHash"));
	const bool bExclude = InProperty->HasMetaData(ExcludeFromHashName);
	
	return FArchiveObjectCrc32::ShouldSkipProperty(InProperty) || InProperty->HasAllPropertyFlags(CPF_Transient) || bExclude;
}

#endif // WITH_EDITORONLY_DATA