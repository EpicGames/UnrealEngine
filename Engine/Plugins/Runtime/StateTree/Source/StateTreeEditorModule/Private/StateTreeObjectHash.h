// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/ArchiveObjectCrc32.h"

#if WITH_EDITORONLY_DATA

/**
 * Archive based object hashing to be used with StateTree data calculation.
 * If a property has "IncludeInHash" meta tag, any of its child properties will be included in the hash.
 * Editor only since it relies on meta data.
 */
class FStateTreeObjectCRC32 : public FArchiveObjectCrc32
{
public:

	virtual bool ShouldSkipProperty(const FProperty* InProperty) const override;
};

#endif // WITH_EDITORONLY_DATA