// Copyright Epic Games, Inc. All Rights Reserved.
#include "StateTreePropertyBindingCompiler.h"
#include "IPropertyAccessEditor.h"
#include "PropertyPathHelpers.h"
#include "StateTreeCompilerLog.h"
#include "StateTreeEditorPropertyBindings.h"
#include "Misc/EnumerateRange.h"
#include "StateTreePropertyBindings.h"
#include "StateTreePropertyRef.h"
#include "StateTreePropertyRefHelpers.h"
#include "StateTreePropertyHelpers.h"
#include "StateTreeDelegate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreePropertyBindingCompiler)

bool FStateTreePropertyBindingCompiler::Init(FStateTreePropertyBindings& InPropertyBindings, FStateTreeCompilerLog& InLog)
{
	Log = &InLog;
	PropertyBindings = &InPropertyBindings;
	PropertyBindings->Reset();

	SourceStructs.Reset();
	CompiledReferences.Reset();
	CompiledDelegateDispatchers.Reset();
	CompiledTaskCompletionDelegateDispatchers.Reset();

	return true;
}

bool FStateTreePropertyBindingCompiler::CompileBatch(const FStateTreeBindableStructDesc& TargetStruct, TConstArrayView<FStateTreePropertyPathBinding> BatchPropertyBindings, FStateTreeIndex16 PropertyFuncsBegin, FStateTreeIndex16 PropertyFuncsEnd, int32& OutBatchIndex)
{
	check(Log);
	check(PropertyBindings);
	OutBatchIndex = INDEX_NONE;

	StoreSourceStructs();

	struct FSortedBinding
	{
		FStateTreePropertyPathBinding Binding;
		TArray<FPropertyBindingPathIndirection> TargetIndirections;
	};
	TArray<FSortedBinding> NewBindings;

	for (const FStateTreePropertyPathBinding& Binding : BatchPropertyBindings)
	{
		if (Binding.GetTargetPath().GetStructID() != TargetStruct.ID)
		{
			continue;
		}
		// Source must be in the source array
		const FStateTreeBindableStructDesc* SourceStruct = GetSourceStructDescByID(Binding.GetSourcePath().GetStructID());
		if (!SourceStruct)
		{
			Log->Reportf(EMessageSeverity::Error, TargetStruct,
				TEXT("Could not find a binding source."));
			return false;
		}

		FString Error;
		TArray<FPropertyBindingPathIndirection> SourceIndirections;
		TArray<FPropertyBindingPathIndirection> TargetIndirections;
		
		if (!Binding.GetSourcePath().ResolveIndirections(SourceStruct->Struct, SourceIndirections, &Error))
		{
			Log->Reportf(EMessageSeverity::Error, TargetStruct, TEXT("Resolving path in %s: %s"), *SourceStruct->ToString(), *Error);
			return false;
		}

		if (!Binding.GetTargetPath().ResolveIndirections(TargetStruct.Struct, TargetIndirections, &Error))
		{
			Log->Reportf(EMessageSeverity::Error, TargetStruct, TEXT("Resolving path in %s: %s"), *TargetStruct.ToString(), *Error);
			return false;
		}

		FPropertyBindingCopyInfo DummyCopy;
		FPropertyBindingPathIndirection LastSourceIndirection = !SourceIndirections.IsEmpty() ? SourceIndirections.Last() : FPropertyBindingPathIndirection(SourceStruct->Struct);
		FPropertyBindingPathIndirection LastTargetIndirection = !TargetIndirections.IsEmpty() ? TargetIndirections.Last() : FPropertyBindingPathIndirection(TargetStruct.Struct);
		if (!PropertyBindings->ResolveBindingCopyInfo(Binding, LastSourceIndirection, LastTargetIndirection, DummyCopy))
		{
			Log->Reportf(EMessageSeverity::Error, TargetStruct,
			TEXT("Cannot copy properties between %s and %s, properties are incompatible."),
				*UE::StateTree::GetDescAndPathAsString(*SourceStruct, Binding.GetSourcePath()),
				*UE::StateTree::GetDescAndPathAsString(TargetStruct, Binding.GetTargetPath()));
			return false;
		}

		FSortedBinding& NewBinding = NewBindings.AddDefaulted_GetRef();
		NewBinding.Binding = FStateTreePropertyPathBinding(SourceStruct->DataHandle, Binding.GetSourcePath(), Binding.GetTargetPath(), Binding.IsOutputBinding());
		NewBinding.TargetIndirections = MoveTemp(TargetIndirections);
	}

	if (!NewBindings.IsEmpty())
	{
		// Sort bindings base on copy target memory layout.
		NewBindings.StableSort([](const FSortedBinding& A, const FSortedBinding& B)
		{
			const int32 MaxSegments = FMath::Min(A.TargetIndirections.Num(), B.TargetIndirections.Num());
			for (int32 Index = 0; Index < MaxSegments; Index++)
			{
				// If property A is in struct before B, copy A first. 
				if (A.TargetIndirections[Index].GetPropertyOffset() < B.TargetIndirections[Index].GetPropertyOffset())
				{
					return true;
				}
				if (A.TargetIndirections[Index].GetPropertyOffset() > B.TargetIndirections[Index].GetPropertyOffset())
				{
					return false;
				}

				// If A and B points to the same property, choose the one that points to an earlier array item.
				// Note: this assumes that INDEX_NONE = -1, which means that binding directly to an array comes before an array access,
				// and non-array access will compare equal (both INDEX_NONE).
				if (A.TargetIndirections[Index].GetArrayIndex() < B.TargetIndirections[Index].GetArrayIndex())
				{
					return true;
				}
				if (A.TargetIndirections[Index].GetArrayIndex() > B.TargetIndirections[Index].GetArrayIndex())
				{
					return false;
				}
			}

			// We get here if the common path is the same, shorter path wins.
			return A.TargetIndirections.Num() <= B.TargetIndirections.Num(); 
		});

		// Store bindings batch.
		const int32 BindingsBegin = PropertyBindings->PropertyPathBindings.Num();
		for (const FSortedBinding& NewBinding : NewBindings)
		{
			PropertyBindings->PropertyPathBindings.Add(NewBinding.Binding);
		}
		const int32 BindingsEnd = PropertyBindings->PropertyPathBindings.Num();

		FPropertyBindingCopyInfoBatch& Batch = PropertyBindings->AddCopyBatch();
		Batch.TargetStruct = TInstancedStruct<FStateTreeBindableStructDesc>::Make(TargetStruct);
		Batch.BindingsBegin = FPropertyBindingIndex16(BindingsBegin);
		Batch.BindingsEnd = FPropertyBindingIndex16(BindingsEnd);
		Batch.PropertyFunctionsBegin = PropertyFuncsBegin;
		Batch.PropertyFunctionsEnd = PropertyFuncsEnd;
		OutBatchIndex = PropertyBindings->GetNumCopyBatches() - 1;
	}

	return true;
}

bool FStateTreePropertyBindingCompiler::CompileDelegateDispatchers(
	const FStateTreeBindableStructDesc& SourceStruct,
	TConstArrayView<FStateTreeEditorDelegateDispatcherCompiledBinding> PreviousCompiledDistpatchers,
	TConstArrayView<FStateTreePropertyPathBinding> DelegateDispatcherBindings,
	FStateTreeDataView InstanceDataView)
{
	check(Log);
	check(PropertyBindings);

	StoreSourceStructs();

	bool bSuccess = true;
	for (const FStateTreePropertyPathBinding& Binding : DelegateDispatcherBindings)
	{
		if (Binding.GetSourcePath().GetStructID() != SourceStruct.ID)
		{
			continue;
		}

		// Source must be in the source array
		const FStateTreeBindableStructDesc* DispatcherStruct = GetSourceStructDescByID(Binding.GetSourcePath().GetStructID());
		if (!DispatcherStruct)
		{
			Log->Reportf(EMessageSeverity::Error, SourceStruct, TEXT("Could not find a binding source."));
			bSuccess = false;
			continue;
		}

		auto FindBySourcePathPredicate = [&Binding](const FStateTreeEditorDelegateDispatcherCompiledBinding& Other)
			{
				return Other.DispatcherPath == Binding.GetSourcePath();
			};

		if (!CompiledDelegateDispatchers.ContainsByPredicate(FindBySourcePathPredicate))
		{
			FString Error;
			TArray<FPropertyBindingPathIndirection> DispatcherIndirections;
			if (!Binding.GetSourcePath().ResolveIndirectionsWithValue(InstanceDataView, DispatcherIndirections, &Error))
			{
				Log->Reportf(EMessageSeverity::Error, SourceStruct, TEXT("Resolving path in %s: %s"), *DispatcherStruct->ToString(), *Error);
				bSuccess = false;
				continue;
			}

			const FPropertyBindingPathIndirection& DispatcherLeafIndirection = DispatcherIndirections.Last();
			const FStructProperty* LeafAsStructProperty = CastField<FStructProperty>(DispatcherLeafIndirection.GetProperty());
			if (LeafAsStructProperty == nullptr || LeafAsStructProperty->Struct != FStateTreeDelegateDispatcher::StaticStruct())
			{
				Log->Reportf(EMessageSeverity::Error, SourceStruct, TEXT("The source is not a valid delegate dispatcher."));
				bSuccess = false;
				continue;
			}

			if (DispatcherLeafIndirection.GetContainerAddress() == nullptr)
			{
				Log->Reportf(EMessageSeverity::Error, SourceStruct, TEXT("The dispatcher can't be initialized."));
				bSuccess = false;
				continue;
			}

			FStateTreeDelegateDispatcher* Dispatcher = reinterpret_cast<FStateTreeDelegateDispatcher*>(DispatcherLeafIndirection.GetMutablePropertyAddress());
			if (Dispatcher == nullptr)
			{
				Log->Reportf(EMessageSeverity::Error, SourceStruct, TEXT("The dispatcher can't be initialized."));
				bSuccess = false;
				continue;
			}

			if (const FStateTreeEditorDelegateDispatcherCompiledBinding* PreviousCompiled = PreviousCompiledDistpatchers.FindByPredicate(FindBySourcePathPredicate))
			{
				// Reuse previous ID
				*Dispatcher = PreviousCompiled->ID;
			}
			else
			{
				Dispatcher->ID = FGuid::NewGuid();
			}

			const bool bFoundID = CompiledDelegateDispatchers.ContainsByPredicate([ID = Dispatcher->ID](const FStateTreeEditorDelegateDispatcherCompiledBinding& Other)
				{
					return Other.ID.ID == ID;
				});
			ensureMsgf(!bFoundID, TEXT("The ID is already used by another delegate dispatcher."));
			CompiledDelegateDispatchers.Add({ Binding.GetSourcePath(), *Dispatcher });
		}
	}

	return bSuccess;
}

bool FStateTreePropertyBindingCompiler::CompileTaskCompletionDelegateDispatchers(const FStateTreeBindableStructDesc& SourceStruct, TConstArrayView<FStateTreePropertyPathBinding> Bindings)
{
	check(Log);
	check(PropertyBindings);

	StoreSourceStructs();

	bool bSuccess = true;
	for (const FStateTreePropertyPathBinding& Binding : Bindings)
	{
		if (Binding.GetSourcePath().GetStructID() != SourceStruct.ID)
		{
			continue;
		}

		if (Binding.IsTaskCompletionBinding())
		{
			// Source must be in the source array
			const FStateTreeBindableStructDesc* DispatcherStruct = GetSourceStructDescByID(Binding.GetSourcePath().GetStructID());
			if (!DispatcherStruct)
			{
				Log->Reportf(EMessageSeverity::Error, SourceStruct, TEXT("Could not find a binding source."));
				bSuccess = false;
				continue;
			}

			if (Binding.GetSourcePath().GetSegments().Num() != 0)
			{
				Log->Reportf(EMessageSeverity::Error, *DispatcherStruct, TEXT("Task completion delegate listener source path should have no segment pointing to the task."));
				bSuccess = false;
				continue;
			}

			// Add it once per condition/ID pair.
			const FGuid TaskNodeID = Binding.GetSourcePath().GetStructID();
			const UE::StateTree::ETaskCompletionCondition Condition = Binding.GetCompletionBinding().GetValue();
			if (!GetTaskCompletionDispatcher(TaskNodeID, Condition).IsValid())
			{
				FStateTreeDelegateDispatcher Dispatcher;
				Dispatcher.ID = FGuid::Combine(TaskNodeID, FGuid::NewDeterministicGuid(*UEnum::GetValueAsString(Condition)));
				CompiledTaskCompletionDelegateDispatchers.Emplace(TaskNodeID, Dispatcher, Condition);
			}
		}
	}

	return bSuccess;
}

bool FStateTreePropertyBindingCompiler::CompileDelegateListeners(
	const FStateTreeBindableStructDesc& TargetStruct,
	TConstArrayView<FStateTreePropertyPathBinding> DelegateSourceBindings,
	FStateTreeDataView InstanceDataView)
{
	check(Log);
	check(PropertyBindings);

	StoreSourceStructs();

	bool bSuccess = true;

	for (const FStateTreePropertyPathBinding& Binding : DelegateSourceBindings)
	{
		if (Binding.GetTargetPath().GetStructID() != TargetStruct.ID)
		{
			continue;
		}

		// Source must be in the source array
		const FStateTreeBindableStructDesc* DispatcherStruct = GetSourceStructDescByID(Binding.GetSourcePath().GetStructID());
		if (!DispatcherStruct)
		{
			Log->Reportf(EMessageSeverity::Error, TargetStruct, TEXT("Could not find a binding source."));
			bSuccess = false;
			continue;
		}

		FString Error;
		TArray<FPropertyBindingPathIndirection> ListenerIndirections;
		if (!Binding.GetTargetPath().ResolveIndirectionsWithValue(InstanceDataView, ListenerIndirections, &Error))
		{
			Log->Reportf(EMessageSeverity::Error, TargetStruct, TEXT("Resolving path in %s: %s"), *TargetStruct.ToString(), *Error);
			bSuccess = false;
			continue;
		}

		FStateTreeDelegateDispatcher Dispatcher;
		if (Binding.IsTaskCompletionBinding())
		{
			const FGuid TaskNodeID = Binding.GetSourcePath().GetStructID();
			const UE::StateTree::ETaskCompletionCondition Condition = Binding.GetCompletionBinding().GetValue();
			Dispatcher = GetTaskCompletionDispatcher(TaskNodeID, Condition);

			if (!Dispatcher.IsValid())
			{
				Log->Reportf(EMessageSeverity::Error, TargetStruct, TEXT("Delegate Listener %s is bound to an unknown task %s"), *TargetStruct.ToString(), *DispatcherStruct->ToString());
				bSuccess = false;
				continue;
			}
		}
		else
		{
			Dispatcher = GetDispatcherFromPath(Binding.GetSourcePath());
			if (!Dispatcher.IsValid())
			{
				Log->Reportf(EMessageSeverity::Error, TargetStruct, TEXT("Delegate Listener %s is bound to an unknown dispatcher %s"), *TargetStruct.ToString(), *DispatcherStruct->ToString());
				bSuccess = false;
				continue;
			}
		}

		FPropertyBindingPathIndirection& ListenerLeafIndirection = ListenerIndirections.Last();
		const FStructProperty* LeafAsStructProperty = CastField<FStructProperty>(ListenerLeafIndirection.GetProperty());
		check(LeafAsStructProperty && LeafAsStructProperty->Struct == FStateTreeDelegateListener::StaticStruct());
		if (LeafAsStructProperty == nullptr || LeafAsStructProperty->Struct != FStateTreeDelegateListener::StaticStruct())
		{
			Log->Reportf(EMessageSeverity::Error, TargetStruct, TEXT("The target is not a valid delegate listener."));
			bSuccess = false;
			continue;
		}

		FStateTreeDelegateListener* Listener = reinterpret_cast<FStateTreeDelegateListener*>(ListenerLeafIndirection.GetMutablePropertyAddress());
		Listener->Dispatcher = Dispatcher;
		Listener->ID = ++ListenersNum;
	}

	return bSuccess;
}

bool FStateTreePropertyBindingCompiler::CompileReferences(const FStateTreeBindableStructDesc& TargetStruct, TConstArrayView<FStateTreePropertyPathBinding> PropertyReferenceBindings, FStateTreeDataView InstanceDataView, const TMap<FGuid, const FStateTreeDataView>& IDToStructValue)
{
	for (const FStateTreePropertyPathBinding& Binding : PropertyReferenceBindings)
	{
		if (Binding.GetTargetPath().GetStructID() != TargetStruct.ID)
		{
			continue;
		}

		// Source must be in the source array
		const FStateTreeBindableStructDesc* SourceStruct = GetSourceStructDescByID(Binding.GetSourcePath().GetStructID());
		if (!SourceStruct)
		{
			Log->Reportf(EMessageSeverity::Error, TargetStruct,
				TEXT("Could not find a binding source."));
			return false;
		}

		const FStateTreeDataView* SourceDataView = IDToStructValue.Find(Binding.GetSourcePath().GetStructID());
		if (!SourceDataView)
		{
			Log->Reportf(EMessageSeverity::Error, TargetStruct,
				TEXT("Could not find a binding source data view."));
			return false;
		}

		FString Error;
		TArray<FPropertyBindingPathIndirection> SourceIndirections;
		if (!Binding.GetSourcePath().ResolveIndirectionsWithValue(*SourceDataView, SourceIndirections, &Error))
		{
			Log->Reportf(EMessageSeverity::Error, TargetStruct, TEXT("Resolving path in %s: %s"), *SourceStruct->ToString(), *Error);
			return false;
		}

		if (!UE::StateTree::PropertyRefHelpers::IsPropertyAccessibleForPropertyRef(SourceIndirections, *SourceStruct))
		{
			Log->Reportf(EMessageSeverity::Error, TargetStruct,
					TEXT("%s cannot reference non-output or property function output %s "),
					*UE::StateTree::GetDescAndPathAsString(TargetStruct, Binding.GetTargetPath()),
					*UE::StateTree::GetDescAndPathAsString(*SourceStruct, Binding.GetSourcePath()));
			return false;
		}

		TArray<FPropertyBindingPathIndirection> TargetIndirections;
		if (!Binding.GetTargetPath().ResolveIndirectionsWithValue(InstanceDataView, TargetIndirections, &Error))
		{
			Log->Reportf(EMessageSeverity::Error, TargetStruct, TEXT("Resolving path in %s: %s"), *TargetStruct.ToString(), *Error);
			return false;
		}

		FPropertyBindingPathIndirection& TargetLeafIndirection = TargetIndirections.Last();
		FStateTreePropertyRef* PropertyRef = static_cast<FStateTreePropertyRef*>(const_cast<void*>(TargetLeafIndirection.GetPropertyAddress()));
		check(PropertyRef);

		FPropertyBindingPathIndirection& SourceLeafIndirection = SourceIndirections.Last();
		if (!UE::StateTree::PropertyRefHelpers::IsPropertyRefCompatibleWithProperty(*TargetLeafIndirection.GetProperty(), *SourceLeafIndirection.GetProperty(), PropertyRef, SourceLeafIndirection.GetPropertyAddress()))
		{
			Log->Reportf(EMessageSeverity::Error, TargetStruct,
				TEXT("%s cannot reference %s, types are incompatible."),		
				*UE::StateTree::GetDescAndPathAsString(TargetStruct, Binding.GetTargetPath()),
				*UE::StateTree::GetDescAndPathAsString(*SourceStruct, Binding.GetSourcePath()));
			return false;
		}

		FStateTreeIndex16 ReferenceIndex;

		// Reuse the index if another PropertyRef already references the same property.
		{
			int32 IndexOfAlreadyExisting = PropertyBindings->PropertyReferencePaths.IndexOfByPredicate([&Binding](const FStateTreePropertyRefPath& RefPath)
			{
				return RefPath.GetSourcePath() == Binding.GetSourcePath();
			});

			if (IndexOfAlreadyExisting != INDEX_NONE)
			{
				ReferenceIndex = FStateTreeIndex16(IndexOfAlreadyExisting);
			}
		}

		if (!ReferenceIndex.IsValid() && UE::StateTree::PropertyRefHelpers::IsPropertyRef(*SourceIndirections.Last().GetProperty()))
		{
			if (SourceStruct->DataHandle.GetSource() == EStateTreeDataSourceType::ExternalGlobalParameterData)
			{
				Log->Reportf(EMessageSeverity::Error, TargetStruct, 
					TEXT("ExternalGlobalParameter doesn't support chained Property Reference from %s to %s."), 
					*UE::StateTree::GetDescAndPathAsString(*SourceStruct, Binding.GetSourcePath()), 
					*UE::StateTree::GetDescAndPathAsString(TargetStruct, Binding.GetTargetPath()));

				return false;
			}

			// If referencing another non-global or non-subtree parameter PropertyRef, reuse it's index.
			if (SourceStruct->DataHandle.GetSource() != EStateTreeDataSourceType::GlobalParameterData
				&& SourceStruct->DataHandle.GetSource() != EStateTreeDataSourceType::SubtreeParameterData)
			{
				const FCompiledReference* ReferencedReference = CompiledReferences.FindByPredicate([&Binding](const FCompiledReference& CompiledReference)
				{
					return CompiledReference.Path == Binding.GetSourcePath();
				});

				if (ReferencedReference)
				{
					ReferenceIndex = ReferencedReference->Index;
				}
				else
				{
					if(!UE::StateTree::PropertyHelpers::HasOptionalMetadata(*TargetLeafIndirection.GetProperty()))
					{
						Log->Reportf(EMessageSeverity::Error, TargetStruct, TEXT("Referenced %s is not bound"), *UE::StateTree::GetDescAndPathAsString(*SourceStruct, Binding.GetSourcePath()));
						return false;
					}
							
					return true;
				}
			}
		}

		if (!ReferenceIndex.IsValid())
		{
			ReferenceIndex = FStateTreeIndex16(PropertyBindings->PropertyReferencePaths.Num());
			PropertyBindings->PropertyReferencePaths.Emplace(SourceStruct->DataHandle, Binding.GetSourcePath());
		}

		// Store index in instance data.
		PropertyRef->RefAccessIndex = ReferenceIndex;

		FCompiledReference& CompiledReference = CompiledReferences.AddDefaulted_GetRef();
		CompiledReference.Path = Binding.GetTargetPath();
		CompiledReference.Index = ReferenceIndex;
	}

	return true;
}

void FStateTreePropertyBindingCompiler::Finalize()
{
	StoreSourceStructs();
}

int32 FStateTreePropertyBindingCompiler::AddSourceStruct(const FStateTreeBindableStructDesc& SourceStruct)
{
	const FStateTreeBindableStructDesc* ExistingStruct = SourceStructs.FindByPredicate([&SourceStruct](const FStateTreeBindableStructDesc& Struct) { return (Struct.ID == SourceStruct.ID); });
	if (ExistingStruct)
	{
		UE_LOGF(LogStateTree, Error, "%ls already exists as %ls using ID '%ls'",
			*SourceStruct.ToString(), *ExistingStruct->ToString(), *ExistingStruct->ID.ToString());
	}

	if (SourceStruct.Struct && !SourceStruct.Struct->IsChildOf<FStateTreeNodeBase>())
	{
		UE_CLOGF(!SourceStruct.DataHandle.IsValid(), LogStateTree, Error, "%ls does not have a valid data handle.", *SourceStruct.ToString())
	}
	
	SourceStructs.Add(SourceStruct);
	return SourceStructs.Num() - 1;
}

FStateTreeDelegateDispatcher FStateTreePropertyBindingCompiler::GetDispatcherFromPath(const FPropertyBindingPath& PathToDispatcher) const
{
	const FStateTreeEditorDelegateDispatcherCompiledBinding* FoundDispatcher = CompiledDelegateDispatchers.FindByPredicate(
		[&PathToDispatcher](const FStateTreeEditorDelegateDispatcherCompiledBinding& Dispatcher)
		{
			return PathToDispatcher == Dispatcher.DispatcherPath;
		});
	if (FoundDispatcher)
	{
		return FoundDispatcher->ID;
	}

	return FStateTreeDelegateDispatcher();
}

FStateTreeDelegateDispatcher FStateTreePropertyBindingCompiler::GetTaskCompletionDispatcher(FGuid TaskNodeID, UE::StateTree::ETaskCompletionCondition Condition) const
{
	const FTaskCompletionDispatcher* FoundDispatcher = CompiledTaskCompletionDelegateDispatchers.FindByPredicate([TaskNodeID, Condition](const FTaskCompletionDispatcher& Other)
		{
			return Other.TaskNodeID == TaskNodeID
				&& Other.Condition == Condition;
		});

	return FoundDispatcher ? FoundDispatcher->Dispatcher : FStateTreeDelegateDispatcher();
}

FStateTreeDelegateDispatcher FStateTreePropertyBindingCompiler::GetDispatcherFromBinding(const FStateTreePropertyPathBinding& Binding) const
{
	if (Binding.IsTaskCompletionBinding())
	{
		return GetTaskCompletionDispatcher(Binding.GetSourcePath().GetStructID(), Binding.GetCompletionBinding().GetValue());
	}
	else
	{
		return GetDispatcherFromPath(Binding.GetSourcePath());
	}
}

TConstArrayView<FStateTreeEditorDelegateDispatcherCompiledBinding> FStateTreePropertyBindingCompiler::GetCompiledDelegateDispatchers() const
{
	return CompiledDelegateDispatchers;
}

TConstArrayView<FStateTreePropertyBindingCompiler::FTaskCompletionDispatcher> FStateTreePropertyBindingCompiler::GetCompiledTaskCompletionDispatchers() const
{
	return CompiledTaskCompletionDelegateDispatchers;
}

void FStateTreePropertyBindingCompiler::StoreSourceStructs()
{
	// Check that existing structs are compatible
	check(PropertyBindings->SourceStructs.Num() <= SourceStructs.Num());
	for (int32 i = 0; i < PropertyBindings->SourceStructs.Num(); i++)
	{
		check(PropertyBindings->SourceStructs[i] == SourceStructs[i]);
	}

	// Add new
	if (SourceStructs.Num() > PropertyBindings->SourceStructs.Num())
	{
		for (int32 i = PropertyBindings->SourceStructs.Num(); i < SourceStructs.Num(); i++)
		{
			PropertyBindings->SourceStructs.Add(SourceStructs[i]);
		}
	}
}
