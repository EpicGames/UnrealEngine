// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreePropertyHelpers.h"
#include "StateTreeEditorNode.h"
#include "ScopedTransaction.h"
#include "Hash/Blake3.h"
#include "Misc/EnumerateRange.h"
#include "Misc/StringBuilder.h"
#include "String/ParseTokens.h"
#include "UObject/Field.h"
#include "StateTreePropertyBindings.h"
#include "StateTreeEditorData.h"

namespace UE::StateTree::PropertyHelpers
{
namespace Internal
{
	void DispatchPostEditToEditorNode(FPropertyChangedChainEvent& InPropertyChangedEvent, const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* InEditorNodeInChain, FStateTreeEditorNode& InEditorNode)
	{
		if (FStateTreeNodeBase* StateTreeNode = InEditorNode.GetNode().GetPtr<FStateTreeNodeBase>())
		{
			// Check that the path contains EditorNode's: Node, Instance or Instance Object
			if (const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* EditorNodeMemberPropNode = InEditorNodeInChain->GetNextNode())
			{
				// Check that we have a changed property on one of the above properties.
				if (const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* ActiveMemberPropNode = EditorNodeMemberPropNode->GetNextNode()) 
				{
					// Update the event
					const FProperty* EditorNodeChildMember = EditorNodeMemberPropNode->GetValue();
					check(EditorNodeChildMember);

					// Take copy of the event, we'll modify it.
					FEditPropertyChain PropertyChainCopy;
					for (const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* Node = InPropertyChangedEvent.PropertyChain.GetHead(); Node; Node = Node->GetNextNode())
					{
						PropertyChainCopy.AddTail(Node->GetValue());
					}
					FPropertyChangedChainEvent PropertyChangedEvent(PropertyChainCopy, InPropertyChangedEvent);

					PropertyChangedEvent.SetActiveMemberProperty(ActiveMemberPropNode->GetValue());
					PropertyChangedEvent.PropertyChain.SetActiveMemberPropertyNode(PropertyChangedEvent.MemberProperty);

					// To be consistent with the other property chain callbacks, do not cross object boundary.
					const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* ActivePropNode = ActiveMemberPropNode;
					while (ActivePropNode->GetNextNode())
					{
						if (CastField<FObjectProperty>(ActivePropNode->GetValue()))
						{
							break;
						}
						ActivePropNode = ActivePropNode->GetNextNode();
					}
							
					PropertyChangedEvent.Property = ActivePropNode->GetValue();
					PropertyChangedEvent.PropertyChain.SetActivePropertyNode(PropertyChangedEvent.Property);

					if (EditorNodeChildMember->GetFName() == GET_MEMBER_NAME_CHECKED(FStateTreeEditorNode, Node))
					{
						StateTreeNode->PostEditNodeChangeChainProperty(PropertyChangedEvent, InEditorNode.GetInstance());
					}
					else if (EditorNodeChildMember->GetFName() == GET_MEMBER_NAME_CHECKED(FStateTreeEditorNode, Instance)
						|| EditorNodeChildMember->GetFName() == GET_MEMBER_NAME_CHECKED(FStateTreeEditorNode, InstanceObject))
					{
						if (InEditorNode.GetInstance().IsValid())
						{
							StateTreeNode->PostEditInstanceDataChangeChainProperty(PropertyChangedEvent, InEditorNode.GetInstance());
						}
					}
					else if (EditorNodeChildMember->GetFName() == GET_MEMBER_NAME_CHECKED(FStateTreeEditorNode, ExecutionRuntimeData)
						|| EditorNodeChildMember->GetFName() == GET_MEMBER_NAME_CHECKED(FStateTreeEditorNode, ExecutionRuntimeDataObject))
					{
						if (InEditorNode.GetExecutionRuntimeData().IsValid())
						{
							StateTreeNode->PostEditInstanceDataChangeChainProperty(PropertyChangedEvent, InEditorNode.GetExecutionRuntimeData());
						}
					}
				}
			}
		}
	}
}


void DispatchPostEditToNodes(UObject& Owner, FPropertyChangedChainEvent& InPropertyChangedEvent, UStateTreeEditorData& EditorData)
{
	// Walk through changed property chain and look for first FStateTreeEditorNode, and call the node specific post edit methods.
	
	const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* CurrentPropNode = InPropertyChangedEvent.PropertyChain.GetHead();
	const FProperty* HeadProperty = CurrentPropNode->GetValue();
	check(HeadProperty);
	if (HeadProperty->GetOwnerClass() != Owner.GetClass())
	{
		return;
	}
	
	FStateTreeEditorNode* LastEditorNode = nullptr;
	const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* LastEditorNodeInChain = nullptr;

	uint8* CurrentAddress = reinterpret_cast<uint8*>(&Owner);
	FPropertyBindingPath TargetPath;
	while (CurrentPropNode)
	{
		const FProperty* CurrentProperty = CurrentPropNode->GetValue();
		check(CurrentProperty);
		CurrentAddress = CurrentAddress + CurrentProperty->GetOffset_ForInternal();

		while (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(CurrentProperty))
		{
			FScriptArrayHelper Helper(ArrayProperty, CurrentAddress);
			const int32 Index = InPropertyChangedEvent.GetArrayIndex(ArrayProperty->GetName());
			if (!Helper.IsValidIndex(Index))
			{
				// It's possible that a property function node is bound directly to an array property
				// In which case the there is no index for the array property while the property chain is still going.
				// We will instead continue traversing the property chain and fetch the property function node property
				break;
			}

			if (TargetPath.GetStructID().IsValid())
			{
				// Skip adding segment for array property itself if we are not modifying its element,
				// As we will not indirect to the inner property and will add segment for array property at the end of the loop
				TargetPath.AddPathSegment(ArrayProperty->GetFName(), Index);
			}

			// Only indirect ArrayProperty to Inner if we are modifying array element
			CurrentAddress = Helper.GetRawPtr(Index);
			CurrentProperty = ArrayProperty->Inner;
		}

		FPropertyBindingPathSegment PathSegment(CurrentProperty->GetFName());
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(CurrentProperty))
		{
			if (StructProperty->Struct == FInstancedStruct::StaticStruct())
			{
				FInstancedStruct& InstancedStruct = *reinterpret_cast<FInstancedStruct*>(CurrentAddress);
				CurrentAddress = InstancedStruct.GetMutableMemory();

				PathSegment.SetInstanceStruct(InstancedStruct.GetScriptStruct());
			}
			else if (StructProperty->Struct == FStateTreeEditorNode::StaticStruct())
			{
				if (TargetPath.GetStructID().IsValid())
				{
					FPropertyBindingBinding* FoundBinding = EditorData.GetPropertyEditorBindings()->GetMutableBindings().FindByPredicate([&TargetPath](const FPropertyBindingBinding& Binding)
					{
						return TargetPath == Binding.GetTargetPath();
					});

					if (!ensure(FoundBinding && FoundBinding->GetPropertyFunctionNode().IsValid()))
					{
						return;
					}

					CurrentAddress = FoundBinding->GetMutablePropertyFunctionNode().GetMemory();
					TargetPath.Reset();
				}

				LastEditorNode = reinterpret_cast<FStateTreeEditorNode*>(CurrentAddress);
				LastEditorNodeInChain = CurrentPropNode;
				TargetPath.SetStructID(LastEditorNode->ID);

				CurrentPropNode = CurrentPropNode->GetNextNode();
				if (CurrentPropNode)
				{
					const FName EditorNodeChildMemberName = CurrentPropNode->GetValue()->GetFName();
					if (EditorNodeChildMemberName == GET_MEMBER_NAME_CHECKED(FStateTreeEditorNode, Instance)
						|| EditorNodeChildMemberName == GET_MEMBER_NAME_CHECKED(FStateTreeEditorNode, InstanceObject))
					{
						CurrentAddress = static_cast<uint8*>(LastEditorNode->GetInstance().GetMutableMemory());
						CurrentPropNode = CurrentPropNode->GetNextNode();
						continue;
					}
					if (EditorNodeChildMemberName == GET_MEMBER_NAME_CHECKED(FStateTreeEditorNode, ExecutionRuntimeData)
						|| EditorNodeChildMemberName == GET_MEMBER_NAME_CHECKED(FStateTreeEditorNode, ExecutionRuntimeDataObject))
					{
						CurrentAddress = static_cast<uint8*>(LastEditorNode->GetExecutionRuntimeData().GetMutableMemory());
						CurrentPropNode = CurrentPropNode->GetNextNode();
						continue;
					}
				}

				break;
			}
			else if (StructProperty->Struct == FStateTreeStateParameters::StaticStruct())
			{
				FStateTreeStateParameters& StateParameters = *reinterpret_cast<FStateTreeStateParameters*>(CurrentAddress);
				check(!TargetPath.GetStructID().IsValid());
				TargetPath.SetStructID(StateParameters.ID);

				CurrentPropNode = CurrentPropNode->GetNextNode();
				if (CurrentPropNode && CurrentPropNode->GetValue()->GetFName() == GET_MEMBER_NAME_CHECKED(FStateTreeStateParameters, Parameters))
				{
					CurrentPropNode = CurrentPropNode->GetNextNode();
					if (CurrentPropNode && CurrentPropNode->GetValue()->GetFName() == TEXT("Value"))
					{
						CurrentAddress = StateParameters.Parameters.GetMutableValue().GetMemory();
						CurrentPropNode = CurrentPropNode->GetNextNode();
						continue;
					}
				}

				return;
			}
		}
		else if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(CurrentProperty))
		{
			if (!TargetPath.GetStructID().IsValid())
			{
				return;
			}

			if (UObject* Object = *reinterpret_cast<UObject**>(CurrentAddress))
			{
				CurrentAddress = reinterpret_cast<uint8*>(Object);
				PathSegment.SetInstanceStruct(Object->GetClass(), EPropertyBindingPropertyAccessType::ObjectInstance);
			}
			else
			{
				break;
			}
		}

		if (TargetPath.GetStructID().IsValid())
		{
			TargetPath.AddPathSegment(PathSegment);
		}

		CurrentPropNode = CurrentPropNode->GetNextNode();
	}

	if (LastEditorNode && LastEditorNodeInChain)
	{
		Internal::DispatchPostEditToEditorNode(InPropertyChangedEvent, LastEditorNodeInChain, *LastEditorNode);
	}
}

void ModifyStateInPreAndPostEdit(
	const FText& TransactionDescription, 
	TNotNull<UStateTreeState*> State,
	TNotNull<UStateTreeEditorData*> EditorData,
	FStringView RelativeNodePath, 
	TFunctionRef<void(TNotNull<UStateTreeState*> OwnerState, TNotNull<UStateTreeEditorData*> EditorData, const FStateTreeEditPropertyPath& PropertyPath)> Func,
	int32 ArrayIndex, 
	EPropertyChangeType::Type ChangeType)
{
	FScopedTransaction ScopedTransaction(TransactionDescription);

	FEditPropertyChain PropertyChain;

	FStateTreeEditPropertyPath PropertyPath = FStateTreeEditPropertyPath(State->GetClass(), RelativeNodePath);
	PropertyPath.MakeEditPropertyChain(PropertyChain);

	State->PreEditChange(PropertyChain);

	Func(State, EditorData, PropertyPath);

	FProperty* ActiveNode = PropertyChain.GetActiveNode()->GetValue();
	TArray<TMap<FString, int32>> ArrayIndicesPerObject;

	if (ArrayIndex != INDEX_NONE)
	{
		ArrayIndicesPerObject.Add(TMap<FString, int32>());
		ArrayIndicesPerObject[0].Add(ActiveNode->GetName(), ArrayIndex);
	}

	FPropertyChangedEvent ChangedEvent(PropertyChain.GetActiveNode()->GetValue(), ChangeType);
	ChangedEvent.SetArrayIndexPerObject(ArrayIndicesPerObject);

	// Set ObjectIteratorIndex to index we used above as FPropertyChangedEvent::GetArrayIndex uses that to read the array indexes.
	if (ArrayIndex != INDEX_NONE)
	{
		ChangedEvent.ObjectIteratorIndex = 0;
	}

	FPropertyChangedChainEvent ChainEvent(PropertyChain, ChangedEvent);
	State->PostEditChangeChainProperty(ChainEvent);
}

FGuid MakeDeterministicID(const UObject& Owner, const FString& PropertyPath, const uint64 Seed)
{
	// From FGuid::NewDeterministicGuid(FStringView ObjectPath, uint64 Seed)
	
	// Convert the objectpath to utf8 so that whether TCHAR is UTF8 or UTF16 does not alter the hash.
	TUtf8StringBuilder<1024> Utf8ObjectPath(InPlace, Owner.GetPathName());
	TUtf8StringBuilder<1024> Utf8PropertyPath(InPlace, PropertyPath);

	FBlake3 Builder;

	// Hash this as the namespace of the Version 3 UUID, to avoid collisions with any other guids created using Blake3.
	static FGuid BaseVersion(TEXT("bf324a38-a445-45a4-8921-249554b58189"));
	Builder.Update(&BaseVersion, sizeof(FGuid));
	Builder.Update(Utf8ObjectPath.GetData(), Utf8ObjectPath.Len() * sizeof(UTF8CHAR));
	Builder.Update(Utf8PropertyPath.GetData(), Utf8PropertyPath.Len() * sizeof(UTF8CHAR));
	Builder.Update(&Seed, sizeof(Seed));

	const FBlake3Hash Hash = Builder.Finalize();

	return FGuid::NewGuidFromHash(Hash);
}

bool HasOptionalMetadata(const FProperty& Property)
{
	return Property.HasMetaData(TEXT("Optional"));
}

}; // UE::StateTree::PropertyHelpers

// ------------------------------------------------------------------------------
// FStateTreeEditPropertyPath
// ------------------------------------------------------------------------------
FStateTreeEditPropertyPath::FStateTreeEditPropertyPath(const UStruct* BaseStruct, FStringView InPath)
{
	TArray<FStringView> PathSegments;
	UE::String::ParseTokens(InPath, TEXT("."), PathSegments, UE::String::EParseTokensOptions::SkipEmpty);

	const UStruct* CurrBase = BaseStruct;
	for (FStringView Segment : PathSegments)
	{
		const FName PropertyName(Segment);
		if (FProperty* Property = CurrBase->FindPropertyByName(PropertyName))
		{
			Path.Emplace(Property, PropertyName);

			if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
			{
				Property = ArrayProperty->Inner;
			}

			if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				CurrBase = StructProperty->Struct;
			}
			else if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
			{
				CurrBase = ObjectProperty->PropertyClass;
			}
		}
		else
		{
			checkf(false, TEXT("Path %s id not part of type %s."), InPath.GetData(), *GetNameSafe(BaseStruct));
			Path.Reset();
			break;
		}
	}
}

FStateTreeEditPropertyPath::FStateTreeEditPropertyPath(const FPropertyChangedChainEvent& PropertyChangedEvent)
{
	FEditPropertyChain::TDoubleLinkedListNode* PropertyNode = PropertyChangedEvent.PropertyChain.GetActiveMemberNode();
	while (PropertyNode != nullptr)
	{
		if (FProperty* Property = PropertyNode->GetValue())
		{
			const FName PropertyName = Property->GetFName(); 
			const int32 ArrayIndex = PropertyChangedEvent.GetArrayIndex(PropertyName.ToString());
			Path.Emplace(Property, PropertyName, ArrayIndex);
		}
		PropertyNode = PropertyNode->GetNextNode();
	}
}

FStateTreeEditPropertyPath::FStateTreeEditPropertyPath(const FEditPropertyChain& PropertyChain)
{
	FEditPropertyChain::TDoubleLinkedListNode* PropertyNode = PropertyChain.GetActiveMemberNode();
	while (PropertyNode != nullptr)
	{
		if (FProperty* Property = PropertyNode->GetValue())
		{
			const FName PropertyName = Property->GetFName(); 
			Path.Emplace(Property, PropertyName, INDEX_NONE);
		}
		PropertyNode = PropertyNode->GetNextNode();
	}
}

void FStateTreeEditPropertyPath::MakeEditPropertyChain(FEditPropertyChain& OutPropertyChain) const
{
	OutPropertyChain.Empty();

	for (const FStateTreeEditPropertySegment& Segment : Path)
	{
		OutPropertyChain.AddTail(Segment.Property);
	}

	OutPropertyChain.SetActiveMemberPropertyNode(Path[0].Property);
}

bool FStateTreeEditPropertyPath::ContainsPath(const FStateTreeEditPropertyPath& InPath) const
{
	if (InPath.Path.Num() > Path.Num())
    {
    	return false;
    }

    for (TConstEnumerateRef<FStateTreeEditPropertySegment> Segment : EnumerateRange(InPath.Path))
    {
    	if (Segment->PropertyName != Path[Segment.GetIndex()].PropertyName)
    	{
    		return false;
    	}
    }
    return true;
}

/** @return true if the property path is exactly the specified path. */
bool FStateTreeEditPropertyPath::IsPathExact(const FStateTreeEditPropertyPath& InPath) const
{
	if (InPath.Path.Num() != Path.Num())
	{
		return false;
	}

	for (TConstEnumerateRef<FStateTreeEditPropertySegment> Segment : EnumerateRange(InPath.Path))
	{
		if (Segment->PropertyName != Path[Segment.GetIndex()].PropertyName)
		{
			return false;
		}
	}
	return true;
}
