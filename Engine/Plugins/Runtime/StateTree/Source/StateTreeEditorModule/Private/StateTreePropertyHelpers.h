// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotNull.h"
#include "Editor.h"
#include "PropertyHandle.h"

struct FStateTreeEditorNode;
struct FGuid;
struct FSlateBrush;
struct FStateTreeEditPropertyPath;

class UStateTreeEditorData;
class UStateTreeState;

#define LOCTEXT_NAMESPACE "StateTreeEditor"

namespace UE::StateTree::PropertyHelpers {

/**
 * Dispatches PostEditChange to all FState
 * Assumes property chain head is member property of Owner. 
 */
void DispatchPostEditToNodes(UObject& Owner, FPropertyChangedChainEvent& PropertyChangedEvent, UStateTreeEditorData& EditorData);

/**
 * Modify a StateTreeState in PreEdit and PostEdit callbacks in a transaction.
 */
void ModifyStateInPreAndPostEdit(
	const FText& TransactionDescription,
	TNotNull<UStateTreeState*> State,
	TNotNull<UStateTreeEditorData*> EditorData,
	FStringView RelativeNodePath,
	TFunctionRef<void(TNotNull<UStateTreeState*>, TNotNull<UStateTreeEditorData*>, const FStateTreeEditPropertyPath&)> Func,
	int32 ArrayIndex = INDEX_NONE,
	EPropertyChangeType::Type ChangeType = EPropertyChangeType::ValueSet);

/** Makes deterministic ID from the owners property path, a property path (or any string), and a seed value (e.g. array index). */
FGuid MakeDeterministicID(const UObject& Owner, const FString& PropertyPath, const uint64 Seed);

/* @return true if the property handle points to struct property of specified type.*/
template<typename T>
bool IsScriptStruct(const TSharedPtr<IPropertyHandle>& PropertyHandle)
{
	if (!PropertyHandle)
	{
		return false;
	}

	FProperty* Property = PropertyHandle->GetProperty();
	if (!Property)
	{
		return false;
	}

	FStructProperty* StructProperty = CastField<FStructProperty>(Property);
	return StructProperty && StructProperty->Struct->IsA(TBaseStructure<T>::Get()->GetClass());
}
/**
 * @return true if provided Property contains "Optional" metadata
 */
bool HasOptionalMetadata(const FProperty& Property);

/**
 * Gets a struct value from property handle, checks type before access. Expects T is struct.
 * @param ValueProperty Handle to property where value is got from.
 * @return Requested value as optional, in case of multiple values the optional is unset.
 */
template<typename T>
FPropertyAccess::Result GetStructValue(const TSharedPtr<const IPropertyHandle>& ValueProperty, T& OutValue)
{
	if (!ValueProperty)
	{
		return FPropertyAccess::Fail;
	}

	FProperty* Property = ValueProperty->GetProperty();
	if (!Property)
	{
		return FPropertyAccess::Fail;
	}

	FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(Property);
	check(StructProperty);
	check(StructProperty->Struct == TBaseStructure<T>::Get());

	T Value = T();
	bool bValueSet = false;

	TArray<const void*> RawData;
	ValueProperty->AccessRawData(RawData);
	for (const void* Data : RawData)
	{
		if (Data)
		{
			const T& CurValue = *static_cast<const T*>(Data);
			if (!bValueSet)
			{
				bValueSet = true;
				Value = CurValue;
			}
			else if (CurValue != Value)
			{
				// Multiple values
				return FPropertyAccess::MultipleValues;
			}
		}
	}

	OutValue = Value;

	return FPropertyAccess::Success;
}

/**
 * Returns const pointer to struct container in the property.
 * @param ValueProperty Handle to property where value is got from.
 * @return Pointer to the struct, or nullptr if type does not match or multiple values.
 */
template<typename T>
const T* GetStructPtr(const TSharedPtr<const IPropertyHandle>& ValueProperty)
{
	if (!ValueProperty)
	{
		return nullptr;
	}

	if (FProperty* Property = ValueProperty->GetProperty())
	{
		FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(Property);
		check(StructProperty);
		check(StructProperty->Struct == TBaseStructure<T>::Get());

		TArray<const void*> RawData;
		ValueProperty->AccessRawData(RawData);
		if (RawData.Num() == 1)
		{
			return static_cast<const T*>(RawData[0]);
		}
	}

	return nullptr;
}

/**
 * Sets a struct property to specific value, checks type before access. Expects T is struct.
 * @param ValueProperty Handle to property where value is got from.
 * @return Requested value as optional, in case of multiple values the optional is unset.
 */
template<typename T>
FPropertyAccess::Result SetStructValue(const TSharedPtr<IPropertyHandle>& ValueProperty, const T& NewValue, EPropertyValueSetFlags::Type Flags = 0)
{
	if (!ValueProperty)
	{
		return FPropertyAccess::Fail;
	}

	FProperty* Property = ValueProperty->GetProperty();
	if (!Property)
	{
		return FPropertyAccess::Fail;
	}

	FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(Property);
	if (!StructProperty)
	{
		return FPropertyAccess::Fail;
	}
	if (StructProperty->Struct != TBaseStructure<T>::Get())
	{
		return FPropertyAccess::Fail;
	}

	const bool bTransactable = (Flags & EPropertyValueSetFlags::NotTransactable) == 0;
	bool bNotifiedPreChange = false;
	TArray<void*> RawData;
	ValueProperty->AccessRawData(RawData);
	for (void* Data : RawData)
	{
		if (Data)
		{
			if (!bNotifiedPreChange)
			{
				if (bTransactable && GEditor)
				{
					GEditor->BeginTransaction(FText::Format(LOCTEXT("SetPropertyValue", "Set {0}"), ValueProperty->GetPropertyDisplayName()));
				}
				ValueProperty->NotifyPreChange();
				bNotifiedPreChange = true;
			}

			T* Value = reinterpret_cast<T*>(Data);
			*Value = NewValue;
		}
	}

	if (bNotifiedPreChange)
	{
		ValueProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
		if (bTransactable && GEditor)
		{
			GEditor->EndTransaction();
		}
	}

	ValueProperty->NotifyFinishedChangingProperties();

	return FPropertyAccess::Success;
}

}; // UE::StateTree::PropertyHelpers

/**
 * Helper class to deal with relative property paths in PostEditChangeChainProperty().
 */
struct FStateTreeEditPropertyPath
{
private:
	struct FStateTreeEditPropertySegment
	{
		FStateTreeEditPropertySegment() = default;
		FStateTreeEditPropertySegment(FProperty* InProperty, const FName InPropertyName, const int32 InArrayIndex = INDEX_NONE)
			: Property(InProperty)
			, PropertyName(InPropertyName)
			, ArrayIndex(InArrayIndex)
		{
		}
	
		FProperty* Property = nullptr;
		FName PropertyName = FName();
		int32 ArrayIndex = INDEX_NONE;
	};
	
public:
	FStateTreeEditPropertyPath() = default;

	/** Makes property path relative to BaseStruct. Checks if the path is not part of the type. */
	explicit FStateTreeEditPropertyPath(const UStruct* BaseStruct, FStringView InPath);

	/** Makes property path from property change event. */
	explicit FStateTreeEditPropertyPath(const FPropertyChangedChainEvent& PropertyChangedEvent);

	/** Makes property path from property chain. */
	explicit FStateTreeEditPropertyPath(const FEditPropertyChain& PropertyChain);

	/**
	 * Makes property chain from property path.
	 * Copy Ctor of FEditPropertyChain is implicitly deleted so a conversion operator is not available here
	 */
	void MakeEditPropertyChain(FEditPropertyChain& OutPropertyChain) const;

	/** @return true if the property path contains specified path. */
	bool ContainsPath(const FStateTreeEditPropertyPath& InPath) const;

	/** @return true if the property path is exactly the specified path. */
	bool IsPathExact(const FStateTreeEditPropertyPath& InPath) const;

	/** @return array index at specified property, or INDEX_NONE, if the property is not array or property not found.  */
	int32 GetPropertyArrayIndex(const FStateTreeEditPropertyPath& InPath) const
	{
		return ContainsPath(InPath) ? Path[InPath.Path.Num() - 1].ArrayIndex : INDEX_NONE;
	}

private:
	TArray<FStateTreeEditPropertySegment> Path;
};

#undef LOCTEXT_NAMESPACE
