// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeState.h"
#include "StateTree.h"
#include "StateTreeEditingSubsystem.h"
#include "StateTreeEditorData.h"
#include "StateTreeConditionBase.h"
#include "StateTreeConsiderationBase.h"
#include "StateTreeTaskBase.h"
#include "StateTreeDelegates.h"
#include "StateTreePropertyHelpers.h"
#include "StateTreeViewModel.h"

#include "Customizations/StateTreeEditorNodeUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeState)

namespace UE::StateTree::Editor
{
	bool bCopyParameterBindingsOnTickByDefault = false;
	FAutoConsoleVariableRef CVarTickParameterBindingsByDefault(
		TEXT("StateTree.Editor.TickParameterBindingsByDefault"),
		bCopyParameterBindingsOnTickByDefault,
		TEXT("Default value for 'Tick Parameter Bindings' on newly created states.")
	);
}

//////////////////////////////////////////////////////////////////////////
// FStateTreeStateParameters

void FStateTreeStateParameters::RemoveUnusedOverrides()
{
	// Remove overrides that do not exists anymore
	if (!PropertyOverrides.IsEmpty())
	{
		if (const UPropertyBag* Bag = Parameters.GetPropertyBagStruct())
		{
			for (TArray<FGuid>::TIterator It = PropertyOverrides.CreateIterator(); It; ++It)
			{
				if (!Bag->FindPropertyDescByID(*It))
				{
					It.RemoveCurrentSwap();
				}
			}
		}
	}
}

//////////////////////////////////////////////////////////////////////////
// FStateTreeTransition

FStateTreeTransition::FStateTreeTransition(const EStateTreeTransitionTrigger InTrigger, const EStateTreeTransitionType InType, const UStateTreeState* InState)
	: Trigger(InTrigger)
{
	State = InState ? InState->GetLinkToState() : FStateTreeStateLink(InType);
}

FStateTreeTransition::FStateTreeTransition(const EStateTreeTransitionTrigger InTrigger, const FGameplayTag InEventTag, const EStateTreeTransitionType InType, const UStateTreeState* InState)
	: Trigger(InTrigger)
	, RequiredEvent{InEventTag}
{
	State = InState ? InState->GetLinkToState() : FStateTreeStateLink(InType);
}

void FStateTreeTransition::PostSerialize(const FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (EventTag_DEPRECATED.IsValid())
	{
		RequiredEvent.Tag = EventTag_DEPRECATED;
		EventTag_DEPRECATED = FGameplayTag();
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
}

void FStateTreeTransition::OnNodeAdded(TNotNull<UStateTreeState*> InState, FStateTreeEditorNode& EditorNode)
{
	if (UStateTree* StateTree = InState->GetTypedOuter<UStateTree>())
	{
		UStateTreeEditingSubsystem::MarkAsModified(StateTree);
	}
}

//////////////////////////////////////////////////////////////////////////
// UStateTreeState

UStateTreeState::UStateTreeState(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if(!IsTemplate())
	{
		ID = FGuid::NewGuid();
		Parameters.ID = FGuid::NewGuid();
		bCopyParameterBindingsOnTick = UE::StateTree::Editor::bCopyParameterBindingsOnTickByDefault;
	}
}

UStateTreeState::~UStateTreeState()
{
	UE::StateTree::Delegates::OnPostCompile.Remove(PostCompileHandle);
}

void UStateTreeState::PostInitProperties()
{
	Super::PostInitProperties();

	// Async load should not be registered; it should only be registered in PostLoad.
	//A newly created object in the editor should register now because it’s not loaded, and PostLoad won’t get called.
	if (!HasAnyFlags(RF_NeedPostLoad | RF_ClassDefaultObject))
	{
		PostCompileHandle = UE::StateTree::Delegates::OnPostCompile.AddUObject(this, &UStateTreeState::OnTreeCompiled);
	}
}

void UStateTreeState::OnTreeCompiled(const UStateTree& StateTree)
{
	if (&StateTree == LinkedAsset)
	{
		UpdateParametersFromLinkedSubtree();
	}
}

void UStateTreeState::PreEditChange(FEditPropertyChain& PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	const FStateTreeEditPropertyPath PropertyChainPath(PropertyAboutToChange);

	static const FStateTreeEditPropertyPath StateTypePath(UStateTreeState::StaticClass(), TEXT("Type"));
	static const FStateTreeEditPropertyPath StateTransitionsPath(UStateTreeState::StaticClass(), TEXT("Transitions"));

	if (PropertyChainPath.IsPathExact(StateTypePath))
	{
		// If transitioning from linked state, reset the parameters
		if (Type == EStateTreeStateType::Linked
			|| Type == EStateTreeStateType::LinkedAsset)
		{
			Parameters.ResetParametersAndOverrides();
		}
	}

	if (PropertyChainPath.IsPathExact(StateTransitionsPath))
	{
		// Copy Transitions so we can do diff compares later
		PreEditTransitions = Transitions;
	}
}

void UStateTreeState::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	const FStateTreeEditPropertyPath ChangePropertyPath(PropertyChangedEvent);

	static const FStateTreeEditPropertyPath StateNamePath(UStateTreeState::StaticClass(), TEXT("Name"));
	static const FStateTreeEditPropertyPath StateTypePath(UStateTreeState::StaticClass(), TEXT("Type"));
	static const FStateTreeEditPropertyPath SelectionBehaviorPath(UStateTreeState::StaticClass(), TEXT("SelectionBehavior"));
	static const FStateTreeEditPropertyPath StateLinkedSubtreePath(UStateTreeState::StaticClass(), TEXT("LinkedSubtree"));
	static const FStateTreeEditPropertyPath StateLinkedAssetPath(UStateTreeState::StaticClass(), TEXT("LinkedAsset"));
	static const FStateTreeEditPropertyPath StateParametersPath(UStateTreeState::StaticClass(), TEXT("Parameters"));
	static const FStateTreeEditPropertyPath StateTasksPath(UStateTreeState::StaticClass(), TEXT("Tasks"));
	static const FStateTreeEditPropertyPath StateEnterConditionsPath(UStateTreeState::StaticClass(), TEXT("EnterConditions"));
	static const FStateTreeEditPropertyPath StateConsiderationsPath(UStateTreeState::StaticClass(), TEXT("Considerations"));
	static const FStateTreeEditPropertyPath StateTransitionsPath(UStateTreeState::StaticClass(), TEXT("Transitions"));
	static const FStateTreeEditPropertyPath StateTransitionsConditionsPath(UStateTreeState::StaticClass(), TEXT("Transitions.Conditions"));
	static const FStateTreeEditPropertyPath StateTransitionsIDPath(UStateTreeState::StaticClass(), TEXT("Transitions.ID"));

	UStateTree* StateTree = GetTypedOuter<UStateTree>();
	ensure(StateTree);

	TSharedPtr<FStateTreeViewModel> StateTreeViewModel = nullptr;
	if (UStateTreeEditorData* TreeData = GetTypedOuter<UStateTreeEditorData>())
	{
		if (UStateTreeEditingSubsystem* StateTreeEditingSubsystem = TreeData ? GEditor->GetEditorSubsystem<UStateTreeEditingSubsystem>() : nullptr)
		{
			// Only find, don't create a view model if it doesn't exist
			StateTreeViewModel = StateTreeEditingSubsystem->FindViewModel(TreeData->GetOuterUStateTree());
		}
	}

	// Broadcast name changes so that the UI can update.
	if (ChangePropertyPath.IsPathExact(StateNamePath))
	{
		if (StateTree)
		{
			UE::StateTree::Delegates::OnIdentifierChanged.Broadcast(*StateTree);
		}
	}

	if (ChangePropertyPath.IsPathExact(SelectionBehaviorPath))
	{
		// Broadcast selection type changes so that the UI can update.
		if (StateTree)
		{
			UE::StateTree::Delegates::OnIdentifierChanged.Broadcast(*StateTree);
		}
	}
	
	if (ChangePropertyPath.IsPathExact(StateTypePath))
	{
		if (Type == EStateTreeStateType::Group)
		{
			// Group should not have tasks.
			Tasks.Reset();
		}

		const bool bHasPredefinedSelectionBehavior = Type == EStateTreeStateType::Linked || Type == EStateTreeStateType::LinkedAsset;
		if (bHasPredefinedSelectionBehavior)
		{
			// Reset Selection Behavior back to Try Enter State for group and linked types
			SelectionBehavior = EStateTreeStateSelectionBehavior::TryEnterState;
			// Remove any tasks when they are not used.
			Tasks.Reset();
		}

		// If transitioning from linked state, reset the linked state.
		if (Type != EStateTreeStateType::Linked)
		{
			LinkedSubtree = FStateTreeStateLink();
		}
		if (Type != EStateTreeStateType::LinkedAsset)
		{
			LinkedAsset = nullptr;
		}

		if (Type == EStateTreeStateType::Linked
			|| Type == EStateTreeStateType::LinkedAsset)
		{
			// Linked parameter layout is fixed, and copied from the linked target state.
			Parameters.bFixedLayout = true;
			UpdateParametersFromLinkedSubtree();
		}
		else
		{
			// Other layouts can be edited
			Parameters.bFixedLayout = false;
		}
	}

	// When switching to new state, update the parameters.
	if (ChangePropertyPath.IsPathExact(StateLinkedSubtreePath))
	{
		if (Type == EStateTreeStateType::Linked)
		{
			UpdateParametersFromLinkedSubtree();
		}
	}
	
	if (ChangePropertyPath.IsPathExact(StateLinkedAssetPath))
	{
		if (Type == EStateTreeStateType::LinkedAsset)
		{
			UpdateParametersFromLinkedSubtree();
		}
	}

	// Broadcast subtree parameter layout edits so that the linked states can adapt, and bindings can update.
	if (ChangePropertyPath.IsPathExact(StateParametersPath))
	{
		if (StateTree)
		{
			UE::StateTree::Delegates::OnStateParametersChanged.Broadcast(*StateTree, ID);
		}
	}

	if (ChangePropertyPath.ContainsPath(StateTransitionsPath))
	{
		const int32 TransitionsIndex = ChangePropertyPath.GetPropertyArrayIndex(StateTransitionsPath);
		if (Transitions.IsValidIndex(TransitionsIndex))
		{
			FStateTreeTransition& Transition = Transitions[TransitionsIndex];

			// Reset delay on completion transitions
			if (EnumHasAnyFlags(Transition.Trigger, EStateTreeTransitionTrigger::OnStateCompleted))
			{
				Transition.bDelayTransition = false;
			}

			// Let ViewModel know the transition was modified
			if (StateTreeViewModel)
			{
				StateTreeViewModel->GetTransitionViewModel().HandleOnTransitionModified(this, Transition.ID);
			}
		}
	}

	// Test exact path for array modifications
	if (ChangePropertyPath.IsPathExact(StateTransitionsPath))
	{
		if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayAdd)
		{
			const int32 TransitionsIndex = ChangePropertyPath.GetPropertyArrayIndex(StateTransitionsPath);
			if (Transitions.IsValidIndex(TransitionsIndex))
			{
				FStateTreeTransition& Transition = Transitions[TransitionsIndex];

				// Only reset Trigger and State to defaults if no explicit state was pre-set.
				if (!Transition.State.ID.IsValid())
				{
					ensureMsgf(Transition.Trigger == EStateTreeTransitionTrigger::OnStateCompleted
						, TEXT("Set transition trigger but did not set target state. Trigger will be overwritten."));

					Transition.Trigger = EStateTreeTransitionTrigger::OnStateCompleted;
					const UStateTreeState* RootState = GetRootState();
					Transition.State = RootState->GetLinkToState();
				}
				Transition.ID = FGuid::NewGuid();

				// Notify Transtions ViewModel of new transtion
				if (StateTreeViewModel)
				{
					StateTreeViewModel->GetTransitionViewModel().HandleOnTransitionsAdded(this, { Transition.ID });
				}
			}
		}
		else if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayRemove || PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayClear)
		{
			TSet<FGuid> RemovedTransitions;
			RemovedTransitions.Reserve(PreEditTransitions.Num());

			for (const FStateTreeTransition& PreEditTransition : PreEditTransitions)
			{
				if (!Transitions.Contains(PreEditTransition))
				{
					RemovedTransitions.Add(PreEditTransition.ID);
				}
			}

			// Notify Transtions ViewModel of removed transtions
			if (StateTreeViewModel)
			{
				StateTreeViewModel->GetTransitionViewModel().HandleOnTransitionsRemoved(this, RemovedTransitions);
			}
		}
		else if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayMove)
		{
			TSet<FGuid> MovedTransitions;
			MovedTransitions.Reserve(PreEditTransitions.Num());

			if (ensure(PreEditTransitions.Num() == Transitions.Num()))
			{
				for (int32 Index = 0; Index < PreEditTransitions.Num(); Index++)
				{
					if (PreEditTransitions[Index].ID != Transitions[Index].ID)
					{
						MovedTransitions.Add(PreEditTransitions[Index].ID);
					}
				}
			}

			// Notify Transtions ViewModel of removed transtions
			if (StateTreeViewModel)
			{
				StateTreeViewModel->GetTransitionViewModel().HandleOnTransitionsMoved(this, MovedTransitions);
			}
		}

		// Always reset temp captured old transitions
		PreEditTransitions.Reset();
	}

	if (StateTree)
	{
		UStateTreeEditingSubsystem::MarkAsModified(StateTree);
	}

	if (UStateTreeEditorData* TreeData = GetTypedOuter<UStateTreeEditorData>())
	{
		UE::StateTree::PropertyHelpers::DispatchPostEditToNodes(*this, PropertyChangedEvent, *TreeData);
	}
}

void UStateTreeState::PostLoad()
{
	Super::PostLoad();

	// Make sure state has transactional flags to make it work with undo (to fix a bug where root states were created without this flag).
	if (!HasAnyFlags(RF_Transactional))
	{
		SetFlags(RF_Transactional);
	}

	if (!PostCompileHandle.IsValid() && !HasAnyFlags(RF_ClassDefaultObject))
	{
		PostCompileHandle = UE::StateTree::Delegates::OnPostCompile.AddUObject(this, &UStateTreeState::OnTreeCompiled);
	}

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const int32 CurrentStateTreeCustomVersion = GetLinkerCustomVersion(FStateTreeCustomVersion::GUID);
	constexpr int32 AddedTransitionIdsVersion = FStateTreeCustomVersion::AddedTransitionIds;
	constexpr int32 OverridableStateParametersVersion = FStateTreeCustomVersion::OverridableStateParameters;
	constexpr int32 AddedCheckingParentsPrerequisitesVersion = FStateTreeCustomVersion::AddedCheckingParentsPrerequisites;
	constexpr int32 TickParameterBindingsVersion = FStateTreeCustomVersion::TickParameterBindings;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (CurrentStateTreeCustomVersion < AddedTransitionIdsVersion)
	{
		// Make guids for transitions. These need to be deterministic when upgrading because of cooking.
		for (int32 Index = 0; Index < Transitions.Num(); Index++)
		{
			FStateTreeTransition& Transition = Transitions[Index];
			Transition.ID = FGuid::NewDeterministicGuid(GetPathName(), Index);
		}
	}

	if (CurrentStateTreeCustomVersion < OverridableStateParametersVersion)
	{
		// In earlier versions, all parameters were overwritten.
		if (const UPropertyBag* Bag = Parameters.Parameters.GetPropertyBagStruct())
		{
			for (const FPropertyBagPropertyDesc& Desc : Bag->GetPropertyDescs())
			{
				Parameters.PropertyOverrides.Add(Desc.ID);
			}
		}
	}

	if (CurrentStateTreeCustomVersion < AddedCheckingParentsPrerequisitesVersion)
	{
		bCheckPrerequisitesWhenActivatingChildDirectly = false;
	}

	if (CurrentStateTreeCustomVersion < TickParameterBindingsVersion)
	{
		// For backward compatibility: Linked and LinkedAsset states always ticked parameter bindings before this option was added.
		bCopyParameterBindingsOnTick = (Type == EStateTreeStateType::Linked || Type == EStateTreeStateType::LinkedAsset);
	}

#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	for (FStateTreeEditorNode& EnterConditionEditorNode : EnterConditions)
	{
		if (FStateTreeNodeBase* ConditionNode = EnterConditionEditorNode.GetNode().GetPtr<FStateTreeNodeBase>())
		{
			UE::StateTreeEditor::EditorNodeUtils::ConditionalUpdateNodeInstanceData(EnterConditionEditorNode, *this);
			ConditionNode->PostLoad(EnterConditionEditorNode.GetInstance());
		}
	}

	for (FStateTreeEditorNode& ConsiderationEditorNode : Considerations)
	{
		if (FStateTreeNodeBase* ConsiderationNode = ConsiderationEditorNode.GetNode().GetPtr<FStateTreeNodeBase>())
		{
			UE::StateTreeEditor::EditorNodeUtils::ConditionalUpdateNodeInstanceData(ConsiderationEditorNode, *this);
			ConsiderationNode->PostLoad(ConsiderationEditorNode.GetInstance());
		}
	}

	for (FStateTreeEditorNode& TaskEditorNode : Tasks)
	{
		if (FStateTreeNodeBase* TaskNode = TaskEditorNode.GetNode().GetPtr<FStateTreeNodeBase>())
		{
			UE::StateTreeEditor::EditorNodeUtils::ConditionalUpdateNodeInstanceData(TaskEditorNode, *this);
			TaskNode->PostLoad(TaskEditorNode.GetInstance());
		}
	}

	if (FStateTreeNodeBase* SingleTaskNode = SingleTask.GetNode().GetPtr<FStateTreeNodeBase>())
	{
		UE::StateTreeEditor::EditorNodeUtils::ConditionalUpdateNodeInstanceData(SingleTask, *this);
		SingleTaskNode->PostLoad(SingleTask.GetInstance());
	}

	for (FStateTreeTransition& Transition : Transitions)
	{
		for (FStateTreeEditorNode& TransitionConditionEditorNode : Transition.Conditions)
		{
			if (FStateTreeNodeBase* ConditionNode = TransitionConditionEditorNode.GetNode().GetPtr<FStateTreeNodeBase>())
			{
				UE::StateTreeEditor::EditorNodeUtils::ConditionalUpdateNodeInstanceData(TransitionConditionEditorNode, *this);
				ConditionNode->PostLoad(TransitionConditionEditorNode.GetInstance());
			}
		}
	}
#endif // WITH_EDITOR
}

void UStateTreeState::UpdateParametersFromLinkedSubtree()
{
	if (const FInstancedPropertyBag* DefaultParameters = GetDefaultParameters())
	{
		Parameters.Parameters.MigrateToNewBagInstanceWithOverrides(*DefaultParameters, Parameters.PropertyOverrides);
		Parameters.RemoveUnusedOverrides();
	}
	else
	{
		Parameters.ResetParametersAndOverrides();
	}
}

void UStateTreeState::SetParametersPropertyOverridden(const FGuid PropertyID, const bool bIsOverridden)
{
	if (bIsOverridden)
	{
		Parameters.PropertyOverrides.AddUnique(PropertyID);
	}
	else
	{
		Parameters.PropertyOverrides.Remove(PropertyID);
		UpdateParametersFromLinkedSubtree();

		// Remove binding when override is removed.
		if (UStateTreeEditorData* EditorData = GetTypedOuter<UStateTreeEditorData>())
		{
			if (FStateTreeEditorPropertyBindings* Bindings = EditorData->GetPropertyEditorBindings())
			{
				if (const UPropertyBag* ParametersBag = Parameters.Parameters.GetPropertyBagStruct())
				{
					if (const FPropertyBagPropertyDesc* Desc = ParametersBag->FindPropertyDescByID(PropertyID))
					{
						check(Desc->CachedProperty);

						EditorData->Modify();

						FPropertyBindingPath Path(Parameters.ID, Desc->CachedProperty->GetFName());
						Bindings->RemoveBindings(Path);
					}
				}
			}
		}
	}

	if (UStateTree* StateTree = GetTypedOuter<UStateTree>())
	{
		UStateTreeEditingSubsystem::MarkAsModified(StateTree);
	}
}

const FInstancedPropertyBag* UStateTreeState::GetDefaultParameters() const
{
	if (Type == EStateTreeStateType::Linked)
	{
		if (const UStateTreeEditorData* TreeData = GetTypedOuter<UStateTreeEditorData>())
		{
			if (const UStateTreeState* LinkTargetState = TreeData->GetStateByID(LinkedSubtree.ID))
			{
				return &LinkTargetState->Parameters.Parameters;
			}
		}
	}
	else if (Type == EStateTreeStateType::LinkedAsset)
	{
		if (LinkedAsset)
		{
			return &LinkedAsset->GetDefaultParameters();
		}
	}

	return nullptr;
}

const UStateTreeState* UStateTreeState::GetRootState() const
{
	const UStateTreeState* RootState = this;
	while (RootState->Parent != nullptr)
	{
		RootState = RootState->Parent;
	}
	return RootState;
}

UStateTreeState* UStateTreeState::GetNextSiblingState() const
{
	if (!Parent)
	{
		return nullptr;
	}
	for (int32 ChildIdx = 0; ChildIdx < Parent->Children.Num(); ChildIdx++)
	{
		if (Parent->Children[ChildIdx] == this)
		{
			const int NextIdx = ChildIdx + 1;

			// Select the next enabled sibling
			if (NextIdx < Parent->Children.Num() && Parent->Children[NextIdx]->bEnabled)
			{
				return Parent->Children[NextIdx];
			}
			break;
		}
	}
	return nullptr;
}

UStateTreeState* UStateTreeState::GetNextSelectableSiblingState() const
{
	if (!Parent)
	{
		return nullptr;
	}

	const int32 StartChildIndex = Parent->Children.IndexOfByKey(this);
	if (StartChildIndex == INDEX_NONE)
	{
		return nullptr;
	}
	
	for (int32 ChildIdx = StartChildIndex + 1; ChildIdx < Parent->Children.Num(); ChildIdx++)
	{
		// Select the next enabled and selectable sibling
		UStateTreeState* State = Parent->Children[ChildIdx];
		if (State->SelectionBehavior != EStateTreeStateSelectionBehavior::None
			&& State->bEnabled)
		{
			return State;
		}
	}
	
	return nullptr;
}

FString UStateTreeState::GetPath() const
{
	TArray<const UStateTreeState*, TInlineAllocator<8>> States;
	for (const UStateTreeState* CurrState = this; CurrState; CurrState = CurrState->Parent)
	{
		States.Add(CurrState);
	}
	Algo::Reverse(States);
	
	TStringBuilder<256> Result;
	for (const UStateTreeState* CurrState : States)
	{
		if (Result.Len() > 0)
		{
			Result.Append(TEXT("/"));
		}
		Result.Append(CurrState->Name.ToString());
	}

	return FString(Result);
}

FStateTreeStateLink UStateTreeState::GetLinkToState() const
{
	FStateTreeStateLink Link(EStateTreeTransitionType::GotoState);
	Link.Name = Name;
	Link.ID = ID;
	return Link;
}

TSubclassOf<UStateTreeSchema> UStateTreeState::GetSchema() const
{
	if (const UStateTreeEditorData* EditorData = GetTypedOuter<UStateTreeEditorData>())
	{
		if (EditorData->Schema)
		{
			return EditorData->Schema->GetClass();
		}
	}
	return nullptr;
}

void UStateTreeState::SetLinkedState(FStateTreeStateLink InStateLink)
{
	check(Type == EStateTreeStateType::Linked);
	LinkedSubtree = InStateLink;

	Tasks.Reset();
	LinkedAsset = nullptr;
	Parameters.bFixedLayout = true;
	UpdateParametersFromLinkedSubtree();
	SelectionBehavior = EStateTreeStateSelectionBehavior::TryEnterState;

	if (UStateTree* StateTree = GetTypedOuter<UStateTree>())
	{
		UStateTreeEditingSubsystem::MarkAsModified(StateTree);
	}
}

void UStateTreeState::SetLinkedStateAsset(UStateTree* InLinkedAsset)
{
	check(Type == EStateTreeStateType::LinkedAsset);
	LinkedAsset = InLinkedAsset;

	Tasks.Reset();
	LinkedSubtree = FStateTreeStateLink();
	Parameters.bFixedLayout = true;
	UpdateParametersFromLinkedSubtree();
	SelectionBehavior = EStateTreeStateSelectionBehavior::TryEnterState;

	if (UStateTree* StateTree = GetTypedOuter<UStateTree>())
	{
		UStateTreeEditingSubsystem::MarkAsModified(StateTree);
	}
}

UStateTreeState& UStateTreeState::AddChildState(const FName ChildName, const EStateTreeStateType StateType)
{
	UStateTreeState* ChildState = NewObject<UStateTreeState>(this, FName(), RF_Transactional);
	check(ChildState);
	ChildState->Name = ChildName;
	ChildState->Parent = this;
	ChildState->Type = StateType;
	Children.Add(ChildState);
	if (UStateTree* StateTree = GetTypedOuter<UStateTree>())
	{
		UStateTreeEditingSubsystem::MarkAsModified(StateTree);
	}
	return *ChildState;
}

void UStateTreeState::OnNodeAdded(FStateTreeEditorNode& EditorNode)
{
	if (UStateTree* StateTree = GetTypedOuter<UStateTree>())
	{
		UStateTreeEditingSubsystem::MarkAsModified(StateTree);
	}
}

FStateTreeTransition& UStateTreeState::AddTransition(const EStateTreeTransitionTrigger InTrigger, const EStateTreeTransitionType InType, const UStateTreeState* InState)
{
	FStateTreeTransition& Transition = Transitions.Emplace_GetRef(InTrigger, InType, InState);
	AddTransitionImpl(Transition);
	return Transition;
}

FStateTreeTransition& UStateTreeState::AddTransition(const EStateTreeTransitionTrigger InTrigger, const FGameplayTag InEventTag, const EStateTreeTransitionType InType, const UStateTreeState* InState)
{
	FStateTreeTransition& Transition = Transitions.Emplace_GetRef(InTrigger, InEventTag, InType, InState);
	AddTransitionImpl(Transition);
	return Transition;
}

void UStateTreeState::AddTransitionImpl(FStateTreeTransition& Transition)
{
	Transition.ID = FGuid::NewGuid();
	if (UStateTree* StateTree = GetTypedOuter<UStateTree>())
	{
		UStateTreeEditingSubsystem::MarkAsModified(StateTree);
	}
}
