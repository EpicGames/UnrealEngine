// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeTransitionViewModel.h"

#include "StateTree.h"
#include "Misc/NotNull.h"
#include "Misc/StringOutputDevice.h"

#define LOCTEXT_NAMESPACE "StateTreeEditor"

void FStateTreeTransitionViewModel::HandleOnTransitionsAdded(const UStateTreeState* OwningState, const TSet<FGuid>& TransitionID) const
{
	OnTransitionsAdded.Broadcast(OwningState, TransitionID);
}

void FStateTreeTransitionViewModel::HandleOnTransitionsRemoved(const UStateTreeState* OwningState, const TSet<FGuid>& TransitionIDs) const
{
	OnTransitionsRemoved.Broadcast(OwningState, TransitionIDs);
}

void FStateTreeTransitionViewModel::HandleOnTransitionsMoved(const UStateTreeState* OwningState, const TSet<FGuid>& TransitionIDs) const
{
	OnTransitionsMoved.Broadcast(OwningState, TransitionIDs);
}

void FStateTreeTransitionViewModel::HandleOnTransitionModified(const UStateTreeState* OwningState, const FGuid& TransitionID) const
{
	OnTransitionModified.Broadcast(OwningState, TransitionID);
}

#undef LOCTEXT_NAMESPACE
