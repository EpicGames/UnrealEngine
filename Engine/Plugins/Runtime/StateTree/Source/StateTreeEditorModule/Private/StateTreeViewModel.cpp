// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeViewModel.h"

#include "StateTree.h"
#include "StateTreeEditorData.h"
#include "StateTreeEditorStyle.h"
#include "StateTreeDelegates.h"
#include "Debugger/StateTreeDebugger.h"
#include "Editor.h"
#include "ScopedTransaction.h"
#include "HAL/PlatformApplicationMisc.h"
#include "UnrealExporter.h"
#include "Exporters/Exporter.h"
#include "Factories.h"
#include "StateTreeTaskBase.h"
#include "StateTreeConditionBase.h"
#include "StateTreeConsiderationBase.h"
#include "StateTreeEditingSubsystem.h"
#include "StateTreePropertyHelpers.h"
#include "StateTreeEditorDataClipboardHelpers.h"
#include "Styling/SlateBrush.h"
#include "Misc/NotNull.h"
#include "Misc/StringOutputDevice.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeViewModel)

#define LOCTEXT_NAMESPACE "StateTreeEditor"

bool bEnableSimpleHierarchyDrivenColors = false;
FAutoConsoleVariableRef CVarEnableSimpleHierarchyDrivenColors(
	TEXT("StateTree.Editor.Experimental.EnableSimpleHierarchyDrivenColors"),
	bEnableSimpleHierarchyDrivenColors,
	TEXT("Set true to use a straightforward approach of replacing base colors for hierarchy colors. Disable hierarchy colors in settings to use.")
);

namespace UE::StateTree::Editor
{
	static const FLinearColor DefaultStateColor = FLinearColor(FColor(31, 151, 167));

	// These defaults only apply when using hierarchy driven colors
	static const FLinearColor DefaultRootColor = FLinearColor(0.347f, 0.347f, 0.347f);
	static const FLinearColor DefaultParentColor = FLinearColor(1.0f, 0.125f, 0.913f);

	class FStateTreeStateTextFactory : public FCustomizableTextObjectFactory
	{
	public:
		FStateTreeStateTextFactory()
			: FCustomizableTextObjectFactory(GWarn)
		{}

		virtual bool CanCreateClass(UClass* InObjectClass, bool& bOmitSubObjs) const override
		{
			return InObjectClass->IsChildOf(UStateTreeState::StaticClass())
				|| InObjectClass->IsChildOf(UStateTreeClipboardBindings::StaticClass());
		}

		virtual void ProcessConstructedObject(UObject* NewObject) override
		{
			if (UStateTreeState* State = Cast<UStateTreeState>(NewObject))
			{
				States.Add(State);
			}
			else if (UStateTreeClipboardBindings* Bindings = Cast<UStateTreeClipboardBindings>(NewObject))
			{
				ClipboardBindings = Bindings;
			}
		}

	public:
		TArray<UStateTreeState*> States;
		UStateTreeClipboardBindings* ClipboardBindings = nullptr;
	};


	void CollectBindingsCopiesRecursive(UStateTreeEditorData* TreeData, UStateTreeState* State, TArray<FStateTreePropertyPathBinding>& AllBindings)
	{
		if (!State)
		{
			return;
		}
		
		TreeData->VisitStateNodes(*State, [TreeData, &AllBindings](const UStateTreeState* State, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)
		{
			TArray<const FPropertyBindingBinding*> NodeBindings;
			TreeData->GetPropertyEditorBindings()->FPropertyBindingBindingCollection::GetBindingsFor(Desc.ID, NodeBindings);
			Algo::Transform(NodeBindings, AllBindings, [](const FPropertyBindingBinding* BindingPtr) { return *static_cast<const FStateTreePropertyPathBinding*>(BindingPtr); });
			return EStateTreeVisitor::Continue;
		});

		for (UStateTreeState* ChildState : State->Children)
		{
			CollectBindingsCopiesRecursive(TreeData, ChildState, AllBindings);
		}
	}

	FString ExportStatesToText(UStateTreeEditorData* TreeData, const TArrayView<UStateTreeState*> States)
	{
		if (States.IsEmpty())
		{
			return FString();
		}

		// Clear the mark state for saving.
		UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));

		FStringOutputDevice Archive;
		const FExportObjectInnerContext Context;

		UStateTreeClipboardBindings* ClipboardBindings = NewObject<UStateTreeClipboardBindings>();
		check(ClipboardBindings);

		for (UStateTreeState* State : States)
		{
			UObject* ThisOuter = State->GetOuter();
			UExporter::ExportToOutputDevice(&Context, State, nullptr, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false, ThisOuter);

			CollectBindingsCopiesRecursive(TreeData, State, ClipboardBindings->Bindings);
		}

		UExporter::ExportToOutputDevice(&Context, ClipboardBindings, nullptr, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false);

		return *Archive;
	}

	void CollectStateLinks(const UStruct* Struct, void* Memory, TArray<FStateTreeStateLink*>& Links)
	{
		for (TPropertyValueIterator<FStructProperty> It(Struct, Memory); It; ++It)
		{
			if (It->Key->Struct == TBaseStructure<FStateTreeStateLink>::Get())
			{
				FStateTreeStateLink* StateLink = static_cast<FStateTreeStateLink*>(const_cast<void*>(It->Value));
				Links.Add(StateLink);
			}
		}
	}

	// todo: Should refactor it into FStateTreeScopedEditorDataFixer
	void FixNodesAfterDuplication(TArrayView<FStateTreeEditorNode> Nodes, TMap<FGuid, FGuid>& IDsMap, TArray<FStateTreeStateLink*>& Links)
	{
		for (FStateTreeEditorNode& Node : Nodes)
		{
			const FGuid NewNodeID = FGuid::NewGuid();
			IDsMap.Emplace(Node.ID, NewNodeID);
			Node.ID = NewNodeID;

			if (Node.GetNode().IsValid())
			{
				CollectStateLinks(Node.GetNode().GetScriptStruct(), Node.GetNode().GetMemory(), Links);
			}
			if (Node.GetInstance().IsValid())
			{
				CollectStateLinks(Node.GetInstance().GetStruct(), Node.GetInstance().GetMutableMemory(), Links);
			}
			if (Node.GetExecutionRuntimeData().IsValid())
			{
				CollectStateLinks(Node.GetExecutionRuntimeData().GetStruct(), Node.GetExecutionRuntimeData().GetMutableMemory(), Links);
			}
		}
	}

	// todo: Should refactor it into FStateTreeScopedEditorDataFixer
	void FixStateAfterDuplication(UStateTreeState* State, UStateTreeState* NewParentState, TMap<FGuid, FGuid>& IDsMap, TArray<FStateTreeStateLink*>& Links, TArray<UStateTreeState*>& NewStates)
	{
		State->Modify();

		const FGuid OldStateEventID = State->GetEventID();

		const FGuid NewStateID = FGuid::NewGuid();
		IDsMap.Emplace(State->ID, NewStateID);
		State->ID = NewStateID;
		
		IDsMap.Emplace(OldStateEventID, State->GetEventID());

		const FGuid NewParametersID = FGuid::NewGuid();
		IDsMap.Emplace(State->Parameters.ID, NewParametersID);
		State->Parameters.ID = NewParametersID;
		
		State->Parent = NewParentState;
		NewStates.Add(State);
		
		if (State->Type == EStateTreeStateType::Linked)
		{
			Links.Emplace(&State->LinkedSubtree);
		}

		FixNodesAfterDuplication(TArrayView<FStateTreeEditorNode>(&State->SingleTask, 1), IDsMap, Links);
		FixNodesAfterDuplication(State->Tasks, IDsMap, Links);
		FixNodesAfterDuplication(State->EnterConditions, IDsMap, Links);
		FixNodesAfterDuplication(State->Considerations, IDsMap, Links);

		for (FStateTreeTransition& Transition : State->Transitions)
		{
			// Transition Ids are not used by nodes so no need to add to 'IDsMap'
			Transition.ID = FGuid::NewGuid();

			FixNodesAfterDuplication(Transition.Conditions, IDsMap, Links);
			Links.Emplace(&Transition.State);
		}

		for (UStateTreeState* Child : State->Children)
		{
			FixStateAfterDuplication(Child, State, IDsMap, Links, NewStates);
		}
	}

	// Removes states from the array which are children of any other state.
	void RemoveContainedChildren(TArray<UStateTreeState*>& States)
	{
		TSet<UStateTreeState*> UniqueStates;
		for (UStateTreeState* State : States)
		{
			UniqueStates.Add(State);
		}

		for (int32 i = 0; i < States.Num(); )
		{
			UStateTreeState* State = States[i];

			// Walk up the parent state sand if the current state
			// exists in any of them, remove it.
			UStateTreeState* StateParent = State->Parent;
			bool bShouldRemove = false;
			while (StateParent)
			{
				if (UniqueStates.Contains(StateParent))
				{
					bShouldRemove = true;
					break;
				}
				StateParent = StateParent->Parent;
			}

			if (bShouldRemove)
			{
				States.RemoveAt(i);
			}
			else
			{
				i++;
			}
		}
	}

	// Returns true if the state is child of parent state.
	bool IsChildOf(const UStateTreeState* ParentState, const UStateTreeState* State)
	{
		for (const UStateTreeState* Child : ParentState->Children)
		{
			if (Child == State)
			{
				return true;
			}
			if (IsChildOf(Child, State))
			{
				return true;
			}
		}
		return false;
	}

namespace Private
{
	/* Short-lived helper struct for node manipulation in the editor */
	struct FStateTreeStateNodeEditorHandle
	{
		const TNotNull<UStateTreeEditorData*> EditorData;
		const TNotNull<UStateTreeState*> OwnerState;

	private:
		FStringView NodePath;
		int32 ArrayIndex = INDEX_NONE;
		void* TargetArray = nullptr;
		void* TargetNode = nullptr;
		bool bIsTransition = false;

	public:
		FStateTreeStateNodeEditorHandle(const TNotNull<UStateTreeEditorData*> InEditorData, const TNotNull<UStateTreeState*> InOwnerState, const FGuid& NodeID)
			: EditorData(InEditorData),
			OwnerState(InOwnerState)
		{
			auto FindNode = [Self = this, &NodeID]<typename T>(T& Nodes, FStringView Path)
			{
				for (int32 Index = 0; Index < Nodes.Num(); ++Index)
				{
					if (NodeID == Nodes[Index].ID)
					{
						Self->ArrayIndex = Index;
						Self->TargetArray = &Nodes;
						Self->TargetNode = &Nodes[Self->ArrayIndex];
						Self->NodePath = Path;

						return true;
					}
				}

				return false;
			};

			if (!NodeID.IsValid())
			{
				return;
			}

			if (FindNode(OwnerState->EnterConditions, TEXT("EnterConditions")))
			{
				return;
			}

			if (FindNode(OwnerState->Tasks, TEXT("Tasks")))
			{
				return;
			}

			if (NodeID == OwnerState->SingleTask.ID)
			{
				NodePath = TEXT("SingleTask");
				TargetNode = &OwnerState->SingleTask;
				return;
			}

			if (FindNode(OwnerState->Considerations, TEXT("Considerations")))
			{
				return;
			}

			if (FindNode(OwnerState->Transitions, TEXT("Transitions")))
			{
				bIsTransition = true;
				return;
			}
		}

		bool IsValid() const
		{
			return TargetNode != nullptr;
		}

		bool IsTransition() const
		{
			return bIsTransition;
		}

		FStateTreeEditorNode& GetEditorNode() const
		{
			check(IsValid() && !IsTransition());

			return *static_cast<FStateTreeEditorNode*>(TargetNode);
		}

		TArray<FStateTreeEditorNode>& GetEditorNodeArray() const
		{
			check(IsValid() && !IsTransition() && GetNodeIndex() != INDEX_NONE);

			return *static_cast<TArray<FStateTreeEditorNode>*>(TargetArray);
		}

		FStateTreeTransition& GetTransition() const
		{
			check(IsValid() && IsTransition());

			return *static_cast<FStateTreeTransition*>(TargetNode);
		}

		TArray<FStateTreeTransition>& GetTransitionArray() const
		{
			check(IsValid() && IsTransition() && GetNodeIndex() != INDEX_NONE);

			return *static_cast<TArray<FStateTreeTransition>*>(TargetArray);
		}

		FStringView GetNodePath() const
		{
			return NodePath;
		}

		int32 GetNodeIndex() const
		{
			return ArrayIndex;
		}
	};

} //namespace Private

} // namespace UE::StateTree::Editor

FStateTreeViewModel::FStateTreeViewModel()
	: TreeDataWeak(nullptr)
#if WITH_STATETREE_TRACE_DEBUGGER
	, Debugger(MakeShareable(new FStateTreeDebugger))
#endif // WITH_STATETREE_TRACE_DEBUGGER
{
}

FStateTreeViewModel::~FStateTreeViewModel()
{
	if (GEditor)
	{
	    GEditor->UnregisterForUndo(this);
	}

	UE::StateTree::Delegates::OnIdentifierChanged.RemoveAll(this);
}

void FStateTreeViewModel::Init(UStateTreeEditorData* InTreeData)
{
	TreeDataWeak = InTreeData;

	GEditor->RegisterForUndo(this);

	UE::StateTree::Delegates::OnIdentifierChanged.AddSP(this, &FStateTreeViewModel::HandleIdentifierChanged);
	
#if WITH_STATETREE_TRACE_DEBUGGER
	UE::StateTree::Delegates::OnBreakpointsChanged.AddSP(this, &FStateTreeViewModel::HandleBreakpointsChanged);
	UE::StateTree::Delegates::OnPostCompile.AddSP(this, &FStateTreeViewModel::HandlePostCompile);

	Debugger->SetAsset(GetStateTree());
	BindToDebuggerDelegates();
	RefreshDebuggerBreakpoints();
#endif // WITH_STATETREE_TRACE_DEBUGGER
}

const UStateTree* FStateTreeViewModel::GetStateTree() const
{
	if (const UStateTreeEditorData* TreeData = TreeDataWeak.Get())
	{
		return TreeData->GetTypedOuter<UStateTree>();
	}

	return nullptr;
}

const UStateTreeEditorData* FStateTreeViewModel::GetStateTreeEditorData() const
{
	return TreeDataWeak.Get();
}

const UStateTreeSchema* FStateTreeViewModel::GetStateTreeSchema() const
{
	if (const UStateTreeEditorData* TreeData = TreeDataWeak.Get())
	{
		return TreeData->Schema;
	}

	return nullptr;
}

const UStateTreeState* FStateTreeViewModel::GetStateByID(const FGuid StateID) const
{
	if (const UStateTreeEditorData* TreeData = TreeDataWeak.Get())
	{
		return const_cast<UStateTreeState*>(TreeData->GetStateByID(StateID));
	}
	return nullptr;
}

UStateTreeState* FStateTreeViewModel::GetMutableStateByID(const FGuid StateID) const
{
	if (UStateTreeEditorData* TreeData = TreeDataWeak.Get())
	{
		return TreeData->GetMutableStateByID(StateID);
	}
	return nullptr;
}

const FStateTreeTransition* FStateTreeViewModel::GetTransitionByID(const FGuid TransitionID) const
{
	return GetMutableTransitionByID(TransitionID);
}

FStateTreeTransition* FStateTreeViewModel::GetMutableTransitionByID(const FGuid TransitionID) const
{
	FStateTreeTransition* Result = nullptr;

	auto FindMatchingTransition = [&](UStateTreeState& HierarchyState, UStateTreeState* ParentState)
	{
		for (FStateTreeTransition& Transition : HierarchyState.Transitions)
		{
			if (Transition.ID == TransitionID)
			{
				Result = &Transition;
				return EStateTreeVisitor::Break;
			}
		}

		return EStateTreeVisitor::Continue;
	};

	if (UStateTreeEditorData* TreeData = TreeDataWeak.Get())
	{
		TreeData->VisitHierarchy(FindMatchingTransition);
	}

	return Result;
}

void FStateTreeViewModel::HandleIdentifierChanged(const UStateTree& StateTree) const
{
	if (GetStateTree() == &StateTree)
	{
		NotifyAssetChangedExternally();
	}
}

#if WITH_STATETREE_TRACE_DEBUGGER

void FStateTreeViewModel::ToggleStateBreakpoints(TConstArrayView<TWeakObjectPtr<>> States, EStateTreeBreakpointType Type)
{
	UStateTreeEditorData* EditorData = TreeDataWeak.Get();
	if (EditorData == nullptr || States.IsEmpty())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("ToggleStateBreakpoint", "Toggle State Breakpoint"));
	EditorData->Modify(/*bAlwaysMarkDirty*/false);

	for (const TWeakObjectPtr<>& WeakStateObject : States)
	{
		if (const UStateTreeState* State = static_cast<const UStateTreeState*>(WeakStateObject.Get()))
		{
			const bool bRemoved = EditorData->RemoveBreakpoint(State->ID, Type);
			if (bRemoved == false)
			{
				EditorData->AddBreakpoint(State->ID, Type);
			}
		}
	}
}

void FStateTreeViewModel::ToggleTaskBreakpoint(const FGuid ID, const EStateTreeBreakpointType Type)
{
	if (UStateTreeEditorData* EditorData = TreeDataWeak.Get())
	{
		const FScopedTransaction Transaction(LOCTEXT("ToggleTaskBreakpoint", "Toggle Task Breakpoint"));
		EditorData->Modify(/*bAlwaysMarkDirty*/false);

		const bool bRemoved = EditorData->RemoveBreakpoint(ID, Type);
		if (bRemoved == false)
		{
			EditorData->AddBreakpoint(ID, Type);
		}
	}
}

void FStateTreeViewModel::ToggleTransitionBreakpoint(const TConstArrayView<TNotNull<const FStateTreeTransition*>> Transitions, const ECheckBoxState ToggledState)
{
	UStateTreeEditorData* EditorData = TreeDataWeak.Get();
	if (EditorData == nullptr)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("TransitionBreakpointToggled", "Transition Breakpoint Toggled"));
	EditorData->Modify(/*bAlwaysMarkDirty*/false);

	for (const FStateTreeTransition* Transition : Transitions)
	{
		const bool bHasBreakpoint = EditorData->HasBreakpoint(Transition->ID, EStateTreeBreakpointType::OnTransition);
		if (ToggledState == ECheckBoxState::Checked && bHasBreakpoint == false)
		{
			EditorData->AddBreakpoint(Transition->ID, EStateTreeBreakpointType::OnTransition);
		}
		else if (ToggledState == ECheckBoxState::Unchecked && bHasBreakpoint)
		{
			EditorData->RemoveBreakpoint(Transition->ID, EStateTreeBreakpointType::OnTransition);
		}
	}
}

bool FStateTreeViewModel::HasBreakpoint(const FGuid ID, const EStateTreeBreakpointType Type)
{
	if (UStateTreeEditorData* TreeData = TreeDataWeak.Get())
	{
		return TreeData->HasBreakpoint(ID, Type);
	}

	return false;
}

bool FStateTreeViewModel::CanProcessBreakpoints() const
{
	return Debugger.Get().CanProcessBreakpoints();
}

bool FStateTreeViewModel::CanAddStateBreakpoint(const EStateTreeBreakpointType Type) const
{
	if (!CanProcessBreakpoints())
	{
		return false;
	}

	const UStateTreeEditorData* EditorData = TreeDataWeak.Get();
	if (!ensure(EditorData != nullptr))
	{
		return false;
	}

	for (const TWeakObjectPtr<UStateTreeState>& WeakState : SelectedStates)
	{
		if (const UStateTreeState* State = WeakState.Get())
		{
			if (EditorData->HasBreakpoint(State->ID, Type) == false)
			{
				return true;
			}
		}
	}

	return false;
}

bool FStateTreeViewModel::CanRemoveStateBreakpoint(const EStateTreeBreakpointType Type) const
{
	const UStateTreeEditorData* EditorData = TreeDataWeak.Get();
	if (!ensure(EditorData != nullptr))
	{
		return false;
	}

	for (const TWeakObjectPtr<UStateTreeState>& WeakState : SelectedStates)
	{
		if (const UStateTreeState* State = WeakState.Get())
		{
			if (EditorData->HasBreakpoint(State->ID, Type))
			{
				return true;
			}
		}
	}

	return false;
}


ECheckBoxState FStateTreeViewModel::GetStateBreakpointCheckState(const EStateTreeBreakpointType Type) const
{
	const bool bCanAdd = CanAddStateBreakpoint(Type);
	const bool bCanRemove = CanRemoveStateBreakpoint(Type);
	if (bCanAdd && bCanRemove)
	{
		return ECheckBoxState::Undetermined;
	}

	if (bCanRemove)
	{
		return ECheckBoxState::Checked;
	}

	if (bCanAdd)
	{
		return ECheckBoxState::Unchecked;
	}

	// Should not happen since action is not visible in this case
	return ECheckBoxState::Undetermined;
}

void FStateTreeViewModel::HandleEnableStateBreakpoint(EStateTreeBreakpointType Type)
{
	TArray<UStateTreeState*> ValidatedSelectedStates;
	GetSelectedStates(ValidatedSelectedStates);
	if (ValidatedSelectedStates.IsEmpty())
	{
		return;
	}

	UStateTreeEditorData* EditorData = TreeDataWeak.Get();
	if (!ensure(EditorData != nullptr))
	{
		return;
	}

	TBitArray<> HasBreakpoint;
	HasBreakpoint.Reserve(ValidatedSelectedStates.Num());
	for (const UStateTreeState* SelectedState : ValidatedSelectedStates)
	{
		HasBreakpoint.Add(SelectedState != nullptr && EditorData->HasBreakpoint(SelectedState->ID, Type));
	}

	check(HasBreakpoint.Num() == ValidatedSelectedStates.Num());

	// Process CanAdd first so in case of undetermined state (mixed selection) we add by default. 
	if (CanAddStateBreakpoint(Type))
	{
		const FScopedTransaction Transaction(LOCTEXT("AddStateBreakpoint", "Add State Breakpoint(s)"));
		EditorData->Modify(/*bAlwaysMarkDirty*/false);
		for (int Index = 0; Index < ValidatedSelectedStates.Num(); ++Index)
		{
			const UStateTreeState* SelectedState = ValidatedSelectedStates[Index];
			if (HasBreakpoint[Index] == false && SelectedState != nullptr)
			{
				EditorData->AddBreakpoint(SelectedState->ID, Type);
			}
		}
	}
	else if (CanRemoveStateBreakpoint(Type))
	{
		const FScopedTransaction Transaction(LOCTEXT("RemoveStateBreakpoint", "Remove State Breakpoint(s)"));
		EditorData->Modify(/*bAlwaysMarkDirty*/false);
		for (int Index = 0; Index < ValidatedSelectedStates.Num(); ++Index)
		{
			const UStateTreeState* SelectedState = ValidatedSelectedStates[Index];
			if (HasBreakpoint[Index] && SelectedState != nullptr)
			{
				EditorData->RemoveBreakpoint(SelectedState->ID, Type);
			}
		}
	}
}

UStateTreeState* FStateTreeViewModel::FindStateAssociatedToBreakpoint(FStateTreeDebuggerBreakpoint Breakpoint) const
{
	UStateTreeEditorData* EditorData = TreeDataWeak.Get();
	if (EditorData == nullptr)
	{
		return nullptr;
	}
	const UStateTree* StateTree = GetStateTree();
	if (StateTree == nullptr)
	{
		return nullptr;
	}

	UStateTreeState* StateTreeState = nullptr;

	if (const FStateTreeStateHandle* StateHandle = Breakpoint.ElementIdentifier.TryGet<FStateTreeStateHandle>())
	{
		const FGuid StateId = StateTree->GetStateIdFromHandle(*StateHandle);
		StateTreeState = EditorData->GetMutableStateByID(StateId);
	}
	else if (const FStateTreeDebuggerBreakpoint::FStateTreeTaskIndex* TaskIndex = Breakpoint.ElementIdentifier.TryGet<FStateTreeDebuggerBreakpoint::FStateTreeTaskIndex>())
	{
		const FGuid TaskId = StateTree->GetNodeIdFromIndex(TaskIndex->Index);

		EditorData->VisitHierarchy([&TaskId, &StateTreeState](UStateTreeState& State, UStateTreeState* /*ParentState*/)
			{
				for (const FStateTreeEditorNode& EditorNode : State.Tasks)
				{
					if (EditorNode.ID == TaskId)
					{
						StateTreeState = &State;
						return EStateTreeVisitor::Break;
					}
				}
				return EStateTreeVisitor::Continue;
			});
	}
	else if (const FStateTreeDebuggerBreakpoint::FStateTreeTransitionIndex* TransitionIndex = Breakpoint.ElementIdentifier.TryGet<FStateTreeDebuggerBreakpoint::FStateTreeTransitionIndex>())
	{
		const FGuid TransitionId = StateTree->GetTransitionIdFromIndex(TransitionIndex->Index);

		EditorData->VisitHierarchy([&TransitionId, &StateTreeState](UStateTreeState& State, UStateTreeState* /*ParentState*/)
			{
				for (const FStateTreeTransition& StateTransition : State.Transitions)
				{
					if (StateTransition.ID == TransitionId)
					{
						StateTreeState = &State;
						return EStateTreeVisitor::Break;
					}
				}
				return EStateTreeVisitor::Continue;
			});
	}

	return StateTreeState;
}

void FStateTreeViewModel::HandleBreakpointsChanged(const UStateTree& StateTree)
{
	if (GetStateTree() == &StateTree)
	{
		RefreshDebuggerBreakpoints();
	}
}

void FStateTreeViewModel::HandlePostCompile(const UStateTree& StateTree)
{
	if (GetStateTree() == &StateTree)
	{
		RefreshDebuggerBreakpoints();
	}
}

void FStateTreeViewModel::RemoveAllBreakpoints()
{
	if (UStateTreeEditorData* TreeData = TreeDataWeak.Get())
	{
		Debugger->ClearAllBreakpoints();

		TreeData->RemoveAllBreakpoints();
	}
}

void FStateTreeViewModel::RefreshDebuggerBreakpoints()
{
	const UStateTree* StateTree = GetStateTree();
	const UStateTreeEditorData* TreeData = TreeDataWeak.Get();
	if (StateTree != nullptr && TreeData != nullptr)
	{
		Debugger->ClearAllBreakpoints();

		for (const FStateTreeEditorBreakpoint& Breakpoint : TreeData->Breakpoints)
		{
			// Test if the ID is associated to a task
			const FStateTreeIndex16 Index = StateTree->GetNodeIndexFromId(Breakpoint.ID);
			if (Index.IsValid())
			{
				Debugger->SetTaskBreakpoint(Index, Breakpoint.BreakpointType);
			}
			else
			{
				// Then test if the ID is associated to a State
				FStateTreeStateHandle StateHandle = StateTree->GetStateHandleFromId(Breakpoint.ID);
				if (StateHandle.IsValid())
				{
					Debugger->SetStateBreakpoint(StateHandle, Breakpoint.BreakpointType);
				}
				else
				{
					// Then test if the ID is associated to a transition
					const FStateTreeIndex16 TransitionIndex = StateTree->GetTransitionIndexFromId(Breakpoint.ID);
					if (TransitionIndex.IsValid())
					{
						Debugger->SetTransitionBreakpoint(TransitionIndex, Breakpoint.BreakpointType);
					}
				}
			}
		}
	}
}

#endif // WITH_STATETREE_TRACE_DEBUGGER

void FStateTreeViewModel::NotifyAssetChangedExternally() const
{
	OnAssetChanged.Broadcast();
}

void FStateTreeViewModel::NotifyStatesChangedExternally(const TSet<UStateTreeState*>& ChangedStates, const FPropertyChangedEvent& PropertyChangedEvent) const
{
	OnStatesChanged.Broadcast(ChangedStates, PropertyChangedEvent);
}

TArray<TObjectPtr<UStateTreeState>>* FStateTreeViewModel::GetSubTrees() const
{
	UStateTreeEditorData* TreeData = TreeDataWeak.Get();
	return TreeData != nullptr ? &TreeData->SubTrees : nullptr;
}

int32 FStateTreeViewModel::GetSubTreeCount() const
{
	UStateTreeEditorData* TreeData = TreeDataWeak.Get();
	return TreeData != nullptr ? TreeData->SubTrees.Num() : 0;
}

void FStateTreeViewModel::GetSubTrees(TArray<TWeakObjectPtr<UStateTreeState>>& OutSubtrees) const
{
	OutSubtrees.Reset();
	if (UStateTreeEditorData* TreeData = TreeDataWeak.Get())
	{
		for (UStateTreeState* Subtree : TreeData->SubTrees)
		{
			OutSubtrees.Add(Subtree);
		}
	}
}

void FStateTreeViewModel::GetLinkStates(FGuid StateID, TArray<FGuid>& LinkingIn, TArray<FGuid>& LinkedOut) const
{
	const UStateTreeState* State = GetStateByID(StateID);
	if (State == nullptr)
	{
		return;
	}

	if (UStateTreeEditorData* TreeData = TreeDataWeak.Get())
	{
		TreeData->VisitHierarchy([&LinkingIn, &LinkedOut, StateID = State->ID](UStateTreeState& State, UStateTreeState* ParentState)
			{
				if (State.ID == StateID)
				{
					return EStateTreeVisitor::Continue;
				}
				if (State.Type == EStateTreeStateType::Linked && StateID == State.LinkedSubtree.ID)
				{
					LinkingIn.AddUnique(State.ID);
				}
				else
				{
					for (const FStateTreeTransition& Transition : State.Transitions)
					{
						if (Transition.State.ID == StateID)
						{
							LinkingIn.AddUnique(State.ID);
						}
					}
				}
				return EStateTreeVisitor::Continue;
			});

		if (State->Type == EStateTreeStateType::Linked)
		{
			LinkedOut.AddUnique(State->LinkedSubtree.ID);
		}

		for (const FStateTreeTransition& Transition : State->Transitions)
		{
			LinkedOut.AddUnique(Transition.State.ID);
		}
	}
}

UStateTreeState* FStateTreeViewModel::GetTransitionToState(TNotNull<const UStateTreeState*> TransitionOwningState, const FGuid TransitionID) const
{
	const FStateTreeTransition* Transition = TransitionOwningState->GetTransitionByID(TransitionID);
	ensure(Transition);
	if (Transition == nullptr)
	{
		return nullptr;
	}

	switch (Transition->State.LinkType)
	{
	case EStateTreeTransitionType::GotoState:
		return GetMutableStateByID(Transition->State.ID);
	case EStateTreeTransitionType::Parent:
		return TransitionOwningState->Parent ? TransitionOwningState->Parent.Get() : nullptr;
	case EStateTreeTransitionType::NextState:
	case EStateTreeTransitionType::NextSelectableState:
		return TransitionOwningState->GetNextSelectableSiblingState();
	case EStateTreeTransitionType::NextParent:
	case EStateTreeTransitionType::NextSelectableParent:
		return TransitionOwningState->Parent ? TransitionOwningState->Parent->GetNextSelectableSiblingState() : nullptr;
	case EStateTreeTransitionType::None:
	case EStateTreeTransitionType::Succeeded:
	case EStateTreeTransitionType::Failed:
	default:
		return nullptr;
	}
}

void FStateTreeViewModel::PostUndo(bool bSuccess)
{
	// TODO: see if we can narrow this down.
	NotifyAssetChangedExternally();

	if (UStateTreeEditorData* TreeData = TreeDataWeak.Get())
	{
		UStateTreeEditingSubsystem::MarkAsModified(TreeData->GetTypedOuter<UStateTree>());
	}
}

void FStateTreeViewModel::PostRedo(bool bSuccess)
{
	NotifyAssetChangedExternally();

	if (UStateTreeEditorData* TreeData = TreeDataWeak.Get())
	{
		UStateTreeEditingSubsystem::MarkAsModified(TreeData->GetTypedOuter<UStateTree>());
	}
}

void FStateTreeViewModel::ClearSelection()
{
	if (SelectedStates.IsEmpty())
	{
		return;
	}

	SelectedStates.Reset();

	const TArray<TWeakObjectPtr<UStateTreeState>> SelectedStatesArr;
	LastSelectedState = FGuid();
	OnSelectionChanged.Broadcast(SelectedStatesArr);
}

void FStateTreeViewModel::SetSelection(UStateTreeState* Selected)
{
	if (SelectedStates.Num() == 1 && SelectedStates.Contains(Selected))
	{
		return;
	}

	SelectedStates.Reset();
	if (Selected != nullptr)
	{
		SelectedStates.Add(Selected);
	}
	
	LastSelectedState = Selected ? Selected->ID : FGuid();
	OnSelectionChanged.Broadcast({ Selected });
}

void FStateTreeViewModel::DeleteNode(TWeakObjectPtr<UStateTreeState> State, const FGuid& ID)
{
	if (UStateTreeEditorData* EditorData = TreeDataWeak.Get())
	{
		if (UStateTreeState* OwnerState = State.Get())
		{
			const UE::StateTree::Editor::Private::FStateTreeStateNodeEditorHandle StateNodeHandle(EditorData, OwnerState, ID);
			// If the op can not be executed, don't even start the transaction
			if (!StateNodeHandle.IsValid())
			{
				return;
			}

			auto DeleteFunc = [&StateNodeHandle](TNotNull<UStateTreeState*> InOwnerState, TNotNull<UStateTreeEditorData*> InEditorData, const FStateTreeEditPropertyPath& InPropertyPath)
				{
					StateNodeHandle.EditorData->Modify();

					if (StateNodeHandle.IsTransition())
					{
						StateNodeHandle.GetTransitionArray().RemoveAt(StateNodeHandle.GetNodeIndex());
					}
					else
					{
						StateNodeHandle.GetEditorNodeArray().RemoveAt(StateNodeHandle.GetNodeIndex());
					}

					InEditorData->RemoveInvalidBindings();
				};

			UE::StateTree::PropertyHelpers::ModifyStateInPreAndPostEdit(
				LOCTEXT("DeleteNodeTransaction", "Delete Node"),
				StateNodeHandle.OwnerState,
				StateNodeHandle.EditorData,
				StateNodeHandle.GetNodePath(),
				DeleteFunc,
				StateNodeHandle.GetNodeIndex(),
				EPropertyChangeType::ArrayRemove);


			OnStateNodesChanged.Broadcast(OwnerState);
			UStateTreeEditingSubsystem::MarkAsModified(EditorData->GetTypedOuter<UStateTree>());
		}
	}
}

void FStateTreeViewModel::DeleteAllNodes(TWeakObjectPtr<UStateTreeState> State, const FGuid& ID)
{
	if (UStateTreeEditorData* EditorData = TreeDataWeak.Get())
	{
		if (UStateTreeState* OwnerState = State.Get())
		{
			UE::StateTree::Editor::Private::FStateTreeStateNodeEditorHandle StateNodeHandle(EditorData, OwnerState, ID);
			// If the op can not be executed, don't even start the transaction
			if (!StateNodeHandle.IsValid())
			{
				return;
			}

			auto DeleteAllFunc = [&StateNodeHandle](const TNotNull<UStateTreeState*> InOwnerState, const TNotNull<UStateTreeEditorData*> InEditorData, const FStateTreeEditPropertyPath& InPropertyPath)
				{
					StateNodeHandle.EditorData->Modify();

					if (StateNodeHandle.IsTransition())
					{
						StateNodeHandle.GetTransitionArray().Empty();
					}
					else
					{
						StateNodeHandle.GetEditorNodeArray().Empty();
					}

					InEditorData->RemoveInvalidBindings();
				};

			UE::StateTree::PropertyHelpers::ModifyStateInPreAndPostEdit(
				LOCTEXT("DeleteAllNodesTransaction", "Delete All Nodes"),
				StateNodeHandle.OwnerState,
				StateNodeHandle.EditorData,
				StateNodeHandle.GetNodePath(),
				DeleteAllFunc,
				INDEX_NONE, // Pass Invalid Index to Array Clear Op
				EPropertyChangeType::ArrayClear);


			OnStateNodesChanged.Broadcast(OwnerState);
			UStateTreeEditingSubsystem::MarkAsModified(EditorData->GetTypedOuter<UStateTree>());
		}
	}
}

void FStateTreeViewModel::CopyNode(TWeakObjectPtr<UStateTreeState> State, const FGuid& ID)
{
	if (UStateTreeEditorData* EditorData = TreeDataWeak.Get())
	{
		if (UStateTreeState* OwnerState = State.Get())
		{
			UE::StateTree::Editor::Private::FStateTreeStateNodeEditorHandle StateNodeHandle(EditorData, OwnerState, ID);
			// If the op can not be executed, don't even start the transaction
			if (!StateNodeHandle.IsValid())
			{
				return;
			}

			UE::StateTreeEditor::FClipboardEditorData ClipboardEditorData;
			if (StateNodeHandle.IsTransition())
			{
				ClipboardEditorData.Append(EditorData, TConstArrayView<FStateTreeTransition>(&StateNodeHandle.GetTransition(), 1));
			}
			else
			{
				ClipboardEditorData.Append(EditorData, TConstArrayView<FStateTreeEditorNode>(&StateNodeHandle.GetEditorNode(), 1));
			}
			UE::StateTreeEditor::ExportTextAsClipboardEditorData(ClipboardEditorData);
		}
	}
}

void FStateTreeViewModel::CopyAllNodes(TWeakObjectPtr<UStateTreeState> State, const FGuid& ID)
{
	if (UStateTreeEditorData* EditorData = TreeDataWeak.Get())
	{
		if (UStateTreeState* OwnerState = State.Get())
		{
			UE::StateTree::Editor::Private::FStateTreeStateNodeEditorHandle StateNodeHandle(EditorData, OwnerState, ID);
			// If the op can not be executed, don't even start the transaction
			if (!StateNodeHandle.IsValid())
			{
				return;
			}

			UE::StateTreeEditor::FClipboardEditorData ClipboardEditorData;
			if (StateNodeHandle.IsTransition())
			{
				ClipboardEditorData.Append(EditorData, StateNodeHandle.GetTransitionArray());
			}
			else
			{
				ClipboardEditorData.Append(EditorData, StateNodeHandle.GetEditorNodeArray());
			}

			UE::StateTreeEditor::ExportTextAsClipboardEditorData(ClipboardEditorData);
		}
	}
}

void FStateTreeViewModel::PasteNode(TWeakObjectPtr<UStateTreeState> State, const FGuid& ID)
{
	if (UStateTreeEditorData* EditorData = TreeDataWeak.Get())
	{
		if (UStateTreeState* OwnerState = State.Get())
		{
			UE::StateTree::Editor::Private::FStateTreeStateNodeEditorHandle StateNodeHandle(EditorData, OwnerState, ID);
			// If the op can not be executed, don't even start the transaction
			if (!StateNodeHandle.IsValid())
			{
				return;
			}

			const UScriptStruct* TargetType = nullptr;
			if (StateNodeHandle.IsTransition())
			{
				TargetType = TBaseStructure<FStateTreeTransition>::Get();
			}
			else
			{
				static const UScriptStruct* TaskBaseStruct = FStateTreeTaskBase::StaticStruct();
				static const UScriptStruct* ConditionBaseStruct = FStateTreeConditionBase::StaticStruct();
				static const UScriptStruct* ConsiderationBaseStruct = FStateTreeConsiderationBase::StaticStruct();

				const UScriptStruct* BaseNodeScriptStruct = nullptr;
				const UScriptStruct* NodeScriptStruct = StateNodeHandle.GetEditorNode().GetNode().GetScriptStruct();
				if (NodeScriptStruct->IsChildOf(TaskBaseStruct))
				{
					BaseNodeScriptStruct = TaskBaseStruct;
				}
				else if (NodeScriptStruct->IsChildOf(ConditionBaseStruct))
				{
					BaseNodeScriptStruct = ConditionBaseStruct;
				}
				else if (NodeScriptStruct->IsChildOf(ConsiderationBaseStruct))
				{
					BaseNodeScriptStruct = ConsiderationBaseStruct;
				}

				TargetType = BaseNodeScriptStruct;
			}

			UE::StateTreeEditor::FClipboardEditorData ClipboardEditorData;
			const bool bSucceeded = UE::StateTreeEditor::ImportTextAsClipboardEditorData(TargetType, EditorData, OwnerState, ClipboardEditorData);

			if (!bSucceeded)
			{
				return;
			}

			if (StateNodeHandle.IsTransition() && ClipboardEditorData.GetTransitionsInBuffer().IsEmpty())
			{
				return;
			}

			if (!StateNodeHandle.IsTransition() && ClipboardEditorData.GetEditorNodesInBuffer().IsEmpty())
			{
				return;
			}

			auto PasteFunc = [&StateNodeHandle, &ClipboardEditorData](const TNotNull<UStateTreeState*> InOwnerState, const TNotNull<UStateTreeEditorData*> InEditorData, const FStateTreeEditPropertyPath& InPropertyPath)
				{
					auto PasteFuncInternal = [InEditorData, &ClipboardEditorData]<typename T>(TArray<T>& Dest, TArrayView<T> Source, int32 DestIndex) requires
						std::is_same_v<T, FStateTreeEditorNode> || std::is_same_v<T, FStateTreeTransition>
					{
						check(Source.Num());

						const int32 DestStartIndex = DestIndex;
						Dest[DestIndex++] = MoveTemp(Source[0]);

						Dest.InsertUninitialized(DestIndex, Source.Num() - 1);
						for (int32 SourceIndex = 1; SourceIndex < Source.Num(); SourceIndex++, DestIndex++)
						{
							new (Dest.GetData() + DestIndex) T(MoveTemp(Source[SourceIndex]));
						}

						for (FStateTreePropertyPathBinding& Binding : ClipboardEditorData.GetBindingsInBuffer())
						{
							InEditorData->GetPropertyEditorBindings()->AddStateTreeBinding(MoveTemp(Binding));
						}
					};

					InEditorData->Modify();

					if (StateNodeHandle.IsTransition())
					{
						PasteFuncInternal(StateNodeHandle.GetTransitionArray(), ClipboardEditorData.GetTransitionsInBuffer(), StateNodeHandle.GetNodeIndex());
					}
					else
					{
						PasteFuncInternal(StateNodeHandle.GetEditorNodeArray(), ClipboardEditorData.GetEditorNodesInBuffer(), StateNodeHandle.GetNodeIndex());
					}
				};

			UE::StateTree::PropertyHelpers::ModifyStateInPreAndPostEdit(
				LOCTEXT("PasteNodeTransaction", "Paste Node"),
				StateNodeHandle.OwnerState,
				StateNodeHandle.EditorData,
				StateNodeHandle.GetNodePath(),
				PasteFunc,
				INDEX_NONE, // Value Set Op, skip the index
				EPropertyChangeType::ValueSet);

			OnStateNodesChanged.Broadcast(OwnerState);
			UStateTreeEditingSubsystem::MarkAsModified(EditorData->GetTypedOuter<UStateTree>());
		}
	}
}

void FStateTreeViewModel::PasteNodesToSelectedStates()
{
	TArray<UStateTreeState*> CurrentlySelectedStates;
	GetSelectedStates(CurrentlySelectedStates);
	PasteNodesToStates(CurrentlySelectedStates);
}

void FStateTreeViewModel::PasteNodesToStates(TConstArrayView<UStateTreeState*> States)
{
	if (UStateTreeEditorData* EditorData = TreeDataWeak.Get())
	{
		if (!States.IsEmpty())
		{
			using namespace UE::StateTreeEditor;
			TArray<FClipboardEditorData, TInlineAllocator<4>> Clipboards;
			Clipboards.AddDefaulted(States.Num());

			for (int32 Idx = 0; Idx < States.Num(); ++Idx)
			{
				FClipboardEditorData& Clipboard = Clipboards[Idx];

				// any editor nodes and transitions
				constexpr const UScriptStruct* TargetType = nullptr;
				ImportTextAsClipboardEditorData(TargetType, EditorData, States[Idx], Clipboard);

				if (!Clipboard.IsValid())
				{
					return;
				}
			}

			FScopedTransaction ScopedTransaction(LOCTEXT("PasteNodesToSelectedStatesTransaction", "Paste Nodes To Selected States"));
			EditorData->Modify();

			for (int32 Idx = 0; Idx < States.Num(); ++Idx)
			{
				UStateTreeState* SelectedState = States[Idx];

				SelectedState->Modify();

				PasteNodesToState(EditorData, SelectedState, Clipboards[Idx]);
			}
		}
	}
}

void FStateTreeViewModel::MoveNode(TWeakObjectPtr<UStateTreeState> From, TWeakObjectPtr<UStateTreeState> To, const FGuid& ID)
{
	if (UStateTreeEditorData* EditorData = TreeDataWeak.Get())
	{
		UStateTreeState* FromState = From.Get();
		UStateTreeState* ToState = To.Get();
		if (FromState && ToState)
		{
			FScopedTransaction ScopedTransaction(LOCTEXT("MoveNodeTransaction", "Move Node To New State"));
			const UE::StateTree::Editor::Private::FStateTreeStateNodeEditorHandle StateNodeHandle(EditorData, FromState, ID);
			if (!StateNodeHandle.IsValid())
			{
				return;
			}
			
			FromState->Modify();
			ToState->Modify();
			StateNodeHandle.EditorData->Modify();
			
			if (StateNodeHandle.IsTransition())
			{
				FStateTreeTransition& Transition = StateNodeHandle.GetTransitionArray()[StateNodeHandle.GetNodeIndex()];
				UE::StateTree::PropertyHelpers::ModifyStateInPreAndPostEdit(
					LOCTEXT("MoveNodeToState", "Move Node To State"),
					ToState,
					EditorData,
					TEXT("Transitions"),
					[&Transition](TNotNull<UStateTreeState*> InOwnerState, TNotNull<UStateTreeEditorData*> InEditorData, const FStateTreeEditPropertyPath& InPropertyPath)
					{
						InOwnerState->Transitions.Add(MoveTemp(Transition));
					},
					INDEX_NONE,
					EPropertyChangeType::ValueSet);
				DeleteNode(From, ID);
			}
			else
			{
				TArray<FStateTreeEditorNode>* TargetArray = nullptr;
				const UScriptStruct* NodeType = StateNodeHandle.GetEditorNode().GetNode().GetScriptStruct();
				FString RelativeNodePath;
				if (NodeType->IsChildOf<FStateTreeTaskBase>())
				{
					TargetArray = &ToState->Tasks;
					RelativeNodePath = "Tasks";
				}
				else if (NodeType->IsChildOf<FStateTreeConditionBase>())
				{
					TargetArray = &ToState->EnterConditions;
					RelativeNodePath = "EnterConditions";
				}
				else if (NodeType->IsChildOf<FStateTreeConsiderationBase>())
				{
					TargetArray = &ToState->Considerations;
					RelativeNodePath = "Considerations";
				}
				
				FStateTreeEditorNode& EditorNode = StateNodeHandle.GetEditorNodeArray()[StateNodeHandle.GetNodeIndex()];
				UE::StateTree::PropertyHelpers::ModifyStateInPreAndPostEdit(
					LOCTEXT("MoveNodeToState", "Move Node To State"),
					ToState,
					EditorData,
					RelativeNodePath,
					[&TargetArray, &EditorNode](TNotNull<UStateTreeState*> InOwnerState, TNotNull<UStateTreeEditorData*> InEditorData, const FStateTreeEditPropertyPath& InPropertyPath)
					{
						TargetArray->Add(MoveTemp(EditorNode));
					},
					INDEX_NONE,
					EPropertyChangeType::ValueSet);
				DeleteNode(From, ID);
			}
			
			OnStateNodesChanged.Broadcast(FromState);
			OnStateNodesChanged.Broadcast(ToState);
			UStateTreeEditingSubsystem::MarkAsModified(EditorData->GetTypedOuter<UStateTree>());
		}
	}
}

void FStateTreeViewModel::DuplicateNode(TWeakObjectPtr<UStateTreeState> State, const FGuid& ID)
{
	if (UStateTreeEditorData* EditorData = TreeDataWeak.Get())
	{
		if (UStateTreeState* OwnerState = State.Get())
		{
			UE::StateTree::Editor::Private::FStateTreeStateNodeEditorHandle StateNodeHandle(EditorData, OwnerState, ID);
			// If the op can not be executed, don't even start the transaction
			if (!StateNodeHandle.IsValid())
			{
				return;
			}

			auto DuplicateFunc = [&StateNodeHandle](const TNotNull<UStateTreeState*> InOwnerState, const TNotNull<UStateTreeEditorData*> InEditorData, const FStateTreeEditPropertyPath& InPropertyPath)
				{
					int32 DestIndex = StateNodeHandle.GetNodeIndex();
					UE::StateTreeEditor::FClipboardEditorData Clipboard;

					auto DuplicateFuncInternal = [InEditorData, &Clipboard]<typename T>(TArray<T>& Dest, int32 Index, TArrayView<T> Source)
						requires std::is_same_v<T, FStateTreeEditorNode> || std::is_same_v<T, FStateTreeTransition>
					{
						check(Source.Num());
						Dest.InsertUninitialized(Index, Source.Num());

						T* BaseAddress = Dest.GetData() + Index;
						for (int32 SourceIdx = 0; SourceIdx < Source.Num(); ++SourceIdx)
						{
							::new (BaseAddress + SourceIdx) T(MoveTemp(Source[SourceIdx]));
						}

						for (FStateTreePropertyPathBinding& Binding : Clipboard.GetBindingsInBuffer())
						{
							InEditorData->GetPropertyEditorBindings()->AddStateTreeBinding(MoveTemp(Binding));
						}
					};

					InEditorData->Modify();

					if (StateNodeHandle.IsTransition())
					{
						FStateTreeTransition& SourceTransition = StateNodeHandle.GetTransition();

						Clipboard.Append(InEditorData, TConstArrayView<FStateTreeTransition>(&SourceTransition, 1));
						Clipboard.ProcessBuffer(nullptr, InEditorData, InOwnerState);

						if (!Clipboard.IsValid())
						{
							return;
						}

						DuplicateFuncInternal(StateNodeHandle.GetTransitionArray(), DestIndex, Clipboard.GetTransitionsInBuffer());
					}
					else
					{
						FStateTreeEditorNode& SourceEditorNode = StateNodeHandle.GetEditorNode();

						Clipboard.Append(InEditorData, TConstArrayView<FStateTreeEditorNode>(&SourceEditorNode, 1));
						Clipboard.ProcessBuffer(nullptr, InEditorData, InOwnerState);

						if (!Clipboard.IsValid())
						{
							return;
						}

						DuplicateFuncInternal(StateNodeHandle.GetEditorNodeArray(), DestIndex, Clipboard.GetEditorNodesInBuffer());
					}
				};

			UE::StateTree::PropertyHelpers::ModifyStateInPreAndPostEdit(
				LOCTEXT("DuplicateNodeTransaction", "Duplicate Node"),
				StateNodeHandle.OwnerState,
				StateNodeHandle.EditorData,
				StateNodeHandle.GetNodePath(),
				DuplicateFunc,
				StateNodeHandle.GetNodeIndex(),
				EPropertyChangeType::Duplicate);

			OnStateNodesChanged.Broadcast(OwnerState);
			UStateTreeEditingSubsystem::MarkAsModified(EditorData->GetTypedOuter<UStateTree>());
		}
	}
}

void FStateTreeViewModel::SetSelection(const TArray<TWeakObjectPtr<UStateTreeState>>& InSelectedStates)
{
	if (SelectedStates.Num() == InSelectedStates.Num() && SelectedStates.Array() == InSelectedStates)
	{
		return;
	}

	SelectedStates.Reset();

	for (const TWeakObjectPtr<UStateTreeState>& State : InSelectedStates)
	{
		if (State.Get())
		{
			SelectedStates.Add(State);
			LastSelectedState = State->ID;
		}
	}

	OnSelectionChanged.Broadcast(InSelectedStates);
}

bool FStateTreeViewModel::IsSelected(const UStateTreeState* State) const
{
	const TWeakObjectPtr<UStateTreeState> WeakState = const_cast<UStateTreeState*>(State);
	return SelectedStates.Contains(WeakState);
}

bool FStateTreeViewModel::IsChildOfSelection(const UStateTreeState* State) const
{
	for (const TWeakObjectPtr<UStateTreeState>& WeakSelectedState : SelectedStates)
	{
		if (const UStateTreeState* SelectedState = Cast<UStateTreeState>(WeakSelectedState.Get()))
		{
			if (SelectedState == State)
			{
				return true;
			}
			
			if (UE::StateTree::Editor::IsChildOf(SelectedState, State))
			{
				return true;
			}
		}
	}
	return false;
}

void FStateTreeViewModel::GetSelectedStates(TArray<UStateTreeState*>& OutSelectedStates) const
{
	OutSelectedStates.Reset();
	for (const TWeakObjectPtr<UStateTreeState>& WeakState : SelectedStates)
	{
		if (UStateTreeState* State = WeakState.Get())
		{
			OutSelectedStates.Add(State);
		}
	}
}

void FStateTreeViewModel::GetSelectedStates(TArray<TWeakObjectPtr<UStateTreeState>>& OutSelectedStates) const
{
	OutSelectedStates.Reset();
	for (const TWeakObjectPtr<UStateTreeState>& WeakState : SelectedStates)
	{
		if (WeakState.Get())
		{
			OutSelectedStates.Add(WeakState);
		}
	}
}

bool FStateTreeViewModel::HasSelection() const
{
	return SelectedStates.Num() > 0;
}

UStateTreeState* FStateTreeViewModel::GetLastSelectedState() const
{
	return GetMutableStateByID(LastSelectedState);
}

void FStateTreeViewModel::BringNodeToFocus(UStateTreeState* State, const FGuid NodeID)
{
	SetSelection(State);
	OnBringNodeToFocus.Broadcast(State, NodeID);
}

void FStateTreeViewModel::BringBindingPathToFocus(const FPropertyBindingPath& InBindingPath)
{
	if (UStateTreeEditorData* TreeData = TreeDataWeak.Get())
	{
		UStateTreeState* BindingPathOwnerState = const_cast<UStateTreeState*>(TreeData->GetStateByStructID(InBindingPath.GetStructID()));
		if (BindingPathOwnerState)
		{
			SetSelection(BindingPathOwnerState);
		}

		OnBringBindingPathToFocus.Broadcast(BindingPathOwnerState, InBindingPath);
	}
}

UStateTreeState* FStateTreeViewModel::GetViewState() const
{
	UStateTreeState* ViewState = GetLastSelectedState();

	// Try to view the first root when we have no selection
	if (!ViewState)
	{
		TArray<TWeakObjectPtr<UStateTreeState>> OutSubtrees;
		GetSubTrees(OutSubtrees);

		if (!OutSubtrees.IsEmpty())
		{
			ViewState = OutSubtrees[0].Get();
		}
	}

	return ViewState;
}

void FStateTreeViewModel::GetPersistentExpandedStates(TSet<TWeakObjectPtr<UStateTreeState>>& OutExpandedStates)
{
	OutExpandedStates.Reset();
	if (UStateTreeEditorData* TreeData = TreeDataWeak.Get())
	{
		for (UStateTreeState* SubTree : TreeData->SubTrees)
		{
			GetExpandedStatesRecursive(SubTree, OutExpandedStates);
		}
	}
}

void FStateTreeViewModel::GetExpandedStatesRecursive(UStateTreeState* State, TSet<TWeakObjectPtr<UStateTreeState>>& OutExpandedStates)
{
	if (State->bExpanded)
	{
		OutExpandedStates.Add(State);
	}
	for (UStateTreeState* Child : State->Children)
	{
		GetExpandedStatesRecursive(Child, OutExpandedStates);
	}
}


void FStateTreeViewModel::SetPersistentExpandedStates(TSet<TWeakObjectPtr<UStateTreeState>>& InExpandedStates)
{
	UStateTreeEditorData* TreeData = TreeDataWeak.Get();
	if (TreeData == nullptr)
	{
		return;
	}

	TreeData->Modify();

	for (TWeakObjectPtr<UStateTreeState>& WeakState : InExpandedStates)
	{
		if (UStateTreeState* State = WeakState.Get())
		{
			State->bExpanded = true;
		}
	}
}


void FStateTreeViewModel::AddState(UStateTreeState* AfterState)
{
	UStateTreeEditorData* TreeData = TreeDataWeak.Get();
	if (TreeData == nullptr)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddStateTransaction", "Add State"));

	UStateTreeState* NewState = NewObject<UStateTreeState>(TreeData, FName(), RF_Transactional);
	UStateTreeState* ParentState = nullptr;

	if (AfterState == nullptr)
	{
		// If no subtrees, add a subtree, or add to the root state.
		if (TreeData->SubTrees.IsEmpty())
		{
			TreeData->Modify();
			TreeData->SubTrees.Add(NewState);
		}
		else
		{
			UStateTreeState* RootState = TreeData->SubTrees[0];
			if (ensureMsgf(RootState, TEXT("%s: Root state is empty."), *GetNameSafe(TreeData->GetOuter())))
			{
				RootState->Modify();
				RootState->Children.Add(NewState);
				NewState->Parent = RootState;
				ParentState = RootState;
			}
		}
	}
	else
	{
		ParentState = AfterState->Parent;
		if (ParentState != nullptr)
		{
			ParentState->Modify();
		}
		else
		{
			TreeData->Modify();
		}

		TArray<TObjectPtr<UStateTreeState>>& ParentArray = ParentState ? ParentState->Children : TreeData->SubTrees;

		const int32 TargetIndex = ParentArray.Find(AfterState);
		if (TargetIndex != INDEX_NONE)
		{
			// Insert After
			ParentArray.Insert(NewState, TargetIndex + 1);
			NewState->Parent = ParentState;
		}
		else
		{
			// Fallback, should never happen.
			ensureMsgf(false, TEXT("%s: Failed to find specified target state %s on state %s while adding new state."), *GetNameSafe(TreeData->GetOuter()), *GetNameSafe(AfterState), *GetNameSafe(ParentState));
			ParentArray.Add(NewState);
			NewState->Parent = ParentState;
		}
	}

	OnStateAdded.Broadcast(ParentState, NewState);
	UStateTreeEditingSubsystem::MarkAsModified(TreeData->GetTypedOuter<UStateTree>());
}

void FStateTreeViewModel::AddChildState(UStateTreeState* ParentState)
{
	UStateTreeEditorData* TreeData = TreeDataWeak.Get();
	if (TreeData == nullptr || ParentState == nullptr)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddChildStateTransaction", "Add Child State"));

	UStateTreeState* NewState = NewObject<UStateTreeState>(ParentState, FName(), RF_Transactional);

	ParentState->Modify();
	
	ParentState->Children.Add(NewState);
	NewState->Parent = ParentState;

	OnStateAdded.Broadcast(ParentState, NewState);
	UStateTreeEditingSubsystem::MarkAsModified(TreeData->GetTypedOuter<UStateTree>());
}

void FStateTreeViewModel::RenameState(UStateTreeState* State, FName NewName)
{
	if (State == nullptr)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("RenameTransaction", "Rename"));
	State->Modify();
	State->Name = NewName;

	TSet<UStateTreeState*> AffectedStates;
	AffectedStates.Add(State);

	FProperty* NameProperty = FindFProperty<FProperty>(UStateTreeState::StaticClass(), GET_MEMBER_NAME_CHECKED(UStateTreeState, Name));
	FPropertyChangedEvent PropertyChangedEvent(NameProperty, EPropertyChangeType::ValueSet);
	NotifyStatesChangedExternally(AffectedStates, PropertyChangedEvent);

	if (UStateTreeEditorData* TreeData = TreeDataWeak.Get())
	{
		UStateTreeEditingSubsystem::MarkAsModified(TreeData->GetTypedOuter<UStateTree>());
	}
}

void FStateTreeViewModel::RemoveSelectedStates()
{
	UStateTreeEditorData* TreeData = TreeDataWeak.Get();
	if (TreeData == nullptr)
	{
		return;
	}

	TArray<UStateTreeState*> States;
	GetSelectedStates(States);

	// Remove items whose parent also exists in the selection.
	UE::StateTree::Editor::RemoveContainedChildren(States);

	if (States.Num() > 0)
	{
		const FScopedTransaction Transaction(LOCTEXT("DeleteStateTransaction", "Delete State"));

		TSet<UStateTreeState*> AffectedParents;

		for (UStateTreeState* StateToRemove : States)
		{
			if (StateToRemove)
			{
				StateToRemove->Modify();

				UStateTreeState* ParentState = StateToRemove->Parent;
				if (ParentState != nullptr)
				{
					AffectedParents.Add(ParentState);
					ParentState->Modify();
				}
				else
				{
					AffectedParents.Add(nullptr);
					TreeData->Modify();
				}
				
				TArray<TObjectPtr<UStateTreeState>>& ArrayToRemoveFrom = ParentState ? ParentState->Children : TreeData->SubTrees;
				const int32 ItemIndex = ArrayToRemoveFrom.Find(StateToRemove);
				if (ItemIndex != INDEX_NONE)
				{
					ArrayToRemoveFrom.RemoveAt(ItemIndex);
					StateToRemove->Parent = nullptr;
				}

			}
		}

		OnStatesRemoved.Broadcast(AffectedParents);
		UStateTreeEditingSubsystem::MarkAsModified(TreeData->GetTypedOuter<UStateTree>());

		ClearSelection();
	}
}

void FStateTreeViewModel::CopySelectedStates()
{
	UStateTreeEditorData* TreeData = TreeDataWeak.Get();
	if (TreeData == nullptr)
	{
		return;
	}

	TArray<UStateTreeState*> States;
	GetSelectedStates(States);
	UE::StateTree::Editor::RemoveContainedChildren(States);
	
	FString ExportedText = UE::StateTree::Editor::ExportStatesToText(TreeData, States);
	
	FPlatformApplicationMisc::ClipboardCopy(*ExportedText);
}

bool FStateTreeViewModel::CanPasteStatesFromClipboard() const
{
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);

	UE::StateTree::Editor::FStateTreeStateTextFactory Factory;
	return Factory.CanCreateObjectsFromText(TextToImport);
}

void FStateTreeViewModel::PasteStatesFromClipboard(UStateTreeState* AfterState)
{
	UStateTreeEditorData* TreeData = TreeDataWeak.Get();
	if (TreeData == nullptr)
	{
		return;
	}
	
	if (AfterState)
	{
		const int32 Index = AfterState->Parent ? AfterState->Parent->Children.Find(AfterState) : TreeData->SubTrees.Find(AfterState);
		if (Index != INDEX_NONE)
		{
			FString TextToImport;
			FPlatformApplicationMisc::ClipboardPaste(TextToImport);
			
			const FScopedTransaction Transaction(LOCTEXT("PasteStatesTransaction", "Paste State(s)"));
			PasteStatesAsChildrenFromText(TextToImport, AfterState->Parent, Index + 1);
		}
	}
}

void FStateTreeViewModel::PasteStatesAsChildrenFromClipboard(UStateTreeState* ParentState)
{
	UStateTreeEditorData* TreeData = TreeDataWeak.Get();
	if (TreeData == nullptr)
	{
		return;
	}
	
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);

	const FScopedTransaction Transaction(LOCTEXT("PasteStatesTransaction", "Paste State(s)"));
	PasteStatesAsChildrenFromText(TextToImport, ParentState, INDEX_NONE);
}

void FStateTreeViewModel::PasteStatesAsChildrenFromText(const FString& TextToImport, UStateTreeState* ParentState, const int32 IndexToInsertAt)
{
	UStateTreeEditorData* TreeData = TreeDataWeak.Get();
	if (TreeData == nullptr)
	{
		return;
	}

	UObject* Outer = ParentState ? static_cast<UObject*>(ParentState) : static_cast<UObject*>(TreeData);
	Outer->Modify();

	UE::StateTree::Editor::FStateTreeStateTextFactory Factory;
	Factory.ProcessBuffer(Outer, RF_Transactional, TextToImport);

	TArray<TObjectPtr<UStateTreeState>>& ParentArray = ParentState ? ParentState->Children : TreeData->SubTrees;
	const int32 TargetIndex = (IndexToInsertAt == INDEX_NONE) ? ParentArray.Num() : IndexToInsertAt;
	ParentArray.Insert(Factory.States, TargetIndex);

	TArray<FStateTreeStateLink*> Links;
	TMap<FGuid, FGuid> IDsMap;
	TArray<UStateTreeState*> NewStates;

	for (UStateTreeState* State : Factory.States)
	{		
		UE::StateTree::Editor::FixStateAfterDuplication(State, ParentState, IDsMap, Links, NewStates);
	}

	// Copy property bindings for the duplicated states.
	if (Factory.ClipboardBindings)
	{
		for (FStateTreePropertyPathBinding& Binding : Factory.ClipboardBindings->Bindings)
		{
			if (Binding.GetPropertyFunctionNode().IsValid())
			{
				UE::StateTree::Editor::FixNodesAfterDuplication(TArrayView<FStateTreeEditorNode>(Binding.GetMutablePropertyFunctionNode().GetPtr<FStateTreeEditorNode>(), 1), IDsMap, Links);
			}
		}

		for (const TPair<FGuid, FGuid>& Entry : IDsMap)
		{
			const FGuid OldTargetID = Entry.Key;
			const FGuid NewTargetID = Entry.Value;
			
			for (FStateTreePropertyPathBinding& Binding : Factory.ClipboardBindings->Bindings)
			{
				if (Binding.GetTargetPath().GetStructID() == OldTargetID)
				{
					Binding.GetMutableTargetPath().SetStructID(NewTargetID);

					if (const FGuid* NewSourceID = IDsMap.Find(Binding.GetSourcePath().GetStructID()))
					{
						Binding.GetMutableSourcePath().SetStructID(*NewSourceID);
					}
					
					TreeData->GetPropertyEditorBindings()->AddStateTreeBinding(MoveTemp(Binding));
				}
			}
		}
	}

	// Patch IDs in state links.
	for (FStateTreeStateLink* Link : Links)
	{
		if (FGuid* NewID = IDsMap.Find(Link->ID))
		{
			Link->ID = *NewID;
		}
	}

	for (UStateTreeState* State : NewStates)
	{
		OnStateAdded.Broadcast(State->Parent, State);
	}
	UStateTreeEditingSubsystem::MarkAsModified(TreeData->GetTypedOuter<UStateTree>());
}

void FStateTreeViewModel::DuplicateSelectedStates()
{
	UStateTreeEditorData* TreeData = TreeDataWeak.Get();
	if (TreeData == nullptr)
	{
		return;
	}

	TArray<UStateTreeState*> States;
	GetSelectedStates(States);
	UE::StateTree::Editor::RemoveContainedChildren(States);

	if (States.IsEmpty())
	{
		return;
	}
	
	FString ExportedText = UE::StateTree::Editor::ExportStatesToText(TreeData, States);

	// Place duplicates after first selected state.
	UStateTreeState* AfterState = States[0];
	
	const int32 Index = AfterState->Parent ? AfterState->Parent->Children.Find(AfterState) : TreeData->SubTrees.Find(AfterState);
	if (Index != INDEX_NONE)
	{
		const FScopedTransaction Transaction(LOCTEXT("DuplicateStatesTransaction", "Duplicate State(s)"));
		PasteStatesAsChildrenFromText(ExportedText, AfterState->Parent, Index + 1);
	}
}


void FStateTreeViewModel::MoveSelectedStatesBefore(UStateTreeState* TargetState)
{
	MoveSelectedStates(TargetState, EStateTreeViewModelInsert::Before);
}

void FStateTreeViewModel::MoveSelectedStatesAfter(UStateTreeState* TargetState)
{
	MoveSelectedStates(TargetState, EStateTreeViewModelInsert::After);
}

void FStateTreeViewModel::MoveSelectedStatesInto(UStateTreeState* TargetState)
{
	MoveSelectedStates(TargetState, EStateTreeViewModelInsert::Into);
}

bool FStateTreeViewModel::CanEnableStates() const
{
	TArray<UStateTreeState*> States;
	GetSelectedStates(States);

	for (const UStateTreeState* State : States)
	{
		// Stop if at least one state can be enabled
		if (State->bEnabled == false)
		{
			return true;
		}
	}

	return false;
}

bool FStateTreeViewModel::CanDisableStates() const
{
	TArray<UStateTreeState*> States;
	GetSelectedStates(States);

	for (const UStateTreeState* State : States)
	{
		// Stop if at least one state can be disabled
		if (State->bEnabled)
		{
			return true;
		}
	}

	return false;
}

bool FStateTreeViewModel::CanPasteNodesToSelectedStates() const
{
	UStateTreeEditorData* TreeData = TreeDataWeak.Get();
	if (!TreeData)
	{
		return false;
	}

	TArray<UStateTreeState*> States;
	GetSelectedStates(States);
	if (States.IsEmpty())
	{
		return false;
	}

	UE::StateTreeEditor::FClipboardEditorData ClipboardEditorData;

	constexpr const UScriptStruct* TargetType = nullptr;
	constexpr bool bProcessBuffer = false;
	const bool bSucceeded = UE::StateTreeEditor::ImportTextAsClipboardEditorData(TargetType, TreeData, States[0], ClipboardEditorData, bProcessBuffer);

	return bSucceeded && (ClipboardEditorData.GetEditorNodesInBuffer().Num() || ClipboardEditorData.GetTransitionsInBuffer().Num());
}

void FStateTreeViewModel::SetSelectedStatesEnabled(const bool bEnable)
{
	TArray<UStateTreeState*> States;
	GetSelectedStates(States);

	if (States.Num() > 0)
	{
		const FScopedTransaction Transaction(LOCTEXT("SetStatesEnabledTransaction", "Set State Enabled"));

		for (UStateTreeState* State : States)
		{
			State->Modify();
			State->bEnabled = bEnable;
		}

		NotifyAssetChangedExternally();
		if (UStateTreeEditorData* TreeData = TreeDataWeak.Get())
		{
			UStateTreeEditingSubsystem::MarkAsModified(TreeData->GetTypedOuter<UStateTree>());
		}
	}
}

void FStateTreeViewModel::ForEachParent(TNotNull<UStateTreeState*> State, TFunctionRef<EStateTreeVisitor(UStateTreeState& ParentState)> InFunc) const
{
	UStateTreeState* ParentState = State->Parent;
	while (ParentState)
	{
		EStateTreeVisitor ContinueVisitor = InFunc(*ParentState);
		if (ContinueVisitor != EStateTreeVisitor::Continue)
		{
			return;
		}

		ParentState = ParentState->Parent;
	}
}

void FStateTreeViewModel::ForEachInTransition(TNotNull<UStateTreeState*> State, TFunctionRef<EStateTreeVisitor(FStateTreeTransition& TransitionToState)> InFunc) const
{
	auto VisitStateTransitionsToState = [&](UStateTreeState& HierarchyState, UStateTreeState* ParentState)
	{
		for (FStateTreeTransition& Transition : HierarchyState.Transitions)
		{
			if (Transition.State.ID == State->GetLinkToState().ID)
			{
				EStateTreeVisitor ContinueVisitor = InFunc(Transition);
				if (ContinueVisitor != EStateTreeVisitor::Continue)
				{
					return EStateTreeVisitor::Break;
				}
			}
		}

		return EStateTreeVisitor::Continue;
	};

	if (const UStateTreeEditorData* TreeData = TreeDataWeak.Get())
	{
		TreeData->VisitHierarchy(VisitStateTransitionsToState);
	}
}

void FStateTreeViewModel::ForEachOutTransition(TNotNull<UStateTreeState*> State, TFunctionRef<EStateTreeVisitor(FStateTreeTransition& TransitionFromState)> InFunc) const
{
	// Note: Yes this is trivial. But the view doesn't need to know that.
	for (FStateTreeTransition& Transition : State->Transitions)
	{
		EStateTreeVisitor ContinueVisitor = InFunc(Transition);
		if (ContinueVisitor != EStateTreeVisitor::Continue)
		{
			return;
		}
	}
}

void FStateTreeViewModel::ForEachChild(TNotNull<UStateTreeState*> State, TFunctionRef<EStateTreeVisitor(UStateTreeState& ChildState)> InFunc) const
{
	// Note: Yes this is trivial. But the view doesn't need to know that.
	for (UStateTreeState* Child : State->Children)
	{
		if (Child)
		{
			EStateTreeVisitor ContinueVisitor = InFunc(*Child);
			if (ContinueVisitor != EStateTreeVisitor::Continue)
			{
				return;
			}
		}
	}
}

TArray<UStateTreeState*> FStateTreeViewModel::GetStateParents(TNotNull<UStateTreeState*> State) const
{
	TArray<UStateTreeState*> ParentStates;
	auto GatherParentStates = [&](UStateTreeState& ParentState)
	{
		ParentStates.Add(&ParentState);
		return EStateTreeVisitor::Continue;
	};

	ForEachParent(State, GatherParentStates);
	return ParentStates;
}

TArray<FStateTreeTransition*> FStateTreeViewModel::GetStateInTransitions(TNotNull<UStateTreeState*> State) const
{
	TArray<FStateTreeTransition*> TransitionsToState;
	auto GatherInTransitions = [&](FStateTreeTransition& Transition)
	{
		TransitionsToState.Add(&Transition);
		return EStateTreeVisitor::Continue;
	};

	ForEachInTransition(State, GatherInTransitions);
	return TransitionsToState;
}

TArray<FStateTreeTransition*> FStateTreeViewModel::GetStateOutTransitions(TNotNull<UStateTreeState*> State) const
{
	TArray<FStateTreeTransition*> TransitionsFromState;
	auto GatherOutTransitions = [&](FStateTreeTransition& Transition)
	{
			TransitionsFromState.Add(&Transition);
		return EStateTreeVisitor::Continue;
	};

	ForEachOutTransition(State, GatherOutTransitions);
	return TransitionsFromState;
}

TArray<UStateTreeState*> FStateTreeViewModel::GetStateChildren(TNotNull<UStateTreeState*> State) const
{
	TArray<UStateTreeState*> ChildStates;
	auto GatherChildStates = [&](UStateTreeState& ChildState)
	{
		ChildStates.Add(&ChildState);
		return EStateTreeVisitor::Continue;
	};

	ForEachChild(State, GatherChildStates);
	return ChildStates;
}

const UStateTreeState* FStateTreeViewModel::GetSourceStateFromTransition(TNotNull<const FStateTreeTransition*> Transition) const
{
	const UStateTreeState* SourceState = nullptr;

	auto VisitStateFindTransitionSource = [&](UStateTreeState& HierarchyState, UStateTreeState* ParentState)
	{
		auto TransitionCompareID = [&](const FStateTreeTransition& Other)
		{
			return Other.ID == Transition->ID;
		};

		if (HierarchyState.Transitions.ContainsByPredicate(TransitionCompareID))
		{
			SourceState = &HierarchyState;
			return EStateTreeVisitor::Break;
		}

		return EStateTreeVisitor::Continue;
	};

	if (const UStateTreeEditorData* TreeData = TreeDataWeak.Get())
	{
		TreeData->VisitHierarchy(VisitStateFindTransitionSource);
	}

	return SourceState;
}

const UStateTreeState* FStateTreeViewModel::GetSourceParentStateFromTransition(TNotNull<const FStateTreeTransition*> Transition) const
{
	if (const UStateTreeState* TransitionState = GetSourceStateFromTransition(Transition))
	{
		return TransitionState->Parent;
	}

	return nullptr;
}

const UStateTreeState* FStateTreeViewModel::GetTargetStateFromTransition(TNotNull<const FStateTreeTransition*> Transition) const
{
	if (const UStateTreeState* TransitionOwningState = GetSourceStateFromTransition(Transition))
	{
		return GetTransitionToState(TransitionOwningState, Transition->ID);
	}

	return nullptr;
}

const UStateTreeState* FStateTreeViewModel::GetTargetParentStateFromTransition(TNotNull<const FStateTreeTransition*> Transition) const
{
	if (const UStateTreeState* TransitionState = GetTargetStateFromTransition(Transition))
	{
		return TransitionState->Parent;
	}

	return nullptr;
}

FStateTreeTransition& FStateTreeViewModel::AddTransition(TNotNull<UStateTreeState*> State, const EStateTreeTransitionTrigger InTrigger, const EStateTreeTransitionType InType, const UStateTreeState* InState) const
{
	FStateTreeTransition* Result = nullptr;
	if (UStateTreeEditorData* EditorData = TreeDataWeak.Get())
	{
		static const FString TransitionsPropertyNodeName = GET_MEMBER_NAME_CHECKED(UStateTreeState, Transitions).ToString();

		auto AddFunc = [&](TNotNull<UStateTreeState*> InOwnerState, TNotNull<UStateTreeEditorData*> InEditorData, const FStateTreeEditPropertyPath& InPropertyPath)
		{
			Result = &State->AddTransition(InTrigger, InType, InState);
		};

		UE::StateTree::PropertyHelpers::ModifyStateInPreAndPostEdit(
			LOCTEXT("AddTransitionTransaction", "Add Transition"),
			State,
			EditorData,
			TransitionsPropertyNodeName,
			AddFunc,
			State->Transitions.Num(),
			EPropertyChangeType::ArrayAdd);
	}

	// No editor, still add the state
	if (!Result)
	{
		Result = &State->AddTransition(InTrigger, InType, InState);
	}

	return *Result;
}

FStateTreeTransition& FStateTreeViewModel::AddTransition(TNotNull<UStateTreeState*> State, const EStateTreeTransitionTrigger InTrigger, const FGameplayTag InEventTag, const EStateTreeTransitionType InType, const UStateTreeState* InState) const
{
	FStateTreeTransition* Result = nullptr;
	if (UStateTreeEditorData* EditorData = TreeDataWeak.Get())
	{
		static const FString TransitionsPropertyNodeName = GET_MEMBER_NAME_CHECKED(UStateTreeState, Transitions).ToString();

		auto AddFunc = [&](TNotNull<UStateTreeState*> InOwnerState, TNotNull<UStateTreeEditorData*> InEditorData, const FStateTreeEditPropertyPath& InPropertyPath)
		{
			Result = &State->AddTransition(InTrigger, InEventTag, InType, InState);
		};

		UE::StateTree::PropertyHelpers::ModifyStateInPreAndPostEdit(
			LOCTEXT("AddTransitionTransaction", "Add Transition"),
			State,
			EditorData,
			TransitionsPropertyNodeName,
			AddFunc,
			State->Transitions.Num(),
			EPropertyChangeType::ArrayAdd);
	}

	// No editor, still add the state
	if (!Result)
	{
		Result = &State->AddTransition(InTrigger, InEventTag, InType, InState);
	}

	return *Result;
}

void FStateTreeViewModel::RemoveTransition(TNotNull<UStateTreeState*> State, const FGuid TransitionID)
{
	// Same as delete node. This method exists to make API more intuitive
	DeleteNode(State, TransitionID);
}

FLinearColor FStateTreeViewModel::GetStateColor(TNotNull<const UStateTreeState*> State, EStateTreeViewModelColorAdjustment ColorAdjustment) const
{
	if (bEnableSimpleHierarchyDrivenColors)
	{
		return GetColorAdjusted(GetHierarchyBasedBackgroundColor(State), ColorAdjustment);
	}

	FLinearColor Result = UE::StateTree::Editor::DefaultStateColor;

	if (const UStateTreeEditorData* TreeData = TreeDataWeak.Get())
	{
		if (const FStateTreeEditorColor* StateColor = TreeData->FindColor(FStateTreeEditorColorRef(State->ColorRef)))
		{
			Result = StateColor->Color;
		}
	}

	return GetColorAdjusted(Result, ColorAdjustment);
}

FLinearColor FStateTreeViewModel::GetHierarchyBasedHighlightColor(TNotNull<const UStateTreeState*> State, EStateTreeViewModelColorAdjustment ColorAdjustment) const
{
	// If we are a leaf, then just use leaf color
	if (State->Children.IsEmpty())
	{
		return GetColorAdjusted(UE::StateTree::Editor::DefaultStateColor, ColorAdjustment);
	}

	int8 NumParents = 0;
	auto CountNumParents = [&](UStateTreeState& ParentState)
	{
		NumParents++;
		return NumParents == 1 ? EStateTreeVisitor::Break : EStateTreeVisitor::Continue;
	};
	ForEachParent(const_cast<UStateTreeState*>(NotNullGet(State)), CountNumParents);

	return GetColorAdjusted(NumParents > 0 ? UE::StateTree::Editor::DefaultParentColor : UE::StateTree::Editor::DefaultRootColor, ColorAdjustment);
}

FLinearColor FStateTreeViewModel::GetHierarchyBasedBackgroundColor(TNotNull<const UStateTreeState*> State, EStateTreeViewModelColorAdjustment ColorAdjustment) const
{
	// If we are a leaf, then just use leaf color
	if (State->Children.IsEmpty())
	{
		return GetColorAdjusted(UE::StateTree::Editor::DefaultStateColor, ColorAdjustment);
	}

	// We have 4 parent value levels and 1 root value level. Find our depth.
	int8 NumParents = 0;
	auto CountNumParents = [&](UStateTreeState& ParentState)
	{
		NumParents++;
		return NumParents == 4 ? EStateTreeVisitor::Break : EStateTreeVisitor::Continue;
	};
	ForEachParent(const_cast<UStateTreeState*>(NotNullGet(State)), CountNumParents);

	const FLinearColor DefaultParentColorHSV = UE::StateTree::Editor::DefaultParentColor.LinearRGBToHSV();
	const FLinearColor BrightnessScale = { 0.0f, 0.0f, 1.0f, 0.0f };

	FLinearColor Result = UE::StateTree::Editor::DefaultStateColor;
	switch (NumParents)
	{
	case 0:
		Result = UE::StateTree::Editor::DefaultRootColor;
		break;
	case 1:
		Result = UE::StateTree::Editor::DefaultParentColor;
		break;
	case 2:
		Result = (DefaultParentColorHSV - DefaultParentColorHSV * BrightnessScale + BrightnessScale * 0.515f).HSVToLinearRGB();
		break;
	case 3:
		Result = (DefaultParentColorHSV - DefaultParentColorHSV * BrightnessScale + BrightnessScale * 0.300f).HSVToLinearRGB();
		break;
	case 4:
		Result = (DefaultParentColorHSV - DefaultParentColorHSV * BrightnessScale + BrightnessScale * 0.150f).HSVToLinearRGB();
		break;
	default:
		Result = (DefaultParentColorHSV - DefaultParentColorHSV * BrightnessScale + BrightnessScale * 0.150f).HSVToLinearRGB();
		break;
	}

	return GetColorAdjusted(Result, ColorAdjustment);
}

FLinearColor FStateTreeViewModel::GetColorAdjusted(const FLinearColor& InColor, EStateTreeViewModelColorAdjustment ColorAdjustment) const
{
	FLinearColor ValueShift;
	FLinearColor ValueScale;
	FLinearColor InColorHSV = InColor.LinearRGBToHSV();

	switch (ColorAdjustment)
	{
	case EStateTreeViewModelColorAdjustment::Normal:
		return InColor;

	case EStateTreeViewModelColorAdjustment::Bright:
		ValueShift = { 0.0f, 0.0f, 0.5f, 0.0f };
		ValueScale = { 1.0f, 1.0f, 0.5f, 1.0f };
		return (InColorHSV * ValueScale + ValueShift).HSVToLinearRGB();

	case EStateTreeViewModelColorAdjustment::Brighter:
		ValueShift = { 0.0f, 0.0f, 0.7f, 0.0f };
		ValueScale = { 1.0f, 1.0f, 0.3f, 1.0f };
		return (InColorHSV * ValueScale + ValueShift).HSVToLinearRGB();

	case EStateTreeViewModelColorAdjustment::Dark:
		ValueScale = { 1.0f, 1.0f, 0.3f, 1.0f };
		return (InColorHSV * ValueScale).HSVToLinearRGB();

	case EStateTreeViewModelColorAdjustment::Darker:
		ValueScale = { 1.0f, 1.0f, 0.18f, 1.0f };
		return (InColorHSV * ValueScale).HSVToLinearRGB();

	default:
		return InColor;
	}
}

const FSlateBrush* FStateTreeViewModel::GetSelectorIcon(TNotNull<const UStateTreeState*> State) const
{
	return FStateTreeEditorStyle::GetBrushForSelectionBehaviorType(State->SelectionBehavior, !State->Children.IsEmpty(), State->Type);
}

FText FStateTreeViewModel::GetNodeDescription(const FStateTreeEditorNode& Node, const EStateTreeNodeFormatting Formatting) const
{
	if (const UStateTreeEditorData* TreeData = TreeDataWeak.Get())
	{
		return TreeData->GetNodeDescription(Node, Formatting);
	}

	return FText::GetEmpty();
}

void FStateTreeViewModel::MoveSelectedStates(UStateTreeState* TargetState, const EStateTreeViewModelInsert RelativeLocation)
{
	UStateTreeEditorData* TreeData = TreeDataWeak.Get();
	if (TreeData == nullptr || TargetState == nullptr)
	{
		return;
	}

	TArray<UStateTreeState*> States;
	GetSelectedStates(States);

	// Remove child items whose parent also exists in the selection.
	UE::StateTree::Editor::RemoveContainedChildren(States);

	// Remove states which contain target state as child.
	States.RemoveAll([TargetState](const UStateTreeState* State)
	{
		return UE::StateTree::Editor::IsChildOf(State, TargetState);
	});

	if (States.Num() > 0 && TargetState != nullptr)
	{
		const FScopedTransaction Transaction(LOCTEXT("MoveTransaction", "Move"));

		TSet<UStateTreeState*> AffectedParents;

		UStateTreeState* TargetParent = TargetState->Parent;
		if (RelativeLocation == EStateTreeViewModelInsert::Into)
		{
			AffectedParents.Add(TargetState);
		}
		else
		{
			AffectedParents.Add(TargetParent);
		}
		
		for (int32 i = States.Num() - 1; i >= 0; i--)
		{
			if (UStateTreeState* State = States[i])
			{
				State->Modify();
				AffectedParents.Add(State->Parent);
			}
		}

		if (RelativeLocation == EStateTreeViewModelInsert::Into)
		{
			// Move into
			TargetState->Modify();
		}
		
		for (UStateTreeState* Parent : AffectedParents)
		{
			if (Parent)
			{
				Parent->Modify();
			}
			else
			{
				TreeData->Modify();
			}
		}

		TSet<UStateTreeState*> AffectedStates;
		// Add in reverse order to keep the original order.
		for (int32 Idx = States.Num() - 1; Idx >= 0; Idx--)
		{
			if (UStateTreeState* SelectedState = States[Idx])
			{
				AffectedStates.Add(SelectedState);

				UStateTreeState* SelectedParent = SelectedState->Parent;

				// Remove from current parent
				TArray<TObjectPtr<UStateTreeState>>& ArrayToRemoveFrom = SelectedParent ? SelectedParent->Children : TreeData->SubTrees;
				const int32 ItemIndex = ArrayToRemoveFrom.Find(SelectedState);
				if (ItemIndex != INDEX_NONE)
				{
					ArrayToRemoveFrom.RemoveAt(ItemIndex);
					SelectedState->Parent = nullptr;
				}

				// Insert to new parent
				if (RelativeLocation == EStateTreeViewModelInsert::Into)
				{
					// Into
					TargetState->Children.Insert(SelectedState, /*Index*/0);
					SelectedState->Parent = TargetState;
				}
				else
				{
					TArray<TObjectPtr<UStateTreeState>>& ArrayToMoveTo = TargetParent ? TargetParent->Children : TreeData->SubTrees;
					const int32 TargetIndex = ArrayToMoveTo.Find(TargetState);
					if (TargetIndex != INDEX_NONE)
					{
						if (RelativeLocation == EStateTreeViewModelInsert::Before)
						{
							// Before
							const int32 IndexToInsertBefore = TargetIndex - (States.Num() - 1 - Idx);
							check(ArrayToMoveTo.IsValidIndex(IndexToInsertBefore));

							ArrayToMoveTo.Insert(SelectedState, IndexToInsertBefore);
							SelectedState->Parent = TargetParent;
						}
						else if (RelativeLocation == EStateTreeViewModelInsert::After)
						{
							// After
							ArrayToMoveTo.Insert(SelectedState, TargetIndex + 1);
							SelectedState->Parent = TargetParent;
						}
					}
					else
					{
						// Fallback, should never happen.
						ensureMsgf(false, TEXT("%s: Failed to find specified target state %s on state %s while moving a state."), *GetNameSafe(TreeData->GetOuter()), *GetNameSafe(TargetState), *GetNameSafe(SelectedParent));
						ArrayToMoveTo.Add(SelectedState);
						SelectedState->Parent = TargetParent;
					}
				}
			}
		}

		OnStatesMoved.Broadcast(AffectedParents, AffectedStates);
		UStateTreeEditingSubsystem::MarkAsModified(TreeData->GetTypedOuter<UStateTree>());

		TArray<TWeakObjectPtr<UStateTreeState>> WeakStates;
		for (UStateTreeState* State : States)
		{
			WeakStates.Add(State);
		}

		SetSelection(WeakStates);
	}
}


void FStateTreeViewModel::BindToDebuggerDelegates()
{
#if WITH_STATETREE_TRACE_DEBUGGER
	Debugger->OnActiveStatesChanged.BindSPLambda(this, [this](const FStateTreeTraceActiveStates& NewActiveStates)
	{
		if (NewActiveStates.PerAssetStates.Num() == 0)
		{
			ActiveStates.Empty();
		}
		else if (const UStateTree* OuterStateTree = GetStateTree())
		{
			for (const FStateTreeTraceActiveStates::FAssetActiveStates& AssetActiveStates : NewActiveStates.PerAssetStates)
			{
				// Only track states owned by the StateTree associated to the view model (skip linked assets)
				if (AssetActiveStates.WeakStateTree == OuterStateTree)
				{
					ActiveStates.Reset(AssetActiveStates.ActiveStates.Num());
					for (const FStateTreeStateHandle Handle : AssetActiveStates.ActiveStates)
					{
						ActiveStates.Add(OuterStateTree->GetStateIdFromHandle(Handle));
					}
				}
			}
		}
	});
#endif // WITH_STATETREE_TRACE_DEBUGGER
}

void FStateTreeViewModel::PasteNodesToState(TNotNull<UStateTreeEditorData*> InEditorData, TNotNull<UStateTreeState*> InState, UE::StateTreeEditor::FClipboardEditorData& InProcessedClipboard)
{
	auto AppendFunc = [&InProcessedClipboard]<typename T>(TNotNull<UStateTreeState*> InOwnerState, TNotNull<UStateTreeEditorData*> InEditorData, const FStateTreeEditPropertyPath & InPropertyPath)
		requires TIsDerivedFrom<T, FStateTreeNodeBase>::IsDerived || std::is_same_v<T, FStateTreeTransition>
	{
		if constexpr (std::is_same_v<T, FStateTreeTransition>)
		{
			TArrayView<FStateTreeTransition> TransitionsInBuffer = InProcessedClipboard.GetTransitionsInBuffer();

			for (FStateTreeTransition& Transition : TransitionsInBuffer)
			{
				InOwnerState->Transitions.Add(MoveTemp(Transition));
			}
		}
		else
		{
			const UScriptStruct* NodeType = TBaseStructure<T>::Get();

			TArray<FStateTreeEditorNode>* TargetArray = nullptr;
			if (NodeType->IsChildOf<FStateTreeTaskBase>())
			{
				TargetArray = &InOwnerState->Tasks;
			}
			else if (NodeType->IsChildOf<FStateTreeConditionBase>())
			{
				TargetArray = &InOwnerState->EnterConditions;
			}
			else if (NodeType->IsChildOf<FStateTreeConsiderationBase>())
			{
				TargetArray = &InOwnerState->Considerations;
			}

			TArrayView<FStateTreeEditorNode> EditorNodesInBuffer = InProcessedClipboard.GetEditorNodesInBuffer();

			for (FStateTreeEditorNode& EditorNode : EditorNodesInBuffer)
			{
				TStructView<FStateTreeNodeBase> NodeView = EditorNode.GetNode();
				if (NodeView.IsValid() && NodeView.GetScriptStruct()->IsChildOf(NodeType))
				{
					TargetArray->Add(MoveTemp(EditorNode));
				}
			}
		}
	};

	FScopedTransaction ScopedTransaction(LOCTEXT("PasteNodesToStateTransaction", "Paste Nodes To State"));

	// Property Change requires one property at a time
	using namespace UE::StateTree::PropertyHelpers;

	// Enter Conditions
	ModifyStateInPreAndPostEdit(
		LOCTEXT("PasteNodesToStateTransaction", "Paste Nodes To State"),
		InState,
		InEditorData,
		TEXT("EnterConditions"),
		[&AppendFunc](TNotNull<UStateTreeState*> InOwnerState, TNotNull<UStateTreeEditorData*> InEditorData, const FStateTreeEditPropertyPath& InPropertyPath)
		{ AppendFunc.operator() < FStateTreeConditionBase > (InOwnerState, InEditorData, InPropertyPath); },
		INDEX_NONE,
		EPropertyChangeType::ValueSet);

	// Considerations
	ModifyStateInPreAndPostEdit(
		LOCTEXT("PasteNodesToStateTransaction", "Paste Nodes To State"),
		InState,
		InEditorData,
		TEXT("Considerations"),
		[&AppendFunc](TNotNull<UStateTreeState*> InOwnerState, TNotNull<UStateTreeEditorData*> InEditorData, const FStateTreeEditPropertyPath& InPropertyPath)
		{ AppendFunc.operator() < FStateTreeConsiderationBase > (InOwnerState, InEditorData, InPropertyPath); },
		INDEX_NONE,
		EPropertyChangeType::ValueSet);

	// Tasks
	ModifyStateInPreAndPostEdit(
		LOCTEXT("PasteNodesToStateTransaction", "Paste Nodes To State"),
		InState,
		InEditorData,
		TEXT("Tasks"),
		[&AppendFunc](TNotNull<UStateTreeState*> InOwnerState, TNotNull<UStateTreeEditorData*> InEditorData, const FStateTreeEditPropertyPath& InPropertyPath)
		{ AppendFunc.operator() < FStateTreeTaskBase > (InOwnerState, InEditorData, InPropertyPath); },
		INDEX_NONE,
		EPropertyChangeType::ValueSet);

	// Transitions
	ModifyStateInPreAndPostEdit(
		LOCTEXT("PasteNodesToStateTransaction", "Paste Nodes To State"),
		InState,
		InEditorData,
		TEXT("Transitions"),
		[&AppendFunc](TNotNull<UStateTreeState*> InOwnerState, TNotNull<UStateTreeEditorData*> InEditorData, const FStateTreeEditPropertyPath& InPropertyPath)
		{ AppendFunc.operator() < FStateTreeTransition > (InOwnerState, InEditorData, InPropertyPath); },
		INDEX_NONE,
		EPropertyChangeType::ValueSet);

	// Dump fixed property bindings in the end to avoid being cleaned up before their corresponding nodes are pushed in
	for (FStateTreePropertyPathBinding& Binding : InProcessedClipboard.GetBindingsInBuffer())
	{
		InEditorData->GetPropertyEditorBindings()->AddStateTreeBinding(MoveTemp(Binding));
	}

	OnStateNodesChanged.Broadcast(InState);
	UStateTreeEditingSubsystem::MarkAsModified(InEditorData->GetTypedOuter<UStateTree>());
}

bool FStateTreeViewModel::IsStateActiveInDebugger(const UStateTreeState& State) const
{
#if WITH_STATETREE_TRACE_DEBUGGER
	return ActiveStates.Contains(State.ID);
#else
	return false;
#endif // WITH_STATETREE_TRACE_DEBUGGER
}

#undef LOCTEXT_NAMESPACE
