// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextStyleDecorator.h"
#include "Framework/Text/SlateTextRun.h"
#include "Styling/SlateTypes.h"

TSharedRef<FTextStyleDecorator> FTextStyleDecorator::Create(FString InName, const FTextBlockStyle& TextStyle)
{
	return MakeShareable(new FTextStyleDecorator(MoveTemp(InName), TextStyle));
}

bool FTextStyleDecorator::Supports(const FTextRunParseResults& RunInfo, const FString& Text) const
{
	return RunInfo.Name == DecoratorName;
}

TSharedRef<ISlateRun> FTextStyleDecorator::Create(const TSharedRef<class FTextLayout>& TextLayout, const FTextRunParseResults& RunParseResult, const FString& OriginalText, const TSharedRef<FString>& ModelText, const ISlateStyle* Style)
{
	FRunInfo RunInfo(RunParseResult.Name);
	for (const TPair<FString, FTextRange>& Pair : RunParseResult.MetaData)
	{
		RunInfo.MetaData.Add(Pair.Key, OriginalText.Mid(Pair.Value.BeginIndex, Pair.Value.EndIndex - Pair.Value.BeginIndex));
	}

	FString Run;
	if (!RunParseResult.ContentRange.IsEmpty())
	{
		Run = OriginalText.Mid(RunParseResult.ContentRange.BeginIndex, RunParseResult.ContentRange.Len());
	}
	else
	{
		// Handles the case when the decorator name is empty (matches the runs without any tags).
		Run = OriginalText.Mid(RunParseResult.OriginalRange.BeginIndex, RunParseResult.OriginalRange.Len());
	}

	// Remove any formatting in between (e.g. when using nested tags).
	int32 TagStartIndex = INDEX_NONE;
	for (int32 Index = 0; Index < Run.Len(); Index++)
	{
		if (TagStartIndex == INDEX_NONE)
		{
			if (Run[Index] == TCHAR('<'))
			{
				TagStartIndex = Index;
			}
		}
		else
		{
			if (Run[Index] == TCHAR('>'))
			{
				Run.RemoveAt(TagStartIndex, Index - TagStartIndex + 1);
				Index = TagStartIndex;
				TagStartIndex = INDEX_NONE;
			}
		}
	}
	
	ModelText->Append(Run);

	return FSlateTextRun::Create(RunInfo, ModelText, TextStyle);
}
