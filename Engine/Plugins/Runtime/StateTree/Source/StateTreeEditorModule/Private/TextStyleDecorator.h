// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Text/ITextDecorator.h"
#include "Styling/SlateTypes.h"

/**
 * Simple decorator for SRichTextBlock, which allows to map simple/shorthand tags to specific style names.
 * The decorator also strips away any tags, e.g. in case the initial text came with nested tags.
 */
class FTextStyleDecorator : public ITextDecorator
{
public:

	static TSharedRef<FTextStyleDecorator> Create(FString InName, const FTextBlockStyle& TextStyle);

	virtual bool Supports(const FTextRunParseResults& RunInfo, const FString& Text) const override;

	virtual TSharedRef<ISlateRun> Create(
		const TSharedRef<class FTextLayout>& TextLayout,
		const FTextRunParseResults& RunParseResult,
		const FString& OriginalText,
		const TSharedRef<FString>& ModelText,
		const ISlateStyle* Style) override;

private:

	FTextStyleDecorator(FString&& InName, const FTextBlockStyle& InTextStyle)
		: DecoratorName(MoveTemp(InName))
		, TextStyle(InTextStyle)
	{
	}

private:

	/** Name of this decorator */
	FString DecoratorName;

	/** Color of this decorator */
	FTextBlockStyle TextStyle;
};