// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/AssetEditorToolkit.h"


/** StateTree Editor public interface */
class IStateTreeEditor : public FAssetEditorToolkit
{
public:

};
