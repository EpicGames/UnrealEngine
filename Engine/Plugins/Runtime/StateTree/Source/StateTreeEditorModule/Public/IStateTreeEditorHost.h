// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"

#include "IStateTreeEditorHost.generated.h"

namespace UE::StateTreeEditor
{
	class FWorkspaceTabHost;
}

class UStateTree;
class IMessageLogListing;
class IDetailsView;

// Interface required for re-using StateTreeEditor mode across different AssetEditors
class IStateTreeEditorHost : public TSharedFromThis<IStateTreeEditorHost>
{
public:
	IStateTreeEditorHost() = default;
	virtual ~IStateTreeEditorHost() = default;

	virtual FName GetCompilerLogName() const = 0;
	virtual FName GetCompilerTabName() const = 0;
	virtual bool ShouldShowCompileButton() const = 0;
	virtual bool CanToolkitSpawnWorkspaceTab() const = 0;

	virtual UStateTree* GetStateTree() const = 0;
	virtual FSimpleMulticastDelegate& OnStateTreeChanged() = 0;

	virtual TSharedPtr<IDetailsView> GetAssetDetailsView() = 0;
	virtual TSharedPtr<IDetailsView> GetDetailsView() = 0;
	virtual TSharedPtr<UE::StateTreeEditor::FWorkspaceTabHost> GetTabHost() const = 0;
};

UCLASS(MinimalAPI)
class UStateTreeEditorContext : public UObject
{
	GENERATED_BODY()
public:
	TSharedPtr<IStateTreeEditorHost> EditorHostInterface;
};

