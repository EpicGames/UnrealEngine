// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTree.h"
#include "StateTreePropertyBindingCompiler.h"

#define UE_API STATETREEEDITORMODULE_API

struct FAppendToClassSchemaContext;
struct FStructView;

enum class EStateTreeExpressionOperand : uint8;
enum class EStateTreePropertyUsage : uint8;
struct FStateTreeDataView;
struct FStateTreeStateHandle;

class UStateTreeState;
class UStateTreeEditorData;
class UStateTreeExtension;
class UStateTreeSchema;
struct FStateTreeEditorNode;
struct FStateTreeStateLink;
struct FStateTreeNodeBase;

namespace UE::StateTree::Compiler
{
	/**
	 * Compiler context for when the state tree fully compiled and succeeded.
	 * This is the last compilation step.
	 */
	struct FPostInternalContext
	{
		virtual ~FPostInternalContext() = default;

		virtual TNotNull<const UStateTree*> GetStateTree() const = 0;
		virtual TNotNull<const UStateTreeEditorData*> GetEditorData() const = 0;
		virtual FStateTreeCompilerLog& GetLog() const = 0;

		virtual UStateTreeExtension* AddExtension(TNotNull<TSubclassOf<UStateTreeExtension>> ExtensionType) const = 0;
	};
}

/**
 * Helper class to convert StateTree editor representation into a compact data.
 * Holds data needed during compiling.
 */
struct FStateTreeCompiler
{
public:

	explicit FStateTreeCompiler(FStateTreeCompilerLog& InLog)
		: Log(InLog)
	{
	}

	/** Compile the public step of a state tree asset. */
	UE_API bool CompilePublic(TNotNull<UStateTree*> StateTree);

	/** Compile the internal step of a state tree asset. */
	UE_API bool CompileInternal(TNotNull<UStateTree*> StateTree);

	/** Compile the public and internal steps of a state tree asset. */
	UE_API bool Compile(TNotNull<UStateTree*> StateTree);

	/** Compile the public and internal steps of a state tree asset. */
	UE_API bool Compile(UStateTree& InStateTree);

	/** Tests that all instanced sub-objects in the compiled StateTree have correct outers. Reports errors to the compiler log. */
	UE_API static void CheckCompiledStateTreeOuters(TNotNull<const UStateTree*> InStateTree, FStateTreeCompilerLog& InLog);

	/**
	 * Append config values that can change how state tree asset compiles when cooked.
	 * When the context is different from the previous result, then a cook is needed.
	 */
	UE_API static void AppendToStateTreeClassSchema(FAppendToClassSchemaContext& Context);

private:
	bool PreCompile(TNotNull<UStateTree*> StateTree);
	bool CompileInternalImpl();
	bool FailCompilation(UStateTree::ECompileStatus Status) const;

	/** Resolves the state a transition points to, and the optional fallback for failing to enter the state. SourceState is nullptr for global tasks. */
	bool ResolveTransitionStateAndFallback(const UStateTreeState* SourceState, const FStateTreeStateLink& Link, FStateTreeStateHandle& OutTransitionHandle, EStateTreeSelectionFallback& OutFallback) const;
	FStateTreeStateHandle GetStateHandle(const FGuid& StateID) const;
	UStateTreeState* GetState(const FGuid& StateID) const;

	void GatherBindingSources();

	bool CreateParameters();
	bool CreateStates();
	bool CreateStateRecursive(UStateTreeState& State, const FStateTreeStateHandle Parent);
	bool CreateEvaluators();
	bool CreateGlobalTasks();
	bool CreateStateTasksAndParameters();

	bool CreateStateTransitions();
	bool CreateStateEnterConditions(int32 StateIndex);

	bool CreateStateConsiderations();
	bool CreateStateConsideration(int32 StateIndex);

	bool CreateBindingsForNodes(TConstArrayView<FStateTreeEditorNode> EditorNodes, FStateTreeIndex16 NodesBegin, TArray<FInstancedStruct>& Instances);
	bool CreateBindingsForStruct(const FStateTreeBindableStructDesc& TargetStruct, FStateTreeDataView TargetValue, FStateTreeIndex16 PropertyFuncsBegin, FStateTreeIndex16 PropertyFuncsEnd, FStateTreeIndex16& OutBatchIndex, FStateTreeIndex16* OutOutputBindingBatchIndex = nullptr);

	bool CreatePropertyFunctionsForStruct(FGuid StructID);
	bool CreatePropertyFunction(const FStateTreeEditorNode& FuncEditorNode);

	bool CreateConditions(UStateTreeState& State, const FString& StatePath, TConstArrayView<FStateTreeEditorNode> Conditions);
	bool CreateCondition(UStateTreeState& State, const FString& StatePath, const FStateTreeEditorNode& CondNode, const EStateTreeExpressionOperand Operand, const int8 DeltaIndent);
	bool CreateConsiderations(UStateTreeState& State, const FString& StatePath, TConstArrayView<FStateTreeEditorNode> Considerations);
	bool CreateConsideration(UStateTreeState& State, const FString& StatePath, const FStateTreeEditorNode& ConsiderationNode, const EStateTreeExpressionOperand Operand, const int8 DeltaIndent);
	bool CreateTask(UStateTreeState* State, const FStateTreeEditorNode& TaskNode, const FStateTreeDataHandle TaskDataHandle);
	bool CreateEvaluator(const FStateTreeEditorNode& EvalNode, const FStateTreeDataHandle EvalDataHandle);

	bool CreateTaskCompletionDispatchers();

	FInstancedStruct* CreateNode(UStateTreeState* State, const FStateTreeEditorNode& EditorNode, FStateTreeBindableStructDesc& InstanceDesc, const FStateTreeDataHandle DataHandle, TArray<FInstancedStruct>& InstancedStructContainer);
	FInstancedStruct* CreateNodeWithSharedInstanceData(UStateTreeState* State, const FStateTreeEditorNode& EditorNode, FStateTreeBindableStructDesc& InstanceDesc);
	TOptional<FStateTreeDataView> CreateNodeInstanceData(const FStateTreeEditorNode& EditorNode, FStateTreeNodeBase& Node, FStateTreeBindableStructDesc& InstanceDesc, const FStateTreeDataHandle DataHandle, TArray<FInstancedStruct>& InstancedStructContainer);

	struct FValidatedPathBindings
	{
		TArray<FStateTreePropertyPathBinding> CopyBindings;
		TArray<FStateTreePropertyPathBinding> OutputCopyBindings;
		TArray<FStateTreePropertyPathBinding> DelegateDispatchers;
		TArray<FStateTreePropertyPathBinding> DelegateListeners;
		TArray<FStateTreePropertyPathBinding> ReferenceBindings;
	};

	bool GetAndValidateBindings(const FStateTreeBindableStructDesc& TargetStruct, FStateTreeDataView TargetValue, FValidatedPathBindings& OutValidatedBindings) const;
	bool ValidateStructRef(const FStateTreeBindableStructDesc& SourceStruct, const FPropertyBindingPath& SourcePath,
	const FStateTreeBindableStructDesc& TargetStruct, const FPropertyBindingPath& TargetPath) const;
	bool ValidateBindingOnNode(const FStateTreeBindableStructDesc& TargetStruct, const FPropertyBindingPath& TargetPath) const;
	bool CompileAndValidateNode(const UStateTreeState* SourceState, const FStateTreeBindableStructDesc& InstanceDesc, FStructView NodeView, const FStateTreeDataView InstanceData);

	void InstantiateStructSubobjects(FStructView Struct);

	bool NotifyInternalPost();
	
	void CreateBindingSourceStructsForNode(const FStateTreeEditorNode& EditorNode, const FStateTreeBindableStructDesc& InstanceDesc);

	const FString& GetStatePath(int32 StateIndex) const;

private:
	FStateTreeCompilerLog& Log;
	TObjectPtr<UStateTree> StateTree = nullptr;
	TObjectPtr<UStateTreeEditorData> EditorData = nullptr;
	TMap<FGuid, int32> IDToNode;
	TMap<FGuid, int32> IDToState;
	TMap<FGuid, int32> IDToTransition;
	TMap<FGuid, const FStateTreeDataView > IDToStructValue;
	TMap<FStateTreeStateHandle, int32> StateHandleToFrameIndex;
	TMultiMap<FGuid, const FPropertyBindingBinding*> BindingsByTargetStructID;
	TArray<TObjectPtr<UStateTreeState>> SourceStates;
	TArray<FString> StatePaths;

	TArray<FInstancedStruct> Nodes;
	TArray<FInstancedStruct> InstanceStructs;
	TArray<FInstancedStruct> SharedInstanceStructs;
	TArray<FInstancedStruct> EvaluationScopeStructs;
	TArray<FInstancedStruct> ExecutionRuntimeStructs;

	/**
	 * Number of binding in the editor data.
	 * Used to check that pointer in BindingsByTargetStructID are valid.
	 */
	int32 EditorBindingsNum = 0;
	
	/** Cached result of MakeCompletionTasksMask for global tasks. Indicates where state tasks should start. */
	int32 GlobalTaskEndBit = 0;

	FStateTreePropertyBindingCompiler BindingsCompiler;

	/** All struct IDs that are used as task completion delegate dispatcher (from all bindings). */
	TSet<FGuid> TaskCompletionDispatcherIDs;

	/** The state tree hash. */
	TOptional<uint32> EditorDataHash;

	/** The Compile function executed. */
	bool bCompiled = false;
};


namespace UE::StateTree::Compiler
{
	struct FValidationResult
	{
		FValidationResult() = default;
		FValidationResult(const bool bInResult, const int32 InValue, const int32 InMaxValue) : bResult(bInResult), Value(InValue), MaxValue(InMaxValue) {}

		/** Validation succeeded */
		bool DidSucceed() const { return bResult == true; }

		/** Validation failed */
		bool DidFail() const { return bResult == false; }

		/**
		 * Logs common validation for IsValidIndex16(), IsValidIndex8(), IsValidCount16(), IsValidCount8().
		 * @param Log reference to the compiler log.
		 * @param ContextText Text identifier for the context where the test is done.
		 * @param ContextStruct Struct identifier for the context where the test is done.
		 */
		void Log(FStateTreeCompilerLog& Log, const TCHAR* ContextText, const FStateTreeBindableStructDesc& ContextStruct = FStateTreeBindableStructDesc()) const;
		
		bool bResult = true;
		int32 Value = 0;
		int32 MaxValue = 0;
	};

	/**
	 * Checks if given index can be represented as uint16, including MAX_uint16 as INDEX_NONE.
	 * @param Index Index to test
	 * @return validation result.
	 */
	inline FValidationResult IsValidIndex16(const int32 Index)
	{
		return FValidationResult(Index == INDEX_NONE || (Index >= 0 && Index < MAX_uint16), Index, MAX_uint16 - 1);
	}

	/**
	 * Checks if given index can be represented as uint8, including MAX_uint8 as INDEX_NONE. 
	 * @param Index Index to test
	 * @return true if the index is valid.
	 */
	inline FValidationResult IsValidIndex8(const int32 Index)
	{
		return FValidationResult(Index == INDEX_NONE || (Index >= 0 && Index < MAX_uint8), Index, MAX_uint8 - 1);
	}

	/**
	 * Checks if given count can be represented as uint16. 
	 * @param Count Count to test
	 * @return true if the count is valid.
	 */
	inline FValidationResult IsValidCount16(const int32 Count)
	{
		return FValidationResult(Count >= 0 && Count <= MAX_uint16, Count, MAX_uint16);
	}

	/**
	 * Checks if given count can be represented as uint8. 
	 * @param Count Count to test
	 * @return true if the count is valid.
	 */
	inline FValidationResult IsValidCount8(const int32 Count)
	{
		return FValidationResult(Count >= 0 && Count <= MAX_uint8, Count, MAX_uint8);
	}

	/**
	 * Returns UScriptStruct defined in "BaseStruct" metadata of given property.
	 * @param Property Handle to property where value is got from.
	 * @param OutBaseStructName Handle to property where value is got from.
	 * @return Script struct defined by the BaseStruct or nullptr if not found.
	 */
	const UScriptStruct* GetBaseStructFromMetaData(const FProperty* Property, FString& OutBaseStructName);

}; // UE::StateTree::Compiler

#undef UE_API
