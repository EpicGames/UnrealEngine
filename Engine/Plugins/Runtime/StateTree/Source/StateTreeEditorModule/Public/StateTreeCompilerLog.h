// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/TokenizedMessage.h"
#include "StateTreePropertyBindings.h"
#include "StateTreeCompilerLog.generated.h"

#define UE_API STATETREEEDITORMODULE_API

class UStateTreeState;

/** StateTree compiler log message */
USTRUCT()
struct FStateTreeCompilerLogMessage
{
	GENERATED_BODY()

	FStateTreeCompilerLogMessage() = default;
	FStateTreeCompilerLogMessage(const EMessageSeverity::Type InSeverity, const UStateTreeState* InState, const FStateTreeBindableStructDesc& InItem, const FString& InMessage)
		: Severity(InSeverity)
		, State(InState)
		, Item(InItem)
		, Message(InMessage)
	{
	}

	/** Severity of the message. */
	UPROPERTY()
	int32 Severity = EMessageSeverity::Error;

	/** (optional) The StateTree state the message refers to. */
	UPROPERTY()
	TObjectPtr<const UStateTreeState> State = nullptr;

	/** (optional) The State tee item (condition/evaluator/task) the message refers to. */
	UPROPERTY()
	FStateTreeBindableStructDesc Item;

	/** The message */
	UPROPERTY()
	FString Message;
};

/** Message log for StateTree compilation */
USTRUCT()
struct FStateTreeCompilerLog
{
	GENERATED_BODY()

	/** Pushes State to be reported along with the message. */
	void PushState(const UStateTreeState* InState)
	{
		StateStack.Add(InState);
	}
	
	/** Pops State to be reported along with the message. @see FStateTreeCompilerLogStateScope */
	void PopState(const UStateTreeState* InState)
	{
		// Check for potentially miss matching push/pop
		check(StateStack.Num() > 0 && StateStack.Last() == InState);
		StateStack.Pop();
	}

	/** Returns current state context. */
	const UStateTreeState* CurrentState() const { return StateStack.Num() > 0 ? StateStack.Last() : nullptr;  }

	/**
	 * Reports a message.
	 * @param InSeverity Severity of the message.
	 * @param InItem StateTree item (condition/evaluator/task) the message affects.
	 * @param InMessage Message to display.
	 */
	void Report(EMessageSeverity::Type InSeverity, const FStateTreeBindableStructDesc& InItem, const FString& InMessage)
	{
		Messages.Emplace(InSeverity, CurrentState(), InItem, InMessage);
	}

	/** Formatted version of the Report(). */
	template <typename... Types>
	void Reportf(EMessageSeverity::Type InSeverity,
						 const FStateTreeBindableStructDesc& InItem,
						 UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt,
						 Types... Args)
	{
		Report(InSeverity, InItem, FString::Printf(Fmt, Args...));
	}

	/** Formatted version of the Report(), omits Item for convenience. */
	template <typename... Types>
	void Reportf(EMessageSeverity::Type InSeverity, UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args)
	{
		Report(InSeverity, FStateTreeBindableStructDesc(), FString::Printf(Fmt, Args...));
	}

	/** Generates tokenized messages from the messages in this log */
	UE_API TArray<TSharedRef<FTokenizedMessage>> ToTokenizedMessages() const;

	/** Appends StateTree log to log listing. */
	UE_API void AppendToLog(class IMessageLogListing* LogListing) const;

	/** Dumps StateTree log to log. */
	UE_API void DumpToLog(const FLogCategoryBase& Category) const;

	/**
	 * Dumps StateTree log to log.
	 * Append the asset to the log if there are any messages.
	 */
	UE_API void DumpToLog(const UStateTree* Asset, const FLogCategoryBase& Category) const;

protected:
	UPROPERTY()
	TArray<TObjectPtr<const UStateTreeState>> StateStack;
	
	UPROPERTY()
	TArray<FStateTreeCompilerLogMessage> Messages;
};

/** Helper struct to manage reported state within a scope. */
struct FStateTreeCompilerLogStateScope
{
	FStateTreeCompilerLogStateScope(const UStateTreeState* InState, FStateTreeCompilerLog& InLog)
		: Log(InLog)
		, State(InState)
	{
		Log.PushState(State);
	}

	~FStateTreeCompilerLogStateScope()
	{
		Log.PopState(State);
	}

	FStateTreeCompilerLog& Log; 
	const UStateTreeState* State = nullptr;
};

#undef UE_API
