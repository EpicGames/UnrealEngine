// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeCompiler.h"

#define UE_API STATETREEEDITORMODULE_API

namespace UE::StateTree::Compiler
{

/**
 * 
 */
class FCompilerManager final
{
public:
	static UE_API void Startup();
	static UE_API void Shutdown();

	/**
	 * Queue the compilation of a dirty state tree asset for later.
	 * The public step will be compiled synchronously if it is dirty.
	 * The internal step will be compiled when requested by the game or asynchronously.
	 * Queuing state tree asset can be disabled with cvar. When disabled, this function behaves like CompileSynchronously.
	 * @note Can only compile on the game thread or async loading thread.
	 */
	static UE_API void QueueForCompilation(TNotNull<UStateTree*> StateTree);

	/**
	 * Compile the internal steps of all queued state tree assets.
	 * Sort them to compile them with regard to their dependencies.
	 * @note Can only flush on the game thread.
	 */
	static UE_API void FlushCompilationQueue();

	/**
	 * Compile the public and internal steps of a state tree asset.
	 * It will compile all the pending state tree asset dependencies if they are queued.
	 * @return whether the compilation succeeded or failed.
	 * @note Can only compile on the game thread or async loading thread.
	 */
	static UE_API bool CompileSynchronously(TNotNull<UStateTree*> StateTree);
	static UE_API bool CompileSynchronously(TNotNull<UStateTree*> StateTree, FStateTreeCompilerLog& Log);

	/**
	 * Compile the public and internal steps of a state tree asset only if the editor data changed.
	 * It will compile all the pending state tree asset dependencies if they are queued.
	 * @return whether the compilation succeeded or failed. The optional is not set if a compilation was not necessary.
	 * @note Can only compile on the game thread or async loading thread.
	 */
	static UE_API TOptional<bool> CompileIfNeededSynchronously(TNotNull<UStateTree*> StateTree);
	
	/*
	 * Cache dependent external objects for tree's editor bindings.
	 * This includes any objects outside StateTree and CoreModule, that editor bindings are dependent on.
	 * This function gets called when EditorData updates its bindings.
	 */
	static UE_API void CacheEditorBindingExternalDependencies(TNotNull<UStateTreeEditorData*> EditorData);

	/** Track a state tree as dirty for compilation. */
	static UE_API void MarkAsModified(TNotNull<UStateTree*> StateTree);

private:
	FCompilerManager() = delete;
	FCompilerManager(const FCompilerManager&) = delete;
	FCompilerManager& operator= (const FCompilerManager&) = delete;
};

} // UE::StateTree::Compiler

#undef UE_API
