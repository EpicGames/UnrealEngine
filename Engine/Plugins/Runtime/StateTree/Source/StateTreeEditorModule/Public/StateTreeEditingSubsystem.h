// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTree.h"
#include "StateTreeViewModel.h"
#include "EditorSubsystem.h"
#include "UObject/ObjectKey.h"

#include "StateTreeEditingSubsystem.generated.h"

#define UE_API STATETREEEDITORMODULE_API

class SWidget;
class FStateTreeViewModel;
class FUICommandList;
struct FStateTreeCompilerLog;

UCLASS(MinimalAPI)
class UStateTreeEditingSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()
public:
	UE_API UStateTreeEditingSubsystem();
	UE_API virtual void BeginDestroy() override;
	
	UE_API TSharedPtr<FStateTreeViewModel> FindViewModel(TNotNull<const UStateTree*> InStateTree) const;
	UE_API TSharedRef<FStateTreeViewModel> FindOrAddViewModel(TNotNull<UStateTree*> InStateTree);

	/** Create a state tree view widget for the viewmodel. */
	static UE_API TSharedRef<SWidget> GetStateTreeView(TSharedRef<FStateTreeViewModel> InViewModel, const TSharedRef<FUICommandList>& TreeViewCommandList);
	
	/**
	 * Compile the state tree asset.
	 * It will compile the public and internal steps.
	 */
	static UE_API bool CompileStateTree(TNotNull<UStateTree*> InStateTree, FStateTreeCompilerLog& InOutLog);

	/** Whether the state tree asset is dirty and requires a compilation. */
	static UE_API bool NeedsRecompile(TNotNull<const UStateTree*> InStateTree);

	/**
	 * Mark the public and the internal steps of the state tree asset as dirty.
	 * It will require the state tree to be compiled before it can be used.
	 * It will affect the other assets that reference the state tree asset.
	 */
	static UE_API void MarkAsPubliclyModified(TNotNull<UStateTree*> InStateTree);

	/**
	 * Mark the internal step of the state tree asset as dirty.
	 * It will require the state tree to be compiled before it can be used.
	 */
	static UE_API void MarkAsModified(TNotNull<UStateTree*> InStateTree);
	
	/**
	 * Validates and applies the schema restrictions on the StateTree.
	 * Also serves as the "safety net" of fixing up editor data following an editor operation.
	 * Updates state's link, removes the unused node while validating the StateTree asset.
	 */
	static UE_API void ValidateStateTree(TNotNull<UStateTree*> InStateTree);

	/** Calculates editor data hash of the asset. */
	static UE_API uint32 CalculateStateTreeHash(TNotNull<const UStateTree*> InStateTree);
	
private:
	void HandlePostGarbageCollect();
	void HandlePostCompile(const UStateTree& InStateTree);

protected:
	TMap<FObjectKey, TSharedPtr<FStateTreeViewModel>> StateTreeViewModels;
	FDelegateHandle PostGarbageCollectHandle;
	FDelegateHandle PostCompileHandle;
};

#undef UE_API
