// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeState.h"
#include "StateTreeEditorPropertyBindings.h"
#include "StateTreeEditorTypes.h"
#include "Debugger/StateTreeDebuggerTypes.h"
#include "StateTreeEditorData.generated.h"

#define UE_API STATETREEEDITORMODULE_API

struct FStateTreeBindableStructDesc;

class UStateTreeEditorDataExtension;
class UStateTreeEditorSchema;
class UStateTreeSchema;

namespace UE::StateTree::Editor
{
	// Name used to describe container of global items (other items use the path to the container State).  
	extern STATETREEEDITORMODULE_API const FString GlobalStateName;

	// Name used to describe container of property functions.
	extern STATETREEEDITORMODULE_API const FString PropertyFunctionStateName;
}

namespace UE::StateTree::Compiler::Private
{
	class FCompilerManagerImpl;
}

USTRUCT()
struct FStateTreeEditorBreakpoint
{
	GENERATED_BODY()

	FStateTreeEditorBreakpoint() = default;
	explicit FStateTreeEditorBreakpoint(const FGuid& ID, const EStateTreeBreakpointType BreakpointType)
		: ID(ID)
		, BreakpointType(BreakpointType)
	{
	}

	/** Unique Id of the Node or State associated to the breakpoint. */
	UPROPERTY()
	FGuid ID;

	/** The event type that should trigger the breakpoint (e.g. OnEnter, OnExit, etc.). */
	UPROPERTY()
	EStateTreeBreakpointType BreakpointType = EStateTreeBreakpointType::Unset;
};

UENUM()
enum class EStateTreeVisitor : uint8
{
	Continue,
	Break,
};

/**
 * Edit time data for StateTree asset. This data gets baked into runtime format before being used by the StateTreeInstance.
 */
UCLASS(MinimalAPI, BlueprintType, EditInlineNew, CollapseCategories, Within = "StateTree", meta = (DisallowLevelActorReference = true))
class UStateTreeEditorData : public UObject, public IStateTreeEditorPropertyBindingsOwner
{
	GENERATED_BODY()
	
public:
	UE_API UStateTreeEditorData();

	/** @return The editor data for a StateTree asset, or nullptr if unavailable. */
	UFUNCTION(BlueprintCallable, Category = "StateTree")
	static UStateTreeEditorData* GetEditorData(UStateTree* StateTree);

	UE_API virtual void PostInitProperties() override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	
	//~ Begin IStateTreeEditorPropertyBindingsOwner interface
	UE_API virtual void GetBindableStructs(const FGuid TargetStructID, TArray<TInstancedStruct<FPropertyBindingBindableStructDescriptor>>& OutStructDescs) const override;
	UE_API virtual bool GetBindableStructByID(const FGuid StructID, TInstancedStruct<FPropertyBindingBindableStructDescriptor>& OutStructDesc) const override;
	UE_API virtual bool GetBindingDataViewByID(const FGuid StructID, FPropertyBindingDataView& OutDataView) const override;
	virtual const FStateTreeEditorPropertyBindings* GetPropertyEditorBindings() const override
	{
		return &EditorBindings;
	}

	virtual FPropertyBindingBindingCollection* GetEditorPropertyBindings() override
	{
		return &EditorBindings;
	}

	virtual const FPropertyBindingBindingCollection* GetEditorPropertyBindings() const override
	{
		return &EditorBindings;
	}

	virtual FStateTreeEditorPropertyBindings* GetPropertyEditorBindings() override
	{
		return &EditorBindings;
	}

	UE_API virtual FStateTreeBindableStructDesc FindContextData(const UStruct* ObjectType, const FString ObjectNameHint) const override;

	UE_API virtual bool CanCreateParameter(const FGuid InStructID) const override;
	UE_API virtual void CreateParametersForStruct(const FGuid InStructID, TArrayView<UE::PropertyBinding::FPropertyCreationDescriptor> InOutCreationDescs) override;

	// todo: (jira UE-337309) we have many sites that manipulate state/node which will change bindings, but not calling this function. Currently it is only called when you change binding from the details view
	UE_API virtual void OnPropertyBindingChanged(const FPropertyBindingPath& InSourcePath, const FPropertyBindingPath& InTargetPath) override;

	UE_API virtual void AppendBindablePropertyFunctionStructs(TArray<TInstancedStruct<FPropertyBindingBindableStructDescriptor>>& InOutStructs) const override;
	//~ End IStateTreeEditorPropertyBindingsOwner interface

	/**
	 * Returns the description for the node for UI.
	 * Handles the name override logic, figures out required data for the GetDescription() call, and handles the fallbacks.
	 * @return description for the node.
	 */
	UFUNCTION(BlueprintCallable, Category = "StateTree")
	UE_API FText GetNodeDescription(const FStateTreeEditorNode& Node, const EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const;

#if WITH_EDITOR
	UE_API void OnParametersChanged(const UStateTree& StateTree);
	UE_API void OnStateParametersChanged(const UStateTree& StateTree, const FGuid StateID);
	UE_API virtual void BeginDestroy() override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
	UE_API virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
#endif

	/** @return the public parameters ID that could be used for bindings within the Tree. */
	FGuid GetRootParametersGuid() const
	{
		return RootParametersGuid;
	}

	/** @return the public parameters that could be used for bindings within the Tree. */
	virtual const FInstancedPropertyBag& GetRootParametersPropertyBag() const
	{
		return RootParameterPropertyBag;
	}

	/** @returns parent state of a struct, or nullptr if not found. */
	UE_API const UStateTreeState* GetStateByStructID(const FGuid TargetStructID) const;

	/** @returns state based on its ID, or nullptr if not found. */
	UE_API const UStateTreeState* GetStateByID(const FGuid StateID) const;

	/** @returns mutable state based on its ID, or nullptr if not found. */
	UE_API UStateTreeState* GetMutableStateByID(const FGuid StateID);

	/** @returns the IDs and instance values of all bindable structs in the StateTree. */
	UE_API void GetAllStructValues(TMap<FGuid, const FStateTreeDataView>& OutAllValues) const;

	/** @returns the IDs and instance values of all bindable structs in the StateTree. */
	UE_API void GetAllStructValues(TMap<FGuid, const FPropertyBindingDataView>& OutAllValues) const;

	/**
	* Iterates over all structs that are related to binding
	* @param InFunc function called at each node, should return Continue if visiting is continued or Break to stop.
	*/
	UE_API EStateTreeVisitor VisitHierarchy(TFunctionRef<EStateTreeVisitor(UStateTreeState& State, UStateTreeState* ParentState)> InFunc) const;

	/**
	 * Iterates over all structs at the global level (context, tree parameters, evaluators, global tasks) that are related to binding.
	 * @param InFunc function called at each node, should return Continue if visiting is continued or Break to stop.
	 */
	UE_API EStateTreeVisitor VisitGlobalNodes(TFunctionRef<EStateTreeVisitor(const UStateTreeState* State, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)> InFunc) const;

	/**
	 * Iterates over all structs in the state hierarchy that are related to binding.
	 * @param InFunc function called at each node, should return Continue if visiting is continued or Break to stop.
	 */
	UE_API EStateTreeVisitor VisitHierarchyNodes(TFunctionRef<EStateTreeVisitor(const UStateTreeState* State, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)> InFunc) const;

	/**
	 * Iterates over all structs that are related to binding.
	 * @param InFunc function called at each node, should return Continue if visiting is continued or Break to stop.
	 */
	UE_API EStateTreeVisitor VisitAllNodes(TFunctionRef<EStateTreeVisitor(const UStateTreeState* State, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)> InFunc) const;

	/**
	 * Iterates over all nodes in a given state.
	 * @param InFunc function called at each node, should return Continue if visiting is continued or Break to stop.
	 */
	UE_API EStateTreeVisitor VisitStateNodes(const UStateTreeState& State, TFunctionRef<EStateTreeVisitor(const UStateTreeState* State, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)> InFunc) const;

	/**
	 * Iterates recursively over all property functions of the provided node. Also nested ones.
	 * @param InFunc function called at each node, should return Continue if visiting is continued or Break to stop.
	 */
	UE_API EStateTreeVisitor VisitStructBoundPropertyFunctions(FGuid StructID, const FString& StatePath, TFunctionRef<EStateTreeVisitor(const FStateTreeEditorNode& EditorNode, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)> InFunc) const;

	/**
	 * Iterates over all bindings whose target path is in a given state.
	 * @param InFunc function called at each binding, should return Continue if visiting is continued or Break to stop.
	 */
	UE_API EStateTreeVisitor VisitStateBindings(TNotNull<const UStateTreeState*> State, TFunctionRef<EStateTreeVisitor(TNotNull<const UStateTreeState*> State, const FStateTreePropertyPathBinding& Binding, const FStateTreeBindableStructDesc& BindingTargetNodeDesc, const FStateTreeDataView BindingTargetNodeValue)> InFunc) const;

	/**
	 * Returns array of nodes along the execution path, up to the TargetStruct.
	 * @param Path The states to visit during the check
	 * @param TargetStructID The ID of the node where to stop.
	 * @param OutStructDescs Array of nodes accessible on the given path.
	 */
	UE_API void GetAccessibleStructsInExecutionPath(const TConstArrayView<const UStateTreeState*> Path, const FGuid TargetStructID, TArray<TInstancedStruct<FPropertyBindingBindableStructDescriptor>>& OutStructDescs) const;

	/**
	 * Iterates over all template property function nodes.
	 * Note it is different from calling VisitStructBoundPropertyFunctions which returns the instanced property functions for a node.
	 * @param InFunc function called at each node, should return Continue if visiting is continued or Break to stop.
	 */
	UE_API virtual EStateTreeVisitor EnumerateBindablePropertyFunctionNodes(TFunctionRef<EStateTreeVisitor(const UScriptStruct* NodeStruct, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)> InFunc) const override;

	UE_DEPRECATED(5.6, "Use GetAccessibleStructsInExecutionPath instead")
	void GetAccessibleStruct(const TConstArrayView<const UStateTreeState*> Path, const FGuid TargetStructID, TArray<TInstancedStruct<FPropertyBindingBindableStructDescriptor>>& OutStructDescs) const
	{
		GetAccessibleStructsInExecutionPath(Path, TargetStructID, OutStructDescs);
	}

	/** Find the first extension of the requested type. */
	template<typename ExtensionType>
	ExtensionType* GetExtension()
	{
		return CastChecked<ExtensionType>(K2_GetExtension(ExtensionType::StaticClass()), ECastCheckedType::NullAllowed);
	}

	/** Find the first extension of the requested type or create it on this editor data if it doesn't exist. */
	template<typename ExtensionType>
	ExtensionType* GetOrCreateExtension()
	{
		static_assert(std::is_base_of_v<UStateTreeEditorDataExtension, ExtensionType>, "Using a non state tree editor data class as extension.");

		if (ExtensionType* FoundExtension = CastChecked<ExtensionType>(K2_GetExtension(ExtensionType::StaticClass()), ECastCheckedType::NullAllowed))
		{
			return FoundExtension;
		}

		UStateTreeEditorDataExtension* Result = NewObject<UStateTreeEditorDataExtension>(this, ExtensionType::StaticClass());
		Extensions.Add(Result);

		return CastChecked<ExtensionType>(Result);
	}

	/** Find the first extension of the requested type. */
	UFUNCTION(BlueprintCallable, Category = "StateTree|Extension", Meta = (DisplayName="Get Extension", DeterminesOutputType = "ExtensionType"))
	UE_API UStateTreeEditorDataExtension* K2_GetExtension(TSubclassOf<UStateTreeEditorDataExtension> ExtensionType);

	/** Make sure the child and parent states are pointing to each other. */
	UE_API void ReparentStates();
	/** Fix any nodes or bindings with duplicated ID. */
	UE_API void FixDuplicateIDs();

	UE_DEPRECATED(5.8, "Use UpdateEditorBindings instead")
	void UpdateBindingsInstanceStructs()
	{
		UpdateBindings();
	}

	/** Update the editor binding segments and cache editor bindings dependency. */
	UE_API void UpdateBindings();

	/** Remove invalid editor property bindings. */
	UE_API void RemoveInvalidBindings();

	// StateTree Builder API

	/**
	 * Adds new Subtree with specified name.
	 * @return Pointer to the new Subtree.
	 */
	UStateTreeState& AddSubTree(const FName Name, EStateTreeStateType StateType = EStateTreeStateType::State)
	{
		UStateTreeState* SubTreeState = NewObject<UStateTreeState>(this, FName(), RF_Transactional);
		check(SubTreeState);
		SubTreeState->Name = Name;
		SubTreeState->Type = StateType;
		SubTrees.Add(SubTreeState);
		return *SubTreeState;
	}

	/**
	 * Adds new Subtree named "Root".
	 * @return Pointer to the new Subtree.
	 */
	UStateTreeState& AddRootState()
	{
		return AddSubTree(FName(TEXT("Root")));
	}

	/**
	 * Adds Evaluator of specified type.
	 * @return reference to the new Evaluator. 
	 */
	template<typename T, typename... TArgs>
	TStateTreeEditorNode<T>& AddEvaluator(TArgs&&... InArgs)
	{
		FStateTreeEditorNode& EditorNode = Evaluators.AddDefaulted_GetRef();
		EditorNode.InitializeAs<T>(this, Forward<TArgs>(InArgs)...);
		OnNodeAdded(EditorNode);
		return static_cast<TStateTreeEditorNode<T>&>(EditorNode);
	}

	/**
	 * Adds Global Task of specified type.
	 * @return reference to the new task. 
	 */
	template<typename T, typename... TArgs>
	TStateTreeEditorNode<T>& AddGlobalTask(TArgs&&... InArgs)
	{
		FStateTreeEditorNode& EditorNode = GlobalTasks.AddDefaulted_GetRef();
		EditorNode.InitializeAs<T>(this, Forward<TArgs>(InArgs)...);
		OnNodeAdded(EditorNode);
		return static_cast<TStateTreeEditorNode<T>&>(EditorNode);
	}

	/**
	 * Adds property binding between two structs.
	 */
	void AddPropertyBinding(const FPropertyBindingPath& SourcePath, const FPropertyBindingPath& TargetPath)
	{
		EditorBindings.AddBinding(SourcePath, TargetPath);
	}

	/**
	 * Adds property binding to PropertyFunction of provided type.
	 */
	void AddPropertyBinding(const UScriptStruct* PropertyFunctionNodeStruct, TConstArrayView<FPropertyBindingPathSegment> SourcePathSegments, const FPropertyBindingPath& TargetPath)
	{
		EditorBindings.AddFunctionBinding(PropertyFunctionNodeStruct, SourcePathSegments, TargetPath);
	}

	/**
	 * Removes property binding of target path.
	 */
	void RemovePropertyBinding(const FPropertyBindingPath& TargetPath)
	{
		constexpr FPropertyBindingBindingCollection::ESearchMode SearchMode = FPropertyBindingBindingCollection::ESearchMode::Includes;
		EditorBindings.RemoveBindings(TargetPath, SearchMode);
	}

	/**
	 * Adds property binding between two structs.
	 */
	bool AddPropertyBinding(const FStateTreeEditorNode& SourceNode, const FString SourcePathStr, const FStateTreeEditorNode& TargetNode, const FString TargetPathStr)
	{
		FPropertyBindingPath SourcePath;
		FPropertyBindingPath TargetPath;
		SourcePath.SetStructID(SourceNode.ID);
		TargetPath.SetStructID(TargetNode.ID);
		if (SourcePath.FromString(SourcePathStr) && TargetPath.FromString(TargetPathStr))
		{
			EditorBindings.AddBinding(SourcePath, TargetPath);
			return true;
		}
		return false;
	}

#if WITH_STATETREE_TRACE_DEBUGGER
	UE_API bool HasAnyBreakpoint(FGuid ID) const;
	UE_API bool HasBreakpoint(FGuid ID, EStateTreeBreakpointType BreakpointType) const;
	UE_API const FStateTreeEditorBreakpoint* GetBreakpoint(FGuid ID, EStateTreeBreakpointType BreakpointType) const;
	UE_API void AddBreakpoint(FGuid ID, EStateTreeBreakpointType BreakpointType);
	UE_API bool RemoveBreakpoint(FGuid ID, EStateTreeBreakpointType BreakpointType);
	UE_API void RemoveAllBreakpoints();
#endif // WITH_STATETREE_TRACE_DEBUGGER

	// ~StateTree Builder API

	/**
	 * Attempts to find a Color matching the provided Color Key
	 */
	const FStateTreeEditorColor* FindColor(const FStateTreeEditorColorRef& ColorRef) const
	{
		return Colors.Find(FStateTreeEditorColor(ColorRef));
	}

	UE_API virtual void CreateRootProperties(TArrayView<UE::PropertyBinding::FPropertyCreationDescriptor> InOutCreationDescs);

private:
	UE_API void FixObjectNodes();
	UE_API void DuplicateIDs();
	UE_API void CallPostLoadOnNodes();
	UE_API void OnNodeAdded(FStateTreeEditorNode& EditorNode);

#if WITH_EDITORONLY_DATA
	FDelegateHandle OnParametersChangedHandle;
	FDelegateHandle OnStateParametersChangedHandle;
#endif

public:
	/** Schema describing which inputs, evaluators, and tasks a StateTree can contain. */
	UPROPERTY(EditDefaultsOnly, Instanced, Category = "Common", NoClear)
	TObjectPtr<UStateTreeSchema> Schema = nullptr;

	/** Schema describing how the editor schema is customized. */
	UPROPERTY(Instanced, NoClear)
	TObjectPtr<UStateTreeEditorSchema> EditorSchema = nullptr;

	/** The editor data extensions. A place to add extra information for plugins. */
	UPROPERTY(EditDefaultsOnly, Instanced, Category = "Extension", NoClear)
	TArray<TObjectPtr<UStateTreeEditorDataExtension>> Extensions;

	/** Public parameters that could be used for bindings within the Tree. */
	UE_DEPRECATED(5.6, "Public access to RootParameters is deprecated. Use GetRootParametersPropertyBag")
	UPROPERTY(meta = (DeprecatedProperty))
	FStateTreeStateParameters RootParameters;

private:
	/** Public parameters ID that could be used for bindings within the Tree. */
	UPROPERTY()
	FGuid RootParametersGuid;

	/** Public parameters property bag that could be used for bindings within the Tree. */
	UPROPERTY(EditDefaultsOnly, Category = Parameters)
	FInstancedPropertyBag RootParameterPropertyBag;

public:
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Evaluators", meta = (BaseStruct = "/Script/StateTreeModule.StateTreeEvaluatorBase", BaseClass = "/Script/StateTreeModule.StateTreeEvaluatorBlueprintBase"))
	TArray<FStateTreeEditorNode> Evaluators;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Global Tasks", meta = (BaseStruct = "/Script/StateTreeModule.StateTreeTaskBase", BaseClass = "/Script/StateTreeModule.StateTreeTaskBlueprintBase"))
	TArray<FStateTreeEditorNode> GlobalTasks;

	UPROPERTY(EditDefaultsOnly, Category = "Global Tasks")
	EStateTreeTaskCompletionType GlobalTasksCompletion = EStateTreeTaskCompletionType::Any;

	UPROPERTY()
	FStateTreeEditorPropertyBindings EditorBindings;

	/** Color Options to assign to a State */
	UPROPERTY(EditDefaultsOnly, Category = "Theme")
	TSet<FStateTreeEditorColor> Colors;

	/** Top level States. */
	UPROPERTY(Instanced, BlueprintReadOnly, Category = "StateTree")
	TArray<TObjectPtr<UStateTreeState>> SubTrees;

	/**
	 * Transient list of breakpoints added in the debugging session.
	 * These will be lost if the asset gets reloaded.
	 * If there is eventually a change to make those persist with the asset
	 * we need to prune all dangling breakpoints after states/tasks got removed.
	 */
	UPROPERTY(Transient)
	TArray<FStateTreeEditorBreakpoint> Breakpoints;

	/**
	 * List of the previous compiled delegate dispatchers.
	 * Saved in the editor data to be duplicated transient.
	 */
	UPROPERTY(DuplicateTransient)
	TArray<FStateTreeEditorDelegateDispatcherCompiledBinding> CompiledDispatchers;

	friend class FStateTreeEditorDataDetails;
	friend class UE::StateTree::Compiler::Private::FCompilerManagerImpl;
};


UCLASS()
class UQAStateTreeEditorData : public UStateTreeEditorData
{
	GENERATED_BODY()
};

#undef UE_API
