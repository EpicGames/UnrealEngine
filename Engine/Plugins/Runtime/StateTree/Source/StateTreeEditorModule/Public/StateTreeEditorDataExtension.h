// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "StateTreeEditorDataExtension.generated.h"

namespace UE::StateTree::Compiler
{
	struct FPostInternalContext;
}

class IDetailLayoutBuilder;
class UStateTreeEditorData;
class UStateTreeState;

/**
 * Extension for the editor data of the state tree asset.
 */
UCLASS(Abstract, DefaultToInstanced, EditInlineNew, Within=StateTreeEditorData, MinimalAPI)
class UStateTreeEditorDataExtension : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * Validates and applies the editor data extension restrictions on the state tree owned by the extension.
	 * This is before the engine validation. The Schema and EditorData will be correctly set.
	 * You are allowed to modify the editor data.
	 */
	virtual void PreValidate()
	{
	}

	/**
	 * Validates and applies the editor data extension restrictions on the state tree owned by the extension.
	 * Also serves as the "safety net" of fixing up editor data following an editor operation.
	 * You are allowed to modify the editor data.
	 */
	virtual void Validate()
	{
	}

	/**
	 * Handled after the asset had properly compiled its public and internal data.
	 * You are allowed to modify the state tree.
	 * You should not modify the editor data. It might create an infinite loop.
	 */
	virtual bool HandlePostInternalCompile(const UE::StateTree::Compiler::FPostInternalContext& Context)
	{
		return true;
	}

	/** Customize the detail view for the selected state. */
	virtual void CustomizeDetails(TNonNullPtr<UStateTreeState> State, IDetailLayoutBuilder& DetailBuilder)
	{
	}

protected:
	UStateTreeEditorData* GetStateTreeEditorData() const
	{
		return GetOuterUStateTreeEditorData();
	}
};
