// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdMode.h"
#include "StateTree.h"
#include "StateTreeEditorData.h"

#include "StateTreeEditorMode.generated.h"

#define UE_API STATETREEEDITORMODULE_API

class FStateTreeBindingExtension;
class FStateTreeBindingsChildrenCustomization;
class IDetailsView;
class IMessageLogListing;

UCLASS(MinimalAPI, Transient)
class UStateTreeEditorMode : public UEdMode
{
	GENERATED_BODY()
public:
	UE_API const static FEditorModeID EM_StateTree;
	
	UE_API UStateTreeEditorMode();

	UE_API virtual void Enter() override;
	UE_API virtual void Exit() override;
	UE_API virtual void CreateToolkit() override;
	UE_API virtual void BindCommands() override;

protected:
	UE_API void OnStateTreeChanged();
	UE_API void BindToolkitCommands(const TSharedRef<FUICommandList>& ToolkitCommands);

	UE_API void OnPropertyBindingChanged(const FPropertyBindingPath& SourcePath, const FPropertyBindingPath& TargetPath);
	UE_API void OnIdentifierChanged(const UStateTree& InStateTree);
	UE_API void OnSchemaChanged(const UStateTree& InStateTree);
	UE_API void ForceRefreshDetailsView() const;
	UE_API void OnRefreshDetailsView(const UStateTree& InStateTree) const;
	UE_API void OnStateParametersChanged(const UStateTree& InStateTree, const FGuid ChangedStateID) const;

	UE_API void HandleMessageTokenClicked(const TSharedRef<IMessageToken>& InMessageToken) const;

	UE_API void OnAssetFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent) const;
	UE_API void OnSelectionFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent);

	UE_API void Compile();
	UE_API bool CanCompile() const;
	UE_API bool IsCompileVisible() const;

	UE_API bool HasValidStateTree() const;
	
	UE_API void HandleModelAssetChanged();
	UE_API void HandleModelSelectionChanged(const TArray<TWeakObjectPtr<UStateTreeState>>& SelectedStates) const;
	UE_API void HandleModelBringNodeToFocus(const UStateTreeState* State, const FGuid NodeID) const;
	UE_API void HandleModelBringBindingPathToFocus(const UStateTreeState* State, const FPropertyBindingPath& InBindingPath) const;

	void HandleStateAdded(UStateTreeState* , UStateTreeState*)
	{
		UpdateAsset();
	}

	void HandleStatesRemoved(const TSet<UStateTreeState*>&)
	{
		UpdateAsset();
	}

	void HandleOnStatesMoved(const TSet<UStateTreeState*>&, const TSet<UStateTreeState*>&)
	{
		UpdateAsset();
	}

	void HandleOnStateNodesChanged(const UStateTreeState*)
	{
		UpdateAsset();

		// when node is being changed from the view model, details view might have already been opened and in that case, we need to rebuild
		ForceRefreshDetailsView();
	}

	/** Resolve the internal editor data and fixup the StateTree nodes. */
	UE_API void UpdateAsset();

	UE_API TSharedPtr<IDetailsView> GetDetailsView() const;
	UE_API TSharedPtr<IDetailsView> GetAssetDetailsView() const;
	
	UE_API TSharedPtr<IMessageLogListing> GetMessageLogListing() const;	
	UE_API void ShowCompilerTab() const;

	UE_API UStateTree* GetStateTree() const;
	
	friend class FStateTreeEditorModeToolkit;

protected:
	UE_DEPRECATED(5.8, "UStateTree::IsEditorDataDirty")
	uint32 EditorDataHash = 0;
	UE_DEPRECATED(5.8, "UStateTree::CompileStatus")
	bool bLastCompileSucceeded = true;
	bool bForceAssetDetailViewToRefresh = false;

	mutable FTimerHandle SetObjectTimerHandle;
	mutable FTimerHandle HighlightTimerHandle;

	TWeakObjectPtr<UStateTree> CachedStateTree = nullptr;

	TSharedPtr<FStateTreeBindingExtension> DetailsViewExtensionHandler;
	TSharedPtr<FStateTreeBindingsChildrenCustomization> DetailsViewChildrenCustomizationHandler;
};

#undef UE_API
