// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeEditorMode.h"
#include "IStateTreeEditorHost.h"
#include "Toolkits/BaseToolkit.h"

#define UE_API STATETREEEDITORMODULE_API

class UStateTree;
class UStateTreeEditorMode;
struct FPropertyBindingBindingCollection;
namespace UE::StateTreeEditor
{
	struct FSpawnedWorkspaceTab;
}

class FStateTreeEditorModeToolkit : public FModeToolkit
{
public:

	UE_API FStateTreeEditorModeToolkit(UStateTreeEditorMode* InEditorMode);
	
	/** IToolkit interface */
	UE_API virtual FName GetToolkitFName() const override;
	UE_API virtual FText GetBaseToolkitName() const override;

	UE_API virtual void Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode) override;
	UE_API virtual void InvokeUI() override;
	UE_API virtual void RequestModeUITabs() override;
	
	UE_API virtual void ExtendSecondaryModeToolbar(UToolMenu* InModeToolbarMenu) override;

	UE_API UStateTree* GetStateTree() const;
	UE_API void OnStateTreeChanged();

protected:
	UE_API FSlateIcon GetCompileStatusImage() const;

	static UE_API FSlateIcon GetNewTaskButtonImage();
	UE_API TSharedRef<SWidget> GenerateTaskBPBaseClassesMenu() const;

	static UE_API FSlateIcon GetNewConditionButtonImage();
	UE_API TSharedRef<SWidget> GenerateConditionBPBaseClassesMenu() const;
    
	static UE_API FSlateIcon GetNewConsiderationButtonImage();
	UE_API TSharedRef<SWidget> GenerateConsiderationBPBaseClassesMenu() const;

	UE_API void OnNodeBPBaseClassPicked(UClass* NodeClass) const;
	
	UE_API FText GetStatisticsText() const;
	UE_API const FPropertyBindingBindingCollection* GetBindingCollection() const;

	UE_API void UpdateStateTreeOutliner();

	UE_API void HandleTabSpawned(UE::StateTreeEditor::FSpawnedWorkspaceTab SpawnedTab);
	UE_API void HandleTabClosed(UE::StateTreeEditor::FSpawnedWorkspaceTab SpawnedTab);

protected:
	TWeakObjectPtr<UStateTreeEditorMode> WeakEditorMode;
	TSharedPtr<IStateTreeEditorHost> EditorHost;

	/** Tree Outliner */
	TSharedPtr<SWidget> StateTreeOutliner = nullptr;
	TWeakPtr<SDockTab> WeakOutlinerTab = nullptr;

#if WITH_STATETREE_TRACE_DEBUGGER
	UE_API void UpdateDebuggerView();
	TWeakPtr<SDockTab> WeakDebuggerTab = nullptr;
#endif // WITH_STATETREE_TRACE_DEBUGGER

};


#undef UE_API
