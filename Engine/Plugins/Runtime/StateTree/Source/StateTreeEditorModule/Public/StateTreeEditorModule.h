// Copyright Epic Games, Inc. All Rights Reserved.

#pragma  once

#include "BlueprintEditorModule.h"
#include "Modules/ModuleInterface.h"
#include "Logging/LogMacros.h"
#include "Templates/NonNullSubclassOf.h"
#include "Templates/PimplPtr.h"
#include "Toolkits/AssetEditorToolkit.h"

#define UE_API STATETREEEDITORMODULE_API

class FAssetRegistryTagsContext;
class UStateTree;
class UStateTreeEditorData;
class UStateTreeEditorSchema;
class UStateTreeSchema;
class UUserDefinedStruct;
class IStateTreeEditor;
struct FStateTreeNodeClassCache;

namespace UE::StateTree::Compiler
{
	struct FPostInternalContext;
}

namespace UE::StateTreeDebugger
{
	class FRewindDebuggerPlaybackExtension;
	class FRewindDebuggerRecordingExtension;
	struct FRewindDebuggerTrackCreator;
}

STATETREEEDITORMODULE_API DECLARE_LOG_CATEGORY_EXTERN(LogStateTreeEditor, Log, All);

/**
* The public interface to this module
*/
class FStateTreeEditorModule : public IModuleInterface, public IHasMenuExtensibility, public IHasToolBarExtensibility
{
public:
	//~Begin IModuleInterface
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;
	//~End IModuleInterface

	/** Gets this module, will attempt to load and should always exist. */
	static UE_API FStateTreeEditorModule& GetModule();

	/** Gets this module, will not attempt to load and may not exist. */
	static UE_API FStateTreeEditorModule* GetModulePtr();

	/** Creates an instance of StateTree editor. Only virtual so that it can be called across the DLL boundary. */
	UE_API virtual TSharedRef<IStateTreeEditor> CreateStateTreeEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UStateTree* StateTree);

	/** Sets the Details View with required State Tree Detail Property Handlers */
	static UE_API void SetDetailPropertyHandlers(IDetailsView& DetailsView);

	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override
	{
		return MenuExtensibilityManager;
	}
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override
	{
		return ToolBarExtensibilityManager;
	}

	UE_API TSharedPtr<FStateTreeNodeClassCache> GetNodeClassCache();
	
	DECLARE_EVENT_OneParam(FStateTreeEditorModule, FOnRegisterLayoutExtensions, FLayoutExtender&);
	FOnRegisterLayoutExtensions& OnRegisterLayoutExtensions()
	{
		return RegisterLayoutExtensions;
	}

	DECLARE_EVENT_TwoParams(FStateTreeEditorModule, FOnAssetRegistryTags, TNotNull<const UStateTree*> /*StateTree*/, FAssetRegistryTagsContext& /*Context*/);
	/** Handle GetAssetRegistryTags for a state tree asset. */
	FOnAssetRegistryTags& OnAssetRegistryTags()
	{
		return AssetRegistryTags;
	}

	DECLARE_EVENT_OneParam(FStateTreeEditorModule, FValidateStateTree, TNotNull<UStateTree*> StateTree);
	/**
	 * Handle the validation a state tree asset.
	 * This is before the engine validation. The Schema and EditorData will be correctly set.
	 * You are allowed to modify the editor data.
	 * @note Use the UStateTreeEditorDataExtension::PreValidate for controlling a single asset.
	 * @note Use the UStateTreeEditorSchema::PreValidate for controlling a type of state tree asset.
	 */
	FValidateStateTree& OnPreValidateStateTree()
	{
		return PreValidateStateTree;
	}

	/**
	 * Handle the validation a state tree asset.
	 * Also serves as the "safety net" of fixing up editor data following an editor operation.
	 * You are allowed to modify the editor data.
	 * @note Use the UStateTreeEditorDataExtension::Validate for controlling a single asset.
	 * @note Use the UStateTreeEditorSchema::Validate for controlling a type of state tree asset.
	 */
	FValidateStateTree& OnValidateStateTree()
	{
		return ValidateStateTree;
	}

	DECLARE_EVENT_OneParam(FStateTreeEditorModule, FPostInternalCompile, const UE::StateTree::Compiler::FPostInternalContext&);
	/**
	 * Handle post internal compilation for a state tree asset.
	 * The state tree asset compiled successfully.
	 * @note Use the UStateTreeEditorDataExtension::HandlePostInternalCompile for controlling a single asset.
	 * @note Use the UStateTreeEditorSchema::HandlePostInternalCompile for controlling a type of state tree asset.
	 */
	FPostInternalCompile& OnPostInternalCompile()
	{
		return PostInternalCompile;
	}

	/** Register the editor data type for a specific schema. */
	UE_API void RegisterEditorDataClass(TNonNullSubclassOf<const UStateTreeSchema> Schema, TNonNullSubclassOf<const UStateTreeEditorData> EditorData);
	/** Unregister the editor data type for a specific schema. */
	UE_API void UnregisterEditorDataClass(TNonNullSubclassOf<const UStateTreeSchema> Schema);
	/** Get the editor data type for a specific schema. */
	UE_API TNonNullSubclassOf<UStateTreeEditorData> GetEditorDataClass(TNonNullSubclassOf<const UStateTreeSchema> Schema) const;

	/** Register the editor schema type for a specific schema. */
	UE_API void RegisterEditorSchemaClass(TNonNullSubclassOf<const UStateTreeSchema> Schema, TNonNullSubclassOf<const UStateTreeEditorSchema> EditorSchema);
	/** Unregister the editor schema type for a specific schema. */
	UE_API void UnregisterEditorSchemaClass(TNonNullSubclassOf<const UStateTreeSchema> Schema);
	/** Get the editor schema type for a specific schema. */
	UE_API TNonNullSubclassOf<UStateTreeEditorSchema> GetEditorSchemaClass(TNonNullSubclassOf<const UStateTreeSchema> Schema) const;

private:
	uint32 HandleRequestEditorHash(const UStateTree& InStateTree);
	void HandleCompileIfChanged(TNotNull<UStateTree*> InStateTree);
	void HandleRequestAssetRegistryTags(TNotNull<const UStateTree*> InStateTree, FAssetRegistryTagsContext& Context);
	void HandleStateTreeAssetLoaded(TNotNull<UStateTree*> InStateTree);
	void HandleStateTreeEditorBindingUpdated(TNotNull<UStateTreeEditorData*> InStateTreeEditorData);
	void HandleAppendToStateTreeClassSchema(FAppendToClassSchemaContext& Context);

protected:
	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;
	TSharedPtr<FStateTreeNodeClassCache> NodeClassCache;

	struct FEditorTypes
	{
		TWeakObjectPtr<const UClass> Schema;
		TWeakObjectPtr<const UClass> EditorData;
		TWeakObjectPtr<const UClass> EditorSchema;
		bool HasData() const;
	};
	TArray<FEditorTypes> EditorTypes;

#if WITH_STATETREE_TRACE_DEBUGGER
	TPimplPtr<UE::StateTreeDebugger::FRewindDebuggerPlaybackExtension> RewindDebuggerPlaybackExtension;
	TPimplPtr<UE::StateTreeDebugger::FRewindDebuggerTrackCreator> RewindDebuggerTrackCreator;
#endif  // WITH_STATETREE_TRACE_DEBUGGER

	FDelegateHandle OnPostEngineInitHandle;
	UE_DEPRECATED(5.7, "OnUserDefinedStructReinstancedHandle is not used.")
	FDelegateHandle OnUserDefinedStructReinstancedHandle;
	FOnRegisterLayoutExtensions RegisterLayoutExtensions;

	/** Handle for unregistering the categories registered with FBlueprintEditorModule. */
	FBlueprintEditorModule::FAdditionalCategoryHandle ContextCategoryHandle;
	FBlueprintEditorModule::FAdditionalCategoryHandle InputCategoryHandle;
	FBlueprintEditorModule::FAdditionalCategoryHandle OutputCategoryHandle;

	FOnAssetRegistryTags AssetRegistryTags;
	FValidateStateTree PreValidateStateTree;
	FValidateStateTree ValidateStateTree;
	FPostInternalCompile PostInternalCompile;
};

#undef UE_API
