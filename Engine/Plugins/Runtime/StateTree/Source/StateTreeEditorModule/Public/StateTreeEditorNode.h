// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "StateTreeNodeBase.h"
#include "StateTreeEditorNode.generated.h"

#define UE_API STATETREEEDITORMODULE_API

UENUM()
enum class EStateTreeNodeType : uint8
{
	EnterCondition,
	Evaluator,
	Task,
	TransitionCondition,
	StateParameters,
	PropertyFunction,
};

/**
 * Base for Evaluator, Task, Condition, Consideration Utilities, Property Function nodes.
 */
USTRUCT(BlueprintType)
struct FStateTreeEditorNode
{
	GENERATED_BODY()

	/** Resets node and instance fields to their default values. */
	UE_API void Reset();

	/**
	 * Initializes the node as the given type, resetting any previous state.
	 * Handles both instance data and execution runtime data types.
	 * @param InOuter Outer object used when creating UObject instance or execution runtime data.
	 */
	template<typename T, typename... TArgs>
	void InitializeAs(TNotNull<UObject*> InOuter, TArgs&&... InArgs)
	{
		Reset();
		Node.InitializeAs<T>(Forward<TArgs>(InArgs)...);
		ID = FGuid::NewGuid();
		InitializeInstanceData(InOuter);
	}

	/**
	 * Initializes the node with the given struct type, resetting any previous state.
	 * Non-templated counterpart to the templated InitializeAs, for use when the node type is only known at runtime.
	 * Handles both instance data and execution runtime data types.
	 * @param InOuter Outer object used when creating UObject instance or execution runtime data.
	 * @param InNodeStruct Node struct type to initialize with.
	 */
	UE_API void InitializeAs(TNotNull<UObject*> InOuter, TNotNull<const UScriptStruct*> InNodeStruct);

	/** Resets and reallocates instance and execution runtime data to match the current node type, preserving the node ID. */
	UE_API void ReallocInstanceData(TNotNull<UObject*> InOuter);

	/**
	 * Fixes UObject instance data of the node, ensuring proper outers and making duplicates unique.
	 * Creates any missing instance or execution runtime data.
	 *
	 * SeenObjects is threaded through all nodes in a single traversal so that shared UObject references
	 * (e.g. from copy-paste) are detected and made unique on the spot, without a separate pre-pass to
	 * collect reference counts. Each node adds its live instance objects to the set after processing,
	 * so any later node holding the same pointer can immediately DuplicateObject it.
	 *
	 * @param SeenObjects Set of UObject instances already assigned to earlier nodes. Updated in place.
	 * @param Outer Outer used for re-parenting and creating new instances.
	 * @return true if any changes were made.
	 */
	UE_API bool FixObjectInstances(TSet<UObject*>& SeenObjects, TNotNull<UObject*> Outer);

private:
	UE_API void InitializeInstanceData(TNotNull<UObject*> InOuter);

public:
	/**
	 * This is used to name nodes for runtime, as well as for error reporting.
	 * If the node has a specified name, used that, or else of return the display name of the node.
	 * @return name of the node.
	 */
	UE_API FName GetName() const;

	/**
	 * Sets the name of the node. Resets the cached node name if the name changed.
	 * @param NewName Name to set on the node.
	 */
	UE_API void SetNodeName(FName NewName);

	/**
	 * Get DataView for the node template.
	 * @return DataView for the node template
	 */
	TStructView<FStateTreeNodeBase> GetNode() const
	{
		return TStructView<FStateTreeNodeBase>(Node.GetScriptStruct(), const_cast<uint8*>(Node.GetMemory()));
	}

	/**
	 * Get ID for the node template, which is different from the ID for the Instance.
	 * @return ID for the node template
	 */
	FGuid GetNodeID() const
	{
		return FGuid::Combine(ID, FGuid::NewDeterministicGuid(TEXT("Node Struct")));
	}

	/**
	 * Get ID for the node instance data, which is different from the ID for the template.
	 * @return ID for the instance data
	 */
	FGuid GetInstanceDataID() const
	{
		return ID;
	}

	FStateTreeDataView GetInstance() const
	{
		return InstanceObject ? FStateTreeDataView(InstanceObject) : FStateTreeDataView(const_cast<FInstancedStruct&>(Instance));
	}

	FStateTreeDataView GetInstance()
	{
		return InstanceObject ? FStateTreeDataView(InstanceObject) : FStateTreeDataView(Instance);
	}

	FStateTreeDataView GetExecutionRuntimeData() const
	{
		return ExecutionRuntimeDataObject ? FStateTreeDataView(ExecutionRuntimeDataObject) : FStateTreeDataView(const_cast<FInstancedStruct&>(ExecutionRuntimeData));
	}

	FStateTreeDataView GetExecutionRuntimeData()
	{
		return ExecutionRuntimeDataObject ? FStateTreeDataView(ExecutionRuntimeDataObject) : FStateTreeDataView(ExecutionRuntimeData);
	}

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = Node)
	FInstancedStruct Node;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = Node)
	FInstancedStruct Instance;

	UPROPERTY(EditDefaultsOnly, Instanced, BlueprintReadOnly, Category = Node)
	TObjectPtr<UObject> InstanceObject = nullptr;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = Node)
	FInstancedStruct ExecutionRuntimeData;

	UPROPERTY(EditDefaultsOnly, Instanced, BlueprintReadOnly, Category = Node)
	TObjectPtr<UObject> ExecutionRuntimeDataObject = nullptr;

	/** ID for the node instance. */
	UPROPERTY(EditDefaultsOnly, Category = Node)
	FGuid ID;

	UPROPERTY(EditDefaultsOnly, Category = Node)
	uint8 ExpressionIndent = 0;

	UPROPERTY(EditDefaultsOnly, Category = Node)
	EStateTreeExpressionOperand ExpressionOperand = EStateTreeExpressionOperand::And;

private:
	mutable TOptional<FName> CachedNodeName;
};

template <typename T>
struct TStateTreeEditorNode : public FStateTreeEditorNode
{
	using NodeType = T;
	inline T& GetNode()
	{
		return Node.template GetMutable<T>();
	}
	inline typename T::FInstanceDataType& GetInstanceData()
	{
		return Instance.template GetMutable<typename T::FInstanceDataType>();
	}
	inline typename T::FExecutionRuntimeDataType& GetExecutionRuntimeData()
	{
		return ExecutionRuntimeData.template GetMutable<typename T::FExecutionRuntimeDataType>();
	}
};

#undef UE_API
