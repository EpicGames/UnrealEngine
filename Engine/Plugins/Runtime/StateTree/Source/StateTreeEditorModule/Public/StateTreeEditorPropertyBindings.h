// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "UObject/ObjectKey.h"
#include "PropertyBindingExtension.h"
#include "PropertyBindingBindingCollectionOwner.h"
#include "StateTreePropertyBindings.h"
#include "StateTreeEditorNode.h"
#include "StateTreeEditorPropertyBindings.generated.h"

#define UE_API STATETREEEDITORMODULE_API

enum class EStateTreeNodeFormatting : uint8;
class IStateTreeEditorPropertyBindingsOwner;
enum class EStateTreeVisitor : uint8;

/**
 * Editor representation of all property bindings in a StateTree
 */
USTRUCT()
struct FStateTreeEditorPropertyBindings : public FPropertyBindingBindingCollection
{
	GENERATED_BODY()

	/** @return const array view to all bindings. */
	TConstArrayView<FStateTreePropertyPathBinding> GetBindings() const
	{
		return PropertyBindings;
	}

	/** @return array view to all bindings. */
	TArrayView<FStateTreePropertyPathBinding> GetMutableBindings()
	{
		return PropertyBindings;
	}

	void AddStateTreeBinding(FStateTreePropertyPathBinding&& InBinding)
	{
		RemoveBindings(InBinding.GetTargetPath(), ESearchMode::Exact);
		PropertyBindings.Add(MoveTemp(InBinding));
	}

	//~ Begin FPropertyBindingBindingCollection overrides
	UE_API virtual int32 GetNumBindableStructDescriptors() const override;
	UE_API virtual const FPropertyBindingBindableStructDescriptor* GetBindableStructDescriptorFromHandle(FConstStructView InSourceHandleView) const override;

	UE_API virtual int32 GetNumBindings() const override;
	UE_API virtual void ForEachBinding(TFunctionRef<void(const FPropertyBindingBinding& Binding)> InFunction) const override;
	UE_API virtual void ForEachBinding(FPropertyBindingIndex16 InBegin, FPropertyBindingIndex16 InEnd, TFunctionRef<void(const FPropertyBindingBinding& Binding, const int32 BindingIndex)> InFunction) const override;
	UE_API virtual void ForEachMutableBinding(TFunctionRef<void(FPropertyBindingBinding& Binding)> InFunction) override;
	UE_API virtual void VisitBindings(TFunctionRef<EVisitResult(const FPropertyBindingBinding& Binding)> InFunction) const override;
	UE_API virtual void VisitBindings(FPropertyBindingIndex16 InBegin, FPropertyBindingIndex16 InEnd, TFunctionRef<EVisitResult(const FPropertyBindingBinding& Binding, const int32 BindingIndex)> InFunction) const override;
	UE_API virtual void VisitMutableBindings(TFunctionRef<EVisitResult(FPropertyBindingBinding& Binding)> InFunction) override;
	//~ End FPropertyBindingBindingCollection overrides

	/**
	 * Adds binding between PropertyFunction of the provided type and destination path.
	 * @param InPropertyFunctionNodeStruct Struct of PropertyFunction.
	 * @param InSourcePathSegments Binding source property path segments.
	 * @param InTargetPath Binding target property path.
	 * @return Constructed binding source property path.
	 */
	UE_API FPropertyBindingPath AddFunctionBinding(const UScriptStruct* InPropertyFunctionNodeStruct, TConstArrayView<FPropertyBindingPathSegment> InSourcePathSegments, const FPropertyBindingPath& InTargetPath);

	/**
	 * Adds an output binding between source and target path.
	 * Output Binding will copy value from target to source
	 * @param InSourcePath Binding source property path segments.
	 * @param InTargetPath Binding target property path.
	 * @return Constructed Binding.
	 */
	UE_API const FStateTreePropertyPathBinding* AddOutputBinding(const FPropertyBindingPath& InSourcePath, const FPropertyBindingPath& InTargetPath);

	/**
	 * Adds a task completion binding between a task and target delegate listener.
	 * Task completion binding, generate a delegate dispatcher when compiled.
	 * @param SourceTask The task that will broadcast the delegate.
	 * @param TargetPath Listener target property path.
	 * @param Completion The condition for the task to broadcast the delegate.
	 * @return Constructed Binding.
	 */
	UE_API const FStateTreePropertyPathBinding* AddTaskCompletionBinding(const FGuid& SourceTask, const FPropertyBindingPath& TargetPath, UE::StateTree::ETaskCompletionCondition Completion);

	/**
	 * Check if any editor binding is dependent on any of the structs
	 * @param InStructsDependedOn Structs to check if any editor binding is dependent on
	 * @return true if any editor binding is dependent on any of the passed in structs
	 */
	UE_API bool IsDependentOn(TConstArrayView<UStruct*> InStructsDependedOn) const;

	/**
	 * Collect all the structs that this editor bindings are dependent on
	 */
	UE_API TArray<TNotNull<const UStruct*>> GatherDependencies() const;

	template<typename Predicate>
	TArray<TNotNull<const UStruct*>> GatherDependenciesByPredicate(Predicate Pred) const
	{
		TArray<TNotNull<const UStruct*>> DependentStructs;
		DependentStructs.Reserve(CachedDependencies.Num());

		for (const TObjectKey<const UStruct> StructObjectKey : CachedDependencies)
		{
			if (const UStruct* Struct = StructObjectKey.ResolveObjectPtr())
			{
				if (::Invoke(Pred, Struct))
				{
					DependentStructs.Emplace(Struct);
				}
			}
		}

		return DependentStructs;
	}

	/**
	 * Set Dependency for Editor Bindings
	 * @param InDependentStructs Structs editor bindings are dependent on
	 */
	UE_API void CacheDependency(const TConstArrayView<TNotNull<const UStruct*>> InDependentStructs);
protected:
	//~ Begin FPropertyBindingBindingCollection overrides
	virtual FPropertyBindingBinding* AddBindingInternal(const FPropertyBindingPath& InSourcePath, const FPropertyBindingPath& InTargetPath) override;
	UE_API virtual void CopyBindingsInternal(const FGuid InFromStructID, const FGuid InToStructID) override;
	UE_API virtual void RemoveBindingsInternal(TFunctionRef<bool(FPropertyBindingBinding&)> InPredicate) override;
	UE_API virtual bool HasBindingInternal(TFunctionRef<bool(const FPropertyBindingBinding&)> InPredicate) const override;
	UE_API virtual const FPropertyBindingBinding* FindBindingInternal(TFunctionRef<bool(const FPropertyBindingBinding&)> InPredicate) const override;
	//~ Begin FPropertyBindingBindingCollection overrides

private:
	UPROPERTY()
	TArray<FStateTreePropertyPathBinding> PropertyBindings;

	TSet<TObjectKey<const UStruct>> CachedDependencies;
};


UINTERFACE(MinimalAPI, meta = (CannotImplementInterfaceInBlueprint))
class UStateTreeEditorPropertyBindingsOwner : public UPropertyBindingBindingCollectionOwner
{
	GENERATED_UINTERFACE_BODY()
};

class IStateTreeEditorPropertyBindingsOwner : public IPropertyBindingBindingCollectionOwner
{
	GENERATED_BODY()
public:

	/**
	 * Finds a bindable context struct based on name and type.
	 * @param ObjectType Object type to match
	 * @param ObjectNameHint Name to use if multiple context objects of same type are found. 
	 */
	virtual FStateTreeBindableStructDesc FindContextData(const UStruct* ObjectType, const FString ObjectNameHint) const PURE_VIRTUAL(IStateTreeEditorPropertyBindingsOwner::FindContextData, return {}; );

	/** @return Pointer to editor property bindings. */
	virtual FStateTreeEditorPropertyBindings* GetPropertyEditorBindings() PURE_VIRTUAL(IStateTreeEditorPropertyBindingsOwner::GetPropertyEditorBindings, return nullptr; );

	/** @return Pointer to editor property bindings. */
	virtual const FStateTreeEditorPropertyBindings* GetPropertyEditorBindings() const PURE_VIRTUAL(IStateTreeEditorPropertyBindingsOwner::GetPropertyEditorBindings, return nullptr; );

	virtual EStateTreeVisitor EnumerateBindablePropertyFunctionNodes(TFunctionRef<EStateTreeVisitor(const UScriptStruct* NodeStruct, const FStateTreeBindableStructDesc& Desc, const FStateTreeDataView Value)> InFunc) const PURE_VIRTUAL(IStateTreeEditorPropertyBindingsOwner::EnumerateBindablePropertyFunctionNodes, return static_cast<EStateTreeVisitor>(0); );
};

// TODO: We should merge this with IStateTreeEditorPropertyBindingsOwner and FStateTreeEditorPropertyBindings.
// Currently FStateTreeEditorPropertyBindings is meant to be used as a member for just to store things,
// IStateTreeEditorPropertyBindingsOwner is meant return model specific stuff,
// and IStateTreeBindingLookup is used in non-editor code and it cannot be in FStateTreeEditorPropertyBindings because bindings don't know about the owner.
struct FStateTreeBindingLookup : public IStateTreeBindingLookup
{
	UE_API FStateTreeBindingLookup(const IStateTreeEditorPropertyBindingsOwner* InBindingOwner);

	const IStateTreeEditorPropertyBindingsOwner* BindingOwner = nullptr;

	UE_API virtual const FPropertyBindingPath* GetPropertyBindingSource(const FPropertyBindingPath& InTargetPath) const override;
	UE_API virtual FText GetPropertyPathDisplayName(const FPropertyBindingPath& InTargetPath, EStateTreeNodeFormatting Formatting) const override;
	UE_API virtual FText GetBindingSourceDisplayName(const FPropertyBindingPath& InTargetPath, EStateTreeNodeFormatting Formatting) const override;
	UE_API virtual const FProperty* GetPropertyPathLeafProperty(const FPropertyBindingPath& InPath) const override;

};

#undef UE_API
