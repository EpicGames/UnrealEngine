// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "StateTreeEditorSchema.generated.h"

#define UE_API STATETREEEDITORMODULE_API

class UStateTree;
class UStateTreeEditorData;
namespace UE::StateTree::Compiler
{
	struct FPostInternalContext;
}

/** Rules specific to editing Transitions. */
UENUM()
enum class EStateTreeTransitionEditingRules : uint32
{
	/** Previous (UE 5.7) rules */
	None = 0,
	/**
	 * When enabled, user can specify re-activation in the editor. See FStateTreeTransition::ReactivateTargetState for more info.
	 */
	AllowReactivation = 1 << 0,

	Default = None,
};

/**
 * Schema describing how a StateTree is edited.
 */
UCLASS(MinimalAPI)
class UStateTreeEditorSchema : public UObject
{
	GENERATED_BODY()

public:
	/** @returns True if modifying extensions in the editor is allowed. An extension can be added by code. */
	virtual bool AllowExtensions() const
	{
		return true;
	}

public:
	/**
	 * @returns the path name of a Property Bag Schema class used by the state tree schema.
	 * It is used by the Global Parameters and the State Parameters.
	 * The format is "/Script/MyModule.MySchema".
	 */
	virtual TOptional<FString> GetParametersSchemaClass() const
	{
		return TOptional<FString>();
	}

public:
	/*
	 * Validates and applies the schema restrictions on the StateTree.
	 * This is before the engine validation. The EditorData will be correctly set.
	 */
	virtual void PreValidate(TNotNull<UStateTree*> StateTree)
	{
	}

	/*
	 * Validates and applies the schema restrictions on the StateTree.
	 * Also serves as the "safety net" of fixing up editor data following an editor operation.
	 */
	UE_API virtual void Validate(TNotNull<UStateTree*> StateTree);


	/**
	 * Handle compilation for the owning state tree asset.
	 * The state tree asset compiled successfully.
	 */
	virtual bool HandlePostInternalCompile(const UE::StateTree::Compiler::FPostInternalContext& Context)
	{
		return true;
	}

	/** Get editing rules for transitions */
	virtual EStateTreeTransitionEditingRules GetTransitionEditingRules(TNotNull<UStateTreeEditorData*> StateTree) const
	{
		return EStateTreeTransitionEditingRules::Default;
	}
};

#undef UE_API
