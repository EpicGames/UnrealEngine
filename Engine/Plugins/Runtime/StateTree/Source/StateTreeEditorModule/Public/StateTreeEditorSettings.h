// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "StateTreeEditorSettings.generated.h"

#define UE_API STATETREEEDITORMODULE_API

UENUM()
enum class EStateTreeSaveOnCompile : uint8
{
	Never UMETA(DisplayName = "Never"),
	SuccessOnly UMETA(DisplayName = "On Success Only"),
	Always UMETA(DisplayName = "Always"),
};

UENUM()
enum class EStateTreeDebuggerTrackNameVerbosity : uint8
{
	/** Display only the StateTree asset name (e.g. "MyStateTree"). */
	Compact UMETA(DisplayName = "Compact"),

	/** Display the instance name and asset name (e.g. "MyInstance StateTree (MyStateTree)"). */
	Verbose UMETA(DisplayName = "Verbose"),
};

UCLASS(MinimalAPI, config = EditorPerProjectUserSettings)
class UStateTreeEditorSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

	static UStateTreeEditorSettings& Get()
	{
		return *GetMutableDefault<UStateTreeEditorSettings>();
	}

	/** Determines when to save StateTrees post-compile */
	UPROPERTY(EditAnywhere, config, Category = "Compiler")
	EStateTreeSaveOnCompile SaveOnCompile = EStateTreeSaveOnCompile::Never;

	/** Controls how StateTree tracks are labeled in the Rewind Debugger. Compact shows only the asset name; Verbose also includes the instance name. */
	UPROPERTY(EditAnywhere, config, Category = "Debugger")
	EStateTreeDebuggerTrackNameVerbosity DebuggerTrackNameVerbosity = EStateTreeDebuggerTrackNameVerbosity::Compact;

	/**
	 * If enabled, debugger window in the StateTree Asset Editor will display all widgets
	 * related to the legacy debugger (recording controls, timelines, frame details, etc.).
	 * Otherwise, it will display options to link to open RewindDebugger and select a given instance
	 */
	UPROPERTY(EditAnywhere, config, Category = "Debugger")
	bool bEnableLegacyDebuggerWindow = false;

	/** If enabled, debugger starts recording information at the start of each PIE session. */
	UPROPERTY(EditAnywhere, config, Category = "Debugger", meta = (EditCondition = "bEnableLegacyDebuggerWindow", EditConditionHides))
	bool bShouldDebuggerAutoRecordOnPIE = true;

	/** If enabled, debugger will clear previous tracks at the start of each PIE session. */
	UPROPERTY(EditAnywhere, config, Category = "Debugger", meta = (EditCondition = "bEnableLegacyDebuggerWindow", EditConditionHides))
	bool bShouldDebuggerResetDataOnNewPIESession = false;

	/**
	 * If enabled, changing the class of a node will try to copy over values of properties with the same name and type.
	 * i.e. if you change one condition for another, and both have a "Target" BB key selector, it'll be kept.
	 */
	UPROPERTY(EditAnywhere, config, Experimental, Category = "Experimental")
	bool bRetainNodePropertyValues = false;

protected:
#if WITH_EDITOR
	UE_API virtual FText GetSectionText() const override;
	UE_API virtual FText GetSectionDescription() const override;
#endif

	UE_API virtual FName GetCategoryName() const override;
};

#undef UE_API
