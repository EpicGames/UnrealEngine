// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeStyle.h"

#define UE_API STATETREEEDITORMODULE_API

enum class EStateTreeStateSelectionBehavior : uint8;
enum class EStateTreeStateType : uint8;

class ISlateStyle;

class FStateTreeEditorStyle : public FStateTreeStyle
{
public:
	static UE_API FStateTreeEditorStyle& Get();

	static UE_API const FSlateBrush* GetBrushForSelectionBehaviorType(EStateTreeStateSelectionBehavior InSelectionBehavior, bool bInHasChildren, EStateTreeStateType InStateType);

protected:
	friend class FStateTreeEditorModule;

	static UE_API void Register();
	static UE_API void Unregister();

private:
	UE_API FStateTreeEditorStyle();
};

#undef UE_API
