// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Math/Color.h"
#include "Misc/Guid.h"
#include "PropertyBindingPath.h"
#include "StateTreeDelegate.h"
#include "UObject/Class.h"
#include "StateTreeEditorTypes.generated.h"

/**
 * Id Struct to uniquely identify an FStateTreeEditorColor instance.
 * An existing FStateTreeEditorColor instance can be found via UStateTreeEditorData::FindColor
 */
USTRUCT()
struct FStateTreeEditorColorRef
{
	GENERATED_BODY()

	FStateTreeEditorColorRef() = default;

	explicit FStateTreeEditorColorRef(const FGuid& ID)
		: ID(ID)
	{
	}

	bool operator==(const FStateTreeEditorColorRef& Other) const
	{
		return ID == Other.ID;
	}

	friend uint32 GetTypeHash(const FStateTreeEditorColorRef& ColorRef)
	{
		return GetTypeHash(ColorRef.ID);
	}

	UPROPERTY(EditDefaultsOnly, Category = "Theme")
	FGuid ID;
};

/**
 * Struct describing a Color, its display name and a unique identifier to get an instance via UStateTreeEditorData::FindColor
 */
USTRUCT()
struct FStateTreeEditorColor
{
	GENERATED_BODY()

	FStateTreeEditorColor()
		: ColorRef(FGuid::NewGuid())
	{
	}

	explicit FStateTreeEditorColor(const FStateTreeEditorColorRef& ColorRef)
		: ColorRef(ColorRef)
	{
	}

	/**
	 * Export Text Item override where properties marked with meta-data "StructExportTransient" are excluded from the exported string
	 * This is so that copy/pasting State Tree Color entries don't have the effect of also copying over these properties into a new entry.
	 * Since it works with meta-data, it's editor-only.
	 * Side note: The existing "TextExportTransient" / "DuplicateTransient" specifiers apply for uclass properties only
	 */ 
	STATETREEEDITORMODULE_API bool ExportTextItem(FString& OutValueString, const FStateTreeEditorColor& DefaultValue, UObject* OwnerObject, int32 PortFlags, UObject* ExportRootScope) const;

	bool operator==(const FStateTreeEditorColor& Other) const
	{
		return ColorRef == Other.ColorRef;
	}

	friend uint32 GetTypeHash(const FStateTreeEditorColor& InColor)
	{
		return GetTypeHash(InColor.ColorRef);
	}

	/** ID unique per State Tree Color Entry. Marked as struct export transient so that copy-pasting this entry does not result in the same repeating ID */
	UPROPERTY(meta=(StructExportTransient))
	FStateTreeEditorColorRef ColorRef;

	UPROPERTY(EditDefaultsOnly, Category = "Theme")
	FString DisplayName;

	UPROPERTY(EditDefaultsOnly, Category = "Theme", meta=(HideAlphaChannel))
	FLinearColor Color = FLinearColor(0.4f, 0.4f, 0.4f);
};

template<>
struct TStructOpsTypeTraits<FStateTreeEditorColor> : TStructOpsTypeTraitsBase2<FStateTreeEditorColor>
{
	enum
	{
		WithExportTextItem = true,
	};
};

/**
 * Struct describing a compiled delegate dispatcher.
 * To help with delta cooking/compilation. We want the same generated ID every time the asset compiles.
 */
USTRUCT()
struct FStateTreeEditorDelegateDispatcherCompiledBinding
{
	GENERATED_BODY()

	UPROPERTY()
	FPropertyBindingPath DispatcherPath;

	UPROPERTY()
	FStateTreeDelegateDispatcher ID;
};