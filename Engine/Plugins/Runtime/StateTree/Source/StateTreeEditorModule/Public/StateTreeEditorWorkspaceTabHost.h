// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Textures/SlateIcon.h"
#include "Framework/Docking/TabManager.h"

#define UE_API STATETREEEDITORMODULE_API

class SDockTab;

namespace UE::StateTreeEditor
{
	struct FMinorWorkspaceTabConfig
	{
		FName ID;
		FText Label;
		FText Tooltip;
		FSlateIcon Icon;
		FName UISystemID;
	};

	struct FSpawnedWorkspaceTab
	{
		FName TabID;
		TWeakPtr<SDockTab> DockTab;
	};

	// Interface required for re-using the same tab management across different AssetEditors
	class FWorkspaceTabHost : public TSharedFromThis<FWorkspaceTabHost>
	{
	public:
		static UE_API const FLazyName BindingTabId;
		static UE_API const FLazyName DebuggerTabId;
		static UE_API const FLazyName OutlinerTabId;
		static UE_API const FLazyName SearchTabId;
		static UE_API const FLazyName StatisticsTabId;

	public:
		virtual ~FWorkspaceTabHost() = default;

		UE_API FOnSpawnTab CreateSpawnDelegate(FName TabID);
		UE_API TConstArrayView<FMinorWorkspaceTabConfig> GetTabConfigs() const;

		TConstArrayView<FSpawnedWorkspaceTab> GetSpawnedTabs() const
		{
			return SpawnedTabs;
		}

		UE_API TSharedRef<SDockTab> Spawn(FName TabID);

		DECLARE_MULTICAST_DELEGATE_OneParam(FOnWorkspaceSpawnTab, FSpawnedWorkspaceTab);
		FOnWorkspaceSpawnTab OnTabSpawned;
		FOnWorkspaceSpawnTab OnTabClosed;

	private:
		UE_API TSharedRef<SDockTab> HandleSpawnDelegate(const FSpawnTabArgs& Args, FName TabID);
		UE_API void HandleTabClosed(TSharedRef<SDockTab>);
		TArray<FSpawnedWorkspaceTab> SpawnedTabs;
	};
}

#undef UE_API
