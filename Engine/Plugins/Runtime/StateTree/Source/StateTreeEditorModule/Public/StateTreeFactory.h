// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "StateTreeFactory.generated.h"

#define UE_API STATETREEEDITORMODULE_API

class UStateTreeSchema;

/**
 * Factory for UStateTree
 */

UCLASS(MinimalAPI)
class UStateTreeFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ UFactory interface
	UE_API virtual bool ConfigureProperties() override;
	UE_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ End of UFactory interface

	UE_API void SetSchemaClass(const TObjectPtr<UClass>& InSchemaClass);  

protected:
	
	UPROPERTY()
	TObjectPtr<UClass> StateTreeSchemaClass;
};

#undef UE_API
