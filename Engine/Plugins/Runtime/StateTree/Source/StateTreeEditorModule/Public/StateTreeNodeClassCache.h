// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/TopLevelAssetPath.h"
#include "StateTreeNodeClassCache.generated.h"

#define UE_API STATETREEEDITORMODULE_API

enum class EReloadCompleteReason;

struct FAssetData;
struct FStateTreeNodeBase;

/**
 * Describes a class or struct.
 * If the class or struct is from a package that is not yet loaded, the data will update on GetStruct/Class/Scripstruct()
 */
USTRUCT()
struct FStateTreeNodeClassData
{
	GENERATED_BODY()

	FStateTreeNodeClassData() = default;
	UE_API FStateTreeNodeClassData(TNotNull<UStruct*> InStruct);
	UE_API FStateTreeNodeClassData(const FTopLevelAssetPath& AssetPath);

	UE_DEPRECATED(5.7, "Use the constructor with FTopLevelAssetPath")
	UE_API FStateTreeNodeClassData(const FString& InClassAssetName, const FString& InClassPackage, const FName InStructName, UStruct* InStruct);

	UE_DEPRECATED(5.7, "Use GetStructPath")
	FName GetStructName() const
	{
		return StructAssetPath.GetAssetName();
	}

	const FTopLevelAssetPath& GetStructPath() const
	{
		return StructAssetPath;
	}

	UE_API UStruct* GetStruct(bool bSilent = false);

	TWeakObjectPtr<UStruct> GetWeakStruct() const
	{
		return CachedStruct;
	}

	UClass* GetClass(bool bSilent = false)
	{
		return Cast<UClass>(GetStruct(bSilent));
	}
	UScriptStruct* GetScriptStruct(bool bSilent = false)
	{
		return Cast<UScriptStruct>(GetStruct(bSilent));
	}

	UE_API const UStruct* GetInstanceDataStruct(bool bSilent = false);

private:
	/** struct or class asset path. */
	FTopLevelAssetPath StructAssetPath;

	/** Pointer to described struct or class. */
	TWeakObjectPtr<UStruct> CachedStruct;

	/** Pointer to described node's instance data struct or class. */
	TWeakObjectPtr<const UStruct> CachedInstanceDataStruct;
};

/**
 * Caches specified classes or structs and reacts to engine events to keep the lists always up to date.
 * All the derived classes or structs are kept in the cache.
 */
struct FStateTreeNodeClassCache
{
	UE_API FStateTreeNodeClassCache();
	UE_API ~FStateTreeNodeClassCache();

	/** Adds a Struct to keep track of */
	UE_API void AddRootStruct(TNotNull<UStruct*> RootStruct);
	
	/** Adds a Class to keep track of */
	void AddRootClass(TNotNull<UClass*> RootClass)
	{
		AddRootStruct(RootClass);
	}
	
	/** Adds a ScriptStruct to keep track of */
	void AddRootScriptStruct(TNotNull<UScriptStruct*> RootStruct)
	{
		AddRootStruct(RootStruct);
	}

	/** Returns know derived Structs based on provided base. If the base Struct is not added as root Struct, nothing is returned. */
	UE_API void GetStructs(TNotNull<const UStruct*> BaseStruct, TArray<TSharedPtr<FStateTreeNodeClassData>>& AvailableClasses);
	
	/** Returns know derived Classes based on provided base. If the base Class is not added as root Class, nothing is returned. */
	void GetClasses(TNotNull<const UStruct*> BaseClass, TArray<TSharedPtr<FStateTreeNodeClassData>>& AvailableClasses)
	{
		GetStructs(BaseClass, AvailableClasses);
	}
	
	/** Returns know derived ScriptStructs based on provided base. If the base struct is not added as root ScriptStruct, nothing is returned. */
	void GetScripStructs(TNotNull<const UScriptStruct*> BaseStruct, TArray<TSharedPtr<FStateTreeNodeClassData>>& AvailableClasses)
	{
		GetStructs(BaseStruct, AvailableClasses);
	}

	/** Invalidates the cache, it will be rebuild on next access. */
	UE_API void InvalidateCache();

protected:
	UE_API void OnAssetAdded(const FAssetData& AssetData);
	UE_API void OnAssetRemoved(const FAssetData& AssetData);
	UE_API void OnReloadComplete(EReloadCompleteReason Reason);
	
private:
	struct FRootClassContainer
	{
		FRootClassContainer() = default;
		FRootClassContainer(TNotNull<UStruct*> InBaseStruct, int32 InIndex)
		 : BaseStruct(InBaseStruct)
		 , Index(InIndex)
		{
		} 
		TWeakObjectPtr<const UStruct> BaseStruct;
		TArray<TSharedPtr<FStateTreeNodeClassData>> ClassData;
		int32 Index = 0;
		bool bUpdated = false;
	};

	UE_API void UpdateBlueprintClass(const FAssetData& AssetData);
	UE_API void CacheClasses();
	UE_API void CacheClasses(FRootClassContainer& RootClass);

	TArray<FRootClassContainer> RootClasses;
	TMap<FString, int32> ClassNameToRootIndex;
};

#undef UE_API
