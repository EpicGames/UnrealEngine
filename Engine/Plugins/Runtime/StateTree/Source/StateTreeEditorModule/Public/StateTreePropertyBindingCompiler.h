// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeDelegate.h"
#include "StateTreeEditorTypes.h"
#include "StateTreePropertyBindings.h"
#include "StateTreePropertyBindingCompiler.generated.h"

#define UE_API STATETREEEDITORMODULE_API

enum class EPropertyAccessCompatibility;
struct FStateTreeCompilerLog;
struct FStateTreePropertyPathBinding;

/**
 * Helper class to compile editor representation of property bindings into runtime representation.
 * TODO: Better error reporting, something that can be shown in the UI.
 */
USTRUCT()
struct FStateTreePropertyBindingCompiler
{
	GENERATED_BODY()

	/**
	  * Initializes the compiler to compile copies to specified Property Bindings.
	  * @param PropertyBindings - Reference to the Property Bindings where all the batches will be stored.
	  * @return true on success.
	  */
	[[nodiscard]] UE_API bool Init(FStateTreePropertyBindings& InPropertyBindings, FStateTreeCompilerLog& InLog);

	/**
	  * Compiles a batch of property copies.
	  * @param TargetStruct - Description of the structs which contains the target properties.
	  * @param PropertyBindings - Array of bindings to compile, all bindings that point to TargetStructs will be added to the batch.
	  * @param PropertyFuncsBegin - Index of the first PropertyFunction belonging to this batch.
	  * @param PropertyFuncsEnd - Index of the last PropertyFunction belonging to this batch.
	  * @param OutBatchIndex - Resulting batch index, if index is INDEX_NONE, no bindings were found and no batch was generated.
	  * @return True on success, false on failure.
	 */
	[[nodiscard]] UE_API bool CompileBatch(const FStateTreeBindableStructDesc& TargetStruct, TConstArrayView<FStateTreePropertyPathBinding> PropertyBindings, FStateTreeIndex16 PropertyFuncsBegin, FStateTreeIndex16 PropertyFuncsEnd, int32& OutBatchIndex);

	 /**
	  * Compiles delegates dispatcher for selected struct.
	  * @param SourceStruct - Description of the structs which contains the delegate dispatchers.
	  * @param PreviousDistpatchers - Array of previous dispatchers IDs.
	  * @param DelegateSourceBindings - Array of bindings dispatcher to compile.
	  * @param InstanceDataView - View to the instance data
	  * @return True on success, false on failure.
	 */
	[[nodiscard]] UE_API bool CompileDelegateDispatchers(const FStateTreeBindableStructDesc& SourceStruct, TConstArrayView<FStateTreeEditorDelegateDispatcherCompiledBinding> PreviousDistpatchers, TConstArrayView<FStateTreePropertyPathBinding> DelegateSourceBindings, FStateTreeDataView InstanceDataView);

	 /**
	  * Compiles delegates dispatcher for selected struct.
	  * @param SourceStruct - Description of the structs which is the delegate dispatchers when it completes.
	  * @param AllBindings - Array of all the bindings.
	  * @return True on success, false on failure.
	 */
	[[nodiscard]] UE_API bool CompileTaskCompletionDelegateDispatchers(const FStateTreeBindableStructDesc& SourceStruct, TConstArrayView<FStateTreePropertyPathBinding> AllBindings);

	 /**
	  * Compiles delegates listener for selected struct.
	  * @param TargetStruct - Description of the structs which contains the delegate listeners.
	  * @param DelegateTargetBindings - Array of bindings listeners to compile.
	  * @param InstanceDataView - View to the instance data
	  * @return True on success, false on failure.
	 */
	[[nodiscard]] UE_API bool CompileDelegateListeners(const FStateTreeBindableStructDesc& TargetStruct, TConstArrayView<FStateTreePropertyPathBinding> DelegateSourceBindings, FStateTreeDataView InstanceDataView);

	/**
	  * Compiles references for selected struct
	  * @param TargetStruct - Description of the structs which contains the target properties.
	  * @param PropertyReferenceBindings - Array of bindings to compile, all bindings that point to TargetStructs will be added.
	  * @param InstanceDataView - view to the instance data
	  * @return True on success, false on failure.
	 */
	[[nodiscard]] UE_API bool CompileReferences(const FStateTreeBindableStructDesc& TargetStruct, TConstArrayView<FStateTreePropertyPathBinding> PropertyReferenceBindings, FStateTreeDataView InstanceDataView, const TMap<FGuid, const FStateTreeDataView>& IDToStructValue);

	/** Finalizes compilation, should be called once all batches are compiled. */
	UE_API void Finalize();

	/**
	  * Adds source struct. When compiling a batch, the bindings can be between any of the defined source structs, and the target struct.
	  * Source structs can be added between calls to Compilebatch().
	  * @param SourceStruct - Description of the source struct to add.
	  * @return Source struct index.
	  */
	UE_API int32 AddSourceStruct(const FStateTreeBindableStructDesc& SourceStruct);

	/** @return delegate dispatcher residing behind given path. */
	UE_API FStateTreeDelegateDispatcher GetDispatcherFromPath(const FPropertyBindingPath& PathToDispatcher) const;

	/** @return the task completion delegate dispatcher given task node ID and condition. */
	UE_API FStateTreeDelegateDispatcher GetTaskCompletionDispatcher(FGuid TaskNodeID, UE::StateTree::ETaskCompletionCondition Condition) const;

	/** @return the delegate dispatcher (can be a task completion) given the binding. */
	UE_API FStateTreeDelegateDispatcher GetDispatcherFromBinding(const FStateTreePropertyPathBinding& Binding) const;

	/** @return the list of compiled delegate dispatcher. */
	UE_API TConstArrayView<FStateTreeEditorDelegateDispatcherCompiledBinding> GetCompiledDelegateDispatchers() const;

	struct FTaskCompletionDispatcher
	{
		FGuid TaskNodeID;
		FStateTreeDelegateDispatcher Dispatcher;
		UE::StateTree::ETaskCompletionCondition Condition = UE::StateTree::ETaskCompletionCondition::Completes;
	};

	/** @return the list of compiled task completion delegate dispatcher. */
	UE_API TConstArrayView<FTaskCompletionDispatcher> GetCompiledTaskCompletionDispatchers() const;

	const FStateTreeBindableStructDesc* GetSourceStructDescByID(const FGuid& ID) const
	{
		return SourceStructs.FindByPredicate([ID](const FStateTreeBindableStructDesc& Structs) { return (Structs.ID == ID); });
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.6, "Use the version taking FPropertyBindingPath instead")
	UE_API FStateTreeIndex16 GetDispatcherIDFromPath(const FStateTreePropertyPath& PathToDispatcher) const;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

protected:

	void StoreSourceStructs();

	UPROPERTY()
	TArray<FStateTreeBindableStructDesc> SourceStructs;

	/** Representation of compiled reference. */
	struct FCompiledReference
	{
		FPropertyBindingPath Path;
		FStateTreeIndex16 Index;
	};

	TArray<FCompiledReference> CompiledReferences;

	/** Dispatcher used by bindings. */
	TArray<FStateTreeEditorDelegateDispatcherCompiledBinding> CompiledDelegateDispatchers;
	TArray<FTaskCompletionDispatcher> CompiledTaskCompletionDelegateDispatchers;
	int32 ListenersNum = 0;

	FStateTreePropertyBindings* PropertyBindings = nullptr;

	FStateTreeCompilerLog* Log = nullptr;
};

#undef UE_API
