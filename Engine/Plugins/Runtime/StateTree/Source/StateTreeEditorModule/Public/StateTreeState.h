// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IStateTreeSchemaProvider.h"
#include "StateTreeNodeBase.h"
#include "StateTreeEditorNode.h"
#include "StateTreeEditorTypes.h"
#include "StateTreeEvents.h"
#include "StateTreeState.generated.h"

#define UE_API STATETREEEDITORMODULE_API

class UStateTreeState;
class UStateTree;

/**
 * Editor representation of an event description.
 */
USTRUCT()
struct FStateTreeEventDesc
{
	GENERATED_BODY()

	FStateTreeEventDesc() = default;

	FStateTreeEventDesc(FGameplayTag InTag)
		: Tag(InTag)
	{}

	/** Event Tag. */
	UPROPERTY(EditDefaultsOnly, Category = "Event")
	FGameplayTag Tag;

	/** Event Payload Struct. */
	UPROPERTY(EditDefaultsOnly, Category = "Event")
	TObjectPtr<const UScriptStruct> PayloadStruct;

	/** If set to true, the event is consumed (later state selection cannot react to it) if state selection can be made. */
	UPROPERTY(EditDefaultsOnly, Category = "Event")
	bool bConsumeEventOnSelect = true;
	
	bool IsValid() const
	{
		return Tag.IsValid() || PayloadStruct;
	}

	FStateTreeEvent& GetTemporaryEvent()
	{
		TemporaryEvent.Tag = Tag;
		TemporaryEvent.Payload = FInstancedStruct(PayloadStruct);

		return TemporaryEvent;
	}

	bool operator==(const FStateTreeEventDesc& Other) const
	{
		return Tag == Other.Tag && PayloadStruct == Other.PayloadStruct;
	}

private:
	/** Temporary event used as a source value in bindings. */
	UPROPERTY(Transient)
	FStateTreeEvent TemporaryEvent;
};

/**
 * StateTree's internal delegate listener used exclusively in transitions.
 */
USTRUCT()
struct FStateTreeTransitionDelegateListener
{
	GENERATED_BODY()
};

/**
 * Editor representation of a transition in StateTree
 */
USTRUCT(BlueprintType)
struct FStateTreeTransition
{
	GENERATED_BODY()

	// Macro needed to avoid deprecation errors with members being copied or created.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FStateTreeTransition() = default;
	UE_API FStateTreeTransition(const EStateTreeTransitionTrigger InTrigger, const EStateTreeTransitionType InType, const UStateTreeState* InState = nullptr);
	UE_API FStateTreeTransition(const EStateTreeTransitionTrigger InTrigger, const FGameplayTag InEventTag, const EStateTreeTransitionType InType, const UStateTreeState* InState = nullptr);
	FStateTreeTransition(const FStateTreeTransition&) = default;
	FStateTreeTransition(FStateTreeTransition&&) = default;
	FStateTreeTransition& operator=(const FStateTreeTransition&) = default;
	FStateTreeTransition& operator=(FStateTreeTransition&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	template<typename T, typename... TArgs>
	TStateTreeEditorNode<T>& AddConditionWithOuter(TNotNull<UStateTreeState*> InState, TArgs&&... InArgs)
	{
		FStateTreeEditorNode& CondNode = Conditions.AddDefaulted_GetRef();
		CondNode.InitializeAs<T>(InState, Forward<TArgs>(InArgs)...);
		OnNodeAdded(InState, CondNode);
		return static_cast<TStateTreeEditorNode<T>&>(CondNode);
	}

	template<typename T, typename... TArgs>
	UE_DEPRECATED(5.8, "Use AddConditionWithOuter that takes a TNotNull<UStateTreeState*> Outer.")
	TStateTreeEditorNode<T>& AddCondition(TArgs&&... InArgs)
	{
		FStateTreeEditorNode& CondNode = Conditions.AddDefaulted_GetRef();
		CondNode.ID = FGuid::NewGuid();
		CondNode.Node.InitializeAs<T>(Forward<TArgs>(InArgs)...);
		const FStateTreeNodeBase& Node = CondNode.Node.GetMutable<FStateTreeNodeBase>();
		if (const UScriptStruct* InstanceType = Cast<const UScriptStruct>(Node.GetInstanceDataType()))
		{
			CondNode.Instance.InitializeAs(InstanceType);
		}
		if (const UScriptStruct* InstanceType = Cast<const UScriptStruct>(Node.GetExecutionRuntimeDataType()))
		{
			CondNode.ExecutionRuntimeData.InitializeAs(InstanceType);
		}
		return static_cast<TStateTreeEditorNode<T>&>(CondNode);
	}

	FGuid GetEventID() const
	{
		return FGuid::Combine(ID, FGuid::NewDeterministicGuid(TEXT("Event")));
	}

	bool operator==(const FStateTreeTransition& Other) const
	{
		return ID == Other.ID;
	}

	UE_API void PostSerialize(const FArchive& Ar);

private:
	UE_API void OnNodeAdded(TNotNull<UStateTreeState*> InState, FStateTreeEditorNode& EditorNode);

public:
	/** When to try trigger the transition. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Transition")
	EStateTreeTransitionTrigger Trigger = EStateTreeTransitionTrigger::OnStateCompleted;

	/** Defines the event required to be present during state selection for the transition to trigger. */
	UPROPERTY(EditDefaultsOnly, Category = "Transition", DisplayName = "Required Event")
	FStateTreeEventDesc RequiredEvent; 

	/** Transition target state. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Transition", meta=(DisplayName="Transition To"))
	FStateTreeStateLink State;

	UPROPERTY(EditDefaultsOnly, Category = "Transition")
	FGuid ID;

	/** Listener to the selected delegate dispatcher. */
	UPROPERTY(EditDefaultsOnly, Category = "Transition", DisplayName = "Delegate")
	FStateTreeTransitionDelegateListener DelegateListener;

	/**
	 * Transition priority when multiple transitions happen at the same time.
	 * During transition handling, the transitions are visited from leaf to root.
	 * The first visited transition, of highest priority, that leads to a state selection, will be activated.
	 */
	UPROPERTY(EditDefaultsOnly, Category = "Transition")
	EStateTreeTransitionPriority Priority = EStateTreeTransitionPriority::Normal;

	/** Delay the triggering of the transition. */
	UPROPERTY(EditDefaultsOnly, Category = "Transition")
	bool bDelayTransition = false;

	/** Transition delay duration in seconds. */
	UPROPERTY(EditDefaultsOnly, Category = "Transition", meta = (EditCondition = "bDelayTransition", UIMin = "0", ClampMin = "0", UIMax = "25", ClampMax = "25", ForceUnits="s"))
	float DelayDuration = 0.0f;

	/** Transition delay random variance in seconds. */
	UPROPERTY(EditDefaultsOnly, Category = "Transition", meta = (EditCondition = "bDelayTransition", UIMin = "0", ClampMin = "0", UIMax = "25", ClampMax = "25", ForceUnits="s"))
	float DelayRandomVariance = 0.0f;

	/** Expression of conditions that need to evaluate to true to allow transition to be triggered. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Transition", meta = (BaseStruct = "/Script/StateTreeModule.StateTreeConditionBase", BaseClass = "/Script/StateTreeModule.StateTreeConditionBlueprintBase"))
	TArray<FStateTreeEditorNode> Conditions;

	/** 
	 * The target state and all the following states will be reactivated. Their instance data will be reinstantiated and will receive EnterState. 
	 * Note: Only enabled when target state is within the nested state (Parent or self). As this only changes behavior for nested states.
	 */
	UPROPERTY(EditDefaultsOnly, Category = "Transition", meta = (HideEditConditionToggle))
	EStateTreeTransitionChangeTypeRules ReactivateTargetState = EStateTreeTransitionChangeTypeRules::Default;

	/** True if the Transition is Enabled (i.e. not explicitly disabled in the asset). */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Debug")
	bool bTransitionEnabled = true;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(all, "Use RequiredEvent.Tag instead.")
	UPROPERTY()
	FGameplayTag EventTag_DEPRECATED;
#endif // WITH_EDITORONLY_DATA
};

template<>
struct TStructOpsTypeTraits<FStateTreeTransition> : public TStructOpsTypeTraitsBase2<FStateTreeTransition>
{
	enum 
	{
		WithPostSerialize = true,
	};
};


USTRUCT()
struct FStateTreeStateParameters
{
	GENERATED_BODY()

	void ResetParametersAndOverrides()
	{
		// Reset just the parameters, keep the bFixedLayout intact.
		Parameters.Reset();
		PropertyOverrides.Reset();
	}

	/** Removes overrides that do appear in Parameters. */
	UE_API void RemoveUnusedOverrides();
	
	UPROPERTY(EditDefaultsOnly, Category = Parameters)
	FInstancedPropertyBag Parameters;

	/** Overrides for parameters. */
	UPROPERTY()
	TArray<FGuid> PropertyOverrides;

	UPROPERTY(EditDefaultsOnly, Category = Parameters)
	bool bFixedLayout = false;

	UPROPERTY(EditDefaultsOnly, Category = Parameters, meta = (IgnoreForMemberInitializationTest))
	FGuid ID;
};

/**
 * Editor representation of a state in StateTree
 */
UCLASS(MinimalAPI, BlueprintType, EditInlineNew, CollapseCategories, meta = (DisallowLevelActorReference = true))
class UStateTreeState : public UObject, public IStateTreeSchemaProvider
{
	GENERATED_BODY()

public:
	UE_API UStateTreeState(const FObjectInitializer& ObjectInitializer);
	UE_API virtual ~UStateTreeState() override;

	UE_API virtual void PostInitProperties() override;
	UE_API virtual void PreEditChange(FEditPropertyChain& PropertyAboutToChange) override;
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
	UE_API virtual void PostLoad() override;;
	UE_API void UpdateParametersFromLinkedSubtree();
	UE_API void OnTreeCompiled(const UStateTree& StateTree);

	UE_API const UStateTreeState* GetRootState() const;
	UE_API UStateTreeState* GetNextSiblingState() const;
	UE_API UStateTreeState* GetNextSelectableSiblingState() const;

	/** @return the path of the state as string. */
	UE_API FString GetPath() const;
	
	/** @return true if the property of specified ID is overridden. */
	bool IsParametersPropertyOverridden(const FGuid PropertyID) const
	{
		return Parameters.PropertyOverrides.Contains(PropertyID);
	}

	/** Sets the override status of specified property by ID. */
	UE_API void SetParametersPropertyOverridden(const FGuid PropertyID, const bool bIsOverridden);

	/** @returns Default parameters from linked state or asset). */
	UE_API const FInstancedPropertyBag* GetDefaultParameters() const;
	
	//~ StateTree Builder API
	/** @return state link to this state. */
	UE_API FStateTreeStateLink GetLinkToState() const;
	
	/** Adds child state with specified name. */
	UE_API UStateTreeState& AddChildState(const FName ChildName, const EStateTreeStateType StateType = EStateTreeStateType::State);

	/**
	 * Adds enter condition of specified type.
	 * @return reference to the new condition.
	 */
	template<typename T, typename... TArgs>
	TStateTreeEditorNode<T>& AddEnterCondition(TArgs&&... InArgs)
	{
		FStateTreeEditorNode& CondNode = EnterConditions.AddDefaulted_GetRef();
		CondNode.InitializeAs<T>(this, Forward<TArgs>(InArgs)...);
		OnNodeAdded(CondNode);
		return static_cast<TStateTreeEditorNode<T>&>(CondNode);
	}

	/**
	 * Adds enter utility consideration of specified type.
	 * @return reference to the new consideration.
	 */
	template<typename T, typename... TArgs>
	TStateTreeEditorNode<T>& AddConsideration(TArgs&&... InArgs)
	{
		FStateTreeEditorNode& ConsiderationNode = Considerations.AddDefaulted_GetRef();
		ConsiderationNode.InitializeAs<T>(this, Forward<TArgs>(InArgs)...);
		OnNodeAdded(ConsiderationNode);
		return static_cast<TStateTreeEditorNode<T>&>(ConsiderationNode);
	}
	
	/**
	 * Adds Task of specified type.
	 * @return reference to the new Task.
	 */
	template<typename T, typename... TArgs>
	TStateTreeEditorNode<T>& AddTask(TArgs&&... InArgs)
	{
		FStateTreeEditorNode& TaskItem = Tasks.AddDefaulted_GetRef();
		TaskItem.InitializeAs<T>(this, Forward<TArgs>(InArgs)...);
		OnNodeAdded(TaskItem);
		return static_cast<TStateTreeEditorNode<T>&>(TaskItem);
	}

private:
	UE_API void OnNodeAdded(FStateTreeEditorNode& EditorNode);

public:
	/** Sets linked state and updates parameters to match the linked state. */
	UE_API void SetLinkedState(FStateTreeStateLink InStateLink);

	/** Sets linked asset and updates parameters to match the linked asset. */
	UE_API void SetLinkedStateAsset(UStateTree* InLinkedAsset);
	
	/**
	 * Adds Transition.
	 * @return reference to the new Transition.
	 */
	UE_API FStateTreeTransition& AddTransition(const EStateTreeTransitionTrigger InTrigger, const EStateTreeTransitionType InType, const UStateTreeState* InState = nullptr);
	/**
	 * Adds Transition.
	 * @return reference to the new Transition.
	 */
	UE_API FStateTreeTransition& AddTransition(const EStateTreeTransitionTrigger InTrigger, const FGameplayTag InEventTag, const EStateTreeTransitionType InType, const UStateTreeState* InState = nullptr);

private:
	void AddTransitionImpl(FStateTreeTransition& Transition);

public:

	const FStateTreeTransition* GetTransitionByID(const FGuid TransitionID) const
	{
		return Transitions.FindByPredicate([TransitionID](const FStateTreeTransition& Other)
			{
				return Other.ID == TransitionID;
			});
	}
	
	const FStateTreeEditorNode* GetNodeByID(const FGuid NodeID) const
	{
		for (const FStateTreeEditorNode& EnterCondition : EnterConditions)
		{
			if (EnterCondition.ID == NodeID)
			{
				return &EnterCondition;
			}
		}
		for (const FStateTreeEditorNode& Task : Tasks)
		{
			if (Task.ID == NodeID)
			{
				return &Task;
			}
		}
		for (const FStateTreeEditorNode& Consideration : Considerations)
		{
			if (Consideration.ID == NodeID)
			{
				return &Consideration;
			}
		}
		
		if (SingleTask.ID == NodeID)
		{
			return &SingleTask;
		}
		
		return nullptr;
	}

	FGuid GetEventID() const
	{
		return FGuid::Combine(ID, FGuid::NewDeterministicGuid(TEXT("Event")));
	}

	//~ IStateTreeSchemaProvider API
	/**  @return Class of schema used by the state tree containing this state. */
	UE_API virtual TSubclassOf<UStateTreeSchema> GetSchema() const override;

	//~IStateTreeSchemaProvider API

	//~ Note: these properties are customized out in FStateTreeStateDetails, adding a new property might require to adjust the customization.
	
	/** Display name of the State */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "State")
	FName Name;

	/** Description of the State */
	UPROPERTY(EditDefaultsOnly, Category = "State", meta=(MultiLine))
	FString Description;

	/** GameplayTag describing the State */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "State")
	FGameplayTag Tag;

	/** Display color of the State */
	UPROPERTY(EditDefaultsOnly, Category = "State", DisplayName = "Color")
	FStateTreeEditorColorRef ColorRef;

	/** Type the State, allows e.g. states to be linked to other States. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "State")
	EStateTreeStateType Type = EStateTreeStateType::State;

	/** How to treat child states when this State is selected.  */
	UPROPERTY(EditDefaultsOnly, Category = "State")
	EStateTreeStateSelectionBehavior SelectionBehavior = EStateTreeStateSelectionBehavior::TrySelectChildrenInOrder;
	
	/** How tasks will complete the state. Only tasks that are considered for completion can complete the state. */
	UPROPERTY(EditDefaultsOnly, Category = "State")
	EStateTreeTaskCompletionType TasksCompletion = EStateTreeTaskCompletionType::Any;

	/** Subtree to run as extension of this State. */
	UPROPERTY(EditDefaultsOnly, Category = "State", Meta=(DirectStatesOnly, SubtreesOnly))
	FStateTreeStateLink LinkedSubtree;

	/** Another State Tree asset to run as extension of this State. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "State")
	TObjectPtr<UStateTree> LinkedAsset = nullptr;

	/** 
	 * Tick rate in seconds the state tasks and transitions should tick.
	 * If set the state cannot sleep.
	 * If set all the other states (children or parents) will also tick at that rate.
	 * If more than one active states has a custom tick rate then the smallest custom tick rate wins.
	 * If not set, the state will tick every frame unless the state tree is allowed to sleep.
	 */
	UPROPERTY(EditDefaultsOnly, Category = "State", meta = (EditCondition = "bHasCustomTickRate", ClampMin = 0.0f))
	float CustomTickRate = 0.0f;

	/** Activate the CustomTickRate. */
	UPROPERTY(EditDefaultsOnly, Category = "State", meta=(InlineEditConditionToggle))
	bool bHasCustomTickRate = false;

	/**
	 * If true, state parameter bindings are copied each task tick, not only when the state is entered or exited.
	 * Has no effect on ScheduledTick.
	 */
	UPROPERTY(EditDefaultsOnly, Category = "State")
	bool bCopyParameterBindingsOnTick = false;

	/** Parameters of this state. If the state is linked to another state or asset, the parameters are for the linked state. */
	UPROPERTY(EditDefaultsOnly, Category = "State")
	FStateTreeStateParameters Parameters;

	/** Should state's required event and enter conditions be evaluated when transition leads directly to it's child. */
	UPROPERTY(EditDefaultsOnly, Category = "Enter Conditions")
	bool bCheckPrerequisitesWhenActivatingChildDirectly = true;

	UPROPERTY(EditDefaultsOnly, Category = "Enter Conditions", meta=(InlineEditConditionToggle))
	bool bHasRequiredEventToEnter = false;

	/** Defines the event required to be present during state selection for the state to be selected. */
	UPROPERTY(EditDefaultsOnly, Category = "Enter Conditions", meta = (EditCondition = "bHasRequiredEventToEnter"))
	FStateTreeEventDesc RequiredEventToEnter;
	
	/** Weight used to scale the normalized final utility score for this state */
	UPROPERTY(EditDefaultsOnly, Category = "Utility", meta=(ClampMin=0))
	float Weight = 1.f;

	/** Expression of enter conditions that needs to evaluate true to allow the state to be selected. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Enter Conditions", meta = (BaseStruct = "/Script/StateTreeModule.StateTreeConditionBase", BaseClass = "/Script/StateTreeModule.StateTreeConditionBlueprintBase"))
	TArray<FStateTreeEditorNode> EnterConditions;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Tasks", meta = (BaseStruct = "/Script/StateTreeModule.StateTreeTaskBase", BaseClass = "/Script/StateTreeModule.StateTreeTaskBlueprintBase"))
	TArray<FStateTreeEditorNode> Tasks;

	/** Expression used to calculate a selection utility score for state weighing. */
	UPROPERTY(EditDefaultsOnly, Category = "Utility", meta = (BaseStruct = "/Script/StateTreeModule.StateTreeConsiderationBase", BaseClass = "/Script/StateTreeModule.StateTreeConsiderationBlueprintBase"))
	TArray<FStateTreeEditorNode> Considerations;

	// Single item used when schema calls for single task per state.
	UPROPERTY(EditDefaultsOnly, Category = "Task", meta = (BaseStruct = "/Script/StateTreeModule.StateTreeTaskBase", BaseClass = "/Script/StateTreeModule.StateTreeTaskBlueprintBase"))
	FStateTreeEditorNode SingleTask;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Transitions")
	TArray<FStateTreeTransition> Transitions;

	UPROPERTY(Instanced, BlueprintReadOnly, Category = "State")
	TArray<TObjectPtr<UStateTreeState>> Children;

	UPROPERTY(EditDefaultsOnly, Category = "State", meta = (IgnoreForMemberInitializationTest))
	FGuid ID;

	UPROPERTY(meta = (ExcludeFromHash))
	bool bExpanded = true;

	UPROPERTY(EditDefaultsOnly, Category = "State")
	bool bEnabled = true;

	UPROPERTY(meta = (ExcludeFromHash))
	TObjectPtr<UStateTreeState> Parent = nullptr;

private:

	/** Transaction temporary. Transitions we captured in PreEdit for remove / move comparision */
	TArray<FStateTreeTransition> PreEditTransitions;
	/** Handle for the UE::StateTree::Delegates::OnPostCompile delegate */
	FDelegateHandle PostCompileHandle;
};

#undef UE_API
