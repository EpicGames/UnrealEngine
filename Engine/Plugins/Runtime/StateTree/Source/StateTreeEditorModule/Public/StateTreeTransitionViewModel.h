// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/WeakObjectPtr.h"
#include "EditorUndoClient.h"
#include "IStateTreeEditorHost.h"

#define UE_API STATETREEEDITORMODULE_API

class UStateTreeState;

/**
 * Viewmodel for editing state transtions.
 */
struct FStateTreeTransitionViewModel
{
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnTransitionsAdded, const UStateTreeState* /*OwningState*/, const TSet<FGuid>& /*TransitionIDs*/);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnTransitionsRemoved, const UStateTreeState* /*OwningState*/, const TSet<FGuid>& /*TransitionIDs*/);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnTransitionsMoved, const UStateTreeState* /*OwningState*/, const TSet<FGuid>& /*TransitionIDs*/);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnTransitionModified, const UStateTreeState* /*OwningState*/, const FGuid& /*TransitionID*/);

public:

	/** Fired externally to let the view model know a transition(s) was added. */
	void HandleOnTransitionsAdded(const UStateTreeState* OwningState, const TSet<FGuid>& TransitionIDs) const;

	/** Fired externally to let the view model know a transition(s) was removed. */
	void HandleOnTransitionsRemoved(const UStateTreeState* OwningState, const TSet<FGuid>& TransitionIDs) const;

	/** Fired externally to let the view model know a transition(s) was moved. */
	void HandleOnTransitionsMoved(const UStateTreeState* OwningState, const TSet<FGuid>& TransitionIDs) const;

	/** Fired externally to let the view model know a transition was modified. */
	void HandleOnTransitionModified(const UStateTreeState* OwningState, const FGuid& TransitionID) const;

	// Called each time a transition(s) is added
	FOnTransitionsAdded& GetOnTransitionsAdded()
	{
		return OnTransitionsAdded;
	}

	// Called each time a transition(s) is removed.
	FOnTransitionsRemoved& GetOnTransitionsRemoved()
	{
		return OnTransitionsRemoved;
	}

	// Called each time a transition(s) is moved.
	FOnTransitionsRemoved& GetOnTransitionsMoved()
	{
		return OnTransitionsMoved;
	}

	// Called each time a transition is modified
	FOnTransitionModified& GetOnTransitionModified()
	{
		return OnTransitionModified;
	}

protected:

	FOnTransitionsAdded OnTransitionsAdded;
	FOnTransitionsRemoved OnTransitionsRemoved;
	FOnTransitionsMoved OnTransitionsMoved;
	FOnTransitionModified OnTransitionModified;
};

#undef UE_API
