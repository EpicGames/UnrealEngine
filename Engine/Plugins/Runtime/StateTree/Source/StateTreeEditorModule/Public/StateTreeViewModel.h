// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/WeakObjectPtr.h"
#include "EditorUndoClient.h"
#include "IStateTreeEditorHost.h"
#include "StateTreeTransitionViewModel.h"

#include "StateTreeViewModel.generated.h"

#define UE_API STATETREEEDITORMODULE_API

namespace UE::StateTreeEditor
{
	struct FClipboardEditorData;
}

struct FStateTreeEditorNode;
struct FStateTreeTransition;
class FMenuBuilder;
class UStateTree;
class UStateTreeEditorData;
class UStateTreeSchema;
class UStateTreeState;

enum class ECheckBoxState : uint8;
enum class EStateTreeBreakpointType : uint8;
enum class EStateTreeNodeFormatting : uint8;
enum class EStateTreeTransitionTrigger : uint8;
enum class EStateTreeTransitionType : uint8;
enum class EStateTreeVisitor : uint8;

struct FGameplayTag;
struct FSlateBrush;
struct FPropertyBindingPath;
struct FPropertyChangedEvent;
struct FStateTreeDebugger;
struct FStateTreeDebuggerBreakpoint;
struct FStateTreeEditorBreakpoint;
struct FStateTreePropertyPathBinding;

enum class EStateTreeViewModelInsert : uint8
{
	Before,
	After,
	Into,
};

enum class EStateTreeViewModelColorAdjustment : uint8
{
	Normal,
	Bright,
	Brighter,
	Dark,
	Darker,
};

enum class UE_DEPRECATED(5.6, "Use the enum with the E prefix") FStateTreeViewModelInsert : uint8
{
	Before,
	After,
	Into,
};

/**
 * ModelView for editing StateTreeEditorData.
 */
class FStateTreeViewModel : public FEditorUndoClient, public TSharedFromThis<FStateTreeViewModel>
{
public:

	DECLARE_MULTICAST_DELEGATE(FOnAssetChanged);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnStatesChanged, const TSet<UStateTreeState*>& /*AffectedStates*/, const FPropertyChangedEvent& /*PropertyChangedEvent*/);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnStateAdded, UStateTreeState* /*ParentState*/, UStateTreeState* /*NewState*/);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnStatesRemoved, const TSet<UStateTreeState*>& /*AffectedParents*/);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnStatesMoved, const TSet<UStateTreeState*>& /*AffectedParents*/, const TSet<UStateTreeState*>& /*MovedStates*/);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnStateNodesChanged, const UStateTreeState* /*AffectedState*/);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSelectionChanged, const TArray<TWeakObjectPtr<UStateTreeState>>& /*SelectedStates*/);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnBringNodeToFocus, const UStateTreeState* /*State*/, const FGuid /*NodeID*/);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnBringBindingPathToFocus, const UStateTreeState* /*State*/, const FPropertyBindingPath& /*BindingPath*/);

	UE_API FStateTreeViewModel();
	UE_API virtual ~FStateTreeViewModel() override;

	UE_API void Init(UStateTreeEditorData* InTreeData);

	//~ FEditorUndoClient
	UE_API virtual void PostUndo(bool bSuccess) override;
	UE_API virtual void PostRedo(bool bSuccess) override;

	// Selection handling.
	UE_API void ClearSelection();
	UE_API void SetSelection(UStateTreeState* Selected);
	UE_API void SetSelection(const TArray<TWeakObjectPtr<UStateTreeState>>& InSelection);
	UE_API bool IsSelected(const UStateTreeState* State) const;
	UE_API bool IsChildOfSelection(const UStateTreeState* State) const;
	UE_API void GetSelectedStates(TArray<UStateTreeState*>& OutSelectedStates) const;
	UE_API void GetSelectedStates(TArray<TWeakObjectPtr<UStateTreeState>>& OutSelectedStates) const;
	UE_API bool HasSelection() const;
	UE_API UStateTreeState* GetLastSelectedState() const;

	UE_API void BringNodeToFocus(UStateTreeState* State, const FGuid NodeID);
	UE_API void BringBindingPathToFocus(const FPropertyBindingPath& InBindingPath);

	// The view state is the last selected state, or first root if that is invalid. Nullptr if neither of those exist.
	UE_API UStateTreeState* GetViewState() const;
	
	// Returns associated state tree asset.
	UE_API const UStateTree* GetStateTree() const;

	UE_API const UStateTreeEditorData* GetStateTreeEditorData() const;
	UE_API const UStateTreeSchema* GetStateTreeSchema() const;

	UE_API const UStateTreeState* GetStateByID(const FGuid StateID) const;
	UE_API UStateTreeState* GetMutableStateByID(const FGuid StateID) const;

	UE_API const FStateTreeTransition* GetTransitionByID(const FGuid TransitionID) const;
	UE_API FStateTreeTransition* GetMutableTransitionByID(const FGuid TransitionID) const;
	
	// Returns array of subtrees to edit.
	UE_API TArray<TObjectPtr<UStateTreeState>>* GetSubTrees() const;
	UE_API int32 GetSubTreeCount() const;
	UE_API void GetSubTrees(TArray<TWeakObjectPtr<UStateTreeState>>& OutSubtrees) const;

	/** Find the states that are linked to the provided StateID. */
	UE_API void GetLinkStates(FGuid StateID, TArray<FGuid>& LinkingIn, TArray<FGuid>& LinkedOut) const;

	/** Find the states that the transition go to. */
	UE_API UStateTreeState* GetTransitionToState(TNotNull<const UStateTreeState*> TransitionOwningState, FGuid TransitionID) const;

	// Gets and sets StateTree view expansion state store in the asset.
	UE_API void SetPersistentExpandedStates(TSet<TWeakObjectPtr<UStateTreeState>>& InExpandedStates);
	UE_API void GetPersistentExpandedStates(TSet<TWeakObjectPtr<UStateTreeState>>& OutExpandedStates);

	// State manipulation
	UE_API void AddState(UStateTreeState* AfterState);
	UE_API void AddChildState(UStateTreeState* ParentState);
	UE_API void RenameState(UStateTreeState* State, FName NewName);
	UE_API void RemoveSelectedStates();
	UE_API void CopySelectedStates();
	UE_API bool CanPasteStatesFromClipboard() const;
	UE_API void PasteStatesFromClipboard(UStateTreeState* AfterState);
	UE_API void PasteStatesAsChildrenFromClipboard(UStateTreeState* ParentState);
	UE_API void DuplicateSelectedStates();
	UE_API void MoveSelectedStatesBefore(UStateTreeState* TargetState);
	UE_API void MoveSelectedStatesAfter(UStateTreeState* TargetState);
	UE_API void MoveSelectedStatesInto(UStateTreeState* TargetState);
	UE_API bool CanEnableStates() const;
	UE_API bool CanDisableStates() const;
	UE_API bool CanPasteNodesToSelectedStates() const;
	UE_API void SetSelectedStatesEnabled(bool bEnable);

	// State utility methods
	UE_API void ForEachParent(TNotNull<UStateTreeState*> State, TFunctionRef<EStateTreeVisitor(UStateTreeState& ParentState)> InFunc) const;
	UE_API void ForEachInTransition(TNotNull<UStateTreeState*> State, TFunctionRef<EStateTreeVisitor(FStateTreeTransition& TransitionToState)> InFunc) const;
	UE_API void ForEachOutTransition(TNotNull<UStateTreeState*> State, TFunctionRef<EStateTreeVisitor(FStateTreeTransition& TransitionFromState)> InFunc) const;
	UE_API void ForEachChild(TNotNull<UStateTreeState*> State, TFunctionRef<EStateTreeVisitor(UStateTreeState& ChildState)> InFunc) const;

	UE_API TArray<UStateTreeState*> GetStateParents(TNotNull<UStateTreeState*> State) const;
	UE_API TArray<FStateTreeTransition*> GetStateInTransitions(TNotNull<UStateTreeState*> State) const;
	UE_API TArray<FStateTreeTransition*> GetStateOutTransitions(TNotNull<UStateTreeState*> State) const;
	UE_API TArray<UStateTreeState*> GetStateChildren(TNotNull<UStateTreeState*> State) const;

	UE_API const UStateTreeState* GetSourceStateFromTransition(TNotNull<const FStateTreeTransition*> Transition) const;
	UE_API const UStateTreeState* GetSourceParentStateFromTransition(TNotNull<const FStateTreeTransition*> Transition) const;
	UE_API const UStateTreeState* GetTargetStateFromTransition(TNotNull<const FStateTreeTransition*> Transition) const;
	UE_API const UStateTreeState* GetTargetParentStateFromTransition(TNotNull<const FStateTreeTransition*> Transition) const;

	// Transition Manipulation
	UE_API FStateTreeTransition& AddTransition(TNotNull<UStateTreeState*> State, const EStateTreeTransitionTrigger InTrigger, const EStateTreeTransitionType InType, const UStateTreeState* InState = nullptr) const;
	UE_API FStateTreeTransition& AddTransition(TNotNull<UStateTreeState*> State, const EStateTreeTransitionTrigger InTrigger, const FGameplayTag InEventTag, const EStateTreeTransitionType InType, const UStateTreeState* InState = nullptr) const;
	UE_API void RemoveTransition(TNotNull<UStateTreeState*> State, const FGuid TransitionID);

	// Color manipulation
	UE_API FLinearColor GetStateColor(TNotNull<const UStateTreeState*> State, EStateTreeViewModelColorAdjustment ColorAdjustment = EStateTreeViewModelColorAdjustment::Normal) const;
	UE_API FLinearColor GetHierarchyBasedHighlightColor(TNotNull<const UStateTreeState*> State, EStateTreeViewModelColorAdjustment ColorAdjustment = EStateTreeViewModelColorAdjustment::Normal) const;
	UE_API FLinearColor GetHierarchyBasedBackgroundColor(TNotNull<const UStateTreeState*> State, EStateTreeViewModelColorAdjustment ColorAdjustment = EStateTreeViewModelColorAdjustment::Normal) const;

	UE_API FLinearColor GetColorAdjusted(const FLinearColor& InColor, EStateTreeViewModelColorAdjustment ColorAdjustment) const;

	UE_API const FSlateBrush* GetSelectorIcon(TNotNull<const UStateTreeState*> State) const;
	UE_API FText GetNodeDescription(const FStateTreeEditorNode& Node, const EStateTreeNodeFormatting Formatting) const;

	// EditorNode and Transition manipulation
	// @todo: support ReplaceWith and Rename
	UE_API void DeleteNode(TWeakObjectPtr<UStateTreeState> State, const FGuid& ID);
	UE_API void DeleteAllNodes(TWeakObjectPtr<UStateTreeState> State, const FGuid& ID);
	UE_API void CopyNode(TWeakObjectPtr<UStateTreeState> State, const FGuid& ID);
	UE_API void CopyAllNodes(TWeakObjectPtr<UStateTreeState> State, const FGuid& ID);
	UE_API void PasteNode(TWeakObjectPtr<UStateTreeState> State, const FGuid& ID);
	UE_API void PasteNodesToSelectedStates();
	UE_API void PasteNodesToStates(TConstArrayView<UStateTreeState*> States);
	UE_API void MoveNode(TWeakObjectPtr<UStateTreeState> From, TWeakObjectPtr<UStateTreeState> To, const FGuid& ID);
	UE_API void DuplicateNode(TWeakObjectPtr<UStateTreeState> State, const FGuid& ID);

	// Force to update the view externally.
	UE_API void NotifyAssetChangedExternally() const;
	UE_API void NotifyStatesChangedExternally(const TSet<UStateTreeState*>& ChangedStates, const FPropertyChangedEvent& PropertyChangedEvent) const;

	// Debugging
#if WITH_STATETREE_TRACE_DEBUGGER
	UE_API bool HasBreakpoint(FGuid ID, EStateTreeBreakpointType Type);
	UE_API bool CanProcessBreakpoints() const;
	UE_API bool CanAddStateBreakpoint(EStateTreeBreakpointType Type) const;
	UE_API bool CanRemoveStateBreakpoint(EStateTreeBreakpointType Type) const;
	UE_API ECheckBoxState GetStateBreakpointCheckState(EStateTreeBreakpointType Type) const;
	UE_API void HandleEnableStateBreakpoint(EStateTreeBreakpointType Type);
	UE_API void ToggleStateBreakpoints(TConstArrayView<TWeakObjectPtr<>> States, EStateTreeBreakpointType Type);
	UE_API void ToggleTaskBreakpoint(FGuid ID, EStateTreeBreakpointType Type);
	UE_API void ToggleTransitionBreakpoint(TConstArrayView<TNotNull<const FStateTreeTransition*>> Transitions, ECheckBoxState ToggledState);

	UE_API UStateTreeState* FindStateAssociatedToBreakpoint(FStateTreeDebuggerBreakpoint Breakpoint) const;

	TSharedRef<FStateTreeDebugger> GetDebugger() const
	{
		return Debugger;
	}

	UE_API void RemoveAllBreakpoints();
	UE_API void RefreshDebuggerBreakpoints();
#endif // WITH_STATETREE_TRACE_DEBUGGER

	UE_API bool IsStateActiveInDebugger(const UStateTreeState& State) const;

	// Called when the whole asset is updated (i.e. undo/redo).
	FOnAssetChanged& GetOnAssetChanged()
	{
		return OnAssetChanged;
	}
	
	// Called when States are changed (i.e. change name or properties).
	FOnStatesChanged& GetOnStatesChanged()
	{
		return OnStatesChanged;
	}
	
	// Called each time a state is added.
	FOnStateAdded& GetOnStateAdded()
	{
		return OnStateAdded;
	}

	// Called each time a states are removed.
	FOnStatesRemoved& GetOnStatesRemoved()
	{
		return OnStatesRemoved;
	}

	// Called each time a state is removed.
	FOnStatesMoved& GetOnStatesMoved()
	{
		return OnStatesMoved;
	}

	// Called each time a state's Editor nodes or transitions are changed except from the DetailsView.
	FOnStateNodesChanged& GetOnStateNodesChanged()
	{
		return OnStateNodesChanged;
	}

	// Called each time the selection changes.
	FOnSelectionChanged& GetOnSelectionChanged()
	{
		return OnSelectionChanged;
	}

	FOnBringNodeToFocus& GetOnBringNodeToFocus()
	{
		return OnBringNodeToFocus;
	}

	FOnBringBindingPathToFocus& GetOnBringBindingPathToFocus()
	{
		return OnBringBindingPathToFocus;
	}

	// Get View Model for transitions. Holds various callbacks for transtions added / modified / etc.
	FStateTreeTransitionViewModel& GetTransitionViewModel()
	{
		return TransitionViewModel;
	}

	const FStateTreeTransitionViewModel& GetTransitionViewModel() const
	{
		return TransitionViewModel;
	}

protected:
	UE_API void GetExpandedStatesRecursive(UStateTreeState* State, TSet<TWeakObjectPtr<UStateTreeState>>& ExpandedStates);

	UE_API void MoveSelectedStates(UStateTreeState* TargetState, const EStateTreeViewModelInsert RelativeLocation);

	UE_API void PasteStatesAsChildrenFromText(const FString& TextToImport, UStateTreeState* ParentState, const int32 IndexToInsertAt);

	UE_API void HandleIdentifierChanged(const UStateTree& StateTree) const;
	
	UE_API void BindToDebuggerDelegates();

	UE_API void PasteNodesToState(TNotNull<UStateTreeEditorData*> InEditorData, TNotNull<UStateTreeState*> InState, UE::StateTreeEditor::FClipboardEditorData& InProcessedClipboard);

	TWeakObjectPtr<UStateTreeEditorData> TreeDataWeak;
	TSet<TWeakObjectPtr<UStateTreeState>> SelectedStates;
	FGuid LastSelectedState;

#if WITH_STATETREE_TRACE_DEBUGGER
	UE_API void HandleBreakpointsChanged(const UStateTree& StateTree);
	UE_API void HandlePostCompile(const UStateTree& StateTree);

	TSharedRef<FStateTreeDebugger> Debugger;
	TArray<FGuid> ActiveStates;
#endif // WITH_STATETREE_TRACE_DEBUGGER
	
	FOnAssetChanged OnAssetChanged;
	FOnStatesChanged OnStatesChanged;
	FOnStateAdded OnStateAdded;
	FOnStatesRemoved OnStatesRemoved;
	FOnStatesMoved OnStatesMoved;
	FOnStateNodesChanged OnStateNodesChanged;
	FOnSelectionChanged OnSelectionChanged;
	FOnBringNodeToFocus OnBringNodeToFocus;
	FOnBringBindingPathToFocus OnBringBindingPathToFocus;

	/** Viewmodel for transition modifications */
	FStateTreeTransitionViewModel TransitionViewModel;
};

/** Helper class to allow to copy bindings into clipboard. */
UCLASS(MinimalAPI, Hidden)
class UStateTreeClipboardBindings : public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY()
	TArray<FStateTreePropertyPathBinding> Bindings;
};

#undef UE_API
