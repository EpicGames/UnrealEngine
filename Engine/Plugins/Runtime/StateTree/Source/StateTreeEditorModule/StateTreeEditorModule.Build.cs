// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class StateTreeEditorModule : ModuleRules
	{
		public StateTreeEditorModule(ReadOnlyTargetRules Target) : base(Target)
		{
			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

			bAllowUETypesInNamespaces = true;

			PublicIncludePaths.AddRange(
			new string[] {
			}
			);

			PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
				"AssetTools",
				"EditorFramework",
				"UnrealEd",
				"Slate",
				"SlateCore",
				"EditorStyle",
				"PropertyEditor",
				"StateTreeDeveloper",
				"StateTreeModule",
				"SourceControl",
				"Projects",
				"BlueprintGraph",
				"PropertyBindingUtils",
				"PropertyBindingUtilsEditor",
				"PropertyAccessEditor",
				"StructUtilsEditor",
				"GameplayTags",
				"EditorSubsystem"
			}
			);

			PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AssetDefinition",
				"RenderCore",
				"GraphEditor",
				"KismetWidgets",
				"PropertyPath",
				"SourceCodeAccess",
				"ToolMenus",
				"ToolWidgets",
				"TraceLog",
				"TraceServices",
				"ApplicationCore",
				"DeveloperSettings",
				"GameplayInsights",
				"RewindDebuggerInterface",
				"DetailCustomizations",
				"AppFramework",
				"Kismet",
				"KismetCompiler",
				"EditorInteractiveToolsFramework",
				"EditorWidgets",
				"InteractiveToolsFramework",
			}
			);

			PrivateIncludePathModuleNames.AddRange(new string[] {
				"MessageLog",
			});

			// Allow debugger trace analysis tools on all non-shipping desktop targets and shipping editors
			if ((Target.Configuration != UnrealTargetConfiguration.Shipping || Target.bBuildEditor)
				&& Target.Platform.IsInGroup(UnrealPlatformGroup.Desktop))
			{
				PublicDefinitions.Add("WITH_STATETREE_TRACE_DEBUGGER=1");
			}
			else
			{
				PublicDefinitions.Add("WITH_STATETREE_TRACE_DEBUGGER=0");
			}
		}
	}
}
