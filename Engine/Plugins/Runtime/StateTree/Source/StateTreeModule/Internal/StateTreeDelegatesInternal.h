// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

#define UE_API STATETREEMODULE_API

class UStateTreeEditorData;
class UStateTree;
#if WITH_EDITOR
struct FAppendToClassSchemaContext;
class FAssetRegistryTagsContext;
#endif

namespace UE::StateTree::Delegates::Private
{
	/** A StateTree asset loads. */
	DECLARE_DELEGATE_OneParam(FOnStateTreeAssetLoaded, TNotNull<UStateTree*> /*StateTree*/);
	extern UE_API FOnStateTreeAssetLoaded OnStateTreeAssetLoaded;

	DECLARE_DELEGATE_OneParam(FOnStateTreeEditorBindingUpdated, TNotNull<UStateTreeEditorData*> /*StateTree*/);
	extern UE_API FOnStateTreeEditorBindingUpdated OnStateTreeEditorBindingUpdated;

#if WITH_EDITOR
	/** A callback for UStateTree::CompileIfChanged. */
	DECLARE_DELEGATE_OneParam(FOnCompileIfChanged, TNotNull<UStateTree*> /*StateTree*/);
	extern UE_API FOnCompileIfChanged OnCompileIfChanged;

	/** A callback for UStateTree::MarkAsModified to notify that a tree was marked dirty. */
	DECLARE_DELEGATE_OneParam(FOnStateTreeMarkedDirty, TNotNull<UStateTree*> /*StateTree*/);
	extern UE_API FOnStateTreeMarkedDirty OnStateTreeMarkedAsModified;

	/** A callback for UStateTree::GetAssetRegistryTags. */
	using FOnStateTreeRequestAssetRegistryTags = TTSDelegate<void(TNotNull<const UStateTree*> /*StateTree*/, FAssetRegistryTagsContext& /*Context*/)>;
	extern UE_API FOnStateTreeRequestAssetRegistryTags OnRequestAssetRegistryTags;

	/** A callback for UStateTree::AppendToClassSchema. */
	DECLARE_DELEGATE_OneParam(FOnAppendToClassSchema, FAppendToClassSchemaContext& /*Context*/);
	extern UE_API FOnAppendToClassSchema OnAppendToClassSchema;

	/** The StateTree asset is going to get cooked. */
	DECLARE_DELEGATE_OneParam(FOnPreCookStateTreeAsset, TNotNull<UStateTree*> /*StateTree*/);
	extern UE_API FOnPreCookStateTreeAsset OnPreCookStateTreeAsset;
#endif
}; // namespace

#undef UE_API
