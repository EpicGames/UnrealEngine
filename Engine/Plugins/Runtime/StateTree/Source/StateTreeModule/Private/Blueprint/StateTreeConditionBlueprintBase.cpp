// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprint/StateTreeConditionBlueprintBase.h"
#include "Blueprint/StateTreeNodeBlueprintBase.h"
#include "StateTreeExecutionContext.h"
#include "BlueprintNodeHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeConditionBlueprintBase)

//----------------------------------------------------------------------//
//  UStateTreeConditionBlueprintBase
//----------------------------------------------------------------------//

UStateTreeConditionBlueprintBase::UStateTreeConditionBlueprintBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bHasTestCondition = BlueprintNodeHelpers::HasBlueprintFunction(TEXT("ReceiveTestCondition"), *this, *StaticClass());
}

bool UStateTreeConditionBlueprintBase::TestCondition(FStateTreeExecutionContext& Context) const
{
	if (bHasTestCondition)
	{
		// Cache the owner and event queue for the duration the condition is evaluated.
		SetCachedInstanceDataFromContext(Context);

		const bool bResult = ReceiveTestCondition();

		ClearCachedInstanceData();

		return bResult;
	}
	return false;
}

//----------------------------------------------------------------------//
//  FStateTreeBlueprintConditionWrapper
//----------------------------------------------------------------------//

bool FStateTreeBlueprintConditionWrapper::TestCondition(FStateTreeExecutionContext& Context) const
{
	UStateTreeConditionBlueprintBase* Condition = Context.GetInstanceDataPtr<UStateTreeConditionBlueprintBase>(*this);
	check(Condition);
	return Condition->TestCondition(Context);
}

#if WITH_EDITOR
FText FStateTreeBlueprintConditionWrapper::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	FText Description;
	if (const UStateTreeConditionBlueprintBase* Instance = InstanceDataView.GetPtr<UStateTreeConditionBlueprintBase>())
	{
		Description = Instance->GetDescription(ID, InstanceDataView, BindingLookup, Formatting);
	}
	if (Description.IsEmpty() && ConditionClass)
	{
		Description = ConditionClass->GetDisplayNameText();
	}
	return Description;
}

FName FStateTreeBlueprintConditionWrapper::GetIconName() const
{
	if (ConditionClass)
	{
		if (const UStateTreeNodeBlueprintBase* NodeCDO = GetDefault<const UStateTreeNodeBlueprintBase>(ConditionClass))
		{
			return NodeCDO->GetIconName();
		}
	}

	return FStateTreeConditionBase::GetIconName();
}

FColor FStateTreeBlueprintConditionWrapper::GetIconColor() const
{
	if (ConditionClass)
	{
		if (const UStateTreeNodeBlueprintBase* NodeCDO = GetDefault<const UStateTreeNodeBlueprintBase>(ConditionClass))
		{
			return NodeCDO->GetIconColor();
		}
	}

	return FStateTreeConditionBase::GetIconColor();
}
#endif