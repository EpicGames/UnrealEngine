// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprint/StateTreeConsiderationBlueprintBase.h"
#include "BlueprintNodeHelpers.h"
#include "StateTreeExecutionContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeConsiderationBlueprintBase)

//----------------------------------------------------------------------//
//  UStateTreeConsiderationBlueprintBase
//----------------------------------------------------------------------//

UStateTreeConsiderationBlueprintBase::UStateTreeConsiderationBlueprintBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bHasGetScore = BlueprintNodeHelpers::HasBlueprintFunction(GET_FUNCTION_NAME_CHECKED(UStateTreeConsiderationBlueprintBase, ReceiveGetScore), *this, *StaticClass());
}

float UStateTreeConsiderationBlueprintBase::GetScore(FStateTreeExecutionContext& Context) const
{
	if (bHasGetScore)
	{
		// Cache the owner and event queue for the duration the consideration is evaluated.
		SetCachedInstanceDataFromContext(Context);

		const float Score = ReceiveGetScore();

		ClearCachedInstanceData();

		return Score;
	}

	return .0f;
}

//----------------------------------------------------------------------//
//  FStateTreeBlueprintConsiderationWrapper
//----------------------------------------------------------------------//

#if WITH_EDITOR
FText FStateTreeBlueprintConsiderationWrapper::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting /*= EStateTreeNodeFormatting::Text*/) const
{
	FText Description;
	if (const UStateTreeConsiderationBlueprintBase* Instance = InstanceDataView.GetPtr<UStateTreeConsiderationBlueprintBase>())
	{
		Description = Instance->GetDescription(ID, InstanceDataView, BindingLookup, Formatting);
	}
	if (Description.IsEmpty() && ConsiderationClass)
	{
		Description = ConsiderationClass->GetDisplayNameText();
	}
	return Description;
}

FName FStateTreeBlueprintConsiderationWrapper::GetIconName() const
{
	if (ConsiderationClass)
	{
		if (const UStateTreeNodeBlueprintBase* NodeCDO = GetDefault<const UStateTreeNodeBlueprintBase>(ConsiderationClass))
		{
			return NodeCDO->GetIconName();
		}
	}

	return FStateTreeConsiderationBase::GetIconName();
}

FColor FStateTreeBlueprintConsiderationWrapper::GetIconColor() const
{
	if (ConsiderationClass)
	{
		if (const UStateTreeNodeBlueprintBase* NodeCDO = GetDefault<const UStateTreeNodeBlueprintBase>(ConsiderationClass))
		{
			return NodeCDO->GetIconColor();
		}
	}

	return FStateTreeConsiderationBase::GetIconColor();
}
#endif //WITH_EDITOR

float FStateTreeBlueprintConsiderationWrapper::GetScore(FStateTreeExecutionContext& Context) const
{
	UStateTreeConsiderationBlueprintBase* Consideration = Context.GetInstanceDataPtr<UStateTreeConsiderationBlueprintBase>(*this);
	check(Consideration);
	return Consideration->GetScore(Context);
}
