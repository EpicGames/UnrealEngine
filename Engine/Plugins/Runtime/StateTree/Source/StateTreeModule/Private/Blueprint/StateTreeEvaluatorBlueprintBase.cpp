// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprint/StateTreeEvaluatorBlueprintBase.h"
#include "Blueprint/StateTreeNodeBlueprintBase.h"
#include "StateTreeExecutionContext.h"
#include "BlueprintNodeHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeEvaluatorBlueprintBase)

//----------------------------------------------------------------------//
//  UStateTreeEvaluatorBlueprintBase
//----------------------------------------------------------------------//

UStateTreeEvaluatorBlueprintBase::UStateTreeEvaluatorBlueprintBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bHasTreeStart = BlueprintNodeHelpers::HasBlueprintFunction(TEXT("ReceiveTreeStart"), *this, *StaticClass());
	bHasTreeStop = BlueprintNodeHelpers::HasBlueprintFunction(TEXT("ReceiveTreeStop"), *this, *StaticClass());
	bHasTick = BlueprintNodeHelpers::HasBlueprintFunction(TEXT("ReceiveTick"), *this, *StaticClass());
}

void UStateTreeEvaluatorBlueprintBase::TreeStart(FStateTreeExecutionContext& Context)
{
	// Evaluator became active, cache event queue and owner.
	SetCachedInstanceDataFromContext(Context);

	if (bHasTreeStart)
	{
		ReceiveTreeStart();
	}
}

void UStateTreeEvaluatorBlueprintBase::TreeStop(FStateTreeExecutionContext& Context)
{
	if (bHasTreeStop)
	{
		ReceiveTreeStop();
	}

	// Evaluator became inactive, clear cached event queue and owner.
	ClearCachedInstanceData();
}

void UStateTreeEvaluatorBlueprintBase::Tick(FStateTreeExecutionContext& Context, const float DeltaTime)
{
	if (bHasTick)
	{
		ReceiveTick(DeltaTime);
	}
}

//----------------------------------------------------------------------//
//  FStateTreeBlueprintEvaluatorWrapper
//----------------------------------------------------------------------//

void FStateTreeBlueprintEvaluatorWrapper::TreeStart(FStateTreeExecutionContext& Context) const
{
	UStateTreeEvaluatorBlueprintBase* Instance = Context.GetInstanceDataPtr<UStateTreeEvaluatorBlueprintBase>(*this);
	check(Instance);
	Instance->TreeStart(Context);
}

void FStateTreeBlueprintEvaluatorWrapper::TreeStop(FStateTreeExecutionContext& Context) const
{
	UStateTreeEvaluatorBlueprintBase* Instance = Context.GetInstanceDataPtr<UStateTreeEvaluatorBlueprintBase>(*this);
	check(Instance);
	Instance->TreeStop(Context);
}

void FStateTreeBlueprintEvaluatorWrapper::Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const
{
	UStateTreeEvaluatorBlueprintBase* Instance = Context.GetInstanceDataPtr<UStateTreeEvaluatorBlueprintBase>(*this);
	check(Instance);
	Instance->Tick(Context, DeltaTime);
}

#if WITH_EDITOR
FText FStateTreeBlueprintEvaluatorWrapper::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	FText Description;
	if (const UStateTreeEvaluatorBlueprintBase* Instance = InstanceDataView.GetPtr<UStateTreeEvaluatorBlueprintBase>())
	{
		Description = Instance->GetDescription(ID, InstanceDataView, BindingLookup, Formatting);
	}
	if (Description.IsEmpty() && EvaluatorClass)
	{
		Description = EvaluatorClass->GetDisplayNameText();
	}
	return Description;
}

FName FStateTreeBlueprintEvaluatorWrapper::GetIconName() const
{
	if (EvaluatorClass)
	{
		if (const UStateTreeNodeBlueprintBase* NodeCDO = GetDefault<const UStateTreeNodeBlueprintBase>(EvaluatorClass))
		{
			return NodeCDO->GetIconName();
		}
	}
	return FStateTreeEvaluatorBase::GetIconName();
}

FColor FStateTreeBlueprintEvaluatorWrapper::GetIconColor() const
{
	if (EvaluatorClass)
	{
		if (const UStateTreeNodeBlueprintBase* NodeCDO = GetDefault<const UStateTreeNodeBlueprintBase>(EvaluatorClass))
		{
			return NodeCDO->GetIconColor();
		}
	}
	return FStateTreeEvaluatorBase::GetIconColor();
}
#endif