// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprint/StateTreeNodeBlueprintBase.h"
#include "AIController.h"
#include "StateTreeExecutionContext.h"
#include "VisualLogger/VisualLogger.h"
#include "StateTreePropertyRef.h"
#include "StateTreePropertyRefHelpers.h"
#include "StateTreeDelegate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeNodeBlueprintBase)

#define LOCTEXT_NAMESPACE "StateTree"

#if WITH_EDITOR
FGuid UStateTreeNodeBlueprintBase::CachedNodeID;
const IStateTreeBindingLookup* UStateTreeNodeBlueprintBase::CachedBindingLookup = nullptr;
#endif

UWorld* UStateTreeNodeBlueprintBase::GetWorld() const
{
	// The items are duplicated as the State Tree execution context as outer, so this should be essentially the same as GetWorld() on StateTree context.
	// The CDO is used by the BP editor to check for certain functionality, make it return nullptr so that the GetWorld() passes as overridden. 
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		if (TStrongObjectPtr<UObject> Owner = WeakExecutionContext.GetOwner())
		{
			return Owner.Get()->GetWorld();
		}
		if (UObject* Outer = GetOuter())
		{
			return Outer->GetWorld();
		}
	}
	
	return nullptr;
}

AActor* UStateTreeNodeBlueprintBase::GetOwnerActor(const FStateTreeExecutionContext& Context) const
{
	if (const AAIController* Controller = Cast<AAIController>(Context.GetOwner()))
	{
		return Controller->GetPawn();
	}
	
	return Cast<AActor>(Context.GetOwner());
}

void UStateTreeNodeBlueprintBase::SetCachedInstanceDataFromContext(const FStateTreeExecutionContext& Context) const
{
	WeakExecutionContext = Context.MakeWeakExecutionContext();
}

void UStateTreeNodeBlueprintBase::ClearCachedInstanceData() const
{
	WeakExecutionContext = FStateTreeWeakExecutionContext();
}

void* UStateTreeNodeBlueprintBase::GetMutablePtrToProperty(const FStateTreeBlueprintPropertyRef& PropertyRef, FProperty*& OutSourceProperty) const
{
	using namespace UE::StateTree;
	const FStateTreeStrongExecutionContext StrongContext = WeakExecutionContext.MakeStrongExecutionContext();
	if (StrongContext.IsValid())
	{
		const Async::FActivePathInfo ActivePath = StrongContext.GetActivePathInfo();
		if (!ActivePath.IsValid())
		{
			UE_VLOG_UELOG(this, LogStateTree, Verbose, TEXT("Trying to GetMutablePtrToProperty while node is not active."));
			return nullptr;
		}

		const FProperty* SourceProperty = nullptr;
		void* PropertyAddress = PropertyRefHelpers::GetMutablePtrToProperty<void>(PropertyRef,
			*const_cast<FStateTreeInstanceStorage*>(StrongContext.GetStorage().Get()),
			StrongContext.GetTemporaryStorage().Get(),
			*ActivePath.Frame,
			&SourceProperty);

		if (PropertyAddress && PropertyRefHelpers::IsBlueprintPropertyRefCompatibleWithProperty(*SourceProperty, &PropertyRef))
		{
			OutSourceProperty = const_cast<FProperty*>(SourceProperty);
			return PropertyAddress;
		}
	}

	OutSourceProperty = nullptr;
	
	return nullptr;
}

void UStateTreeNodeBlueprintBase::SendEvent(const FStateTreeEvent& Event)
{
	if (!WeakExecutionContext.SendEvent(Event.Tag, Event.Payload, Event.Origin))
	{
		UE_VLOG_UELOG(this, LogStateTree, Error, TEXT("Failed to send the event. The instance probably stopped."));
	}
}

void UStateTreeNodeBlueprintBase::RequestTransition(const FStateTreeStateLink& TargetState, const EStateTreeTransitionPriority Priority)
{
	if (!WeakExecutionContext.RequestTransition(TargetState.StateHandle, Priority, TargetState.Fallback))
	{
		UE_VLOG_UELOG(this, LogStateTree, Error, TEXT("Failed to request a transition. The instance probably stopped."));
	}
}

bool UStateTreeNodeBlueprintBase::IsPropertyRefValid(const FStateTreeBlueprintPropertyRef& PropertyRef) const
{
	FProperty* SourceProperty = nullptr;
	return GetMutablePtrToProperty(PropertyRef, SourceProperty) != nullptr;
}

DEFINE_FUNCTION(UStateTreeNodeBlueprintBase::execGetPropertyReference)
{
	P_GET_STRUCT_REF(FStateTreeBlueprintPropertyRef, PropertyRef);
	Stack.StepCompiledIn<FProperty>(nullptr);
	P_FINISH;

	FProperty* SourceProperty = nullptr;
	if (void* PropertyAddress = P_THIS->GetMutablePtrToProperty(PropertyRef, SourceProperty))
	{
		Stack.MostRecentPropertyAddress = reinterpret_cast<uint8*>(PropertyAddress);
		Stack.MostRecentProperty = const_cast<FProperty*>(SourceProperty);
		if (RESULT_PARAM)
		{
			Stack.MostRecentProperty->CopyCompleteValueToScriptVM(RESULT_PARAM, Stack.MostRecentPropertyAddress);
		}
	}
	else
	{
		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentProperty = nullptr;
	}
}

#if WITH_EDITOR
FText UStateTreeNodeBlueprintBase::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	FText Result;
	
	const FGuid OldCachedNodeID = CachedNodeID;
	const IStateTreeBindingLookup* OldCachedBindingLookup = CachedBindingLookup;

	CachedNodeID = ID;
	CachedBindingLookup = &BindingLookup;

	Result = Description;
	if (Result.IsEmpty())
	{
		Result = ReceiveGetDescription(Formatting);
	}
	if (Result.IsEmpty())
	{
		Result = GetClass()->GetDisplayNameText();
	}

	CachedNodeID = OldCachedNodeID;
	CachedBindingLookup = OldCachedBindingLookup;
	
	return Result;
}
#endif

FText UStateTreeNodeBlueprintBase::GetPropertyDescriptionByPropertyName(FName PropertyName) const
{
	FText Result;
#if WITH_EDITOR
	// Try property binding first
	if (CachedBindingLookup)
	{
		const FPropertyBindingPath Path(CachedNodeID, PropertyName);
		Result = CachedBindingLookup->GetBindingSourceDisplayName(Path);
	}

	// No binding, get the value.
	if (Result.IsEmpty())
	{
		if (const FProperty* Property = GetClass()->FindPropertyByName(PropertyName))
		{
			FString	Value;
			Property->ExportText_InContainer(0, Value, this, this, nullptr, PPF_PropertyWindow | PPF_BlueprintDebugView);
			Result = FText::FromString(Value);
		}
	}
#endif
	return Result;
}

#undef LOCTEXT_NAMESPACE
