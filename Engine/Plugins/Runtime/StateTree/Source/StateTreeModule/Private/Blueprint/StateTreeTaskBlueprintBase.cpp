// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprint/StateTreeTaskBlueprintBase.h"
#include "StateTreeExecutionContext.h"
#include "BlueprintNodeHelpers.h"
#include "Engine/World.h"
#include "TimerManager.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeTaskBlueprintBase)

//----------------------------------------------------------------------//
//  UStateTreeTaskBlueprintBase
//----------------------------------------------------------------------//
UStateTreeTaskBlueprintBase::UStateTreeTaskBlueprintBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bShouldStateChangeOnReselect(true)
	, bShouldCallTick(false) // for when a child overrides the Tick
	, bShouldCallTickOnlyOnEvents(false)
	, bShouldCopyBoundPropertiesOnTick(true)
	, bShouldCopyBoundPropertiesOnExitState(true)
#if WITH_EDITORONLY_DATA
	, bConsideredForCompletion(true)
	, bCanEditConsideredForCompletion(true)
#endif
	, bIsProcessingEnterStateOrTick(false)
{
	bHasExitState = BlueprintNodeHelpers::HasBlueprintFunction(TEXT("ReceiveExitState"), *this, *StaticClass());
	bHasStateCompleted = BlueprintNodeHelpers::HasBlueprintFunction(TEXT("ReceiveStateCompleted"), *this, *StaticClass());
	bHasLatentEnterState = BlueprintNodeHelpers::HasBlueprintFunction(TEXT("ReceiveLatentEnterState"), *this, *StaticClass());
	bHasLatentTick = BlueprintNodeHelpers::HasBlueprintFunction(TEXT("ReceiveLatentTick"), *this, *StaticClass());
PRAGMA_DISABLE_DEPRECATION_WARNINGS		
	bHasEnterState_DEPRECATED = BlueprintNodeHelpers::HasBlueprintFunction(TEXT("ReceiveEnterState"), *this, *StaticClass());
	bHasTick_DEPRECATED = BlueprintNodeHelpers::HasBlueprintFunction(TEXT("ReceiveTick"), *this, *StaticClass());
PRAGMA_ENABLE_DEPRECATION_WARNINGS	
}

EStateTreeRunStatus UStateTreeTaskBlueprintBase::EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition)
{
	// Task became active, cache event queue and owner.
	SetCachedInstanceDataFromContext(Context);

	FGuardValue_Bitfield(bIsProcessingEnterStateOrTick, true);

	// Reset status to running since the same task may be restarted.
	RunStatus = EStateTreeRunStatus::Running;

	if (bHasLatentEnterState)
	{
		// Note: the name contains latent just to differentiate it from the deprecated version (the old version did not allow latent actions to be started).
		ReceiveLatentEnterState(Transition);
	}
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	else if (bHasEnterState_DEPRECATED)
	{
		RunStatus = ReceiveEnterState(Transition);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	return RunStatus;
}

void UStateTreeTaskBlueprintBase::ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition)
{
	if (bHasExitState)
	{
		ReceiveExitState(Transition);
	}

	if (UWorld* CurrentWorld = GetWorld())
	{
		CurrentWorld->GetLatentActionManager().RemoveActionsForObject(this);
		CurrentWorld->GetTimerManager().ClearAllTimersForObject(this);
	}

	// Task became inactive, clear cached event queue and owner.
	ClearCachedInstanceData();
}

void UStateTreeTaskBlueprintBase::StateCompleted(FStateTreeExecutionContext& Context, const EStateTreeRunStatus CompletionStatus, const FStateTreeActiveStates& CompletedActiveStates)
{
	if (bHasStateCompleted)
	{
		ReceiveStateCompleted(CompletionStatus, CompletedActiveStates);
	}
}

EStateTreeRunStatus UStateTreeTaskBlueprintBase::Tick(FStateTreeExecutionContext& Context, const float DeltaTime)
{
	FGuardValue_Bitfield(bIsProcessingEnterStateOrTick, true);

	if (bHasLatentTick)
	{
		// Note: the name contains latent just to differentiate it from the deprecated version (the old version did not allow latent actions to be started).
		ReceiveLatentTick(DeltaTime);
	}
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	else if (bHasTick_DEPRECATED)
	{
		RunStatus = ReceiveTick(DeltaTime);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	return RunStatus;
}

void UStateTreeTaskBlueprintBase::FinishTask(const bool bSucceeded)
{
	RunStatus = bSucceeded ? EStateTreeRunStatus::Succeeded : EStateTreeRunStatus::Failed;
	if (!bIsProcessingEnterStateOrTick)
	{
		EStateTreeFinishTaskType CompletionType = bSucceeded ? EStateTreeFinishTaskType::Succeeded : EStateTreeFinishTaskType::Failed;
		GetWeakExecutionContext().FinishTask(CompletionType);
	}
}

void UStateTreeTaskBlueprintBase::BroadcastDelegate(FStateTreeDelegateDispatcher Dispatcher)
{
	if (!GetWeakExecutionContext().BroadcastDelegate(Dispatcher)) 
	{
		UE_VLOG_UELOG(this, LogStateTree, Error, TEXT("Failed to broadcast the delegate. The instance probably stopped."));
	}
}

void UStateTreeTaskBlueprintBase::BindDelegate(const FStateTreeDelegateListener& Listener, const FStateTreeDynamicDelegate& Delegate)
{
	FSimpleDelegate SimpleDelegate = FSimpleDelegate::CreateUFunction(const_cast<UObject*>(Delegate.GetUObject()), Delegate.GetFunctionName());
	if (!GetWeakExecutionContext().BindDelegate(Listener, MoveTemp(SimpleDelegate)))
	{
		UE_VLOG_UELOG(this, LogStateTree, Error, TEXT("Failed to bind the delegate. The instance probably stopped."));
	}
}

void UStateTreeTaskBlueprintBase::UnbindDelegate(const FStateTreeDelegateListener& Listener)
{
	if (!GetWeakExecutionContext().UnbindDelegate(Listener))
	{
		UE_VLOG_UELOG(this, LogStateTree, Error, TEXT("Failed to unbind the delegate. The instance probably stopped."));
	}
}

//----------------------------------------------------------------------//
//  FStateTreeBlueprintTaskWrapper
//----------------------------------------------------------------------//

bool FStateTreeBlueprintTaskWrapper::Link(FStateTreeLinker& Linker)
{
	bShouldStateChangeOnReselect = (TaskFlags & 0x01) != 0;
	bShouldCallTick = (TaskFlags & 0x02) != 0;
	bShouldCallTickOnlyOnEvents = (TaskFlags & 0x04) != 0;
	bShouldCopyBoundPropertiesOnTick = (TaskFlags & 0x08) != 0;
	bShouldCopyBoundPropertiesOnExitState = (TaskFlags & 0x10) != 0;

	return Super::Link(Linker);
}

EStateTreeRunStatus FStateTreeBlueprintTaskWrapper::EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	UStateTreeTaskBlueprintBase* Instance = Context.GetInstanceDataPtr<UStateTreeTaskBlueprintBase>(*this);
	check(Instance);
	return Instance->EnterState(Context, Transition);
}

void FStateTreeBlueprintTaskWrapper::ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	UStateTreeTaskBlueprintBase* Instance = Context.GetInstanceDataPtr<UStateTreeTaskBlueprintBase>(*this);
	check(Instance);
	Instance->ExitState(Context, Transition);
}

void FStateTreeBlueprintTaskWrapper::StateCompleted(FStateTreeExecutionContext& Context, const EStateTreeRunStatus CompletionStatus, const FStateTreeActiveStates& CompletedActiveStates) const
{
	UStateTreeTaskBlueprintBase* Instance = Context.GetInstanceDataPtr<UStateTreeTaskBlueprintBase>(*this);
	check(Instance);
	Instance->StateCompleted(Context, CompletionStatus, CompletedActiveStates);
}

EStateTreeRunStatus FStateTreeBlueprintTaskWrapper::Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const
{
	UStateTreeTaskBlueprintBase* Instance = Context.GetInstanceDataPtr<UStateTreeTaskBlueprintBase>(*this);
	check(Instance);
	return Instance->Tick(Context, DeltaTime);
}

#if WITH_EDITOR
EDataValidationResult FStateTreeBlueprintTaskWrapper::Compile(UE::StateTree::ICompileNodeContext& Context)
{
	const UStateTreeTaskBlueprintBase& InstanceData = Context.GetInstanceDataView().Get<UStateTreeTaskBlueprintBase>();

	// Copy over ticking related options.
	bShouldStateChangeOnReselect = InstanceData.bShouldStateChangeOnReselect;

	bShouldCallTick = InstanceData.bShouldCallTick || InstanceData.bHasLatentTick;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bShouldCallTick |= InstanceData.bHasTick_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
		bShouldCallTickOnlyOnEvents = InstanceData.bShouldCallTickOnlyOnEvents;
	bShouldCopyBoundPropertiesOnTick = InstanceData.bShouldCopyBoundPropertiesOnTick;
	bShouldCopyBoundPropertiesOnExitState = InstanceData.bShouldCopyBoundPropertiesOnExitState;

	// The flags on the FStateTreeTaskBase are not saved.
	TaskFlags = (bShouldStateChangeOnReselect ? 0x01 : 0)
		| (bShouldCallTick ? 0x02 : 0)
		| (bShouldCallTickOnlyOnEvents ? 0x04 : 0)
		| (bShouldCopyBoundPropertiesOnTick ? 0x08 : 0)
		| (bShouldCopyBoundPropertiesOnExitState ? 0x10 : 0);

	return EDataValidationResult::Valid;
}

FText FStateTreeBlueprintTaskWrapper::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	FText Description;
	if (const UStateTreeTaskBlueprintBase* Instance = InstanceDataView.GetPtr<UStateTreeTaskBlueprintBase>())
	{
		Description = Instance->GetDescription(ID, InstanceDataView, BindingLookup, Formatting);
	}
	if (Description.IsEmpty() && TaskClass)
	{
		Description = TaskClass->GetDisplayNameText();
	}
	return Description;
}

FName FStateTreeBlueprintTaskWrapper::GetIconName() const
{
	if (TaskClass)
	{
		if (const UStateTreeNodeBlueprintBase* NodeCDO = GetDefault<const UStateTreeNodeBlueprintBase>(TaskClass))
		{
			return NodeCDO->GetIconName();
		}
	}

	return FStateTreeTaskBase::GetIconName();
}

FColor FStateTreeBlueprintTaskWrapper::GetIconColor() const
{
	if (TaskClass)
	{
		if (const UStateTreeNodeBlueprintBase* NodeCDO = GetDefault<const UStateTreeNodeBlueprintBase>(TaskClass))
		{
			return NodeCDO->GetIconColor();
		}
	}

	return FStateTreeTaskBase::GetIconColor();
}
#endif