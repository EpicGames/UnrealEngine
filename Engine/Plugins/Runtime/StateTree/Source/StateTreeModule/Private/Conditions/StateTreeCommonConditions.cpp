// Copyright Epic Games, Inc. All Rights Reserved.

#include "Conditions/StateTreeCommonConditions.h"

#include "AITypes.h"
#include "Conditions/StateTreeConditionHelpers.h"
#include "StateTreeExecutionContext.h"
#include "UObject/EnumProperty.h"
#include "StateTreeNodeDescriptionHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeCommonConditions)

#define LOCTEXT_NAMESPACE "StateTree"

namespace UE::StateTree::Conditions
{
	// Deprecated
	EComparisonOperator GenericAICheckToComparisonOperator(EGenericAICheck Operator)
	{
		switch (Operator)
		{
		case EGenericAICheck::Equal:
			return EComparisonOperator::Equal;
		case EGenericAICheck::NotEqual:
			return EComparisonOperator::NotEqual;
		case EGenericAICheck::Less:
			return EComparisonOperator::Less;
		case EGenericAICheck::LessOrEqual:
			return EComparisonOperator::LessOrEqual;
		case EGenericAICheck::Greater:
			return EComparisonOperator::Greater;
		case EGenericAICheck::GreaterOrEqual:
			return EComparisonOperator::GreaterOrEqual;
		default:
			ensureMsgf(false, TEXT("Unhandled operator %d"), EnumToUnderlyingType(Operator));
			return EComparisonOperator::Equal;
		}
	}
}

//----------------------------------------------------------------------//
//  FStateTreeCompareIntCondition
//----------------------------------------------------------------------//

// Deprecated
PRAGMA_DISABLE_DEPRECATION_WARNINGS
FStateTreeCompareIntCondition::FStateTreeCompareIntCondition(const EGenericAICheck InOperator, const EStateTreeCompare InInverts)
	: FStateTreeCompareIntCondition(UE::StateTree::Conditions::GenericAICheckToComparisonOperator(InOperator), InInverts)
{}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool FStateTreeCompareIntCondition::TestCondition(FStateTreeExecutionContext& Context) const
{
	const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	const bool bResult = UE::StateTree::Conditions::CompareNumbers<int32>(InstanceData.Left, InstanceData.Right, Operator);

	SET_NODE_CUSTOM_TRACE_TEXT(Context, Override, TEXT("%s%s %s %s")
		, *UE::StateTree::DescHelpers::GetInvertText(bInvert, EStateTreeNodeFormatting::Text).ToString()
		, *LexToString(InstanceData.Left)
		, *UE::StateTree::DescHelpers::GetOperatorText(Operator, EStateTreeNodeFormatting::Text).ToString()
		, *LexToString(InstanceData.Right));

	return bResult ^ bInvert;
}

#if WITH_EDITOR
FText FStateTreeCompareIntCondition::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	const FInstanceDataType* InstanceData = InstanceDataView.GetPtr<FInstanceDataType>();
	check(InstanceData);

	FText LeftValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, Left)), Formatting);
	if (LeftValue.IsEmpty())
	{
		LeftValue = FText::AsNumber(InstanceData->Left);
	}

	FText RightValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, Right)), Formatting);
	if (RightValue.IsEmpty())
	{
		RightValue = FText::AsNumber(InstanceData->Right);
	}

	const FText InvertText = UE::StateTree::DescHelpers::GetInvertText(bInvert, Formatting);
	const FText OperatorText = UE::StateTree::DescHelpers::GetOperatorText(Operator, Formatting);

	return FText::FormatNamed(LOCTEXT("CompareInt", "{EmptyOrNot}{Left} {Op} {Right}"),
		TEXT("EmptyOrNot"), InvertText,
		TEXT("Left"),LeftValue,
		TEXT("Op"), OperatorText,
		TEXT("Right"),RightValue);
}
#endif

//----------------------------------------------------------------------//
//  FStateTreeCompareFloatCondition
//----------------------------------------------------------------------//

// Deprecated
PRAGMA_DISABLE_DEPRECATION_WARNINGS
FStateTreeCompareFloatCondition::FStateTreeCompareFloatCondition(const EGenericAICheck InOperator, const EStateTreeCompare InInverts)
	: FStateTreeCompareFloatCondition(UE::StateTree::Conditions::GenericAICheckToComparisonOperator(InOperator), InInverts)
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool FStateTreeCompareFloatCondition::TestCondition(FStateTreeExecutionContext& Context) const
{
	const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	const bool bResult = UE::StateTree::Conditions::CompareNumbers<double>(InstanceData.Left, InstanceData.Right, Operator);

	SET_NODE_CUSTOM_TRACE_TEXT(Context, Override, TEXT("%s%s %s %s")
		, *UE::StateTree::DescHelpers::GetInvertText(bInvert, EStateTreeNodeFormatting::Text).ToString()
		, *LexToString(InstanceData.Left)
		, *UE::StateTree::DescHelpers::GetOperatorText(Operator, EStateTreeNodeFormatting::Text).ToString()
		, *LexToString(InstanceData.Right));

	return bResult ^ bInvert;
}

#if WITH_EDITOR
FText FStateTreeCompareFloatCondition::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	const FInstanceDataType* InstanceData = InstanceDataView.GetPtr<FInstanceDataType>();
	check(InstanceData);

	FNumberFormattingOptions Options;
	Options.MinimumFractionalDigits = 1;
	Options.MaximumFractionalDigits = 3;

	FText LeftValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, Left)), Formatting);
	if (LeftValue.IsEmpty())
	{
		LeftValue = FText::AsNumber(InstanceData->Left, &Options);
	}

	FText RightValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, Right)), Formatting);
	if (RightValue.IsEmpty())
	{
		RightValue = FText::AsNumber(InstanceData->Right, &Options);
	}

	const FText InvertText = UE::StateTree::DescHelpers::GetInvertText(bInvert, Formatting);
	const FText OperatorText = UE::StateTree::DescHelpers::GetOperatorText(Operator, Formatting);

	return FText::FormatNamed(LOCTEXT("CompareFloat", "{EmptyOrNot}{Left} {Op} {Right}"),
		TEXT("EmptyOrNot"), InvertText,
		TEXT("Left"),LeftValue,
		TEXT("Op"), OperatorText,
		TEXT("Right"),RightValue);
}
#endif

//----------------------------------------------------------------------//
//  FStateTreeCompareBoolCondition
//----------------------------------------------------------------------//

bool FStateTreeCompareBoolCondition::TestCondition(FStateTreeExecutionContext& Context) const
{
	const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	SET_NODE_CUSTOM_TRACE_TEXT(Context, Override, TEXT("%s%s is %s")
		, *UE::StateTree::DescHelpers::GetInvertText(bInvert, EStateTreeNodeFormatting::Text).ToString()
		, *UE::StateTree::DescHelpers::GetBoolText(InstanceData.bLeft, EStateTreeNodeFormatting::Text).ToString()
		, *UE::StateTree::DescHelpers::GetBoolText(InstanceData.bRight, EStateTreeNodeFormatting::Text).ToString());

	return (InstanceData.bLeft == InstanceData.bRight) ^ bInvert;
}

#if WITH_EDITOR
FText FStateTreeCompareBoolCondition::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	const FInstanceDataType* InstanceData = InstanceDataView.GetPtr<FInstanceDataType>();
	check(InstanceData);

	FText LeftValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, bLeft)), Formatting);
	if (LeftValue.IsEmpty())
	{
		LeftValue = UE::StateTree::DescHelpers::GetBoolText(InstanceData->bLeft, Formatting);
	}

	FText RightValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, bRight)), Formatting);
	if (RightValue.IsEmpty())
	{
		RightValue = UE::StateTree::DescHelpers::GetBoolText(InstanceData->bRight, Formatting);
	}

	const FText InvertText = UE::StateTree::DescHelpers::GetInvertText(bInvert, Formatting);

	const FText Format = (Formatting == EStateTreeNodeFormatting::RichText)
		? LOCTEXT("CompareBoolRich", "{EmptyOrNot}{Left} <s>is</> {Right}")
		: LOCTEXT("CompareBool", "{EmptyOrNot}{Left} is {Right}");

	return FText::FormatNamed(Format,
		TEXT("EmptyOrNot"), InvertText,
		TEXT("Left"),LeftValue,
		TEXT("Right"),RightValue);
}
#endif

//----------------------------------------------------------------------//
//  FStateTreeCompareEnumCondition
//----------------------------------------------------------------------//

bool FStateTreeCompareEnumCondition::TestCondition(FStateTreeExecutionContext& Context) const
{
	const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	SET_NODE_CUSTOM_TRACE_TEXT(Context, Override, TEXT("%s%s is %s")
		, *UE::StateTree::DescHelpers::GetInvertText(bInvert, EStateTreeNodeFormatting::Text).ToString()
		, *InstanceData.Left.Enum->GetNameStringByValue(InstanceData.Left.Value)
		, *InstanceData.Right.Enum->GetNameStringByValue(InstanceData.Right.Value));

	return (InstanceData.Left == InstanceData.Right) ^ bInvert;
}

#if WITH_EDITOR
FText FStateTreeCompareEnumCondition::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	const FInstanceDataType* InstanceData = InstanceDataView.GetPtr<FInstanceDataType>();
	check(InstanceData);

	FText LeftValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, Left)), Formatting);
	if (LeftValue.IsEmpty())
	{
		if (InstanceData->Left.Enum)
		{
			LeftValue = InstanceData->Left.Enum->GetDisplayNameTextByValue(InstanceData->Left.Value);
		}
		else
		{
			LeftValue = LOCTEXT("None", "None");
		}
	}

	FText RightValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, Right)), Formatting);
	if (RightValue.IsEmpty())
	{
		if (InstanceData->Right.Enum)
		{
			RightValue = InstanceData->Right.Enum->GetDisplayNameTextByValue(InstanceData->Right.Value);
		}
		else
		{
			RightValue = LOCTEXT("None", "None");
		}
	}

	const FText InvertText = UE::StateTree::DescHelpers::GetInvertText(bInvert, Formatting);

	const FText Format = (Formatting == EStateTreeNodeFormatting::RichText)
		? LOCTEXT("CompareEnumRich", "{EmptyOrNot}{Left} <s>is</> {Right}")
		: LOCTEXT("CompareEnum", "{EmptyOrNot}{Left} is {Right}");

	return FText::FormatNamed(Format,
		TEXT("EmptyOrNot"), InvertText,
		TEXT("Left"),LeftValue,
		TEXT("Right"),RightValue);
}

void FStateTreeCompareEnumCondition::OnBindingChanged(const FGuid& ID, FStateTreeDataView InstanceData, const FPropertyBindingPath& SourcePath, const FPropertyBindingPath& TargetPath, const IStateTreeBindingLookup& BindingLookup)
{
	if (!TargetPath.GetStructID().IsValid())
	{
		return;
	}

	FInstanceDataType& Instance = InstanceData.GetMutable<FInstanceDataType>();

	// Left has changed, update enums from the leaf property.
	if (!TargetPath.IsPathEmpty()
		&& TargetPath.GetSegments().Last().GetName() == GET_MEMBER_NAME_CHECKED(FInstanceDataType, Left))
	{
		if (const FProperty* LeafProperty = BindingLookup.GetPropertyPathLeafProperty(SourcePath))
		{
			// Handle both old stype namespace enums and new class enum properties.
			UEnum* NewEnum = nullptr;
			if (const FByteProperty* ByteProperty = CastField<FByteProperty>(LeafProperty))
			{
				NewEnum = ByteProperty->GetIntPropertyEnum();
			}
			else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(LeafProperty))
			{
				NewEnum = EnumProperty->GetEnum();
			}

			if (Instance.Left.Enum != NewEnum)
			{
				Instance.Left.Initialize(NewEnum);
			}
		}
		else
		{
			Instance.Left.Initialize(nullptr);
		}

		if (Instance.Right.Enum != Instance.Left.Enum)
		{
			Instance.Right.Initialize(Instance.Left.Enum);
		}
	}
}

#endif


//----------------------------------------------------------------------//
//  FStateTreeCompareDistanceCondition
//----------------------------------------------------------------------//

// Deprecated
PRAGMA_DISABLE_DEPRECATION_WARNINGS
FStateTreeCompareDistanceCondition::FStateTreeCompareDistanceCondition(const EGenericAICheck InOperator, const EStateTreeCompare InInverts)
	: FStateTreeCompareDistanceCondition(UE::StateTree::Conditions::GenericAICheckToComparisonOperator(InOperator), InInverts)
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool FStateTreeCompareDistanceCondition::TestCondition(FStateTreeExecutionContext& Context) const
{
	const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	const FVector::FReal Left = FVector::DistSquared(InstanceData.Source, InstanceData.Target);
	const FVector::FReal Right = FMath::Square(InstanceData.Distance);
	const bool bResult = UE::StateTree::Conditions::CompareNumbers<FVector::FReal>(Left, Right, Operator);

	SET_NODE_CUSTOM_TRACE_TEXT(Context, Override, TEXT("%sDistance %s %s %s (from [%s] to [%s])")
		, *UE::StateTree::DescHelpers::GetInvertText(bInvert, EStateTreeNodeFormatting::Text).ToString()
		, *LexToString(FMath::Sqrt(Left))
		, *UE::StateTree::DescHelpers::GetOperatorText(Operator, EStateTreeNodeFormatting::Text).ToString()
		, *LexToString(InstanceData.Distance)
		, *InstanceData.Source.ToString()
		, *InstanceData.Target.ToString());

	return bResult ^ bInvert;
}

#if WITH_EDITOR
FText FStateTreeCompareDistanceCondition::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	const FInstanceDataType* InstanceData = InstanceDataView.GetPtr<FInstanceDataType>();
	check(InstanceData);

	FNumberFormattingOptions Options;
	Options.MinimumFractionalDigits = 1;
	Options.MaximumFractionalDigits = 3;

	FText SourceValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, Source)), Formatting);
	if (SourceValue.IsEmpty())
	{
		SourceValue = InstanceData->Source.ToText();
	}

	FText TargetValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, Target)), Formatting);
	if (TargetValue.IsEmpty())
	{
		TargetValue = InstanceData->Target.ToText();
	}

	FText DistanceValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, Distance)), Formatting);
	if (DistanceValue.IsEmpty())
	{
		DistanceValue = FText::AsNumber(InstanceData->Distance, &Options);
	}

	const FText OperatorText = UE::StateTree::DescHelpers::GetOperatorText(Operator, Formatting);
	const FText InvertText = UE::StateTree::DescHelpers::GetInvertText(bInvert, Formatting);

	const FText Format = (Formatting == EStateTreeNodeFormatting::RichText)
		? LOCTEXT("CompareDistanceRich", "{EmptyOrNot}<s>Distance from</> {Source} <s>to</> {Target} {Op} {Distance}")
		: LOCTEXT("CompareDistance", "{EmptyOrNot}Distance from {Source} to {Target} {Op} {Distance}");

	return FText::FormatNamed(Format,
		TEXT("EmptyOrNot"), InvertText,
		TEXT("Source"), SourceValue,
		TEXT("Target"), TargetValue,
		TEXT("Op"), OperatorText,
		TEXT("Distance"), DistanceValue);
}
#endif

//----------------------------------------------------------------------//
//  FStateTreeCompareNameCondition
//----------------------------------------------------------------------//

bool FStateTreeCompareNameCondition::TestCondition(FStateTreeExecutionContext& Context) const
{
	const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	SET_NODE_CUSTOM_TRACE_TEXT(Context, Override, TEXT("%s%s is %s")
							   , *UE::StateTree::DescHelpers::GetInvertText(bInvert, EStateTreeNodeFormatting::Text).ToString()
							   , *InstanceData.Left.ToString()
							   , *InstanceData.Right.ToString());

	return InstanceData.Left.IsEqual(InstanceData.Right) ^ bInvert;
}

#if WITH_EDITOR
FText FStateTreeCompareNameCondition::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	const FInstanceDataType* InstanceData = InstanceDataView.GetPtr<FInstanceDataType>();
	check(InstanceData);

	FText LeftValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, Left)), Formatting);
	if (LeftValue.IsEmpty())
	{
		LeftValue = FText::FromName(InstanceData->Left);
	}

	FText RightValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, Right)), Formatting);
	if (RightValue.IsEmpty())
	{
		RightValue = FText::FromName(InstanceData->Right);
	}

	const FText InvertText = UE::StateTree::DescHelpers::GetInvertText(bInvert, Formatting);

	const FText Format = (Formatting == EStateTreeNodeFormatting::RichText)
		? LOCTEXT("CompareNameRich", "{EmptyOrNot}{Left} <s>is</> {Right}")
	: LOCTEXT("CompareName", "{EmptyOrNot}{Left} is {Right}");

	return FText::FormatNamed(Format,
							  TEXT("EmptyOrNot"), InvertText,
							  TEXT("Left"),LeftValue,
							  TEXT("Right"),RightValue);
}
#endif

//----------------------------------------------------------------------//
//  FStateTreeRandomCondition
//----------------------------------------------------------------------//

bool FStateTreeRandomCondition::TestCondition(FStateTreeExecutionContext& Context) const
{
	const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	const float RandomValue = FMath::FRandRange(0.0f, 1.0f);

	SET_NODE_CUSTOM_TRACE_TEXT(Context, Override, TEXT("Random value %f < %s")
		, RandomValue
		, *LexToString(InstanceData.Threshold));

	return RandomValue < InstanceData.Threshold;
}

#if WITH_EDITOR
FText FStateTreeRandomCondition::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	const FInstanceDataType* InstanceData = InstanceDataView.GetPtr<FInstanceDataType>();
	check(InstanceData);

	FNumberFormattingOptions Options;
	Options.MinimumFractionalDigits = 1;
	Options.MaximumFractionalDigits = 3;

	FText ThresholdValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, Threshold)), Formatting);
	if (ThresholdValue.IsEmpty())
	{
		ThresholdValue = FText::AsNumber(InstanceData->Threshold, &Options);
	}

	const FText Format = (Formatting == EStateTreeNodeFormatting::RichText)
		? LOCTEXT("RandomRich", "<s>Random [0..1] &lt;</> {Threshold}")
		: LOCTEXT("Random", "Random [0..1] &lt; {Threshold}");

	return FText::FormatNamed(Format,
		TEXT("Threshold"), ThresholdValue);
}
#endif

#undef LOCTEXT_NAMESPACE

