// Copyright Epic Games, Inc. All Rights Reserved.

#include "Conditions/StateTreeGameplayTagConditions.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeNodeDescriptionHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeGameplayTagConditions)

#if WITH_EDITOR
#define LOCTEXT_NAMESPACE "StateTreeEditor"

namespace UE::StateTree::Conditions
{


}

#endif// WITH_EDITOR

//----------------------------------------------------------------------//
//  FGameplayTagSingleMatchCondition
//----------------------------------------------------------------------//

bool FGameplayTagSingleMatchCondition::TestCondition(FStateTreeExecutionContext& Context) const
{
	const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	SET_NODE_CUSTOM_TRACE_TEXT(Context, Override, TEXT("%s'%s' matches '%s%s'")
		, *UE::StateTree::DescHelpers::GetInvertText(bInvert, EStateTreeNodeFormatting::Text).ToString()
		, *InstanceData.Left.ToString()
		, *UE::StateTree::DescHelpers::GetExactMatchText(bExactMatch, EStateTreeNodeFormatting::Text).ToString()
		, *InstanceData.Right.ToString());

	return (bExactMatch ? InstanceData.Left.MatchesTagExact(InstanceData.Right) : InstanceData.Left.MatchesTag(InstanceData.Right)) ^ bInvert;
}

#if WITH_EDITOR
FText FGameplayTagSingleMatchCondition::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	const FInstanceDataType* InstanceData = InstanceDataView.GetPtr<FInstanceDataType>();
	check(InstanceData);

	FText LeftValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, Left)), Formatting);
	if (LeftValue.IsEmpty())
	{
		LeftValue = FText::FromString(InstanceData->Left.ToString());
	}

	FText RightValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, Right)), Formatting);
	if (RightValue.IsEmpty())
	{
		RightValue = FText::FromString(InstanceData->Right.ToString());
	}

	const FText InvertText = UE::StateTree::DescHelpers::GetInvertText(bInvert, Formatting);
	const FText ExactMatchText = UE::StateTree::DescHelpers::GetExactMatchText(bExactMatch, Formatting);

	const FText Format = (Formatting == EStateTreeNodeFormatting::RichText)
		? LOCTEXT("GameplayTagSingleMatchRich", "{EmptyOrNot}{Left} <s>matches</> {EmptyOrExactly}{Right}")
		: LOCTEXT("GameplayTagSingleMatch", "{EmptyOrNot}{Left} matches {EmptyOrExactly}{Right}");

	return FText::FormatNamed(Format,
		TEXT("EmptyOrNot"), InvertText,
		TEXT("Left"), LeftValue,
		TEXT("EmptyOrExactly"), ExactMatchText,
		TEXT("Right"), RightValue);
}
#endif // WITH_EDITOR


//----------------------------------------------------------------------//
//  FGameplayTagMatchCondition
//----------------------------------------------------------------------//

bool FGameplayTagMatchCondition::TestCondition(FStateTreeExecutionContext& Context) const
{
	const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	SET_NODE_CUSTOM_TRACE_TEXT(Context, Override, TEXT("%s'%s' contains '%s%s'")
		, *UE::StateTree::DescHelpers::GetInvertText(bInvert, EStateTreeNodeFormatting::Text).ToString()
		, *UE::StateTree::DescHelpers::GetGameplayTagContainerAsText(InstanceData.TagContainer).ToString()
		, *UE::StateTree::DescHelpers::GetExactMatchText(bExactMatch, EStateTreeNodeFormatting::Text).ToString()
		, *InstanceData.Tag.ToString());

	return (bExactMatch ? InstanceData.TagContainer.HasTagExact(InstanceData.Tag) : InstanceData.TagContainer.HasTag(InstanceData.Tag)) ^ bInvert;
}

#if WITH_EDITOR
FText FGameplayTagMatchCondition::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	const FInstanceDataType* InstanceData = InstanceDataView.GetPtr<FInstanceDataType>();
	check(InstanceData);

	FText ContainerValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, TagContainer)), Formatting);
	if (ContainerValue.IsEmpty())
	{
		ContainerValue = UE::StateTree::DescHelpers::GetGameplayTagContainerAsText(InstanceData->TagContainer);
	}

	FText TagValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, Tag)), Formatting);
	if (TagValue.IsEmpty())
	{
		TagValue = FText::FromString(InstanceData->Tag.ToString());
	}

	const FText InvertText = UE::StateTree::DescHelpers::GetInvertText(bInvert, Formatting);
	const FText ExactMatchText = UE::StateTree::DescHelpers::GetExactMatchText(bExactMatch, Formatting);

	const FText Format = (Formatting == EStateTreeNodeFormatting::RichText)
		? LOCTEXT("GameplayTagMatchRich", "{EmptyOrNot}{TagContainer} <s>contains</> {EmptyOrExactly}{Tag}")
		: LOCTEXT("GameplayTagMatch", "{EmptyOrNot}{TagContainer} contains {EmptyOrExactly}{Tag}");

	return FText::FormatNamed(Format,
		TEXT("EmptyOrNot"), InvertText,
		TEXT("TagContainer"), ContainerValue,
		TEXT("EmptyOrExactly"), ExactMatchText,
		TEXT("Tag"), TagValue);
}
#endif

//----------------------------------------------------------------------//
//  FGameplayTagContainerMatchCondition
//----------------------------------------------------------------------//

bool FGameplayTagContainerMatchCondition::TestCondition(FStateTreeExecutionContext& Context) const
{
	const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	bool bResult = false;
	switch (MatchType)
	{
	case EGameplayContainerMatchType::Any:
		bResult = bExactMatch ? InstanceData.TagContainer.HasAnyExact(InstanceData.OtherContainer) : InstanceData.TagContainer.HasAny(InstanceData.OtherContainer);
		break;
	case EGameplayContainerMatchType::All:
		bResult = bExactMatch ? InstanceData.TagContainer.HasAllExact(InstanceData.OtherContainer) : InstanceData.TagContainer.HasAll(InstanceData.OtherContainer);
		break;
	default:
		ensureMsgf(false, TEXT("Unhandled match type %s."), *UEnum::GetValueAsString(MatchType));
	}

	SET_NODE_CUSTOM_TRACE_TEXT(Context, Override, TEXT("%s'%s' contains '%s %s%s'")
		, *UE::StateTree::DescHelpers::GetInvertText(bInvert, EStateTreeNodeFormatting::Text).ToString()
		, *UE::StateTree::DescHelpers::GetGameplayTagContainerAsText(InstanceData.TagContainer).ToString()
		, *UEnum::GetValueAsString(MatchType)
		, *UE::StateTree::DescHelpers::GetExactMatchText(bExactMatch, EStateTreeNodeFormatting::Text).ToString()
		, *UE::StateTree::DescHelpers::GetGameplayTagContainerAsText(InstanceData.OtherContainer).ToString());

	return bResult ^ bInvert;
}

#if WITH_EDITOR
FText FGameplayTagContainerMatchCondition::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	const FInstanceDataType* InstanceData = InstanceDataView.GetPtr<FInstanceDataType>();
	check(InstanceData);

	FText ContainerValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, TagContainer)), Formatting);
	if (ContainerValue.IsEmpty())
	{
		ContainerValue = UE::StateTree::DescHelpers::GetGameplayTagContainerAsText(InstanceData->TagContainer);
	}

	FText OtherContainerValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, OtherContainer)), Formatting);
	if (OtherContainerValue.IsEmpty())
	{
		OtherContainerValue = UE::StateTree::DescHelpers::GetGameplayTagContainerAsText(InstanceData->OtherContainer);
	}

	const FText InvertText = UE::StateTree::DescHelpers::GetInvertText(bInvert, Formatting);
	const FText ExactMatchText = UE::StateTree::DescHelpers::GetExactMatchText(bExactMatch, Formatting);
	const FText MatchTypeText = UEnum::GetDisplayValueAsText(MatchType);

	const FText Format = (Formatting == EStateTreeNodeFormatting::RichText)
		? LOCTEXT("GameplayTagContainerMatchRich", "{EmptyOrNot}{TagContainer} <s>contains {AnyOrAll}</> {EmptyOrExactly}{OtherTagContainer}")
		: LOCTEXT("GameplayTagContainerMatch", "{EmptyOrNot}{TagContainer} contains {AnyOrAll} {EmptyOrExactly}{OtherTagContainer}");

	return FText::FormatNamed(Format,
		TEXT("EmptyOrNot"), InvertText,
		TEXT("TagContainer"), ContainerValue,
		TEXT("AnyOrAll"), MatchTypeText,
		TEXT("EmptyOrExactly"), ExactMatchText,
		TEXT("OtherTagContainer"), OtherContainerValue);
}
#endif

//----------------------------------------------------------------------//
//  FGameplayTagQueryCondition
//----------------------------------------------------------------------//

bool FGameplayTagQueryCondition::TestCondition(FStateTreeExecutionContext& Context) const
{
	const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	SET_NODE_CUSTOM_TRACE_TEXT(Context, Override, TEXT("%s'%s' matches %s")
		, *UE::StateTree::DescHelpers::GetInvertText(bInvert, EStateTreeNodeFormatting::Text).ToString()
		, *UE::StateTree::DescHelpers::GetGameplayTagContainerAsText(InstanceData.TagContainer).ToString()
		, *UE::StateTree::DescHelpers::GetGameplayTagQueryAsText(TagQuery).ToString());

	return TagQuery.Matches(InstanceData.TagContainer) ^ bInvert;
}

#if WITH_EDITOR
FText FGameplayTagQueryCondition::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	const FInstanceDataType* InstanceData = InstanceDataView.GetPtr<FInstanceDataType>();
	check(InstanceData);

	FText ContainerValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, TagContainer)), Formatting);
	if (ContainerValue.IsEmpty())
	{
		ContainerValue = UE::StateTree::DescHelpers::GetGameplayTagContainerAsText(InstanceData->TagContainer);
	}

	const FText QueryValue = UE::StateTree::DescHelpers::GetGameplayTagQueryAsText(TagQuery);

	const FText InvertText = UE::StateTree::DescHelpers::GetInvertText(bInvert, Formatting);


	const FText Format = (Formatting == EStateTreeNodeFormatting::RichText)
		? LOCTEXT("GameplayTagQueryRich", "{EmptyOrNot}{TagContainer} <s>matches</> {TagQuery}")
		: LOCTEXT("GameplayTagQuery", "{EmptyOrNot}{TagContainer} matches {TagQuery}");

	return FText::FormatNamed(Format,
		TEXT("EmptyOrNot"), InvertText,
		TEXT("TagContainer"), ContainerValue,
		TEXT("TagQuery"), QueryValue);
}
#endif

#if WITH_EDITOR
#undef LOCTEXT_NAMESPACE
#endif // WITH_EDITOR

