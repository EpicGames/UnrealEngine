// Copyright Epic Games, Inc. All Rights Reserved.

#include "Conditions/StateTreeObjectConditions.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeNodeDescriptionHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeObjectConditions)

#define LOCTEXT_NAMESPACE "StateTreeObjectCondition"


//----------------------------------------------------------------------//
//  FStateTreeCondition_ObjectIsValid
//----------------------------------------------------------------------//

bool FStateTreeObjectIsValidCondition::TestCondition(FStateTreeExecutionContext& Context) const
{
	const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	const bool bResult = IsValid(InstanceData.Object);
	return bResult ^ bInvert;
}

#if WITH_EDITOR
FText FStateTreeObjectIsValidCondition::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	const FText InvertText = UE::StateTree::DescHelpers::GetInvertText(bInvert, Formatting);
	const FText Format = (Formatting == EStateTreeNodeFormatting::RichText)
		? LOCTEXT("ObjectIsValidConditionsRich", "{EmptyOrNot}<s>Is Object Valid</>")
		: LOCTEXT("ObjectIsValidCondition", "{EmptyOrNot}Is Object Valid");
	return FText::FormatNamed(Format,
		TEXT("EmptyOrNot"), InvertText);
}
#endif

//----------------------------------------------------------------------//
//  FStateTreeCondition_ObjectEquals
//----------------------------------------------------------------------//

bool FStateTreeObjectEqualsCondition::TestCondition(FStateTreeExecutionContext& Context) const
{
	const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	const bool bResult = InstanceData.Left == InstanceData.Right;
	return bResult ^ bInvert;
}

#if WITH_EDITOR
FText FStateTreeObjectEqualsCondition::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	const FText InvertText = UE::StateTree::DescHelpers::GetInvertText(bInvert, Formatting);
	const FText Format = (Formatting == EStateTreeNodeFormatting::RichText)
		? LOCTEXT("ObjectEqualsConditionRich", "{EmptyOrNot}<s>Is Object Equals</>")
		: LOCTEXT("ObjectEqualsCondition", "{EmptyOrNot}Is Object Equals");
	return FText::FormatNamed(Format,
		TEXT("EmptyOrNot"), InvertText);
}
#endif

//----------------------------------------------------------------------//
//  FStateTreeCondition_ObjectIsChildOfClass
//----------------------------------------------------------------------//

bool FStateTreeObjectIsChildOfClassCondition::TestCondition(FStateTreeExecutionContext& Context) const
{
	const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	const bool bResult = InstanceData.Object && InstanceData.Class
						&& InstanceData.Object->GetClass()->IsChildOf(InstanceData.Class);

	SET_NODE_CUSTOM_TRACE_TEXT(Context, Override, TEXT("%s of type '%s' is%s child of '%s'")
		, *GetNameSafe(InstanceData.Object)
		, *GetNameSafe(InstanceData.Object ? InstanceData.Object->GetClass() : nullptr)
		, bResult ? TEXT("") : TEXT(" not")
		, *GetNameSafe(InstanceData.Class));

	return bResult ^ bInvert;
}

#if WITH_EDITOR
FText FStateTreeObjectIsChildOfClassCondition::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	const FText InvertText = UE::StateTree::DescHelpers::GetInvertText(bInvert, Formatting);
	const FText Format = (Formatting == EStateTreeNodeFormatting::RichText)
		? LOCTEXT("ObjectIsChildOfConditionsRich", "{EmptyOrNot}<s>Is Child Of Class</>")
		: LOCTEXT("ObjectIsChildOfCondition", "{EmptyOrNot}Is Child Of Class");
	return FText::FormatNamed(Format,
		TEXT("EmptyOrNot"), InvertText);
}
#endif

#undef LOCTEXT_NAMESPACE
