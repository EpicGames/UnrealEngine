// Copyright Epic Games, Inc. All Rights Reserved.

#include "Considerations/StateTreeCommonConsiderations.h"
#include "Algo/Sort.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeNodeDescriptionHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeCommonConsiderations)

#define LOCTEXT_NAMESPACE "StateTree"

#if WITH_EDITOR
FText FStateTreeFloatInputConsideration::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting /*= EStateTreeNodeFormatting::Text*/) const
{
	const FInstanceDataType* InstanceData = InstanceDataView.GetPtr<FInstanceDataType>();
	check(InstanceData);

	FText InputText = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, Input)), Formatting);
	if (InputText.IsEmpty())
	{
		FNumberFormattingOptions Options;
		Options.MinimumFractionalDigits = 1;
		Options.MaximumFractionalDigits = 2;

		InputText = FText::AsNumber(InstanceData->Input, &Options);
	}
	
	const FFloatInterval& Interval = InstanceData->Interval;
	FText IntervalText = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, Interval)), Formatting);
	if (IntervalText.IsEmpty())
	{
		IntervalText = UE::StateTree::DescHelpers::GetIntervalText(Interval, Formatting);
	}
	
	if (Formatting == EStateTreeNodeFormatting::RichText)
	{
		return FText::FormatNamed(LOCTEXT("InputInIntervalRich", "{Input} <s>in</> {Interval}"),
			TEXT("Input"), InputText,
			TEXT("Interval"), IntervalText
		);
	}
	else //EStateTreeNodeFormatting::Text
	{
		return FText::FormatNamed(LOCTEXT("InputInInterval", "{Input} in {Interval}"),
			TEXT("Input"), InputText,
			TEXT("Interval"), IntervalText
		);
	}
}
#endif //WITH_EDITOR

float FStateTreeFloatInputConsideration::GetScore(FStateTreeExecutionContext& Context) const
{
	const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	const float Input = InstanceData.Input;
	const FFloatInterval& Interval = InstanceData.Interval;
	const float NormalizedInput = FMath::Clamp(Interval.GetRangePct(Input), 0.f, 1.f);

	return ResponseCurve.Evaluate(NormalizedInput);
}

#if WITH_EDITOR
FText FStateTreeConstantConsideration::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting /*= EStateTreeNodeFormatting::Text*/) const
{
	const FInstanceDataType* InstanceData = InstanceDataView.GetPtr<FInstanceDataType>();
	check(InstanceData);

	FText ConstantText = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, Constant)), Formatting);
	if (ConstantText.IsEmpty())
	{
		FNumberFormattingOptions Options;
		Options.MinimumFractionalDigits = 1;
		Options.MaximumFractionalDigits = 2;

		ConstantText = FText::AsNumber(InstanceData->Constant, &Options);
	}

	return FText::FormatNamed(LOCTEXT("Constant", "{ConstantValue}"), 
		TEXT("ConstantValue"), ConstantText);
}
#endif //WITH_EDITOR

float FStateTreeConstantConsideration::GetScore(FStateTreeExecutionContext& Context) const
{
	const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	return InstanceData.Constant;
}

float FStateTreeEnumInputConsideration::GetScore(FStateTreeExecutionContext& Context) const
{
	const FStateTreeAnyEnum& Input = Context.GetInstanceData(*this).Input;
	const uint32 EnumValue = Input.Value;
	const TArray<FStateTreeEnumValueScorePair>& Data = EnumValueScorePairs.Data;

	for (int32 Index = 0; Index < Data.Num(); ++Index)
	{
		const FStateTreeEnumValueScorePair& Pair = Data[Index];
		if (EnumValue == Pair.EnumValue)
		{
			return Pair.Score;
		}
	}

	return 0.f;
}

#if WITH_EDITOR
EDataValidationResult FStateTreeEnumInputConsideration::Compile(UE::StateTree::ICompileNodeContext& Context)
{
	const FInstanceDataType& InstanceData = Context.GetInstanceDataView().Get<FInstanceDataType>();
	check(InstanceData.Input.Enum == EnumValueScorePairs.Enum);
	const int32 NumPairs = EnumValueScorePairs.Data.Num();

	//Validate uniqueness of keys
	const auto SortedPairs = EnumValueScorePairs.Data;
	Algo::Sort(SortedPairs, 
		[](const FStateTreeEnumValueScorePair& A, const FStateTreeEnumValueScorePair& B) {
		return A.EnumValue < B.EnumValue;
		});
	for (int32 Idx = 1; Idx < NumPairs; ++Idx)
	{
		if (SortedPairs[Idx].EnumValue == SortedPairs[Idx - 1].EnumValue)
		{
			Context.AddValidationError(LOCTEXT("DuplicateEnumValues", "Duplicate Enum Values found."));

			return EDataValidationResult::Invalid;
		}
	}

	return EDataValidationResult::Valid;
}

void FStateTreeEnumInputConsideration::OnBindingChanged(const FGuid& ID, FStateTreeDataView InstanceData, const FPropertyBindingPath& SourcePath, const FPropertyBindingPath& TargetPath, const IStateTreeBindingLookup& BindingLookup)
{
	if (!TargetPath.GetStructID().IsValid())
	{
		return;
	}

	FInstanceDataType& Instance = InstanceData.GetMutable<FInstanceDataType>();

	// Left has changed, update enums from the leaf property.
	if (!TargetPath.IsPathEmpty()
		&& TargetPath.GetSegments().Last().GetName() == GET_MEMBER_NAME_CHECKED(FInstanceDataType, Input))
	{
		if (const FProperty* LeafProperty = BindingLookup.GetPropertyPathLeafProperty(SourcePath))
		{
			// Handle both old type namespace enums and new class enum properties.
			UEnum* NewEnum = nullptr;
			if (const FByteProperty* ByteProperty = CastField<FByteProperty>(LeafProperty))
			{
				NewEnum = ByteProperty->GetIntPropertyEnum();
			}
			else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(LeafProperty))
			{
				NewEnum = EnumProperty->GetEnum();
			}

			if (Instance.Input.Enum != NewEnum)
			{
				Instance.Input.Initialize(NewEnum);
			}
		}
		else
		{
			Instance.Input.Initialize(nullptr);
		}

		if (EnumValueScorePairs.Enum != Instance.Input.Enum)
		{
			EnumValueScorePairs.Initialize(Instance.Input.Enum);
		}
	}
}

FText FStateTreeEnumInputConsideration::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting /*= EStateTreeNodeFormatting::Text*/) const
{
	const FInstanceDataType* InstanceData = InstanceDataView.GetPtr<FInstanceDataType>();
	check(InstanceData);

	FText InputText = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, Input)), Formatting);
	
	return FText::FormatNamed(LOCTEXT("EnumInput", "{Input}"),
		TEXT("Input"), InputText);
}

#endif //WITH_EDITOR

#undef LOCTEXT_NAMESPACE
