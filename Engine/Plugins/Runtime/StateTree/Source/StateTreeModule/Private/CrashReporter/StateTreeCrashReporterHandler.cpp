// Copyright Epic Games, Inc. All Rights Reserved.

#include "CrashReporter/StateTreeCrashReporterHandler.h"

#include "Misc/ScopeLock.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "HAL/PlatformTLS.h"

#include "StateTree.h"

#include <atomic>

#if UE_WITH_STATETREE_CRASHREPORTER

namespace UE::StateTree::Private
{
static bool bCrashHandlerEnabled = true;
static FAutoConsoleVariableRef CVarCrashHandlerEnabled(
	TEXT("StateTree.CrashHandlerEnabled"),
	bCrashHandlerEnabled,
	TEXT("StateTree will add context into the crash reporter."),
	ECVF_Default);

struct FCrashReporterHandlerImpl : FCrashReporterHandler
{
public:
	struct FContext
	{
		explicit FContext(TNotNull<const UObject*> InOwner, TNotNull<const UStateTree*> InStateTree, FName InContext, uint32 InThreadID)
			: Owner(InOwner)
			, StateTree(InStateTree)
			, Context(InContext)
			, ThreadID(InThreadID)
		{
			ID = GenerateID();
		}

		UE_AUTORTFM_ALWAYS_OPEN
		static uint32 GenerateID()
		{
			// This ID only needs to be unique; it doesn't need to be rolled back if an AutoRTFM transaction fails.
			static std::atomic<uint32> IDGenerator = 0;
			return ++IDGenerator;
		}

		TWeakObjectPtr<const UObject> Owner;
		TWeakObjectPtr<const UStateTree> StateTree;
		FName Context;
		uint32 ThreadID;
		uint32 ID;
	};

public:
	uint32 PushInfo(TNotNull<const UObject*> InOwner, TNotNull<const UStateTree*> InStateTree, FName InContext)
	{
		UE::TScopeLock LockGuard(CriticalSection);
		return Contexts.Emplace_GetRef(InOwner, InStateTree, InContext, FPlatformTLS::GetCurrentThreadId()).ID;
	}

	void PopInfo(uint32 ID)
	{
		UE::TScopeLock LockGuard(CriticalSection);
		const int32 FoundIndex = Contexts.IndexOfByPredicate([ID](const FContext& Other)
			{
				return Other.ID == ID;
			});
		if (FoundIndex != INDEX_NONE)
		{
			Contexts.RemoveAtSwap(FoundIndex);
		}
	}

	void DumpInfo(FCrashContextExtendedWriter& Writer)
	{
		// Callback from the crash reporter.Avoids memory allocation in case the crash is an OOM.
		UE::TScopeLock LockGuard(CriticalSection);

		for (int32 Index = 0; Index < Contexts.Num(); ++Index)
		{
			const FContext& Context = Contexts[Index];
			Builder.Reset();

			if (const UObject* Owner = Context.Owner.Get())
			{
				Owner->GetFullName(Builder);
			}
			else
			{
				Builder.Append(TEXT("<Invalid>"));
			}

			Builder << TEXT('\n');
			if (const UStateTree* StateTree = Context.StateTree.Get())
			{
				StateTree->GetFullName(Builder);
			}
			else
			{
				Builder.Append(TEXT("<Invalid>"));
			}

			Builder << TEXT('\n');
			Builder << Context.Context;

			Builder << TEXT('\n');
			Builder << Context.ThreadID;

			// Avoid allocating a new name for each entry.
			constexpr int32 BufferSize = 16;
			TCHAR Identifier[BufferSize];
			FCString::Snprintf(Identifier, UE_ARRAY_COUNT(Identifier), TEXT("StateTree%d"), Index);
			Writer.AddString(Identifier, Builder.ToString());
		}
	}

private:
	TArray<FContext, TInlineAllocator<32>> Contexts;
	FTransactionallySafeCriticalSection CriticalSection;
	TStringBuilder<1024> Builder;
};


//~ Use a shared ptr instead of manually deleting it in case we crash while unregistering the StateTree module.
static TSharedPtr<FCrashReporterHandlerImpl> Instance;
static FDelegateHandle AdditionalCrashContextDelegateHandle;

} // namespace UE::StateTree::Private

namespace UE::StateTree
{

void FCrashReporterHandler::Register()
{
	Private::Instance = MakeShared<Private::FCrashReporterHandlerImpl>();
	Private::AdditionalCrashContextDelegateHandle = FGenericCrashContext::OnAdditionalCrashContextDelegate().AddSP(Private::Instance.ToSharedRef(), &Private::FCrashReporterHandlerImpl::DumpInfo);
}

void FCrashReporterHandler::Unregister()
{
	FGenericCrashContext::OnAdditionalCrashContextDelegate().Remove(Private::AdditionalCrashContextDelegateHandle);
	// Keep the AdditionalCrashContextDelegateHandle valid in case it currently crashing. The pointer will be automatically removed (shared ptr) when the module unloads.
}

FCrashReporterScope::FCrashReporterScope(TNotNull<const UObject*> Owner, TNotNull<const UStateTree*> StateTree, FName Context)
{
	bWasEnabled = Private::bCrashHandlerEnabled;
	if (bWasEnabled)
	{
		check(Private::Instance);
		ID = Private::Instance->PushInfo(Owner, StateTree, Context);
	}
}


FCrashReporterScope::~FCrashReporterScope()
{
	if (bWasEnabled)
	{
		Private::Instance->PopInfo(ID);
	}
}

} //namespace UE::StateTree

#endif
