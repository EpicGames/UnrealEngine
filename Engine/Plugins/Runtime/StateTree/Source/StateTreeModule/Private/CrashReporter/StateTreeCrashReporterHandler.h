// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "Misc/NotNull.h"

#ifndef UE_WITH_STATETREE_CRASHREPORTER
	#define UE_WITH_STATETREE_CRASHREPORTER WITH_ADDITIONAL_CRASH_CONTEXTS
#endif

class UObject;
class UStateTree;

#if UE_WITH_STATETREE_CRASHREPORTER

namespace UE::StateTree
{

/**
 * Additional crash context for StateTree.
 * Will add extra information on the asset when crashing inside a StateTree callstack.
 */
struct FCrashReporterHandler
{
	static void Register();
	static void Unregister();
};

/** Helper object allowing easy tracking of StateTree code in crash reporter. */
class FCrashReporterScope
{
public:
	explicit FCrashReporterScope(TNotNull<const UObject*> Owner, TNotNull<const UStateTree*> StateTree, FName Context);
	~FCrashReporterScope();

	FCrashReporterScope() = delete;
	FCrashReporterScope(const FCrashReporterScope&) = delete;
	FCrashReporterScope& operator=(const FCrashReporterScope&) = delete;

private:
	bool bWasEnabled = false;
	uint32 ID = 0;
};

} //namespace

#endif //UE_WITH_STATETREE_CRASHREPORTER

#if UE_WITH_STATETREE_CRASHREPORTER
#define UE_STATETREE_CRASH_REPORTER_SCOPE(InOwner, InStateTree, InContext) ::UE::StateTree::FCrashReporterScope ANONYMOUS_VARIABLE(AddCrashContext) {(InOwner), (InStateTree), (InContext)}
#else //UE_WITH_STATETREE_CRASHREPORTER
#define UE_STATETREE_CRASH_REPORTER_SCOPE(InOwner, InStateTree, InContext)
#endif //UE_WITH_STATETREE_CRASHREPORTER
