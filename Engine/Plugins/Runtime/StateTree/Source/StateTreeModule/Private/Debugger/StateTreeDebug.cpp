// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debugger/StateTreeDebug.h"
#include "Debugger/StateTreeRuntimeValidationInstanceData.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeInstanceData.h"

#if WITH_STATETREE_DEBUG

namespace UE::StateTree::Debug
{
FPhaseDelegate OnBeginUpdatePhase_AnyThread;
FPhaseDelegate OnEndUpdatePhase_AnyThread;

FStateDelegate OnStateEvent_AnyThread;

FTransitionDelegate OnTransitionEvent_AnyThread;

FNodeDelegate OnConditionEnterState_AnyThread;
FNodeDelegate OnTestCondition_AnyThread;
FNodeDelegate OnConditionExitState_AnyThread;
FNodeDelegate OnEvaluatorEnterTree_AnyThread;
FNodeDelegate OnTickEvaluator_AnyThread;
FNodeDelegate OnEvaluatorExitTree_AnyThread;
FNodeDelegate OnTaskEnterState_AnyThread;
FNodeDelegate OnTickTask_AnyThread;
FNodeDelegate OnTaskExitState_AnyThread;
FEventSentDelegate OnEventSent_AnyThread;
FEventConsumedDelegate OnEventConsumed_AnyThread;
FStateUtilityEvaluatedDelegate OnStateUtilityEvaluated_AnyThread;

FNodeReference::FNodeReference(TNotNull<const UStateTree*> InStateTree, const FStateTreeIndex16 InNodeIndex)
	: StateTree(InStateTree)
	, Index(InNodeIndex)
{
}

namespace Private
{
void UpdatePhaseEnter(const FStateTreeExecutionContext& ExecutionContext, const EStateTreeUpdatePhase Phase, const FStateTreeStateHandle StateHandle, const FPhaseDelegate& Delegate)
{
	Delegate.Broadcast(ExecutionContext, Phase, StateHandle);
}

void UpdatePhaseExit(const FStateTreeExecutionContext& ExecutionContext, const EStateTreeUpdatePhase Phase, const FStateTreeStateHandle StateHandle, const FPhaseDelegate& Delegate)
{
	Delegate.Broadcast(ExecutionContext, Phase, StateHandle);
}

void NodeEnter(const FStateTreeExecutionContext& ExecutionContext, const FNodeReference Node, const FNodeDelegate& Delegate)
{
	FStateTreeInstanceData* InstanceData = ExecutionContext.GetMutableInstanceData();
	FGuid NodeId = Node.StateTree->GetNodeIdFromIndex(Node.Index);
	if (ensure(NodeId.IsValid()))
	{
		if (FRuntimeValidationInstanceData* RuntimeValidation = InstanceData->GetRuntimeValidation().GetInstanceData())
		{
			const FActiveFrameID FrameID = ExecutionContext.GetCurrentlyProcessedFrame() ? ExecutionContext.GetCurrentlyProcessedFrame()->FrameID : FActiveFrameID();
			RuntimeValidation->NodeEnterState(NodeId, FrameID);
		}
		Delegate.Broadcast(ExecutionContext, FNodeDelegateArgs{ .Node = Node, .NodeId = NodeId });
	}
}

void NodeExit(const FStateTreeExecutionContext& ExecutionContext, const FNodeReference Node, const FNodeDelegate& Delegate)
{
	FStateTreeInstanceData* InstanceData = ExecutionContext.GetMutableInstanceData();
	FGuid NodeId = Node.StateTree->GetNodeIdFromIndex(Node.Index);
	if (ensure(NodeId.IsValid()))
	{
		if (FRuntimeValidationInstanceData* RuntimeValidation = InstanceData->GetRuntimeValidation().GetInstanceData())
		{
			const FActiveFrameID FrameID = ExecutionContext.GetCurrentlyProcessedFrame() ? ExecutionContext.GetCurrentlyProcessedFrame()->FrameID : FActiveFrameID();
			RuntimeValidation->NodeExitState(NodeId, FrameID);
		}
		Delegate.Broadcast(ExecutionContext, FNodeDelegateArgs{ .Node = Node, .NodeId = NodeId });
	}
}

void NodeTick(const FStateTreeExecutionContext& ExecutionContext, const FNodeReference Node, const FNodeDelegate& Delegate)
{
	const FGuid NodeId = Node.StateTree->GetNodeIdFromIndex(Node.Index);
	if (ensure(NodeId.IsValid()))
	{
		Delegate.Broadcast(ExecutionContext, FNodeDelegateArgs{ .Node = Node, .NodeId = NodeId });
	}
}
} //namespace Private

void EnterPhase(const FStateTreeExecutionContext& ExecutionContext, const EStateTreeUpdatePhase Phase, const FStateTreeStateHandle StateHandle)
{
	Private::UpdatePhaseEnter(ExecutionContext, Phase, StateHandle, OnBeginUpdatePhase_AnyThread);
}

void ExitPhase(const FStateTreeExecutionContext& ExecutionContext, const EStateTreeUpdatePhase Phase, const FStateTreeStateHandle StateHandle)
{
	Private::UpdatePhaseExit(ExecutionContext, Phase, StateHandle, OnEndUpdatePhase_AnyThread);
}

void StateEvent(const FStateTreeExecutionContext& ExecutionContext, const FStateTreeStateHandle StateHandle, const EStateTreeTraceEventType EventType)
{
	OnStateEvent_AnyThread.Broadcast(ExecutionContext, StateHandle, EventType);
}

void TransitionEvent(const FStateTreeExecutionContext& ExecutionContext, const FStateTreeTransitionSource& TransitionSource, const EStateTreeTraceEventType EventType)
{
	OnTransitionEvent_AnyThread.Broadcast(ExecutionContext, TransitionSource, EventType);
}

void ConditionEnterState(const FStateTreeExecutionContext& ExecutionContext, const FNodeReference Node)
{
	Private::NodeEnter(ExecutionContext, Node, OnConditionEnterState_AnyThread);
}

void ConditionTest(const FStateTreeExecutionContext& ExecutionContext, const FNodeReference Node)
{
	Private::NodeTick(ExecutionContext, Node, OnTestCondition_AnyThread);
}

void ConditionExitState(const FStateTreeExecutionContext& ExecutionContext, const FNodeReference Node)
{
	Private::NodeExit(ExecutionContext, Node, OnConditionExitState_AnyThread);
}

void EvaluatorEnterTree(const FStateTreeExecutionContext& ExecutionContext, const FNodeReference Node)
{
	Private::NodeEnter(ExecutionContext, Node, OnEvaluatorEnterTree_AnyThread);
}

void EvaluatorTick(const FStateTreeExecutionContext& ExecutionContext, const FNodeReference Node)
{
	Private::NodeTick(ExecutionContext, Node, OnTickEvaluator_AnyThread);
}

void EvaluatorExitTree(const FStateTreeExecutionContext& ExecutionContext, const FNodeReference Node)
{
	Private::NodeExit(ExecutionContext, Node, OnEvaluatorExitTree_AnyThread);
}

void TaskEnterState(const FStateTreeExecutionContext& ExecutionContext, const FNodeReference Node)
{
	Private::NodeEnter(ExecutionContext, Node, OnTaskEnterState_AnyThread);
}

void TaskTick(const FStateTreeExecutionContext& ExecutionContext, const FNodeReference Node)
{
	Private::NodeTick(ExecutionContext, Node, OnTickTask_AnyThread);
}

void TaskExitState(const FStateTreeExecutionContext& ExecutionContext, const FNodeReference Node)
{
	Private::NodeExit(ExecutionContext, Node, OnTaskExitState_AnyThread);
}

void EventSent(const FStateTreeMinimalExecutionContext& ExecutionContext, TNotNull<const UStateTree*> StateTree, const FGameplayTag Tag, const FConstStructView Payload, const FName Origin)
{
	OnEventSent_AnyThread.Broadcast(ExecutionContext, FEventSentDelegateArgs{ .StateTree = StateTree, .Tag = Tag, .Payload = Payload, .Origin = Origin });
}

void EventConsumed(const FStateTreeExecutionContext& ExecutionContext, const FStateTreeSharedEvent& Event)
{
	OnEventConsumed_AnyThread.Broadcast(ExecutionContext, Event);
}

void StateUtilityEvaluated(const FStateTreeExecutionContext& ExecutionContext, FStateTreeStateHandle StateHandle, float Score)
{
	OnStateUtilityEvaluated_AnyThread.Broadcast(ExecutionContext, StateHandle, Score);
}

}//namespace UE::StateTree::Debug

#endif // WITH_STATETREE_DEBUG
