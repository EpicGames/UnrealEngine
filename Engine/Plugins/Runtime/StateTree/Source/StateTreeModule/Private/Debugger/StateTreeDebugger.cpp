// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_STATETREE_TRACE_DEBUGGER

#include "Debugger/StateTreeDebugger.h"
#include "Common/ProviderLock.h"
#include "Debugger/IStateTreeTraceProvider.h"
#include "Debugger/StateTreeTraceProvider.h"
#include "Debugger/StateTreeTraceTypes.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "IRewindDebugger.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "StateTreeDelegates.h"
#include "StateTreeModule.h"
#include "Trace/Analysis.h"
#include "Trace/Analyzer.h"
#include "Trace/StoreClient.h"
#include "TraceServices/AnalysisService.h"
#include "TraceServices/ITraceServicesModule.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/Diagnostics.h"
#include "TraceServices/Model/Frames.h"

#define LOCTEXT_NAMESPACE "StateTreeDebugger"

//----------------------------------------------------------------//
// UE::StateTreeDebugger
//----------------------------------------------------------------//
namespace UE::StateTreeDebugger
{
	struct FDiagnosticsSessionAnalyzer : UE::Trace::IAnalyzer
	{
		virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override
		{
			auto& Builder = Context.InterfaceBuilder;
			Builder.RouteEvent(RouteId_Session2, "Diagnostics", "Session2");
		}

		virtual bool OnEvent(const uint16 RouteId, EStyle, const FOnEventContext& Context) override
		{
			const FEventData& EventData = Context.EventData;

			switch (RouteId)
			{
			case RouteId_Session2:
				{
					EventData.GetString("Platform", SessionInfo.Platform);
					EventData.GetString("AppName", SessionInfo.AppName);
					EventData.GetString("CommandLine", SessionInfo.CommandLine);
					EventData.GetString("Branch", SessionInfo.Branch);
					EventData.GetString("BuildVersion", SessionInfo.BuildVersion);
					SessionInfo.Changelist = EventData.GetValue<uint32>("Changelist", 0);
					SessionInfo.ConfigurationType = static_cast<EBuildConfiguration>(EventData.GetValue<uint8>("ConfigurationType"));
					SessionInfo.TargetType = static_cast<EBuildTargetType>(EventData.GetValue<uint8>("TargetType"));

					return false;
				}
			default: ;
			}

			return true;
		}

		enum : uint16
		{
			RouteId_Session2,
		};

		TraceServices::FSessionInfo SessionInfo;
	};

} // UE::StateTreeDebugger


//----------------------------------------------------------------//
// FStateTreeDebugger
//----------------------------------------------------------------//
FStateTreeDebugger::FStateTreeDebugger()
	: StateTreeModule(FModuleManager::GetModuleChecked<IStateTreeModule>("StateTreeModule"))
{
	TracingStateChangedHandle = UE::StateTree::Delegates::OnTracingStateChanged.AddLambda([this](const EStateTreeTraceStatus TraceStatus)
		{
			// StateTree traces got enabled in the current process so let's analyse it if not already analysing something.
			if (TraceStatus == EStateTreeTraceStatus::TracesStarted && !IsAnalysisSessionActive())
			{
				RequestAnalysisOfLatestTrace();
			}
		});

	TraceAnalysisStateChangedHandle = UE::StateTree::Delegates::OnTraceAnalysisStateChanged.AddLambda([this](const EStateTreeTraceAnalysisStatus TraceAnalysisStatus)
		{
			if (TraceAnalysisStatus == EStateTreeTraceAnalysisStatus::Cleared
				|| TraceAnalysisStatus == EStateTreeTraceAnalysisStatus::Stopped)
			{
				StopSessionAnalysis();
			}
		});

	TracingTimelineScrubbedHandle = UE::StateTree::Delegates::OnTracingTimelineScrubbed.AddLambda([this](const double InScrubTime)
		{
			SetScrubTime(InScrubTime);
		});
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FStateTreeDebugger::~FStateTreeDebugger()
{
	UE::StateTree::Delegates::OnTracingStateChanged.Remove(TracingStateChangedHandle);
	UE::StateTree::Delegates::OnTraceAnalysisStateChanged.Remove(TraceAnalysisStateChangedHandle);
	UE::StateTree::Delegates::OnTracingTimelineScrubbed.Remove(TracingTimelineScrubbedHandle);

	StopSessionAnalysis();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FStateTreeDebugger::Tick(const float DeltaTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStateTreeDebugger::Tick);

	if (RetryLoadNextLiveSessionTimer > 0.0f)
	{
		// We are still not connected to the last live session.
		// Update polling timer and retry with remaining time; 0 or less will stop retries.
		if (TryStartNewLiveSessionAnalysis(RetryLoadNextLiveSessionTimer - DeltaTime))
		{
			RetryLoadNextLiveSessionTimer = 0.0f;
			LastLiveSessionId = INDEX_NONE;
		}
	}

	// QOL: try to select an instance by default
	if (GetSelectedInstanceId().IsInvalid() && IsAnalysisSessionActive())
	{
		TArray<const TSharedRef<const UE::StateTreeDebugger::FInstanceDescriptor>> InstanceDescriptors;
		GetSessionInstanceDescriptors(InstanceDescriptors);

		for (const TSharedRef<const UE::StateTreeDebugger::FInstanceDescriptor>& InstanceDescriptor : InstanceDescriptors)
		{
			const UE::StateTreeDebugger::FInstanceDescriptor& Descriptor = InstanceDescriptor.Get();
			// Add instance to the list if the asset associated to the view is the instance's main asset, or one of its linked assets.
			if (Descriptor.StateTree != GetAsset()
				&& !Descriptor.LinkedStateTrees.Contains(GetAsset()))
			{
				continue;
			}
			SelectInstance(Descriptor.Id);
		}
	}

	if (StateTreeAsset.IsValid()
		&& IsAnalysisSessionActive()
		&& !IsAnalysisSessionPaused())
	{
		SyncToCurrentSessionDuration();
	}
}

void FStateTreeDebugger::StopSessionAnalysis()
{
	// HitBreakpoint is normally reset when resuming the session analysis, but it is also
	// possible to stop the session analysis while it is paused from a breakpoint.
	// In this case we make sure to reset it before forcing the last update since the breakpoint
	// should no longer be in effect.
	HitBreakpoint.Reset();

	if (IsAnalysisSessionActive())
	{
		// Force one last update to process events emitted while closing the game session (e.g., EndPlay in PIE)
		// Note that we can only perform this if the associated asset is still loaded which might not be the case
		// during the standalone game shutdown.
		if (StateTreeAsset.IsValid())
		{
			SyncToCurrentSessionDuration();
		}

		// We are not responsible of the external analysis so we simply release it, otherwise we stop it.
		if (bIsExternalSessionAnalysis)
		{
			AnalysisSession = nullptr;
		}
		else
		{
			AnalysisSession->Stop(/*WaitOnAnalysis*/true);
		}
	}

	CurrentFrameStateTreeAsset.Reset();
	bIsExternalSessionAnalysis = false;
	bSessionAnalysisActive = false;
	bSessionAnalysisPaused = false;
	LastProcessedRecordedWorldTime = 0;
}

void FStateTreeDebugger::SyncToCurrentSessionDuration()
{
	if (const TraceServices::IAnalysisSession* Session = GetAnalysisSession())
	{
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session);
			AnalysisDuration = Session->GetDurationSeconds();
		}
		ReadTrace(AnalysisDuration);
	}
}

TSharedPtr<const UE::StateTreeDebugger::FInstanceDescriptor> FStateTreeDebugger::GetDescriptor(const FStateTreeInstanceDebugId InstanceId) const
{
	if (const TraceServices::IAnalysisSession* Session = GetAnalysisSession())
	{
		if (const IStateTreeTraceProvider* Provider = Session->ReadProvider<IStateTreeTraceProvider>(FStateTreeTraceProvider::ProviderName))
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*Provider);
			return Provider->GetInstanceDescriptor(InstanceId);
		}
	}

	return nullptr;
}

FText FStateTreeDebugger::GetInstanceName(const FStateTreeInstanceDebugId InstanceId) const
{
	const TSharedPtr<const UE::StateTreeDebugger::FInstanceDescriptor> FoundDescriptor = GetDescriptor(InstanceId);
	return (FoundDescriptor != nullptr) ? FText::FromString(FoundDescriptor->Name) : LOCTEXT("InstanceNotFound","Instance not found");
}

FText FStateTreeDebugger::GetInstanceDescription(const FStateTreeInstanceDebugId InstanceId) const
{
	const TSharedPtr<const UE::StateTreeDebugger::FInstanceDescriptor> FoundDescriptor = GetDescriptor(InstanceId);
	return (FoundDescriptor != nullptr) ? DescribeInstance(*FoundDescriptor) : LOCTEXT("InstanceNotFound","Instance not found");
}

void FStateTreeDebugger::SelectInstance(const FStateTreeInstanceDebugId InstanceId)
{
	if (SelectedInstanceId != InstanceId)
	{
		SelectedInstanceId = InstanceId;

		// The debugger can be associated to a StateTree asset used as a linked StateTree in active instances.
		// Those active instances are available for the user to select and in that case we need to fetch the main
		// StateTree (i.e., associated to the runtime instance) to read the proper event timeline.
		InstanceRootAsset = StateTreeAsset;
		if (const TSharedPtr<const UE::StateTreeDebugger::FInstanceDescriptor> FoundDescriptor = GetDescriptor(SelectedInstanceId))
		{
			if (const UStateTree* FoundStateTree = FoundDescriptor->StateTree.Get())
			{
				InstanceRootAsset = FoundStateTree;
			}
		}

		// Update event collection for newly debugged instance
		SetScrubStateCollection(GetMutableEventCollection(InstanceId));
	}
}

// Deprecated
void FStateTreeDebugger::GetSessionInstances(TArray<UE::StateTreeDebugger::FInstanceDescriptor>& OutInstances) const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (const TraceServices::IAnalysisSession* Session = GetAnalysisSession())
	{
		if (const IStateTreeTraceProvider* Provider = Session->ReadProvider<IStateTreeTraceProvider>(FStateTreeTraceProvider::ProviderName))
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*Provider);
			Provider->GetInstances(OutInstances);
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FStateTreeDebugger::GetSessionInstanceDescriptors(TArray<const TSharedRef<const UE::StateTreeDebugger::FInstanceDescriptor>>& OutInstances) const
{
	if (const TraceServices::IAnalysisSession* Session = GetAnalysisSession())
	{
		if (const IStateTreeTraceProvider* Provider = Session->ReadProvider<IStateTreeTraceProvider>(FStateTreeTraceProvider::ProviderName))
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*Provider);
			Provider->GetInstances(OutInstances);
		}
	}
}

bool FStateTreeDebugger::RequestAnalysisOfEditorSession()
{
	// Get snapshot of current trace to help identify the next live one
	TArray<FTraceDescriptor> TraceDescriptors;
	GetLiveTraces(TraceDescriptors);
	LastLiveSessionId = TraceDescriptors.Num() ? TraceDescriptors.Last().TraceId : INDEX_NONE;

	// 0 is the invalid value used for Trace Id
	constexpr int32 InvalidTraceId = 0;
	int32 ActiveTraceId = InvalidTraceId;

	// StartTraces returns true if a new connection was created.
	// In this case we will receive OnTracingStateChanged and try to start an analysis on that new connection as soon as possible.
	// Otherwise, it might have been able to use an active connection in which case it was returned in the output parameter.
	if (StateTreeModule.StartTraces(ActiveTraceId))
	{
		return true;
	}

	// Otherwise we start analysis of the already active trace, if any.
	if (ActiveTraceId != InvalidTraceId)
	{
		if (const FTraceDescriptor* Descriptor = TraceDescriptors.FindByPredicate([ActiveTraceId](const FTraceDescriptor& Descriptor)
			{
				return Descriptor.TraceId == ActiveTraceId;
			}))
		{
			return RequestSessionAnalysis(*Descriptor);
		}
	}

	return false;
}

void FStateTreeDebugger::RequestAnalysisOfLatestTrace()
{
	// Invalidate our current active session
	ActiveSessionTraceDescriptor = FTraceDescriptor();

	// Invalidate current selected instance so breakpoint can be hit by any instances in the next analysis
	ClearSelection();

	// Stop current analysis if any
	StopSessionAnalysis();

	// This might not succeed immediately but will schedule next retry if necessary
	TryStartNewLiveSessionAnalysis(1.0f);
}


bool FStateTreeDebugger::TryStartNewLiveSessionAnalysis(const float RetryPollingDuration)
{
	// RewindDebugger manages its session analysis without using the unreal insights module
	//  so we need to use its analysis session first.
	if (const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance())
	{
		if (const TSharedPtr<const TraceServices::IAnalysisSession> ExternalSession = RewindDebugger->GetAnalysisSessionAsShared())
		{
			AnalysisSession = ExternalSession;
			bIsExternalSessionAnalysis = true;
			bSessionAnalysisActive = true;
			AnalysisDuration = 0;
			LastTraceReadTime = 0;
			return true;
		}
	}

	TArray<FTraceDescriptor> Traces;
	GetLiveTraces(Traces);

	if (Traces.Num() && Traces.Last().TraceId != LastLiveSessionId)
	{
		// Intentional call to StartSessionAnalysis instead of RequestSessionAnalysis since we want
		// to set 'bIsAnalyzingNextEditorSession' before calling OnNewSession delegate.
		const bool bStarted = StartSessionAnalysis(Traces.Last());
		if (bStarted)
		{
			UpdateAnalysisTransitionType(EAnalysisSourceType::EditorSession);

			SetScrubStateCollection(nullptr);
			OnNewSession.ExecuteIfBound();
		}

		return bStarted;
	}

	RetryLoadNextLiveSessionTimer = RetryPollingDuration;
	UE_CLOGF(RetryLoadNextLiveSessionTimer > 0, LogStateTree, Log, "Unable to start analysis for the most recent live session.");

	return false;
}

bool FStateTreeDebugger::StartSessionAnalysis(const FTraceDescriptor& TraceDescriptor)
{
	if (ActiveSessionTraceDescriptor == TraceDescriptor)
	{
		return ActiveSessionTraceDescriptor.IsValid();
	}

	ActiveSessionTraceDescriptor = FTraceDescriptor();

	// Make sure any active analysis is stopped
	StopSessionAnalysis();

	UE::Trace::FStoreClient* StoreClient = GetStoreClient();
	if (StoreClient == nullptr)
	{
		return false;
	}

	// If new trace descriptor is not valid no need to continue
	if (TraceDescriptor.IsValid() == false)
	{
		return false;
	}

	AnalysisDuration = 0;
	LastTraceReadTime = 0;

	const uint32 TraceId = TraceDescriptor.TraceId;

	// Make sure it is still live
	const UE::Trace::FStoreClient::FSessionInfo* SessionInfo = StoreClient->GetSessionInfoByTraceId(TraceId);
	if (SessionInfo != nullptr)
	{
		UE::Trace::FStoreClient::FTraceData TraceData = StoreClient->ReadTrace(TraceId);
		if (!TraceData)
		{
			return false;
		}

		FString TraceName(StoreClient->GetStatus()->GetStoreDir());
		const UE::Trace::FStoreClient::FTraceInfo* TraceInfo = StoreClient->GetTraceInfoById(TraceId);
		if (TraceInfo != nullptr)
		{
			FString Name(TraceInfo->GetName());
			if (!Name.EndsWith(TEXT(".utrace")))
			{
				Name += TEXT(".utrace");
			}
			TraceName = FPaths::Combine(TraceName, Name);
			FPaths::NormalizeFilename(TraceName);
		}

		ITraceServicesModule& TraceServicesModule = FModuleManager::LoadModuleChecked<ITraceServicesModule>("TraceServices");
		if (const TSharedPtr<TraceServices::IAnalysisService> TraceAnalysisService = TraceServicesModule.GetAnalysisService())
		{
			checkf(!IsAnalysisSessionActive(), TEXT("Must make sure that current session was properly stopped before starting a new one otherwise it can cause threading issues"));
			AnalysisSession = TraceAnalysisService->StartAnalysis(TraceId, *TraceName, MoveTemp(TraceData));
		}

		if (AnalysisSession.IsValid())
		{
			bSessionAnalysisActive = true;
			ActiveSessionTraceDescriptor = TraceDescriptor;
		}
	}

	return ActiveSessionTraceDescriptor.IsValid();
}

void FStateTreeDebugger::SetScrubStateCollection(const UE::StateTreeDebugger::FInstanceEventCollection* Collection)
{
	ScrubState.SetEventCollection(Collection);

	OnScrubStateChanged.ExecuteIfBound(ScrubState);

	RefreshActiveStates();
}

void FStateTreeDebugger::GetLiveTraces(TArray<FTraceDescriptor>& OutTraceDescriptors) const
{
	UE::Trace::FStoreClient* StoreClient = GetStoreClient();
	if (StoreClient == nullptr)
	{
		return;
	}

	OutTraceDescriptors.Reset();

	const uint32 SessionCount = StoreClient->GetSessionCount();
	for (uint32 SessionIndex = 0; SessionIndex < SessionCount; ++SessionIndex)
	{
		const UE::Trace::FStoreClient::FSessionInfo* SessionInfo = StoreClient->GetSessionInfo(SessionIndex);
		if (SessionInfo != nullptr)
		{
			const uint32 TraceId = SessionInfo->GetTraceId();
			const UE::Trace::FStoreClient::FTraceInfo* TraceInfo = StoreClient->GetTraceInfoById(TraceId);
			if (TraceInfo != nullptr)
			{
				FTraceDescriptor& Trace = OutTraceDescriptors.AddDefaulted_GetRef();
				Trace.TraceId = TraceId;
				Trace.Name = FString(TraceInfo->GetName());
				UpdateMetadata(Trace);
			}
		}
	}
}

void FStateTreeDebugger::UpdateMetadata(FTraceDescriptor& TraceDescriptor) const
{
	UE::Trace::FStoreClient* StoreClient = GetStoreClient();
	if (StoreClient == nullptr)
	{
		return;
	}

	const UE::Trace::FStoreClient::FTraceData TraceData = StoreClient->ReadTrace(TraceDescriptor.TraceId);
	if (!TraceData)
	{
		return;
	}

	// inspired from FStoreBrowser
	struct FDataStream : UE::Trace::IInDataStream
	{
		enum class EReadStatus
		{
			Ready = 0,
			StoppedByReadSizeLimit
		};

		virtual int32 Read(void* Data, const uint32 Size) override
		{
			if (BytesRead >= 1024 * 1024)
			{
				Status = EReadStatus::StoppedByReadSizeLimit;
				return 0;
			}
			const int32 InnerBytesRead = Inner->Read(Data, Size);
			BytesRead += InnerBytesRead;

			return InnerBytesRead;
		}

		virtual void Close() override
		{
			Inner->Close();
		}

		IInDataStream* Inner = nullptr;
		int32 BytesRead = 0;
		EReadStatus Status = EReadStatus::Ready;
	};

	FDataStream DataStream;
	DataStream.Inner = TraceData.Get();

	UE::StateTreeDebugger::FDiagnosticsSessionAnalyzer Analyzer;
	UE::Trace::FAnalysisContext Context;
	Context.AddAnalyzer(Analyzer);
	Context.Process(DataStream).Wait();

	TraceDescriptor.SessionInfo = Analyzer.SessionInfo;
}

FText FStateTreeDebugger::GetSelectedTraceDescription() const
{
	if (ActiveSessionTraceDescriptor.IsValid())
	{
		return DescribeTrace(ActiveSessionTraceDescriptor);
	}

	return LOCTEXT("NoSelectedTraceDescriptor", "No trace selected");
}

void FStateTreeDebugger::SetScrubTime(const double ScrubTime)
{
	if (ScrubState.SetScrubTime(ScrubTime))
	{
		OnScrubStateChanged.ExecuteIfBound(ScrubState);

		RefreshActiveStates();
	}
}

bool FStateTreeDebugger::IsActiveInstance(const double Time, const FStateTreeInstanceDebugId InstanceId) const
{
	if (const TraceServices::IAnalysisSession* Session = GetAnalysisSession())
	{
		if (const IStateTreeTraceProvider* Provider = Session->ReadProvider<IStateTreeTraceProvider>(FStateTreeTraceProvider::ProviderName))
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*Provider);
			const TSharedPtr<const UE::StateTreeDebugger::FInstanceDescriptor> Descriptor = Provider->GetInstanceDescriptor(InstanceId);
			return Descriptor.IsValid() && Descriptor->Lifetime.Contains(Time);
		}
	}
	return false;
}

FText FStateTreeDebugger::DescribeTrace(const FTraceDescriptor& TraceDescriptor)
{
	if (TraceDescriptor.IsValid())
	{
		const TraceServices::FSessionInfo& SessionInfo = TraceDescriptor.SessionInfo;

		return FText::FromString(FString::Printf(TEXT("%s-%s-%s-%s-%s"),
			*LexToString(TraceDescriptor.TraceId),
			*SessionInfo.Platform,
			*SessionInfo.AppName,
			LexToString(SessionInfo.ConfigurationType),
			LexToString(SessionInfo.TargetType)));
	}

	return LOCTEXT("InvalidTraceDescriptor", "Invalid");
}

FText FStateTreeDebugger::DescribeInstance(const UE::StateTreeDebugger::FInstanceDescriptor& InstanceDesc)
{
	if (InstanceDesc.IsValid() == false)
	{
		return LOCTEXT("NoSelectedInstanceDescriptor", "No instance selected");
	}
	return FText::FromString(LexToString(InstanceDesc));
}

void FStateTreeDebugger::SetActiveStates(const FStateTreeTraceActiveStates& NewActiveStates)
{
	ActiveStates = NewActiveStates;
	OnActiveStatesChanged.ExecuteIfBound(ActiveStates);
}

void FStateTreeDebugger::RefreshActiveStates()
{
	if (ScrubState.IsPointingToValidActiveStates())
	{
		const UE::StateTreeDebugger::FInstanceEventCollection& EventCollection = ScrubState.GetEventCollection();
		const int32 EventIndex = EventCollection.ActiveStatesChanges[ScrubState.GetActiveStatesIndex()].EventIndex;
		SetActiveStates(EventCollection.Events[EventIndex].Get<FStateTreeTraceActiveStatesEvent>().ActiveStates);
	}
	else
	{
		SetActiveStates(FStateTreeTraceActiveStates());
	}
}

bool FStateTreeDebugger::CanStepBackToPreviousStateWithEvents() const
{
	return ScrubState.HasPreviousFrame();
}

void FStateTreeDebugger::StepBackToPreviousStateWithEvents()
{
	ScrubState.GotoPreviousFrame();
	OnScrubStateChanged.Execute(ScrubState);

	RefreshActiveStates();
}

bool FStateTreeDebugger::CanStepForwardToNextStateWithEvents() const
{
	return ScrubState.HasNextFrame();
}

void FStateTreeDebugger::StepForwardToNextStateWithEvents()
{
	ScrubState.GotoNextFrame();
	OnScrubStateChanged.Execute(ScrubState);

	RefreshActiveStates();
}

bool FStateTreeDebugger::CanStepBackToPreviousStateChange() const
{
	return ScrubState.HasPreviousActiveStates();
}

void FStateTreeDebugger::StepBackToPreviousStateChange()
{
	ScrubState.GotoPreviousActiveStates();
	OnScrubStateChanged.Execute(ScrubState);

	RefreshActiveStates();
}

bool FStateTreeDebugger::CanStepForwardToNextStateChange() const
{
	return ScrubState.HasNextActiveStates();
}

void FStateTreeDebugger::StepForwardToNextStateChange()
{
	ScrubState.GotoNextActiveStates();
	OnScrubStateChanged.Execute(ScrubState);

	RefreshActiveStates();
}

bool FStateTreeDebugger::HasStateBreakpoint(const FStateTreeStateHandle StateHandle, const EStateTreeBreakpointType BreakpointType) const
{
	return Breakpoints.ContainsByPredicate([StateHandle, BreakpointType](const FStateTreeDebuggerBreakpoint& Breakpoint)
		{
			if (Breakpoint.BreakpointType == BreakpointType)
			{
				const FStateTreeStateHandle* BreakpointStateHandle = Breakpoint.ElementIdentifier.TryGet<FStateTreeStateHandle>();
				return (BreakpointStateHandle != nullptr && *BreakpointStateHandle == StateHandle);
			}
			return false;
		});
}

bool FStateTreeDebugger::HasTaskBreakpoint(const FStateTreeIndex16 Index, const EStateTreeBreakpointType BreakpointType) const
{
	return Breakpoints.ContainsByPredicate([Index, BreakpointType](const FStateTreeDebuggerBreakpoint& Breakpoint)
	{
		if (Breakpoint.BreakpointType == BreakpointType)
		{
			const FStateTreeDebuggerBreakpoint::FStateTreeTaskIndex* BreakpointTaskIndex = Breakpoint.ElementIdentifier.TryGet<FStateTreeDebuggerBreakpoint::FStateTreeTaskIndex>();
			return (BreakpointTaskIndex != nullptr && BreakpointTaskIndex->Index == Index);
		}
		return false;
	});
}

bool FStateTreeDebugger::HasTransitionBreakpoint(const FStateTreeIndex16 Index, const EStateTreeBreakpointType BreakpointType) const
{
	return Breakpoints.ContainsByPredicate([Index, BreakpointType](const FStateTreeDebuggerBreakpoint& Breakpoint)
	{
		if (Breakpoint.BreakpointType == BreakpointType)
		{
			const FStateTreeDebuggerBreakpoint::FStateTreeTransitionIndex* BreakpointTransitionIndex = Breakpoint.ElementIdentifier.TryGet<FStateTreeDebuggerBreakpoint::FStateTreeTransitionIndex>();
			return (BreakpointTransitionIndex != nullptr && BreakpointTransitionIndex->Index == Index);
		}
		return false;
	});
}

void FStateTreeDebugger::SetStateBreakpoint(const FStateTreeStateHandle StateHandle, const EStateTreeBreakpointType BreakpointType)
{
	Breakpoints.Emplace(StateHandle, BreakpointType);
}

void FStateTreeDebugger::SetTransitionBreakpoint(const FStateTreeIndex16 TransitionIndex, const EStateTreeBreakpointType BreakpointType)
{
	Breakpoints.Emplace(FStateTreeDebuggerBreakpoint::FStateTreeTransitionIndex(TransitionIndex), BreakpointType);
}

void FStateTreeDebugger::SetTaskBreakpoint(const FStateTreeIndex16 NodeIndex, const EStateTreeBreakpointType BreakpointType)
{
	Breakpoints.Emplace(FStateTreeDebuggerBreakpoint::FStateTreeTaskIndex(NodeIndex), BreakpointType);
}

void FStateTreeDebugger::ClearBreakpoint(const FStateTreeIndex16 NodeIndex, const EStateTreeBreakpointType BreakpointType)
{
	const int32 Index = Breakpoints.IndexOfByPredicate([NodeIndex, BreakpointType](const FStateTreeDebuggerBreakpoint& Breakpoint)
		{
			const FStateTreeDebuggerBreakpoint::FStateTreeTaskIndex* IndexPtr = Breakpoint.ElementIdentifier.TryGet<FStateTreeDebuggerBreakpoint::FStateTreeTaskIndex>();
			return (IndexPtr != nullptr && IndexPtr->Index == NodeIndex && Breakpoint.BreakpointType == BreakpointType);
		});

	if (Index != INDEX_NONE)
	{
		Breakpoints.RemoveAtSwap(Index);
	}
}

void FStateTreeDebugger::ClearAllBreakpoints()
{
	Breakpoints.Empty();
}

const TraceServices::IAnalysisSession* FStateTreeDebugger::GetAnalysisSession() const
{
	return AnalysisSession.Get();
}

bool FStateTreeDebugger::RequestSessionAnalysis(const FTraceDescriptor& TraceDescriptor)
{
	if (StartSessionAnalysis(TraceDescriptor))
	{
		UpdateAnalysisTransitionType(EAnalysisSourceType::SelectedSession);

		SetScrubStateCollection(nullptr);
		OnNewSession.ExecuteIfBound();
		return true;
	}
	return false;
}

void FStateTreeDebugger::UpdateAnalysisTransitionType(const EAnalysisSourceType SourceType)
{
	switch (AnalysisTransitionType)
	{
	case EAnalysisTransitionType::Unset:
		AnalysisTransitionType = (SourceType == EAnalysisSourceType::SelectedSession)
				? EAnalysisTransitionType::NoneToSelected
				: EAnalysisTransitionType::NoneToEditor;
		break;

	case EAnalysisTransitionType::NoneToSelected:
	case EAnalysisTransitionType::EditorToSelected:
	case EAnalysisTransitionType::SelectedToSelected:
		AnalysisTransitionType = (SourceType == EAnalysisSourceType::SelectedSession)
				? EAnalysisTransitionType::SelectedToSelected
				: EAnalysisTransitionType::SelectedToEditor;
		break;

	case EAnalysisTransitionType::NoneToEditor:
	case EAnalysisTransitionType::EditorToEditor:
	case EAnalysisTransitionType::SelectedToEditor:
		AnalysisTransitionType = (SourceType == EAnalysisSourceType::SelectedSession)
				? EAnalysisTransitionType::EditorToSelected
				: EAnalysisTransitionType::EditorToEditor;
		break;
	default:
		ensureMsgf(false, TEXT("Unhandled transition type."));
	}
}

UE::Trace::FStoreClient* FStateTreeDebugger::GetStoreClient() const
{
	return StateTreeModule.GetStoreClient();
}

void FStateTreeDebugger::ReadTrace(const uint64 FrameIndex)
{
	if (const TraceServices::IAnalysisSession* Session = GetAnalysisSession())
	{
		// UE::StateTreeDebugger::ReadTrace also uses the FrameProvider so we scope the lock for the whole operation
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session);

		const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(*Session);

		if (const TraceServices::FFrame* TargetFrame = FrameProvider.GetFrame(TraceFrameType_Game, FrameIndex))
		{
			UE::StateTreeDebugger::ReadTrace(*Session
				, FrameProvider
				, *TargetFrame
				, /*ITraceReader*/this
				, UE::StateTreeDebugger::FTraceFilter(InstanceRootAsset.Get())
				, LastTraceReadTime);
		}
	}

	// Notify outside session read scope
	SendNotifications();
}

void FStateTreeDebugger::ReadTrace(const double ScrubTime)
{
	if (const TraceServices::IAnalysisSession* Session = GetAnalysisSession())
	{
		UE::StateTreeDebugger::ReadTrace(
			*Session
			, ScrubTime
			, this
			, UE::StateTreeDebugger::FTraceFilter(InstanceRootAsset.Get())
			, LastTraceReadTime);

		SendNotifications();
	}
}

void FStateTreeDebugger::SendNotifications()
{
	if (NewInstances.Num() > 0)
	{
		for (const FStateTreeInstanceDebugId NewInstanceId : NewInstances)
		{
			OnNewInstance.ExecuteIfBound(NewInstanceId);
		}
		NewInstances.Reset();
	}

	if (HitBreakpoint.IsSet())
	{
		check(HitBreakpoint.InstanceId.IsValid());
		check(Breakpoints.IsValidIndex(HitBreakpoint.Index));

		// Force scrub time to latest simulation time to reflect most recent events.
		// This will notify scrub position changed and active states
		SetScrubTime(HitBreakpoint.Time);

		// Make sure the instance is selected in case the breakpoint was set for any instances
		if (SelectedInstanceId != HitBreakpoint.InstanceId)
		{
			SelectInstance(HitBreakpoint.InstanceId);
		}

		OnBreakpointHit.ExecuteIfBound(HitBreakpoint.InstanceId, Breakpoints[HitBreakpoint.Index]);

		PauseSessionAnalysis();
	}
}

bool FStateTreeDebugger::EvaluateBreakpoints(const FStateTreeInstanceDebugId InstanceId, const FStateTreeTraceEventVariantType& Event)
{
	// Keep track of the StateTree associated to the last frame update so breakpoints are tested against the right asset.
	if (const FStateTreeTraceInstanceFrameEvent* FrameEvent = Event.TryGet<FStateTreeTraceInstanceFrameEvent>())
	{
		CurrentFrameStateTreeAsset = FrameEvent->WeakStateTree;
	}

	if (StateTreeAsset == nullptr // asset is required to properly match state handles
		|| CurrentFrameStateTreeAsset != StateTreeAsset // ignore events from a different asset (required for linked statetree)
		|| HitBreakpoint.IsSet() // only consider first hit breakpoint in the frame
		|| Breakpoints.IsEmpty()
		|| (SelectedInstanceId.IsValid() && InstanceId != SelectedInstanceId)) // ignore events not for the selected instances
	{
		return false;
	}

	for (int BreakpointIndex = 0; BreakpointIndex < Breakpoints.Num(); ++BreakpointIndex)
	{
		const FStateTreeDebuggerBreakpoint Breakpoint = Breakpoints[BreakpointIndex];
		if (Breakpoint.IsMatchingEvent(Event))
		{
			HitBreakpoint.Index = BreakpointIndex;
			HitBreakpoint.InstanceId = InstanceId;
			HitBreakpoint.Time = LastProcessedRecordedWorldTime;
		}
	}

	return HitBreakpoint.IsSet();
}

UE::StateTreeDebugger::FInstanceEventCollection* FStateTreeDebugger::GetOrCreateEventCollection(FStateTreeInstanceDebugId InstanceId)
{
	UE::StateTreeDebugger::FInstanceEventCollection* ExistingCollection = GetMutableEventCollection(InstanceId);

	// Create missing EventCollection if necessary
	if (ExistingCollection == nullptr)
	{
		// Push deferred notification for new instance Id
		NewInstances.Push(InstanceId);

		ExistingCollection = new UE::StateTreeDebugger::FInstanceEventCollection(InstanceId);
		EventCollections.Add(ExistingCollection);

		// Update the active event collection when it's newly created for the currently debugged instance.
		// Otherwise (i.e. EventCollection already exists) it is updated when switching instance (i.e. SelectInstance)
		if (SelectedInstanceId == InstanceId && ScrubState.GetEventCollection().IsInvalid())
		{
			SetScrubStateCollection(ExistingCollection);
		}
	}
	return ExistingCollection;
}

void FStateTreeDebugger::OnTraceEventProcessed(const FStateTreeInstanceDebugId InstanceId, const FStateTreeTraceEventVariantType& Event)
{
	Visit([&WorldTime = LastProcessedRecordedWorldTime](auto& TypedEvent)
		{
			WorldTime = TypedEvent.RecordingWorldTime;
		}, Event);

	EvaluateBreakpoints(InstanceId, Event);
}

const UE::StateTreeDebugger::FInstanceEventCollection& FStateTreeDebugger::GetEventCollection(const FStateTreeInstanceDebugId InstanceId) const
{
	for (const UE::StateTreeDebugger::FInstanceEventCollection& EventCollection : EventCollections)
	{
		if (EventCollection.InstanceId == InstanceId)
		{
			return EventCollection;
		}
	}

	return UE::StateTreeDebugger::FInstanceEventCollection::Invalid;
}

UE::StateTreeDebugger::FInstanceEventCollection* FStateTreeDebugger::GetMutableEventCollection(const FStateTreeInstanceDebugId InstanceId)
{
	for (UE::StateTreeDebugger::FInstanceEventCollection& EventCollection : EventCollections)
	{
		if (EventCollection.InstanceId == InstanceId)
		{
			// Note that returning the pointer to the EventCollection element here is fine.
			// Using a TDereferencingIterator on a TIndirectArray returns us the actual collection instance.
			return &EventCollection;
		}
	}

	return nullptr;
}

void FStateTreeDebugger::ResetEventCollections()
{
	EventCollections.Reset();
	SetScrubStateCollection(nullptr);
	LastProcessedRecordedWorldTime = 0;
}

//----------------------------------------------------------------//
// UE::StateTreeDebugger
//----------------------------------------------------------------//
namespace UE::StateTreeDebugger
{

bool ProcessEvent(
	FInstanceEventCollection& InEventCollection,
	const TraceServices::FFrame& InFrame,
	const FStateTreeTraceEventVariantType& InEvent,
	TNotNull<ITraceReader*> TraceReader)
{
	TArray<FStateTreeTraceEventVariantType>& Events = InEventCollection.Events;

	TraceServices::FFrame FrameToAddInSpans = InFrame;
	bool bShouldAddFrameToSpans = false;

	// Add new frame span if none added yet
	if (InEventCollection.FrameSpans.IsEmpty())
	{
		bShouldAddFrameToSpans = true;
	}
	else
	{
		const FFrameSpan& LastSpan = InEventCollection.FrameSpans.Last();
		const TraceServices::FFrame& LastFrame = LastSpan.Frame;
		const uint64 FrameIndexOffset = InEventCollection.ContiguousTracesData.IsEmpty()
			? 0
			: (InEventCollection.FrameSpans[InEventCollection.ContiguousTracesData.Last().LastSpanIndex].Frame.Index + 1);

		// Add new frame span for new larger frame index
		if (InFrame.Index + FrameIndexOffset > LastFrame.Index)
		{
			bShouldAddFrameToSpans = true;

			// Apply current offset to the frame index
			FrameToAddInSpans.Index += FrameIndexOffset;
		}
		else if (InFrame.Index < LastFrame.Index && InFrame.StartTime > LastFrame.StartTime)
		{
			// Frame index will restart at 0 if a new session is started,
			// in that case we offset the frame we store to append to existing data
			bShouldAddFrameToSpans = true;

			const FInstanceEventCollection::FContiguousTraceInfo& TraceInfo =
				InEventCollection.ContiguousTracesData.Emplace_GetRef(
					FInstanceEventCollection::FContiguousTraceInfo(InEventCollection.FrameSpans.Num() - 1));
			FrameToAddInSpans.Index += InEventCollection.FrameSpans[TraceInfo.LastSpanIndex].Frame.Index + 1;
		}
	}

	if (bShouldAddFrameToSpans)
	{
		double RecordingWorldTime = 0;
		Visit([&RecordingWorldTime](auto& TypedEvent)
			{
				RecordingWorldTime = TypedEvent.RecordingWorldTime;
			}, InEvent);

		InEventCollection.FrameSpans.Add(FFrameSpan(FrameToAddInSpans, RecordingWorldTime, Events.Num()));
	}

	// Add activate states change info
	if (InEvent.IsType<FStateTreeTraceActiveStatesEvent>())
	{
		checkf(InEventCollection.FrameSpans.Num() > 0, TEXT("Expecting to always be in a frame span at this point."));
		const int32 FrameSpanIndex = InEventCollection.FrameSpans.Num() - 1;

		// Add new entry for the first event or if the last event is for a different frame
		if (InEventCollection.ActiveStatesChanges.IsEmpty()
			|| InEventCollection.ActiveStatesChanges.Last().SpanIndex != FrameSpanIndex)
		{
			InEventCollection.ActiveStatesChanges.Push({ FrameSpanIndex, Events.Num() });
		}
		else
		{
			// Multiple events for change of active states in the same frame, keep the last one until we implement scrubbing within a frame
			InEventCollection.ActiveStatesChanges.Last().EventIndex = Events.Num();
		}
	}

	// Store event in the collection
	Events.Emplace(InEvent);

	// Notify the reader
	TraceReader->OnTraceEventProcessed(InEventCollection.InstanceId, InEvent);

	return /*bKeepProcessing*/true;
}

void AddEvents(
	const double InStartTime,
	const double InEndTime,
	const TraceServices::IAnalysisSession& Session,
	const TraceServices::IFrameProvider& InFrameProvider,
	const FStateTreeInstanceDebugId InInstanceId,
	const IStateTreeTraceProvider::FEventsTimeline& TimelineData,
	TNotNull<ITraceReader*> TraceReader)
{
	FInstanceEventCollection* EventCollection = TraceReader->GetOrCreateEventCollection(InInstanceId);
	if (EventCollection == nullptr)
	{
		return;
	}

	// Keep track of the frames containing events. Starting with an invalid frame.
	TraceServices::FFrame Frame;
	Frame.Index = INDEX_NONE;

	TimelineData.EnumerateEvents(InStartTime, InEndTime,
		[&InFrameProvider, &Frame, EventCollection, TraceReader, &Session](const double EventStartTime, const double EventEndTime, uint32 InDepth, const FStateTreeTraceEventVariantType& Event)
		{
			bool bValidFrame = true;

			// Fetch frame when not set yet or if events no longer part of the current one
			if (Frame.Index == INDEX_NONE ||
				(EventEndTime < Frame.StartTime || Frame.EndTime < EventStartTime))
			{
				TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);
				bValidFrame = InFrameProvider.GetFrameFromTime(TraceFrameType_Game, EventStartTime, Frame);

				if (bValidFrame == false)
				{
					// Edge case for events from a missing first complete frame.
					// (i.e. FrameProvider didn't get BeginFrame event but StateTreeEvent were sent in that frame)
					// Doing this will merge our two first frames of state tree events using the same recording world time
					// but this should happen only for late start recording.
					const TraceServices::FFrame* FirstFrame = InFrameProvider.GetFrame(TraceFrameType_Game, 0);
					if (FirstFrame != nullptr && EventEndTime < FirstFrame->StartTime)
					{
						Frame = *FirstFrame;
						bValidFrame = true;
					}
				}
			}

			if (bValidFrame)
			{
				const bool bKeepProcessing = ProcessEvent(*EventCollection, Frame, Event, TraceReader);
				return bKeepProcessing ? TraceServices::EEventEnumerate::Continue : TraceServices::EEventEnumerate::Stop;
			}

			// Skip events outside of game frames
			return TraceServices::EEventEnumerate::Continue;
		});
}

void ReadTrace(
	const TraceServices::IAnalysisSession& Session,
	const double ScrubTime,
	const TNotNull<ITraceReader*> TraceReader,
	const FTraceFilter& Filter,
	double& OutLastTraceReadTime)
{
	TraceServices::FFrame TargetFrame;
	bool bValidFrame = true;
	const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(Session);

	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);
		if (FrameProvider.GetFrameFromTime(TraceFrameType_Game, ScrubTime, TargetFrame))
		{
			// Process only completed frames

			if (TargetFrame.EndTime == std::numeric_limits<double>::infinity())
			{
				if (const TraceServices::FFrame* PreviousCompleteFrame = FrameProvider.GetFrame(TraceFrameType_Game, TargetFrame.Index - 1))
				{
					TargetFrame = *PreviousCompleteFrame;
				}
				else
				{
					bValidFrame = false;
				}
			}
		}
	}

	if (bValidFrame)
	{
		ReadTrace(Session, FrameProvider, TargetFrame, TraceReader, Filter, OutLastTraceReadTime);
	}
}

void ReadTrace(
	const TraceServices::IAnalysisSession& Session,
	const TraceServices::IFrameProvider& FrameProvider,
	const TraceServices::FFrame& Frame,
	TNotNull<ITraceReader*> TraceReader,
	const FTraceFilter& Filter,
	double& OutLastTraceReadTime)
{
	bool bValidLastReadFrame;
	TraceServices::FFrame LastReadFrame;
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);
		bValidLastReadFrame = FrameProvider.GetFrameFromTime(TraceFrameType_Game, OutLastTraceReadTime, LastReadFrame);
	}

	// Last time is invalid (first time processing the frame, or the buffer was cleared), and it's a different frame (scrubbing forward).
	if (!bValidLastReadFrame || Frame.Index > LastReadFrame.Index)
	{
		if (const IStateTreeTraceProvider* Provider = Session.ReadProvider<IStateTreeTraceProvider>(FStateTreeTraceProvider::ProviderName))
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*Provider);
			if (Filter.Instance.IsValid())
			{
				Provider->ReadTimelines(Filter.Instance,
					[StartTime = OutLastTraceReadTime, EndTime = Frame.EndTime, &FrameProvider, TraceReader, &Session](const FStateTreeInstanceDebugId InstanceId, const IStateTreeTraceProvider::FEventsTimeline& TimelineData)
					{
						AddEvents(StartTime, EndTime, Session, FrameProvider, InstanceId, TimelineData, TraceReader);
					});
			}
			else if (Filter.Asset)
			{
				Provider->ReadTimelines(*Filter.Asset,
					[StartTime = OutLastTraceReadTime, EndTime = Frame.EndTime, &FrameProvider, TraceReader, &Session](const FStateTreeInstanceDebugId InstanceId, const IStateTreeTraceProvider::FEventsTimeline& TimelineData)
					{
						AddEvents(StartTime, EndTime, Session, FrameProvider, InstanceId, TimelineData, TraceReader);
					});
			}

			OutLastTraceReadTime = Frame.EndTime;
		}
	}
}

} // UE::StateTreeDebugger

#undef LOCTEXT_NAMESPACE

#endif // WITH_STATETREE_TRACE_DEBUGGER
