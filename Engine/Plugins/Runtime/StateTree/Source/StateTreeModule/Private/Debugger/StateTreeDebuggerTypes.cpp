// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debugger/StateTreeDebuggerTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeDebuggerTypes)

#if WITH_STATETREE_TRACE_DEBUGGER

namespace UE::StateTreeDebugger
{

//----------------------------------------------------------------//
// FInstanceDescriptor
//----------------------------------------------------------------//
FInstanceDescriptor::FInstanceDescriptor(const UStateTree* InStateTree, const FStateTreeInstanceDebugId InId, const FString& InName, const TRange<double>& InLifetime)
	: Lifetime(InLifetime)
	, StateTree(InStateTree)
	, Name(InName)
	, Id(InId)
{
}

bool FInstanceDescriptor::IsValid() const
{
	return StateTree.IsValid() && Name.Len() && Id.IsValid();
}


//----------------------------------------------------------------//
// FInstanceEventCollection
//----------------------------------------------------------------//
const FInstanceEventCollection FInstanceEventCollection::Invalid;


//----------------------------------------------------------------//
// FScrubState
//----------------------------------------------------------------//
bool FScrubState::SetScrubTime(const double NewScrubTime, const bool bForceRefresh)
{
	if (!bForceRefresh
		&& NewScrubTime == ScrubTime)
	{
		return false;
	}

	ScrubTimeBoundState = EScrubTimeBoundState::Unset;
	TraceFrameIndex = INDEX_NONE;
	FrameSpanIndex = INDEX_NONE;
	ActiveStatesIndex = INDEX_NONE;

	if (EventCollection)
	{
		const TArray<FFrameSpan>& Spans = EventCollection->FrameSpans;
		if (Spans.Num() > 0)
		{
			const double SpansLowerBound = Spans[0].GetWorldTimeStart();
			const double SpansUpperBound =  Spans.Last().GetWorldTimeEnd();
			
			if (NewScrubTime < SpansLowerBound)
			{
				ScrubTimeBoundState = EScrubTimeBoundState::BeforeLowerBound;
			}
			else if (NewScrubTime > SpansUpperBound)
			{
				ScrubTimeBoundState = EScrubTimeBoundState::AfterHigherBound;
				UpdateActiveStatesIndex(Spans.Num() - 1);
			}
			else
			{
				const uint32 NextFrameSpanIndex = Spans.IndexOfByPredicate([NewScrubTime](const FFrameSpan& Span)
					{
						return Span.GetWorldTimeStart() > NewScrubTime;
					});

				
				ensure(NextFrameSpanIndex == INDEX_NONE || NextFrameSpanIndex > 0);
				SetFrameSpanIndex(NextFrameSpanIndex != INDEX_NONE ? NextFrameSpanIndex-1 : Spans.Num() - 1);
			}
		}
	}

	// This will set back to the exact value provided since SetFrameSpanIndex will snap it to the start time of the matching frame.
	// It will be consistent with the case where EventCollectionIndex is not set.
	ScrubTime = NewScrubTime;

	return true;
}

void FScrubState::SetEventCollection(const FInstanceEventCollection* InEventCollection)
{
	EventCollection = InEventCollection;

	// Force refresh of internal indices with current time applied on the new event collection.
	constexpr bool bForceRefresh = true;
	SetScrubTime(ScrubTime, bForceRefresh);
}

void FScrubState::SetFrameSpanIndex(const int32 NewFrameSpanIndex)
{
	FrameSpanIndex = NewFrameSpanIndex;
	checkf(EventCollection != nullptr, TEXT("Internal method expecting validity checks before getting called."));
	checkf(EventCollection->FrameSpans.IsValidIndex(FrameSpanIndex), TEXT("Internal method expecting validity checks before getting called."));

	ScrubTime = EventCollection->FrameSpans[FrameSpanIndex].GetWorldTimeStart();
	TraceFrameIndex = EventCollection->FrameSpans[FrameSpanIndex].Frame.Index;
	ScrubTimeBoundState = EScrubTimeBoundState::InBounds;
	UpdateActiveStatesIndex(NewFrameSpanIndex);
}

void FScrubState::SetActiveStatesIndex(const int32 NewActiveStatesIndex)
{
	ActiveStatesIndex = NewActiveStatesIndex;

	checkf(EventCollection != nullptr, TEXT("Internal method expecting validity checks before getting called."));
	checkf(EventCollection->ActiveStatesChanges.IsValidIndex(ActiveStatesIndex), TEXT("Internal method expecting validity checks before getting called."));

	FrameSpanIndex = EventCollection->ActiveStatesChanges[ActiveStatesIndex].SpanIndex;
	ScrubTime = EventCollection->FrameSpans[FrameSpanIndex].GetWorldTimeStart();
	TraceFrameIndex = EventCollection->FrameSpans[FrameSpanIndex].Frame.Index;
	ScrubTimeBoundState = EScrubTimeBoundState::InBounds;
}

bool FScrubState::HasPreviousFrame() const
{
	if (EventCollection != nullptr)
	{
		return IsInBounds() ? EventCollection->FrameSpans.IsValidIndex(FrameSpanIndex - 1) : ScrubTimeBoundState == EScrubTimeBoundState::AfterHigherBound;
	}
	return false;
}

double FScrubState::GotoPreviousFrame()
{
	SetFrameSpanIndex(IsInBounds() ? (FrameSpanIndex - 1) : EventCollection->FrameSpans.Num() - 1);
	return ScrubTime;
}

bool FScrubState::HasNextFrame() const
{
	if (EventCollection != nullptr)
	{
		return IsInBounds() ? EventCollection->FrameSpans.IsValidIndex(FrameSpanIndex + 1) : ScrubTimeBoundState == EScrubTimeBoundState::BeforeLowerBound;
	}
	return false;
}

double FScrubState::GotoNextFrame()
{
	SetFrameSpanIndex(IsInBounds() ? (FrameSpanIndex + 1) : 0);
	return ScrubTime;
}

bool FScrubState::HasPreviousActiveStates() const
{
	if (EventCollection == nullptr || ActiveStatesIndex == INDEX_NONE)
	{
		return false;
	}

	const TArray<FInstanceEventCollection::FActiveStatesChangePair>& ActiveStatesChanges = EventCollection->ActiveStatesChanges;
	if (ScrubTimeBoundState == EScrubTimeBoundState::AfterHigherBound && ActiveStatesChanges.Num() > 0)
	{
		return true;
	}

	if (ActiveStatesChanges.IsValidIndex(ActiveStatesIndex) && ActiveStatesChanges[ActiveStatesIndex].SpanIndex < FrameSpanIndex)
	{
		return true;
	}

	return ActiveStatesChanges.IsValidIndex(ActiveStatesIndex - 1);
}

double FScrubState::GotoPreviousActiveStates()
{
	const TArray<FInstanceEventCollection::FActiveStatesChangePair>& ActiveStatesChanges = EventCollection->ActiveStatesChanges;
	if (ScrubTimeBoundState == EScrubTimeBoundState::AfterHigherBound)
	{
		SetActiveStatesIndex(ActiveStatesChanges.Num() - 1);
	}
	else if (ActiveStatesChanges.IsValidIndex(ActiveStatesIndex) && ActiveStatesChanges[ActiveStatesIndex].SpanIndex < FrameSpanIndex)
	{
		SetActiveStatesIndex(ActiveStatesIndex);
	}
	else
	{
		SetActiveStatesIndex(ActiveStatesIndex - 1);
	}

	return ScrubTime;
}

bool FScrubState::HasNextActiveStates() const
{
	if (EventCollection == nullptr)
	{
		return false;
	}

	const TArray<FInstanceEventCollection::FActiveStatesChangePair>& ActiveStatesChanges = EventCollection->ActiveStatesChanges;
	if (ScrubTimeBoundState == EScrubTimeBoundState::BeforeLowerBound && ActiveStatesChanges.Num() > 0)
	{
		return true;
	}

	return ActiveStatesIndex != INDEX_NONE && EventCollection->ActiveStatesChanges.IsValidIndex(ActiveStatesIndex + 1);
}

double FScrubState::GotoNextActiveStates()
{
	if (ScrubTimeBoundState == EScrubTimeBoundState::BeforeLowerBound)
	{
		SetActiveStatesIndex(0);
	}
	else
	{
		SetActiveStatesIndex(ActiveStatesIndex + 1);
	}
	return ScrubTime;
}

const FInstanceEventCollection& FScrubState::GetEventCollection() const
{
	return EventCollection != nullptr ? *EventCollection : FInstanceEventCollection::Invalid;
}

void FScrubState::UpdateActiveStatesIndex(const int32 SpanIndex)
{
	check(EventCollection != nullptr);

	// Need to find the index of a frame span that contains an active states changed event; either the current one has it otherwise look backward to find the last one
	ActiveStatesIndex = EventCollection->ActiveStatesChanges.FindLastByPredicate(
		[SpanIndex](const FInstanceEventCollection::FActiveStatesChangePair& SpanAndEventIndices)
		{
			return SpanAndEventIndices.SpanIndex <= SpanIndex;
		});
}
} // UE::StateTreeDebugger

FStateTreeDebuggerBreakpoint::FStateTreeDebuggerBreakpoint()
	: BreakpointType(EStateTreeBreakpointType::Unset)
	, EventType(EStateTreeTraceEventType::Unset)
{
}

FStateTreeDebuggerBreakpoint::FStateTreeDebuggerBreakpoint(const FStateTreeStateHandle StateHandle, const EStateTreeBreakpointType BreakpointType)
	: ElementIdentifier(TInPlaceType<FStateTreeStateHandle>(), StateHandle)
	, BreakpointType(BreakpointType)
{
	EventType = GetMatchingEventType(BreakpointType);
}

FStateTreeDebuggerBreakpoint::FStateTreeDebuggerBreakpoint(const FStateTreeTaskIndex Index, const EStateTreeBreakpointType BreakpointType)
	: ElementIdentifier(TInPlaceType<FStateTreeTaskIndex>(), Index)
	, BreakpointType(BreakpointType)
{
	EventType = GetMatchingEventType(BreakpointType);
}

FStateTreeDebuggerBreakpoint::FStateTreeDebuggerBreakpoint(const FStateTreeTransitionIndex Index, const EStateTreeBreakpointType BreakpointType)
	: ElementIdentifier(TInPlaceType<FStateTreeTransitionIndex>(), Index)
	, BreakpointType(BreakpointType)
{
	EventType = GetMatchingEventType(BreakpointType);
}

bool FStateTreeDebuggerBreakpoint::IsMatchingEvent(const FStateTreeTraceEventVariantType& Event) const
{
	EStateTreeTraceEventType ReceivedEventType = EStateTreeTraceEventType::Unset;
	Visit([&ReceivedEventType](auto& TypedEvent) { ReceivedEventType = TypedEvent.EventType; }, Event);

	bool bIsMatching = false;
	if (EventType == ReceivedEventType)
	{
		if (const FStateTreeStateHandle* StateHandle = ElementIdentifier.TryGet<FStateTreeStateHandle>())
		{
			const FStateTreeTraceStateEvent* StateEvent = Event.TryGet<FStateTreeTraceStateEvent>();
			bIsMatching = StateEvent != nullptr && StateEvent->GetStateHandle() == *StateHandle;
		}
		else if (const FStateTreeTaskIndex* TaskIndex = ElementIdentifier.TryGet<FStateTreeTaskIndex>())
		{
			const FStateTreeTraceTaskEvent* TaskEvent = Event.TryGet<FStateTreeTraceTaskEvent>();
			bIsMatching = TaskEvent != nullptr && TaskEvent->Index == TaskIndex->Index;
		}
		else if (const FStateTreeTransitionIndex* TransitionIndex = ElementIdentifier.TryGet<FStateTreeTransitionIndex>())
		{
			const FStateTreeTraceTransitionEvent* TransitionEvent = Event.TryGet<FStateTreeTraceTransitionEvent>();
			bIsMatching = TransitionEvent != nullptr && TransitionEvent->TransitionSource.TransitionIndex == TransitionIndex->Index;
		}
	}
	return bIsMatching;
}

EStateTreeTraceEventType FStateTreeDebuggerBreakpoint::GetMatchingEventType(const EStateTreeBreakpointType BreakpointType)
{
	switch (BreakpointType)
	{
	case EStateTreeBreakpointType::OnEnter:
		return EStateTreeTraceEventType::OnEntered;
	case EStateTreeBreakpointType::OnExit:
		return EStateTreeTraceEventType::OnExited;
	case EStateTreeBreakpointType::OnTransition:
		return EStateTreeTraceEventType::OnTransition;
	default:
		return EStateTreeTraceEventType::Unset;
	}
}

#endif // WITH_STATETREE_TRACE_DEBUGGER

