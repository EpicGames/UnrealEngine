// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debugger/StateTreeRuntimeValidation.h"
#include "Debugger/StateTreeRuntimeValidationInstanceData.h"

#if WITH_STATETREE_DEBUG
#include "HAL/IConsoleManager.h"
#include "StateTree.h"
#endif

namespace UE::StateTree::Debug
{
#if WITH_STATETREE_DEBUG

namespace Private
{
bool bRuntimeValidationContext = true;
static FAutoConsoleVariableRef CVarRuntimeValidationContext(
	TEXT("StateTree.RuntimeValidation.Context"),
	bRuntimeValidationContext,
	TEXT("Test if the context creation parameters are the same between each creation of StateTreeExecutionContext.")
);

bool bRuntimeValidationDoesNewerVersionExists = true;
static FAutoConsoleVariableRef CVarRuntimeValidationDoesNewerVersionExists(
	TEXT("StateTree.RuntimeValidation.DoesNewerVersionExists"),
	bRuntimeValidationDoesNewerVersionExists,
	TEXT("Test if a StateTreeExecutionContext started with an old version of a blueprint type.")
);

bool bRuntimeValidationEnterExitState = false;
static FAutoConsoleVariableRef CVarRuntimeValidationEnterExitState(
	TEXT("StateTree.RuntimeValidation.EnterExitState"),
	bRuntimeValidationEnterExitState,
	TEXT("Test that if a node get a EnterState, it will receive an ExitState.\n"
		"Test that if a node get a ExitState, it did receive an EnterState before.")
);

bool bRuntimeValidationInstanceData = false;
static FAutoConsoleVariableRef CVarRuntimeValidationInstanceData(
	TEXT("StateTree.RuntimeValidation.InstanceData"),
	bRuntimeValidationInstanceData,
	TEXT("Test if the state tree instance data and shared instance data are valid.")
);

FString NodeToString(const UObject* Obj, FGuid Id)
{
	TStringBuilderWithBuffer<TCHAR, 128> Buffer;
	if (Obj)
	{
		Obj->GetPathName(nullptr, Buffer);
	}
	Buffer << TEXT(':');
	Buffer << Id;
	return Buffer.ToString();
}
} // namespace Private

FRuntimeValidationInstanceData::~FRuntimeValidationInstanceData()
{
	if (Private::bRuntimeValidationEnterExitState && UObjectInitialized() && !IsEngineExitRequested())
	{
		if (NodeStates.Num() > 0)
		{
			const FNodeStatePair& Pair = NodeStates[0];
			ensureAlwaysMsgf(false, TEXT("Tree exited. Missing ExitState on %s."), *Private::NodeToString(StateTree.Get(), Pair.NodeID));
			Private::bRuntimeValidationEnterExitState = false;
		}
	}
}

void FRuntimeValidationInstanceData::SetContext(TNotNull<const UObject*> InNewOwner, TNotNull<const UStateTree*> InNewStateTree, const bool bInInstanceDataWriteAccessAcquired)
{
	TWeakObjectPtr<const UStateTree> NewStateTree = InNewStateTree;
	TWeakObjectPtr<const UObject> NewOwner = InNewOwner;
	if (Private::bRuntimeValidationContext)
	{
		if (StateTree.IsValid() && StateTree != NewStateTree && bInstanceDataWriteAccessAcquired)
		{
			ensureAlwaysMsgf(false, TEXT("StateTree runtime check failed: The StateTree '%s' is different from the previously set '%s'.\n"
				"Make sure you initialize FStateTreeExecutionContext with the same value every time.\n"
				"Auto deactivate Runtime check StateTree.RuntimeValidation.Context to prevent reporting the same error multiple times.")
				, *InNewStateTree->GetFullName(), *StateTree.Get()->GetFullName());
			Private::bRuntimeValidationContext = false;
		}
		if (Owner.IsValid() && Owner != NewOwner)
		{
			ensureAlwaysMsgf(false, TEXT("StateTree runtime check failed: The owner '%s' is different from the previously set '%s'.\n"
				"Make sure you initialize FStateTreeExecutionContext with the same values every time.\n"
				"Auto deactivate Runtime check StateTree.RuntimeValidation.Context to prevent reporting the same error multiple times.")
				, *InNewOwner->GetFullName(), *Owner.Get()->GetFullName());
			Private::bRuntimeValidationContext = false;
		}
	}

	ValidateTreeNodes(InNewStateTree);
	ValidateInstanceData(InNewStateTree);

	StateTree = NewStateTree;
	Owner = NewOwner;
	bInstanceDataWriteAccessAcquired |= bInInstanceDataWriteAccessAcquired;
	if (!bMaintainNodeStates.IsSet())
	{
		bMaintainNodeStates = Private::bRuntimeValidationEnterExitState;
	}
}

void FRuntimeValidationInstanceData::NodeEnterState(const FGuid& NodeID, FActiveFrameID FrameID)
{
	if (bMaintainNodeStates.Get(false))
	{
		FNodeStatePair* Found = NodeStates.FindByPredicate([&NodeID, &FrameID](const FNodeStatePair& Other)
			{
				return Other.NodeID == NodeID && Other.FrameID == FrameID;
			});
		if (Found)
		{
			if (Private::bRuntimeValidationEnterExitState)
			{
				ensureAlwaysMsgf(false, TEXT("StateTree runtime check failed: EnterState executed on node %s without an ExitState.\n"
					"Auto deactivate Runtime check StateTree.RuntimeValidation.EnterExitState to prevent reporting the same error multiple times.")
					, *Private::NodeToString(Owner.Get(), NodeID));
				Private::bRuntimeValidationEnterExitState = false;
			}
		}
		else
		{
			NodeStates.Add(FNodeStatePair{ .NodeID = NodeID, .FrameID = FrameID });
		}
	}
}

void FRuntimeValidationInstanceData::NodeExitState(const FGuid& NodeID, FActiveFrameID FrameID)
{
	if (bMaintainNodeStates.Get(false))
	{
		const int32 FoundIndex = NodeStates.IndexOfByPredicate([&NodeID, &FrameID](const FNodeStatePair& Other)
			{
				return Other.NodeID == NodeID && Other.FrameID == FrameID;
			});
		if (FoundIndex != INDEX_NONE)
		{
			NodeStates.RemoveAtSwap(FoundIndex);
		}
		else if (Private::bRuntimeValidationEnterExitState)
		{
			ensureAlwaysMsgf(false, TEXT("StateTree runtime check failed: ExitState executed on node %s without an EnterState.\n"
				"Auto deactivate Runtime check StateTree.RuntimeValidation.EnterExitState to prevent reporting the same error multiple times.")
				, *Private::NodeToString(Owner.Get(), NodeID));
			Private::bRuntimeValidationEnterExitState = false;
		}
	}
}

void FRuntimeValidationInstanceData::ValidateTreeNodes(TNotNull<const UStateTree*> InNewStateTree) const
{
	if (Private::bRuntimeValidationDoesNewerVersionExists)
	{
		if (InNewStateTree->IsReadyToRun())
		{
			auto DoesNewerVersionExists = [](const UObject* InstanceDataType)
			{
				// Is the class/scriptstruct a blueprint that got replaced by another class.
				bool bHasNewerVersionExistsFlag = InstanceDataType->HasAnyFlags(RF_NewerVersionExists);
				if (!bHasNewerVersionExistsFlag)
				{
					if (const UClass* InstanceDataClass = Cast<UClass>(InstanceDataType))
					{
						bHasNewerVersionExistsFlag = InstanceDataClass->HasAnyClassFlags(CLASS_NewerVersionExists);
					}
					else if (const UScriptStruct* InstanceDataStruct = Cast<UScriptStruct>(InstanceDataType))
					{
						bHasNewerVersionExistsFlag = (InstanceDataStruct->StructFlags & STRUCT_NewerVersionExists) != 0;
					}
				}
				return bHasNewerVersionExistsFlag;
			};
			{
				const FStateTreeInstanceData& InstanceData = InNewStateTree->GetDefaultInstanceData();
				const int32 InstanceDataNum = InstanceData.Num();
				for (int32 Index = 0; Index < InstanceDataNum; ++Index)
				{
					bool bFailed = false;
					const UObject* InstanceObject = nullptr;
					if (InstanceData.IsObject(Index))
					{
						InstanceObject = InstanceData.GetObject(Index);
						bFailed = DoesNewerVersionExists(InstanceObject)
							|| (InstanceObject && DoesNewerVersionExists(InstanceObject->GetClass()));
					}
					else
					{
						InstanceObject = InstanceData.GetStruct(Index).GetScriptStruct();
						bFailed = DoesNewerVersionExists(InstanceObject);
					}

					if (bFailed)
					{
						ensureAlwaysMsgf(false, TEXT("StateTree runtime check failed: The data '%s' has a newest version.\n"
							"It should be detected in StateTree::Link.\n"
							"Auto deactivate Runtime check StateTree.RuntimeValidation.DoesNewerVersionExists to prevent reporting the same error multiple times.")
							, *InstanceObject->GetFullName());
						Private::bRuntimeValidationDoesNewerVersionExists = false;
					}
				}
			}

			for (FConstStructView NodeView : InNewStateTree->GetNodes())
			{
				const FStateTreeNodeBase* Node = NodeView.GetPtr<const FStateTreeNodeBase>();
				if (Node)
				{
					const UStruct* DesiredInstanceDataType = Node->GetInstanceDataType();
					if (DoesNewerVersionExists(DesiredInstanceDataType))
					{
						ensureAlwaysMsgf(false, TEXT("StateTree runtime check failed: The node '%s' has a newest version.\n"
							"It should be detected in StateTree::Link.\n"
							"Auto deactivate Runtime check StateTree.RuntimeValidation.DoesNewerVersionExists to prevent reporting the same error multiple times.")
							, *DesiredInstanceDataType->GetFullName());
						Private::bRuntimeValidationDoesNewerVersionExists = false;
					}
				}
			}
		}
	}
}

void FRuntimeValidationInstanceData::ValidateInstanceData(TNotNull<const UStateTree*> NewStateTree)
{
	if (Private::bRuntimeValidationInstanceData)
	{
		if (!NewStateTree->GetDefaultInstanceData().GetStorage().AreAllInstancesValid())
		{
			ensureAlwaysMsgf(false, TEXT("The instance data has invalid data."));
			Private::bRuntimeValidationInstanceData = false;
		}

		TSharedPtr<FStateTreeInstanceData> SharedPtr = NewStateTree->GetSharedInstanceData();
		if (SharedPtr)
		{
			if (!SharedPtr->GetStorage().AreAllInstancesValid())
			{
				ensureAlwaysMsgf(false, TEXT("The shared instance data has invalid data."));
				Private::bRuntimeValidationInstanceData = false;
			}
		}
	}
}
#endif //WITH_STATETREE_DEBUG

/**
 * 
 */


#if WITH_STATETREE_DEBUG
FRuntimeValidation::FRuntimeValidation(TNotNull<FRuntimeValidationInstanceData*> Instance)
	: RuntimeValidationData(Instance)
{
}

FRuntimeValidationInstanceData* FRuntimeValidation::GetInstanceData() const
{
	return RuntimeValidationData;
}

void FRuntimeValidation::SetContext(TNotNull<const UObject*> Owner, TNotNull<const UStateTree*> StateTree, bool bInstanceDataWriteAccessAcquired) const
{
	if (RuntimeValidationData)
	{
		RuntimeValidationData->SetContext(Owner, StateTree, bInstanceDataWriteAccessAcquired);
	}
}

#else

void FRuntimeValidation::SetContext(TNotNull<const UObject*>, TNotNull<const UStateTree*>, bool) const
{
}

#endif //WITH_STATETREE_DEBUG

} // UE::StateTree::Debug


