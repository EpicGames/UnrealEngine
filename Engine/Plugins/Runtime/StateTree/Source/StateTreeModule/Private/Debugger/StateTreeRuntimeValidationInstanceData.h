// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeStatePath.h"

class UStateTree;
class UObject;

#if WITH_STATETREE_DEBUG

namespace UE::StateTree::Debug
{
/**
 * For debugging purposes. Data used for runtime check.
*/
class FRuntimeValidationInstanceData
{
public:
	FRuntimeValidationInstanceData() = default;
	~FRuntimeValidationInstanceData();

	void SetContext(TNotNull<const UObject*> InOwner, TNotNull<const UStateTree*> InStateTree, bool bInInstanceDataWriteAccessAcquired);

	void NodeEnterState(const FGuid& NodeID, FActiveFrameID FrameID);
	void NodeExitState(const FGuid& NodeID, FActiveFrameID FrameID);
	void NodeTick(const FGuid& NodeID, FActiveFrameID FrameID);

private:
	void ValidateTreeNodes(TNotNull<const UStateTree*> NewStateTree) const;
	void ValidateInstanceData(TNotNull<const UStateTree*> NewStateTree);

private:
	struct FNodeStatePair
	{
		FGuid NodeID;
		FActiveFrameID FrameID;
	};
	TArray<FNodeStatePair> NodeStates;
	TWeakObjectPtr<const UStateTree> StateTree;
	TWeakObjectPtr<const UObject> Owner;

	/** An ExecContext with write access to the Instance Storage data this is debugging on has been created */
	bool bInstanceDataWriteAccessAcquired = false;
	/**
	 * The execution started with bRuntimeValidationEnterExitState set to true.
	 * It will keep maintaining the NodeStates array even if bRuntimeValidationEnterExitState is set to false afterward.
	 * The bRuntimeValidationEnterExitState can be reset to true after a fail, and we need to keep maintaining the array.
	 */
	TOptional<bool> bMaintainNodeStates;
};

} // UE::StateTree::Debug

#endif // WITH_STATETREE_DEBUG