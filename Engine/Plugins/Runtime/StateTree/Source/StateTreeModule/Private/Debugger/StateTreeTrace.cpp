// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_STATETREE_TRACE

#include "Debugger/StateTreeTrace.h"
#include "Debugger/StateTreeTraceTypes.h"
#include "Exporters/Exporter.h"
#include "Misc/StringOutputDevice.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Misc/MTTransactionallySafeAccessDetector.h"
#include "ObjectTrace.h"
#include "Serialization/BufferArchive.h"
#include "StateTree.h"
#include "StateTreeDelegates.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeExecutionTypes.h"
#include "UObject/Package.h"
#include "Trace/Trace.inl"

#if WITH_EDITOR
#include "Editor.h"
#endif // WITH_EDITOR

UE_TRACE_CHANNEL_DEFINE(StateTreeDebugChannel, "Traces StateTree state machine execution for debugging including world timestamps, asset IDs, instance \
events, frame events, phase transitions, log messages, state changes, task execution, evaluator execution, transition \
conditions, condition evaluation, and active state tracking with deferred buffering support.")

UE_TRACE_EVENT_BEGIN(StateTreeDebugger, WorldTimestampEvent)
	UE_TRACE_EVENT_FIELD(double, WorldTime)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(StateTreeDebugger, AssetDebugIdEvent)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, TreeName)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, TreePath)
	UE_TRACE_EVENT_FIELD(uint32, CompiledDataHash)
	UE_TRACE_EVENT_FIELD(uint16, AssetDebugId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(StateTreeDebugger, InstanceEvent)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint32, InstanceId)
	UE_TRACE_EVENT_FIELD(uint32, InstanceSerial)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, InstanceName)
	UE_TRACE_EVENT_FIELD(std::underlying_type_t<EStateTreeTraceEventType>, EventType)
	UE_TRACE_EVENT_FIELD(uint16, AssetDebugId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(StateTreeDebugger, InstanceFrameEvent)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint32, InstanceId)
	UE_TRACE_EVENT_FIELD(uint32, InstanceSerial)
	UE_TRACE_EVENT_FIELD(uint16, AssetDebugId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(StateTreeDebugger, PhaseEvent)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint32, InstanceId)
	UE_TRACE_EVENT_FIELD(uint32, InstanceSerial)
	UE_TRACE_EVENT_FIELD(std::underlying_type_t<EStateTreeUpdatePhase>, Phase)
	UE_TRACE_EVENT_FIELD(uint16, StateIndex)
	UE_TRACE_EVENT_FIELD(std::underlying_type_t<EStateTreeTraceEventType>, EventType)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(StateTreeDebugger, LogEvent)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint32, InstanceId)
	UE_TRACE_EVENT_FIELD(uint32, InstanceSerial)
	UE_TRACE_EVENT_FIELD(std::underlying_type_t<ELogVerbosity::Type>, Verbosity)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Message)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(StateTreeDebugger, StateEvent)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint32, InstanceId)
	UE_TRACE_EVENT_FIELD(uint32, InstanceSerial)
	UE_TRACE_EVENT_FIELD(uint16, StateIndex)
	UE_TRACE_EVENT_FIELD(std::underlying_type_t<EStateTreeTraceEventType>, EventType)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(StateTreeDebugger, TaskEvent)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint32, InstanceId)
	UE_TRACE_EVENT_FIELD(uint32, InstanceSerial)
	UE_TRACE_EVENT_FIELD(uint16, NodeIndex)
	UE_TRACE_EVENT_FIELD(uint8[], DataView)
	UE_TRACE_EVENT_FIELD(std::underlying_type_t<EStateTreeTraceEventType>, EventType)
	UE_TRACE_EVENT_FIELD(uint8, Status)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(StateTreeDebugger, EvaluatorEvent)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint32, InstanceId)
	UE_TRACE_EVENT_FIELD(uint32, InstanceSerial)
	UE_TRACE_EVENT_FIELD(uint16, NodeIndex)
	UE_TRACE_EVENT_FIELD(uint8[], DataView)
	UE_TRACE_EVENT_FIELD(std::underlying_type_t<EStateTreeTraceEventType>, EventType)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(StateTreeDebugger, TransitionEvent)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint32, InstanceId)
	UE_TRACE_EVENT_FIELD(uint32, InstanceSerial)
	UE_TRACE_EVENT_FIELD(uint8, SourceType)
	UE_TRACE_EVENT_FIELD(uint16, TransitionIndex)
	UE_TRACE_EVENT_FIELD(uint16, TargetStateIndex)
	UE_TRACE_EVENT_FIELD(uint8, Priority)
	UE_TRACE_EVENT_FIELD(std::underlying_type_t<EStateTreeTraceEventType>, EventType)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(StateTreeDebugger, ConditionEvent)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint32, InstanceId)
	UE_TRACE_EVENT_FIELD(uint32, InstanceSerial)
	UE_TRACE_EVENT_FIELD(uint16, NodeIndex)
	UE_TRACE_EVENT_FIELD(uint8[], DataView)
	UE_TRACE_EVENT_FIELD(std::underlying_type_t<EStateTreeTraceEventType>, EventType)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(StateTreeDebugger, ActiveStatesEvent)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint32, InstanceId)
	UE_TRACE_EVENT_FIELD(uint32, InstanceSerial)
	UE_TRACE_EVENT_FIELD(uint16[], ActiveStates)
	UE_TRACE_EVENT_FIELD(uint16[], AssetDebugIds)
UE_TRACE_EVENT_END()

namespace UE::StateTreeTrace
{
struct FBufferedDataList;
FDelegateHandle GOnWorldTickStartDelegateHandle;
FDelegateHandle GTracingStateChangedDelegateHandle;

/** Struct to keep track if a given phase was traced or not. */
struct FPhaseTraceStatusPair
{
	explicit FPhaseTraceStatusPair(const EStateTreeUpdatePhase Phase, const FStateTreeStateHandle StateHandle)
	: Phase(Phase)
	, StateHandle(StateHandle)
	{
	}

	EStateTreeUpdatePhase Phase = EStateTreeUpdatePhase::Unset;
	FStateTreeStateHandle StateHandle = FStateTreeStateHandle::Invalid;
	bool bTraced = false;
};

/**
 * Struct to hold data for asset debug id events until we are ready to trace the events (i.e. traces are active and channel is enabled).
 */
struct FAssetDebugIdEventBufferedData
{
	FAssetDebugIdEventBufferedData() = default;
	explicit FAssetDebugIdEventBufferedData(const UStateTree* StateTree, const FStateTreeIndex16 AssetDebugId) : WeakStateTree(StateTree), AssetDebugId(AssetDebugId)
	{
	}

	TWeakObjectPtr<const UStateTree> WeakStateTree;
	FStateTreeIndex16 AssetDebugId;
};

/**
 * Struct to hold data for active states events until we are ready to trace the events (i.e. traces are active and channel is enabled).
 */
struct FInstanceEventBufferedData
{
	struct FActiveStates
	{
		FActiveStates() = default;
		explicit FActiveStates(FBufferedDataList& BufferedData, const TConstArrayView<FStateTreeExecutionFrame> ActiveFrames);
		bool IsValid() const { return StatesIndices.Num() > 0 && StatesIndices.Num() == AssetDebugIds.Num(); }
		void Output(const FStateTreeInstanceDebugId InInstanceId) const
		{
			UE_TRACE_LOG(StateTreeDebugger, ActiveStatesEvent, StateTreeDebugChannel)
				<< ActiveStatesEvent.Cycle(FPlatformTime::Cycles64())
				<< ActiveStatesEvent.InstanceId(InInstanceId.Id)
				<< ActiveStatesEvent.InstanceSerial(InInstanceId.SerialNumber)
				<< ActiveStatesEvent.ActiveStates(StatesIndices.GetData(), StatesIndices.Num())
				<< ActiveStatesEvent.AssetDebugIds(AssetDebugIds.GetData(), AssetDebugIds.Num());
		}

		TArray<uint16> StatesIndices;
		TArray<uint16> AssetDebugIds;
	};

	FInstanceEventBufferedData() = default;
	explicit FInstanceEventBufferedData(const FStateTreeInstanceDebugId InstanceId)
		: InstanceId(InstanceId)
	{
	}

	void CloseRecording(const double WorldTime) const
	{
		// Output and empty active states event at the last recorded world time to close active states
		UE_TRACE_LOG(StateTreeDebugger, WorldTimestampEvent, StateTreeDebugChannel) << WorldTimestampEvent.WorldTime(WorldTime);
		FActiveStates EmptyState;
		EmptyState.Output(InstanceId);
	}

	FStateTreeInstanceDebugId InstanceId;

	/** Stack to keep track of all received phase events so matching "closing" events (i.e., Pop) will control "IF" and "WHEN" a given phase will be sent. */
	TArray<FPhaseTraceStatusPair> PhaseStack;
};

/** Struct to keep track of the buffered event data and flush them. */
struct FBufferedDataList
{
	/** Traces required events (world time, phase events, etc.) for a given instance. */
	void Flush_AnyThread(const FStateTreeInstanceDebugId InstanceId);

	/** Traces closing events and resets some data for subsequent traces. */
	void OnStoppingTraces_GameThread();

	/**  Returns an existing ID or create one for a given StateTree asset. */
	FStateTreeIndex16 FindOrAddDebugIdForAsset_AnyThread(const UStateTree* StateTree);

	/**
	 * Keeps track of Pushed/Popped phase for a given instance
	 * @return The pair that got popped when processing an event of type 'Pop' for a currently active event ('Push').
	 */
	TOptional<FPhaseTraceStatusPair> UpdatePhaseScope_AnyThread(const FStateTreeInstanceDebugId InstanceId, const EStateTreeUpdatePhase Phase, const EStateTreeTraceEventType EventType, const FStateTreeStateHandle StateHandle);

	/** Converts the provided list of execution frames to traceable data format and outputs it to the trace. */
	void OutputActiveStates_AnyThread(const FStateTreeInstanceDebugId InstanceId, const TConstArrayView<FStateTreeExecutionFrame> ActiveFrames);

	/** Initializes recording world time to 0 if not initialized. */
	void InitializeWorldTimeIfNeeded_AnyThread();

	/** Sets the world time that might need to be traced by worker threads. */
	void SetWorldTime_GameThread(const double WorldTime);

	void Reset_GameThread()
	{
		checkf(IsInGameThread(), TEXT("Expecting to only be called on the main thread before worker threads trace events."));
		{
			FRWScopeLock RWScopeLock(TracedInstancesRWLock, SLT_Write);
			TracedInstances.Empty();
		}

		RecordingWorldTime = UNINITIALIZED_WORLD_TIME;
		TracedRecordingWorldTime = UNINITIALIZED_WORLD_TIME;

		AssetDebugIdMap.Reset();
		BufferedEventsMap.Reset();
		NextAssetDebugId = 1;
	}

	bool ShouldTraceInstance(const FStateTreeInstanceDebugId InstanceId) const
	{
		// Note this pattern is the same as AnimNextTrace; both should be updated if a change/fix is needed
		FRWScopeLock RWScopeLock(TracedInstancesRWLock, SLT_ReadOnly);
		if (!TracedInstances.Contains(InstanceId))
		{
			RWScopeLock.ReleaseReadOnlyLockAndAcquireWriteLock_USE_WITH_CAUTION();

			// Note that we might double add here due to 2x people writing at the same time, but it doesn't matter
			TracedInstances.Add(InstanceId);
			return true;
		}

		return false;
	}

private:
	static constexpr double UNINITIALIZED_WORLD_TIME = -1;

	/** World time provided by the game thread at the beginning of the frame */
	std::atomic<double> RecordingWorldTime = UNINITIALIZED_WORLD_TIME;

	/** World time traced only once by the first worker thread that needs to trace an event */
	std::atomic<double> TracedRecordingWorldTime = UNINITIALIZED_WORLD_TIME;

	/** List of traced instances */
	mutable TSet<FStateTreeInstanceDebugId> TracedInstances;
	mutable FRWLock TracedInstancesRWLock;

	/** List of asset debug ids events that will be output if channel gets enabled. */
	TArray<FAssetDebugIdEventBufferedData> AssetDebugIdMap;
	FTransactionallySafeRWLock AssetDebugIdMapLock;

	/** List of lifetime events that will be output if channel gets enabled in the Push - Pop lifetime window of an instance. */
	TMap<FStateTreeInstanceDebugId, FInstanceEventBufferedData> BufferedEventsMap;
	FTransactionallySafeRWLock BufferedEventsMapLock;

	/** MT access detector used to validate that AssetDebugIdEvents and InstanceLifetimeEvents are never flushed while getting accessed by worker threads. */
	UE_MT_DECLARE_TS_RW_ACCESS_DETECTOR(AssetDebugIdMapMTDetector);

	/** MT access detector used to validate that AssetDebugIdEvents and InstanceLifetimeEvents are never flushed while getting accessed by worker threads. */
	UE_MT_DECLARE_TS_RW_ACCESS_DETECTOR(BufferedEventsMapMTDetector);

	uint16 NextAssetDebugId = 1;

	/** Called by TraceBufferedEvents from the OutputXYZ methods to make sure current world time was traced. */
	void TraceWorldTimeIfNeeded_AnyThread();

	/**
	 * Called by TraceBufferedEvents from the OutputXYZ methods to flush pending phase events.
	 * Phases popped before TraceStackedPhases gets called will never produce any trace since
	 * they will not be required for the analysis.
	 */
	void TraceStackedPhases_AnyThread(const FStateTreeInstanceDebugId InstanceId);
};

/**
 * Buffered events (e.g. lifetime, active state, scoped phase) in case channel is not active yet or phase are empty and don't need to be traced.
 * This can be accessed from any thread and based on the assumption that a given instance data can never be accessed by multiple thread simultaneously in the same frame
 * so we only need to protect access to the main maps.
 */
FBufferedDataList GBufferedEvents;

FInstanceEventBufferedData::FActiveStates::FActiveStates(FBufferedDataList& BufferedData, const TConstArrayView<FStateTreeExecutionFrame> ActiveFrames)
{
	for (const FStateTreeExecutionFrame& Frame : ActiveFrames)
	{
		const FStateTreeIndex16 AssetDebugId = BufferedData.FindOrAddDebugIdForAsset_AnyThread(Frame.StateTree.Get());

		const int32 RequiredSize = StatesIndices.Num() + Frame.ActiveStates.Num();
		StatesIndices.Reserve(RequiredSize);
		AssetDebugIds.Reserve(RequiredSize);

		for (const FStateTreeStateHandle StateHandle : Frame.ActiveStates)
		{
			StatesIndices.Add(StateHandle.Index);
			AssetDebugIds.Add(AssetDebugId.Get());
		}
	}
}

void FBufferedDataList::Flush_AnyThread(const FStateTreeInstanceDebugId InstanceId)
{
	if (InstanceId.IsValid())
	{
		TraceWorldTimeIfNeeded_AnyThread();
		TraceStackedPhases_AnyThread(InstanceId);
	}
}

void FBufferedDataList::TraceWorldTimeIfNeeded_AnyThread()
{
	InitializeWorldTimeIfNeeded_AnyThread();

	double Expected = UNINITIALIZED_WORLD_TIME;
	if (TracedRecordingWorldTime.compare_exchange_strong(Expected, RecordingWorldTime))
	{
		UE_TRACE_LOG(StateTreeDebugger, WorldTimestampEvent, StateTreeDebugChannel)
			<< WorldTimestampEvent.WorldTime(RecordingWorldTime.load());
	}
}

void FBufferedDataList::TraceStackedPhases_AnyThread(const FStateTreeInstanceDebugId InstanceId)
{
	// Trace pushes phase events and marked them as traced only if not already traced and our channel is enabled.
	// We need PhaseEvent:Pop to be sent only in this case to enforce complementary events in case of
	// late recording (e.g. recording gets started, or channel gets enabled, while simulation is already running and instances were ticked)
	if (!UE_TRACE_CHANNELEXPR_IS_ENABLED(StateTreeDebugChannel))
	{
		return;
	}

	TArray<FPhaseTraceStatusPair> PhaseStackToTrace;
	{
		// We can use a read scope even if we modify entries since the lock
		// is only required to access the buffered data struct associated 'InstanceId'
		// which can only happen in a single worker thread each frame.
		TReadScopeLock ReadSection(BufferedEventsMapLock);
		UE_MT_SCOPED_READ_ACCESS(BufferedEventsMapMTDetector);
		if (FInstanceEventBufferedData* Data = BufferedEventsMap.Find(InstanceId))
		{
			// Keep local copy to trace outside the critical section and only perform the
			// minimal required changes while we hold the lock (i.e., updating that status)
			PhaseStackToTrace = Data->PhaseStack;
			for (FPhaseTraceStatusPair& StackEntry : Data->PhaseStack)
			{
				StackEntry.bTraced = true;
			}
		}
	}

	// We can now safely send PhaseEvents from the local copy
	for (FPhaseTraceStatusPair& StackEntry : PhaseStackToTrace)
	{
		if (StackEntry.bTraced == false)
		{
			UE_TRACE_LOG(StateTreeDebugger, PhaseEvent, StateTreeDebugChannel)
					<< PhaseEvent.Cycle(FPlatformTime::Cycles64())
					<< PhaseEvent.InstanceId(InstanceId.Id)
					<< PhaseEvent.InstanceSerial(InstanceId.SerialNumber)
					<< PhaseEvent.Phase(static_cast<std::underlying_type_t<EStateTreeUpdatePhase>>(StackEntry.Phase))
					<< PhaseEvent.StateIndex(StackEntry.StateHandle.Index)
					<< PhaseEvent.EventType(static_cast<std::underlying_type_t<EStateTreeTraceEventType>>(EStateTreeTraceEventType::Push));
		}
	}
}

void FBufferedDataList::OnStoppingTraces_GameThread()
{
	checkf(IsInGameThread(), TEXT("Expecting to only be called by the statetree delegate on the main thread after worker threads traced events."));

	UE_MT_SCOPED_READ_ACCESS(BufferedEventsMapMTDetector);
	for (TPair<FStateTreeInstanceDebugId, FInstanceEventBufferedData>& Pair : BufferedEventsMap)
	{
		Pair.Value.CloseRecording(TracedRecordingWorldTime);
	}

	// Reset for next trace
	RecordingWorldTime = UNINITIALIZED_WORLD_TIME;
}

FStateTreeIndex16 FBufferedDataList::FindOrAddDebugIdForAsset_AnyThread(const UStateTree* StateTree)
{
	FStateTreeIndex16 AssetDebugId;
	if (!ensure(StateTree != nullptr))
	{
		return AssetDebugId;
	}

	// Return DebugId from existing entry (read-only operation)
	{
		TReadScopeLock ReadSection(AssetDebugIdMapLock);
		UE_MT_SCOPED_READ_ACCESS(AssetDebugIdMapMTDetector);
		const FAssetDebugIdEventBufferedData* ExistingPair = AssetDebugIdMap.FindByPredicate([StateTree](const FAssetDebugIdEventBufferedData& BufferedData)
		{
			return BufferedData.WeakStateTree == StateTree;
		});

		if (ExistingPair != nullptr)
		{
			return ExistingPair->AssetDebugId;
		}
	}

	// Assign new DebugId, store it and trace it (write operation)
	{
		TWriteScopeLock WriteSection(AssetDebugIdMapLock);
		UE_MT_SCOPED_WRITE_ACCESS(AssetDebugIdMapMTDetector);
		AssetDebugId = FStateTreeIndex16(++NextAssetDebugId);
		AssetDebugIdMap.Emplace(StateTree, AssetDebugId);
	}

	OutputAssetDebugIdEvent(StateTree, AssetDebugId);

	return AssetDebugId;
}

TOptional<FPhaseTraceStatusPair> FBufferedDataList::UpdatePhaseScope_AnyThread(const FStateTreeInstanceDebugId InstanceId, const EStateTreeUpdatePhase Phase
	, const EStateTreeTraceEventType EventType, const FStateTreeStateHandle StateHandle)
{
	TOptional<FPhaseTraceStatusPair> RemovedPair;
	auto UpdateStackFunc = [&RemovedPair, Phase, EventType, StateHandle](FInstanceEventBufferedData& Data)
	{
		if (EventType == EStateTreeTraceEventType::Push)
		{
			Data.PhaseStack.Push(FPhaseTraceStatusPair(Phase, StateHandle));
		}
		else if (ensureAlwaysMsgf(Data.PhaseStack.IsEmpty() == false, TEXT("Not expected to pop phases that never got pushed.")))
		{
			RemovedPair = Data.PhaseStack.Pop();
		}
	};

	// Update existing data (read-only operation on the event container)
	{
		TReadScopeLock ReadSection(BufferedEventsMapLock);
		UE_MT_SCOPED_READ_ACCESS(BufferedEventsMapMTDetector);
		FInstanceEventBufferedData* ExistingBufferedData = BufferedEventsMap.Find(InstanceId);
		if (ExistingBufferedData != nullptr)
		{
			UpdateStackFunc(*ExistingBufferedData);
			return RemovedPair;
		}
	}

	// Need to create a new entry in the container
	{
		TWriteScopeLock WriteSection(BufferedEventsMapLock);
		UE_MT_SCOPED_WRITE_ACCESS(BufferedEventsMapMTDetector);
		FInstanceEventBufferedData& NewBufferedData = BufferedEventsMap.Add(InstanceId);
		UpdateStackFunc(NewBufferedData);
	}
	return RemovedPair;
}

void FBufferedDataList::OutputActiveStates_AnyThread(const FStateTreeInstanceDebugId InstanceId, const TConstArrayView<FStateTreeExecutionFrame> ActiveFrames)
{
	const FInstanceEventBufferedData::FActiveStates ActiveStates = FInstanceEventBufferedData::FActiveStates(*this, ActiveFrames);
	ActiveStates.Output(InstanceId);
}

void FBufferedDataList::SetWorldTime_GameThread(const double WorldTime)
{
	checkf(IsInGameThread(), TEXT("Expecting to only be called by the world delegate on the main thread before worker threads trace events."));
	RecordingWorldTime = WorldTime;

	// Reset traced time so the first worker thread that will need it will set and trace it
	TracedRecordingWorldTime = UNINITIALIZED_WORLD_TIME;
}

void FBufferedDataList::InitializeWorldTimeIfNeeded_AnyThread()
{
	constexpr double InitialValue = 0;
	double Expected = UNINITIALIZED_WORLD_TIME;
	RecordingWorldTime.compare_exchange_strong(Expected, InitialValue);
}

/**
 * Pushed or pops an entry on the Phase stack for a given Instance.
 * Will send the Pop events for phases popped if their associated Push events were sent.
 */
void OutputPhaseScopeEvent(
	TNotNull<const FStateTreeReadOnlyExecutionContext*> Context,
	const EStateTreeUpdatePhase Phase,
	const EStateTreeTraceEventType EventType,
	const FStateTreeStateHandle StateHandle)
{
	const FStateTreeInstanceDebugId InstanceId = Context->GetInstanceDebugId();

	const TOptional<FPhaseTraceStatusPair> RemovedPair = GBufferedEvents.UpdatePhaseScope_AnyThread(InstanceId, Phase, EventType, StateHandle);

	// Phase was previously traced (i.e., other events were traced in that scope so we need to trace the closing (i.e., Pop) event)
	if (RemovedPair.IsSet()
		&& ensureMsgf(RemovedPair.GetValue().Phase == Phase, TEXT("Not expected to pop a phase that is not on the top of the stack."))
		&& RemovedPair.GetValue().bTraced)
	{
		UE_TRACE_LOG(StateTreeDebugger, PhaseEvent, StateTreeDebugChannel)
			<< PhaseEvent.Cycle(FPlatformTime::Cycles64())
			<< PhaseEvent.InstanceId(InstanceId.Id)
			<< PhaseEvent.InstanceSerial(InstanceId.SerialNumber)
			<< PhaseEvent.Phase(static_cast<std::underlying_type_t<EStateTreeUpdatePhase>>(Phase))
			<< PhaseEvent.StateIndex(StateHandle.Index)
			<< PhaseEvent.EventType(static_cast<std::underlying_type_t<EStateTreeTraceEventType>>(EStateTreeTraceEventType::Pop));
	}
}

/**
 * Called by the OutputXYZ methods to flush pending events (e.g. Push or WorldTime).
 */
void TraceBufferedEvents(TNotNull<const FStateTreeReadOnlyExecutionContext*> Context)
{
	const UObject* Owner = Context->GetOwner();
	if (CANNOT_TRACE_OBJECT(Owner))
	{
		return;
	}

	const FStateTreeInstanceDebugId InstanceId = Context->GetInstanceDebugId();
	if (GBufferedEvents.ShouldTraceInstance(InstanceId))
	{
		const UStateTree* StateTree = Context->GetStateTree();
		const FString InstanceName = Context->GetInstanceDebugDescription();
		const uint64 OuterTraceId = Context->GetOuterTraceId();

#if OBJECT_TRACE_ENABLED
		const uint64 OuterId = OuterTraceId == 0 ? FObjectTrace::GetObjectId(Owner) : OuterTraceId;
#else
		const uint64 OuterId = OuterTraceId;
#endif

		// Trace the owner (Object trace specific)
		TRACE_OBJECT(Owner);

		// Trace the struct instance (Object trace specific)
		TRACE_INSTANCE(Owner, InstanceId.ToUint64(), OuterId, FStateTreeInstanceData::StaticStruct(), StateTree->GetName());

		// Will trace Asset debug Id if needed (StateTree trace specific)
		const FStateTreeIndex16 AssetDebugId = GBufferedEvents.FindOrAddDebugIdForAsset_AnyThread(StateTree);

		// Trace instance (StateTree trace specific)
		UE_TRACE_LOG(StateTreeDebugger, InstanceEvent, StateTreeDebugChannel)
			<< InstanceEvent.Cycle(FPlatformTime::Cycles64())
			<< InstanceEvent.InstanceId(InstanceId.Id)
			<< InstanceEvent.InstanceSerial(InstanceId.SerialNumber)
			<< InstanceEvent.InstanceName(*InstanceName, InstanceName.Len())
			<< InstanceEvent.EventType(static_cast<std::underlying_type_t<EStateTreeTraceEventType>>(EStateTreeTraceEventType::Push))
			<< InstanceEvent.AssetDebugId(AssetDebugId.Get());

		// Trace active states (StateTree trace specific)
		OutputActiveStatesEventTrace(Context, Context->GetActiveFrames());
	}

	GBufferedEvents.Flush_AnyThread(InstanceId);
}

void SerializeDebugDataToArchive(FBufferArchive& Ar, FNodeCustomDebugData&& CustomDebugData, const FStateTreeDataView DataView)
{
	constexpr uint32 PortFlags =
		PPF_PropertyWindow // limit to properties visible in Editor
		| PPF_ExportsNotFullyQualified
		| PPF_Delimited // property data should be wrapped in quotes
		| PPF_ExternalEditor // uses authored names instead of internal names and default values are always written out
		| PPF_SimpleObjectText // object property values should be exported without the package or class information
		| PPF_ForDiff // do not emit object path
		| PPF_IncludeTransient; // transient properties should be exported

	FString TypePath;
	FString InstanceDataAsText;
	FString DebugText;

	if (const UScriptStruct* ScriptStruct = Cast<const UScriptStruct>(DataView.GetStruct()))
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::StateTree::ExportStructAsText);
		TypePath = ScriptStruct->GetPathName();

		if (!CustomDebugData.ShouldOverrideDataView())
		{
			ScriptStruct->ExportText(InstanceDataAsText, DataView.GetMemory(), DataView.GetMemory(), /*OwnerObject*/nullptr, PortFlags | PPF_SeparateDefine, /*ExportRootScope*/nullptr);
		}
	}
	else if (const UClass* Class = Cast<const UClass>(DataView.GetStruct()))
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::StateTree::ExportObjectAsText);
		TypePath = Class->GetPathName();

		if (!CustomDebugData.ShouldOverrideDataView())
		{
			FStringOutputDevice OutputDevice;
			UObject* Object = DataView.GetMutablePtr<UObject>();

			// Not using on scope FExportObjectInnerContext since it is very costly to build.
			// Passing a null context will make the export use an already built thread local context.
			UExporter::ExportToOutputDevice(nullptr, Object, /*Exporter*/nullptr, OutputDevice, TEXT("copy"), 0, PortFlags, false, Object->GetOuter());
			InstanceDataAsText = OutputDevice;
		}
	}

	if (CustomDebugData.IsSet())
	{
		DebugText = CustomDebugData.GetTraceDebuggerString();
	}

	Ar << TypePath;
	Ar << InstanceDataAsText;
	Ar << DebugText;
}

void RegisterGlobalDelegates()
{
	GOnWorldTickStartDelegateHandle = FWorldDelegates::OnWorldTickStart.AddLambda([](const UWorld* TickedWorld, ELevelTick TickType, float DeltaTime)
		{
#if OBJECT_TRACE_ENABLED
			GBufferedEvents.SetWorldTime_GameThread(FObjectTrace::GetWorldElapsedTime(TickedWorld));
#endif// OBJECT_TRACE_ENABLED
		});

	GTracingStateChangedDelegateHandle = UE::StateTree::Delegates::OnTracingStateChanged.AddLambda([](const EStateTreeTraceStatus TraceStatus)
	{
		// Traces are about to be stopped so allow the buffered events to react.
		if (TraceStatus == EStateTreeTraceStatus::StoppingTrace)
		{
			GBufferedEvents.OnStoppingTraces_GameThread();
		}
		else if (TraceStatus == EStateTreeTraceStatus::Cleared)
		{
			GBufferedEvents.Reset_GameThread();
		}
	});
}

void UnregisterGlobalDelegates()
{
	FWorldDelegates::OnWorldTickStart.Remove(GOnWorldTickStartDelegateHandle);
	GOnWorldTickStartDelegateHandle.Reset();

	UE::StateTree::Delegates::OnTracingStateChanged.Remove(GTracingStateChangedDelegateHandle);
	GTracingStateChangedDelegateHandle.Reset();
}

void TraceInstance(TNotNull<const FStateTreeReadOnlyExecutionContext*> Context)
{
	if (!UE_TRACE_CHANNELEXPR_IS_ENABLED(StateTreeDebugChannel))
	{
		return;
	}

	TraceBufferedEvents(Context);
}

void OutputAssetDebugIdEvent(
	const UStateTree* StateTree,
	const FStateTreeIndex16 AssetDebugId)
{
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(StateTreeDebugChannel))
	{
		check(StateTree);
		const FString TreeName = StateTree->GetName();
		const FString TreePath = StateTree->GetPathName();

		UE_TRACE_LOG(StateTreeDebugger, AssetDebugIdEvent, StateTreeDebugChannel)
			<< AssetDebugIdEvent.Cycle(FPlatformTime::Cycles64())
			<< AssetDebugIdEvent.TreeName(*TreeName, TreeName.Len())
			<< AssetDebugIdEvent.TreePath(*TreePath, TreePath.Len())
			<< AssetDebugIdEvent.CompiledDataHash(StateTree->LastCompiledEditorDataHash)
			<< AssetDebugIdEvent.AssetDebugId(AssetDebugId.Get());
	}
}

void OutputInstanceLifetimeEvent(
	TNotNull<const FStateTreeReadOnlyExecutionContext*> Context,
	const EStateTreeTraceEventType EventType)
{
	const UObject* Owner = Context->GetOwner();

	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(StateTreeDebugChannel))
	{
		if (EventType == EStateTreeTraceEventType::Push)
		{
			TraceInstance(Context);
		}
		else if (EventType == EStateTreeTraceEventType::Pop)
		{
			const FStateTreeInstanceDebugId InstanceId = Context->GetInstanceDebugId();

			// Emit the StateTree-specific InstanceEvent so the trace provider
			// sets Lifetime upper bound (making IsInstanceActive() return false).
			UE_TRACE_LOG(StateTreeDebugger, InstanceEvent, StateTreeDebugChannel)
				<< InstanceEvent.Cycle(FPlatformTime::Cycles64())
				<< InstanceEvent.InstanceId(InstanceId.Id)
				<< InstanceEvent.InstanceSerial(InstanceId.SerialNumber)
				<< InstanceEvent.EventType(static_cast<std::underlying_type_t<EStateTreeTraceEventType>>(EStateTreeTraceEventType::Pop));

			TRACE_INSTANCE_LIFETIME_END(Owner, InstanceId.ToUint64());
		}
	}
}

void OutputInstanceAssetEvent(
	TNotNull<const FStateTreeReadOnlyExecutionContext*> Context,
	const UStateTree* StateTree)
{
	check(StateTree != nullptr);

	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(StateTreeDebugChannel))
	{
		const FStateTreeInstanceDebugId InstanceId = Context->GetInstanceDebugId();

		TraceInstance(Context);

		const FStateTreeIndex16 AssetDebugId = GBufferedEvents.FindOrAddDebugIdForAsset_AnyThread(StateTree);

		UE_TRACE_LOG(StateTreeDebugger, InstanceFrameEvent, StateTreeDebugChannel)
			<< InstanceFrameEvent.Cycle(FPlatformTime::Cycles64())
			<< InstanceFrameEvent.InstanceId(InstanceId.Id)
			<< InstanceFrameEvent.InstanceSerial(InstanceId.SerialNumber)
			<< InstanceFrameEvent.AssetDebugId(AssetDebugId.Get());
	}
}

void OutputInstanceFrameEvent(
	TNotNull<const FStateTreeReadOnlyExecutionContext*> Context,
	const FStateTreeExecutionFrame* Frame)
{
	check(Frame != nullptr);
	OutputInstanceAssetEvent(Context, Frame->StateTree.Get());
}

void OutputLogEventTrace(
	TNotNull<const FStateTreeReadOnlyExecutionContext*> Context,
	const ELogVerbosity::Type Verbosity,
	const TCHAR* Fmt, ...)
{
	const FStateTreeInstanceDebugId InstanceId = Context->GetInstanceDebugId();

	static TCHAR TraceStaticBuffer[8192];
	GET_TYPED_VARARGS(TCHAR, TraceStaticBuffer, UE_ARRAY_COUNT(TraceStaticBuffer), UE_ARRAY_COUNT(TraceStaticBuffer) - 1, Fmt, Fmt);

	TraceInstance(Context);

	UE_TRACE_LOG(StateTreeDebugger, LogEvent, StateTreeDebugChannel)
		<< LogEvent.Cycle(FPlatformTime::Cycles64())
		<< LogEvent.InstanceId(InstanceId.Id)
		<< LogEvent.InstanceSerial(InstanceId.SerialNumber)
		<< LogEvent.Verbosity(Verbosity)
		<< LogEvent.Message(TraceStaticBuffer);
}

void OutputStateEventTrace(
	TNotNull<const FStateTreeReadOnlyExecutionContext*> Context,
	const FStateTreeStateHandle StateHandle,
	const EStateTreeTraceEventType EventType)
{
	const FStateTreeInstanceDebugId InstanceId = Context->GetInstanceDebugId();

	TraceInstance(Context);

	UE_TRACE_LOG(StateTreeDebugger, StateEvent, StateTreeDebugChannel)
		<< StateEvent.Cycle(FPlatformTime::Cycles64())
		<< StateEvent.InstanceId(InstanceId.Id)
		<< StateEvent.InstanceSerial(InstanceId.SerialNumber)
		<< StateEvent.StateIndex(StateHandle.Index)
		<< StateEvent.EventType(static_cast<std::underlying_type_t<EStateTreeTraceEventType>>(EventType));
}

void OutputTaskEventTrace(
	TNotNull<const FStateTreeReadOnlyExecutionContext*> Context,
	const FStateTreeIndex16 TaskIdx,
	const FStateTreeDataView DataView,
	EStateTreeTraceEventType EventType,
	EStateTreeRunStatus Status)
{
	const FStateTreeInstanceDebugId InstanceId = Context->GetInstanceDebugId();

	FBufferArchive Archive;
	SerializeDebugDataToArchive(Archive, Context->StealNodeCustomDebugTraceData(), DataView);

	TraceInstance(Context);

	UE_TRACE_LOG(StateTreeDebugger, TaskEvent, StateTreeDebugChannel)
		<< TaskEvent.Cycle(FPlatformTime::Cycles64())
		<< TaskEvent.InstanceId(InstanceId.Id)
		<< TaskEvent.InstanceSerial(InstanceId.SerialNumber)
		<< TaskEvent.NodeIndex(TaskIdx.Get())
		<< TaskEvent.DataView(Archive.GetData(), Archive.Num())
		<< TaskEvent.EventType(static_cast<std::underlying_type_t<EStateTreeTraceEventType>>(EventType))
		<< TaskEvent.Status(static_cast<std::underlying_type_t<EStateTreeRunStatus>>(Status));
}

void OutputEvaluatorEventTrace(
	TNotNull<const FStateTreeReadOnlyExecutionContext*> Context,
	const FStateTreeIndex16 EvaluatorIdx,
	const FStateTreeDataView DataView,
	EStateTreeTraceEventType EventType)
{
	const FStateTreeInstanceDebugId InstanceId = Context->GetInstanceDebugId();

	FBufferArchive Archive;
	SerializeDebugDataToArchive(Archive, Context->StealNodeCustomDebugTraceData(), DataView);

	TraceInstance(Context);

	UE_TRACE_LOG(StateTreeDebugger, EvaluatorEvent, StateTreeDebugChannel)
		<< EvaluatorEvent.Cycle(FPlatformTime::Cycles64())
		<< EvaluatorEvent.InstanceId(InstanceId.Id)
		<< EvaluatorEvent.InstanceSerial(InstanceId.SerialNumber)
		<< EvaluatorEvent.NodeIndex(EvaluatorIdx.Get())
		<< EvaluatorEvent.DataView(Archive.GetData(), Archive.Num())
		<< EvaluatorEvent.EventType(static_cast<std::underlying_type_t<EStateTreeTraceEventType>>(EventType));
}

void OutputTransitionEventTrace(
	TNotNull<const FStateTreeReadOnlyExecutionContext*> Context,
	const FStateTreeTransitionSource Source,
	const EStateTreeTraceEventType EventType
)
{
	const FStateTreeInstanceDebugId InstanceId = Context->GetInstanceDebugId();

	// Output an instance Frame/Asset event to put the transition data in the proper context when it gets read.
	if (const UStateTree* StateTree = Source.Asset.Get())
	{
		OutputInstanceAssetEvent(Context, StateTree);
	}

	FBufferArchive Archive;
	Archive << EventType;

	TraceInstance(Context);

	UE_TRACE_LOG(StateTreeDebugger, TransitionEvent, StateTreeDebugChannel)
	<< TransitionEvent.Cycle(FPlatformTime::Cycles64())
	<< TransitionEvent.InstanceId(InstanceId.Id)
	<< TransitionEvent.InstanceSerial(InstanceId.SerialNumber)
	<< TransitionEvent.SourceType(static_cast<std::underlying_type_t<EStateTreeTransitionSourceType>>(Source.SourceType))
	<< TransitionEvent.TransitionIndex(Source.TransitionIndex.Get())
	<< TransitionEvent.TargetStateIndex(Source.TargetState.Index)
	<< TransitionEvent.Priority(static_cast<std::underlying_type_t<EStateTreeTransitionPriority>>(Source.Priority))
	<< TransitionEvent.EventType(static_cast<std::underlying_type_t<EStateTreeTraceEventType>>(EventType));
}

void OutputConditionEventTrace(
	TNotNull<const FStateTreeReadOnlyExecutionContext*> Context,
	const FStateTreeIndex16 ConditionIdx,
	const FStateTreeDataView DataView,
	EStateTreeTraceEventType EventType)
{
	const FStateTreeInstanceDebugId InstanceId = Context->GetInstanceDebugId();

	FBufferArchive Archive;
	SerializeDebugDataToArchive(Archive, Context->StealNodeCustomDebugTraceData(), DataView);

	TraceInstance(Context);

	UE_TRACE_LOG(StateTreeDebugger, ConditionEvent, StateTreeDebugChannel)
	<< ConditionEvent.Cycle(FPlatformTime::Cycles64())
	<< ConditionEvent.InstanceId(InstanceId.Id)
	<< ConditionEvent.InstanceSerial(InstanceId.SerialNumber)
	<< ConditionEvent.NodeIndex(ConditionIdx.Get())
	<< ConditionEvent.DataView(Archive.GetData(), Archive.Num())
	<< ConditionEvent.EventType(static_cast<std::underlying_type_t<EStateTreeTraceEventType>>(EventType));
}

void OutputActiveStatesEventTrace(
	TNotNull<const FStateTreeReadOnlyExecutionContext*> Context,
	const TConstArrayView<FStateTreeExecutionFrame> ActiveFrames
)
{
	const FStateTreeInstanceDebugId InstanceId = Context->GetInstanceDebugId();

	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(StateTreeDebugChannel))
	{
		TraceInstance(Context);

		GBufferedEvents.OutputActiveStates_AnyThread(InstanceId, ActiveFrames);
	}
}

// Deprecated
FStateTreeIndex16 FindOrAddDebugIdForAsset(const UStateTree* StateTree)
{
	return GBufferedEvents.FindOrAddDebugIdForAsset_AnyThread(StateTree);
}

// Deprecated
void OutputPhaseScopeEvent(const FStateTreeInstanceDebugId InstanceId, const EStateTreeUpdatePhase Phase, const EStateTreeTraceEventType EventType, const FStateTreeStateHandle StateHandle)
{
}

// Deprecated
void OutputInstanceLifetimeEvent(const FStateTreeInstanceDebugId InstanceId, const UStateTree* StateTree, const TCHAR* InstanceName, const EStateTreeTraceEventType EventType)
{
}

// Deprecated
void OutputInstanceAssetEvent(const FStateTreeInstanceDebugId InstanceId, const UStateTree* StateTree)
{
}
// Deprecated
void OutputInstanceFrameEvent(const FStateTreeInstanceDebugId InstanceId, const FStateTreeExecutionFrame* Frame)
{
}

// Deprecated
void OutputLogEventTrace(const FStateTreeInstanceDebugId InstanceId, const ELogVerbosity::Type Verbosity, const TCHAR* Fmt, ...)
{
}

// Deprecated
void OutputStateEventTrace(const FStateTreeInstanceDebugId InstanceId, const FStateTreeStateHandle StateHandle, const EStateTreeTraceEventType EventType)
{
}

// Deprecated
void OutputTaskEventTrace(const FStateTreeInstanceDebugId InstanceId, FNodeCustomDebugData&& DebugData, const FStateTreeIndex16 TaskIdx, const FStateTreeDataView DataView, const EStateTreeTraceEventType EventType, const EStateTreeRunStatus Status)
{
}

// Deprecated
void OutputEvaluatorEventTrace(const FStateTreeInstanceDebugId InstanceId, FNodeCustomDebugData&& DebugData, const FStateTreeIndex16 EvaluatorIdx, const FStateTreeDataView DataView, const EStateTreeTraceEventType EventType)
{
}

// Deprecated
void OutputConditionEventTrace(const FStateTreeInstanceDebugId InstanceId, FNodeCustomDebugData&& DebugData, const FStateTreeIndex16 ConditionIdx, const FStateTreeDataView DataView, const EStateTreeTraceEventType EventType)
{
}

// Deprecated
void OutputTransitionEventTrace(const FStateTreeInstanceDebugId InstanceId, const FStateTreeTransitionSource TransitionSource, const EStateTreeTraceEventType EventType)
{
}

// Deprecated
void OutputActiveStatesEventTrace(const FStateTreeInstanceDebugId InstanceId, const TConstArrayView<FStateTreeExecutionFrame> ActiveFrames)
{
}

} // UE::StateTreeTrace

#endif // WITH_STATETREE_TRACE
