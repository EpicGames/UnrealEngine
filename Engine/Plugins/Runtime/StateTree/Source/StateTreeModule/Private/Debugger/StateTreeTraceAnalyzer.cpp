// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_STATETREE_TRACE_DEBUGGER

#include "Debugger/StateTreeTraceAnalyzer.h"
#include "Common/ProviderLock.h"
#include "Debugger/StateTreeDebugger.h"
#include "Debugger/StateTreeTraceProvider.h"
#include "Debugger/StateTreeTraceTypes.h"
#include "Serialization/MemoryReader.h"
#include "TraceServices/Model/AnalysisSession.h"

FStateTreeTraceAnalyzer::FStateTreeTraceAnalyzer(TraceServices::IAnalysisSession& InSession, FStateTreeTraceProvider& InProvider)
	: Session(InSession)
	, Provider(InProvider)
{
}

void FStateTreeTraceAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_AssetDebugId, "StateTreeDebugger", "AssetDebugIdEvent");
	Builder.RouteEvent(RouteId_WorldTimestamp, "StateTreeDebugger", "WorldTimestampEvent");
	Builder.RouteEvent(RouteId_Instance, "StateTreeDebugger", "InstanceEvent");
	Builder.RouteEvent(RouteId_InstanceFrame, "StateTreeDebugger", "InstanceFrameEvent");
	Builder.RouteEvent(RouteId_Phase, "StateTreeDebugger", "PhaseEvent");
	Builder.RouteEvent(RouteId_LogMessage, "StateTreeDebugger", "LogEvent");
	Builder.RouteEvent(RouteId_State, "StateTreeDebugger", "StateEvent");
	Builder.RouteEvent(RouteId_Task, "StateTreeDebugger", "TaskEvent");
	Builder.RouteEvent(RouteId_Evaluator, "StateTreeDebugger", "EvaluatorEvent");
	Builder.RouteEvent(RouteId_Transition, "StateTreeDebugger", "TransitionEvent");
	Builder.RouteEvent(RouteId_Condition, "StateTreeDebugger", "ConditionEvent");
	Builder.RouteEvent(RouteId_ActiveStates, "StateTreeDebugger", "ActiveStatesEvent");
}

bool FStateTreeTraceAnalyzer::OnEvent(const uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FStateTreeAnalyzer"));

	TraceServices::FProviderEditScopeLock ProviderEditScope(Provider);

	const auto& EventData = Context.EventData;
	switch (RouteId)
	{
	case RouteId_WorldTimestamp:
		{
			WorldTime = EventData.GetValue<double>("WorldTime");
			break;
		}
	case RouteId_AssetDebugId:
		{
			FString ObjectName, ObjectPathName;
			EventData.GetString("TreeName", ObjectName);
			EventData.GetString("TreePath", ObjectPathName);

			TWeakObjectPtr<const UStateTree> WeakStateTree;
			{
				// This might not work when using a debugger on a client but should be fine in Editor as long as
				// we are not trying to find the object during GC. We might not currently be in the game thread.  
				// @todo STDBG: eventually errors should be reported in the UI
				FGCScopeGuard Guard;
				WeakStateTree = FindObject<UStateTree>(nullptr, *ObjectPathName);
			}

			if (const UStateTree* StateTree = WeakStateTree.Get())
			{
				const uint32 CompiledDataHash = EventData.GetValue<uint32>("CompiledDataHash");
				if (StateTree->LastCompiledEditorDataHash == CompiledDataHash)
				{
					Provider.AppendAssetDebugId(StateTree, FStateTreeIndex16(EventData.GetValue<uint16>("AssetDebugId")));
				}
				else
				{
					UE_LOGF(LogStateTree, Warning, "Traces are not using the same StateTree asset version as the current asset.");
				}
			}
			else
			{
				UE_LOGF(LogStateTree, Warning, "Unable to find StateTree asset: %ls : %ls", *ObjectPathName, *ObjectName);
			}
			break;
		}
	case RouteId_Instance:
		{
			FString InstanceName;
			EventData.GetString("InstanceName", InstanceName);

			Provider.AppendInstanceEvent(FStateTreeIndex16(EventData.GetValue<uint16>("AssetDebugId")),
				FStateTreeInstanceDebugId(EventData.GetValue<uint32>("InstanceId"), EventData.GetValue<uint32>("InstanceSerial")),
				*InstanceName,
				Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Cycle")),
				WorldTime,
				EventData.GetValue<EStateTreeTraceEventType>("EventType"));
			break;
		}
	case RouteId_InstanceFrame:
		{
			TWeakObjectPtr<const UStateTree> WeakStateTree;
			if (Provider.GetAssetFromDebugId(FStateTreeIndex16(EventData.GetValue<uint16>("AssetDebugId")), WeakStateTree))
			{
				const FStateTreeTraceInstanceFrameEvent Event(WorldTime, EStateTreeTraceEventType::Push, WeakStateTree.Get());
			
				Provider.AppendEvent(FStateTreeInstanceDebugId(EventData.GetValue<uint32>("InstanceId"), EventData.GetValue<uint32>("InstanceSerial")),
					Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Cycle")),
					FStateTreeTraceEventVariantType(TInPlaceType<FStateTreeTraceInstanceFrameEvent>(), Event));
			}
			else
			{
				UE_LOGF(LogStateTree, Error, "Instance frame event refers to an asset Id that wasn't added previously.");
			}

			break;
		}
	case RouteId_Phase:
		{
			const FStateTreeTracePhaseEvent Event(WorldTime,
				EventData.GetValue<EStateTreeUpdatePhase>("Phase"),
				EventData.GetValue<EStateTreeTraceEventType>("EventType"),
				FStateTreeStateHandle(EventData.GetValue<uint16>("StateIndex")));

			Provider.AppendEvent(FStateTreeInstanceDebugId(EventData.GetValue<uint32>("InstanceId"), EventData.GetValue<uint32>("InstanceSerial")),
				Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Cycle")),
				FStateTreeTraceEventVariantType(TInPlaceType<FStateTreeTracePhaseEvent>(), Event));
			break;
		}
	case RouteId_LogMessage:
		{
			FString Message;
        	EventData.GetString("Message", Message);
			const FStateTreeTraceLogEvent Event(WorldTime, EventData.GetValue<ELogVerbosity::Type>("Verbosity"), Message);

			Provider.AppendEvent(FStateTreeInstanceDebugId(EventData.GetValue<uint32>("InstanceId"), EventData.GetValue<uint32>("InstanceSerial")),
				Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Cycle")),
				FStateTreeTraceEventVariantType(TInPlaceType<FStateTreeTraceLogEvent>(), Event));
			break;
		}
	case RouteId_State:
		{
			const FStateTreeTraceStateEvent Event(WorldTime,
				FStateTreeIndex16(EventData.GetValue<uint16>("StateIndex")),
				EventData.GetValue<EStateTreeTraceEventType>("EventType"));

			Provider.AppendEvent(FStateTreeInstanceDebugId(EventData.GetValue<uint32>("InstanceId"), EventData.GetValue<uint32>("InstanceSerial")),
				Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Cycle")),
				FStateTreeTraceEventVariantType(TInPlaceType<FStateTreeTraceStateEvent>(), Event));
			break;
		}
	case RouteId_Task:
		{
			FString TypePath, DataAsText, DebugText;
			FMemoryReaderView Archive(EventData.GetArrayView<uint8>("DataView"));
			Archive << TypePath;
			Archive << DataAsText;
			Archive << DebugText;

			const FStateTreeTraceTaskEvent Event(WorldTime,
				FStateTreeIndex16(EventData.GetValue<uint16>("NodeIndex")),
				EventData.GetValue<EStateTreeTraceEventType>("EventType"),
				EventData.GetValue<EStateTreeRunStatus>("Status"),
				TypePath, DataAsText, DebugText);

			Provider.AppendEvent(FStateTreeInstanceDebugId(EventData.GetValue<uint32>("InstanceId"), EventData.GetValue<uint32>("InstanceSerial")),
				Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Cycle")),
				FStateTreeTraceEventVariantType(TInPlaceType<FStateTreeTraceTaskEvent>(), Event));
			break;
		}
	case RouteId_Evaluator:
		{
			FString TypePath, DataAsText, DebugText;
			FMemoryReaderView Archive(EventData.GetArrayView<uint8>("DataView"));
			Archive << TypePath;
			Archive << DataAsText;
			Archive << DebugText;

			const FStateTreeTraceEvaluatorEvent Event(WorldTime,
				FStateTreeIndex16(EventData.GetValue<uint16>("NodeIndex")),
				EventData.GetValue<EStateTreeTraceEventType>("EventType"),
				TypePath, DataAsText, DebugText);

			Provider.AppendEvent(FStateTreeInstanceDebugId(EventData.GetValue<uint32>("InstanceId"), EventData.GetValue<uint32>("InstanceSerial")),
				Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Cycle")),
				FStateTreeTraceEventVariantType(TInPlaceType<FStateTreeTraceEvaluatorEvent>(), Event));
			break;
		}
	case RouteId_Condition:
		{
			FString TypePath, DataAsText, DebugText;
			FMemoryReaderView Archive(EventData.GetArrayView<uint8>("DataView"));
			Archive << TypePath;
			Archive << DataAsText;
			Archive << DebugText;

			const FStateTreeTraceConditionEvent Event(WorldTime,
				FStateTreeIndex16(EventData.GetValue<uint16>("NodeIndex")),
				EventData.GetValue<EStateTreeTraceEventType>("EventType"),
				TypePath, DataAsText, DebugText);
			
			Provider.AppendEvent(FStateTreeInstanceDebugId(EventData.GetValue<uint32>("InstanceId"), EventData.GetValue<uint32>("InstanceSerial")),
				Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Cycle")),
				FStateTreeTraceEventVariantType(TInPlaceType<FStateTreeTraceConditionEvent>(), Event));
			break;
		}
	case RouteId_Transition:
		{
			const FStateTreeTraceTransitionEvent Event(WorldTime,
				FStateTreeTransitionSource(
					nullptr, // we simply reuse the struct to hold the received data, we don't need to provide a StateTree
					EventData.GetValue<EStateTreeTransitionSourceType>("SourceType"),
					FStateTreeIndex16(EventData.GetValue<uint16>("TransitionIndex")),
					FStateTreeStateHandle(EventData.GetValue<uint16>("TargetStateIndex")),
					EventData.GetValue<EStateTreeTransitionPriority>("Priority")
					),
				EventData.GetValue<EStateTreeTraceEventType>("EventType"));
			
			Provider.AppendEvent(FStateTreeInstanceDebugId(EventData.GetValue<uint32>("InstanceId"), EventData.GetValue<uint32>("InstanceSerial")),
				Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Cycle")),
				FStateTreeTraceEventVariantType(TInPlaceType<FStateTreeTraceTransitionEvent>(), Event));
			break;
		}
	case RouteId_ActiveStates:
		{
			FStateTreeTraceActiveStatesEvent Event(WorldTime);

			TArray<FStateTreeStateHandle> ActiveStates(EventData.GetArrayView<uint16>("ActiveStates"));
			TArray<uint16> AssetDebugIds(EventData.GetArrayView<uint16>("AssetDebugIds"));

			if (ensureMsgf(ActiveStates.Num() == AssetDebugIds.Num(), TEXT("Each state is expected to have a matching asset id")))
			{
				FStateTreeInstanceDebugId InstanceDebugId(EventData.GetValue<uint32>("InstanceId"), EventData.GetValue<uint32>("InstanceSerial"));
				TWeakObjectPtr<const UStateTree> WeakStateTree;

				// If empty, we create the event with an empty list of states for the main StateTree.
				if (ActiveStates.IsEmpty())
				{
					if (Provider.GetAssetFromInstanceId(InstanceDebugId, WeakStateTree))
					{
						FStateTreeTraceActiveStates::FAssetActiveStates& NewPair = Event.ActiveStates.PerAssetStates.Emplace_GetRef();
						NewPair.WeakStateTree = WeakStateTree;
					}
				}
				else
				{
					FStateTreeIndex16 LastAssetDebugId;
					FStateTreeTraceActiveStates::FAssetActiveStates* AssetActiveStates = nullptr;
					for (int Index = 0; Index < ActiveStates.Num(); ++Index)
					{
						FStateTreeIndex16 AssetDebugId(AssetDebugIds[Index]);
						if (AssetDebugId != LastAssetDebugId)
						{
							if (Provider.GetAssetFromDebugId(AssetDebugId, WeakStateTree))
							{
								FStateTreeTraceActiveStates::FAssetActiveStates& NewPair = Event.ActiveStates.PerAssetStates.Emplace_GetRef();
								NewPair.WeakStateTree = WeakStateTree;
								AssetActiveStates = &NewPair;
								LastAssetDebugId = AssetDebugId;
							}
							else
							{
								UE_LOGF(LogStateTree, Error, "Instance frame event refers to an asset Id that wasn't added previously.");
								continue;
							}
						}

						check(AssetActiveStates != nullptr);
						AssetActiveStates->ActiveStates.Push(ActiveStates[Index]);
					}
				}

				Provider.AppendEvent(InstanceDebugId,
					Context.EventTime.AsSeconds(EventData.GetValue<uint64>("Cycle")),
					FStateTreeTraceEventVariantType(TInPlaceType<FStateTreeTraceActiveStatesEvent>(), Event));
			}
			break;
		}
	default:
		ensureMsgf(false, TEXT("Unhandle route id: %u"), RouteId);
	}

	return true;
}

#endif // WITH_STATETREE_TRACE_DEBUGGER
