// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_STATETREE_TRACE_DEBUGGER

#include "Debugger/StateTreeTraceModule.h"
#include "Debugger/StateTreeTraceProvider.h"
#include "Debugger/StateTreeTraceAnalyzer.h"
#include "Debugger/StateTreeDebugger.h"  // Required to compile TArray<UE::StateTreeDebugger::FInstanceDescriptor> from StateTreeTraceProvider
#include "TraceServices/Model/AnalysisSession.h"

FName FStateTreeTraceModule::ModuleName("TraceModule_StateTree");

void FStateTreeTraceModule::GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo)
{
	OutModuleInfo.Name = ModuleName;
	OutModuleInfo.DisplayName = TEXT("StateTree");
}

void FStateTreeTraceModule::OnAnalysisBegin(TraceServices::IAnalysisSession& InSession)
{
	const TSharedPtr<FStateTreeTraceProvider> Provider = MakeShared<FStateTreeTraceProvider>(InSession);
	InSession.AddProvider(FStateTreeTraceProvider::ProviderName, Provider);
	InSession.AddAnalyzer(new FStateTreeTraceAnalyzer(InSession, *Provider));
}

void FStateTreeTraceModule::GetLoggers(TArray<const TCHAR *>& OutLoggers)
{
	OutLoggers.Add(TEXT("StateTree"));
}

#endif // WITH_STATETREE_TRACE_DEBUGGER