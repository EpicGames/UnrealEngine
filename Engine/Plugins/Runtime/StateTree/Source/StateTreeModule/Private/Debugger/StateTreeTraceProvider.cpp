// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_STATETREE_TRACE_DEBUGGER

#include "Debugger/StateTreeTraceProvider.h"
#include "Debugger/StateTreeDebugger.h"

namespace TraceServices
{
	thread_local FProviderLock::FThreadLocalState GStateTreeProviderLockState;
}

FName FStateTreeTraceProvider::ProviderName("StateTreeDebuggerProvider");

#define LOCTEXT_NAMESPACE "StateTreeDebuggerProvider"

FStateTreeTraceProvider::FStateTreeTraceProvider(TraceServices::IAnalysisSession& InSession)
	: Session(InSession)
{
}

bool FStateTreeTraceProvider::ReadTimelines(const FStateTreeInstanceDebugId InstanceId, const TFunctionRef<void(const FStateTreeInstanceDebugId ProcessedInstanceId, const FEventsTimeline&)> Callback) const
{
	ReadAccessCheck();

	// Read specific timeline if specified
	if (InstanceId.IsValid())
	{
		const uint32* IndexPtr = InstanceIdToDebuggerEntryTimelines.Find(InstanceId);
		if (IndexPtr != nullptr && EventsTimelines.IsValidIndex(*IndexPtr))
		{
			Callback(InstanceId, *EventsTimelines[*IndexPtr]);
			return true;
		}
	}
	else
	{
		for(auto It = InstanceIdToDebuggerEntryTimelines.CreateConstIterator(); It; ++It)
		{
			if (EventsTimelines.IsValidIndex(It.Value()))
			{
				Callback(It.Key(), *EventsTimelines[It.Value()]);
			}
		}

		return EventsTimelines.Num() > 0;
	}

	return false;
}

bool FStateTreeTraceProvider::ReadTimelines(const UStateTree& StateTree, TFunctionRef<void(const FStateTreeInstanceDebugId ProcessedInstanceId, const FEventsTimeline&)> Callback) const
{
	ReadAccessCheck();

	for (auto It = InstanceIdToDebuggerEntryTimelines.CreateConstIterator(); It; ++It)
	{
		check(EventsTimelines.IsValidIndex(It.Value()));
		check(Descriptors.Num() == EventsTimelines.Num());

		if (Descriptors[It.Value()].Get().StateTree == &StateTree)
		{
			Callback(Descriptors[It.Value()].Get().Id, *EventsTimelines[It.Value()]);
		}
	}

	return EventsTimelines.Num() > 0;
}

void FStateTreeTraceProvider::AppendEvent(const FStateTreeInstanceDebugId InInstanceId, const double InTime, const FStateTreeTraceEventVariantType& InEvent)
{
	EditAccessCheck();

	// It is currently possible to receive events from an instance without receiving event `EStateTreeTraceEventType::Push` first
	// (i.e. traces were activated after the statetree instance execution was started).    
	// We plan to buffer the Instance events (Started/Stopped) to address this but for now we ignore events related to that instance.
	if (const uint32* IndexPtr = InstanceIdToDebuggerEntryTimelines.Find(InInstanceId))
	{
		EventsTimelines[*IndexPtr]->AppendEvent(InTime, InEvent);
	}

	{
		TraceServices::FAnalysisSessionEditScope SessionEditScope(Session);
		Session.UpdateDurationSeconds(InTime);
	}
}

void FStateTreeTraceProvider::AppendInstanceEvent(
	const FStateTreeIndex16 AssetDebugId,
	const FStateTreeInstanceDebugId InInstanceId,
	const TCHAR* InInstanceName,
	const double InTime,
	const double InWorldRecordingTime,
	const EStateTreeTraceEventType InEventType)
{
	EditAccessCheck();
	using namespace UE::StateTreeDebugger;

	if (InEventType == EStateTreeTraceEventType::Push)
	{
		TWeakObjectPtr<const UStateTree> WeakStateTree;
		if (GetAssetFromDebugId(AssetDebugId, WeakStateTree))
		{
			if (const UStateTree* StateTree = WeakStateTree.Get())
			{
				const TSharedRef<FInstanceDescriptor>* Descriptor = Descriptors.FindByPredicate(
					[InInstanceId](const TSharedRef<const FInstanceDescriptor>& Descriptor)
					{
						return Descriptor.Get().Id == InInstanceId;
					});

				// Possible to receive new Push when stopping/starting traces during the same session.
				// In that case we can reuse the existing entries
				if (Descriptor == nullptr)
				{
					TSharedRef<FInstanceDescriptor>& NewDescriptor = Descriptors.Emplace_GetRef(MakeShared<FInstanceDescriptor>(
						StateTree
						, InInstanceId
						, InInstanceName
						, TRange<double>(InWorldRecordingTime, FInstanceDescriptor::ActiveInstanceEndTime)));

					// Build list of statically linked assets
					// This can then be used when processing events and breakpoints set in linked assets
					auto GatherLinkedStateTrees = [](const auto& Self, const UStateTree* InStateTree, TArray<TWeakObjectPtr<const UStateTree>>& OutAssets)->void
						{
							for (const FCompactStateTreeState& State : InStateTree->GetStates())
							{
								if (State.LinkedAsset)
								{
									OutAssets.Add(State.LinkedAsset);
									Self(Self, State.LinkedAsset, OutAssets);
								}
							}
						};
					GatherLinkedStateTrees(GatherLinkedStateTrees, StateTree, NewDescriptor.Get().LinkedStateTrees);

					check(InstanceIdToDebuggerEntryTimelines.Find(InInstanceId) == nullptr);
					InstanceIdToDebuggerEntryTimelines.Add(InInstanceId, EventsTimelines.Num());

					EventsTimelines.Emplace(MakeShared<TraceServices::TPointTimeline<FStateTreeTraceEventVariantType>>(Session.GetLinearAllocator()));
				}
			}
			else
			{
				UE_LOGF(LogStateTree, Error, "Instance event refers to an unloaded asset.");
			}
		}
		else
		{
			UE_LOGF(LogStateTree, Error, "Instance event refers to an asset Id that wasn't added previously.");
		}
	}
	else if (InEventType == EStateTreeTraceEventType::Pop)
	{
		// Process only if timeline can be found. See details in AppendEvent comment.
		if (const uint32* Index = InstanceIdToDebuggerEntryTimelines.Find(InInstanceId))
		{
			check(Descriptors.IsValidIndex(*Index));
			Descriptors[*Index].Get().Lifetime.SetUpperBound(InWorldRecordingTime);
		}
	}

	{
		TraceServices::FAnalysisSessionEditScope SessionEditScope(Session);
		Session.UpdateDurationSeconds(InTime);
	}
}

void FStateTreeTraceProvider::AppendAssetDebugId(const UStateTree* InStateTree, const FStateTreeIndex16 AssetDebugId)
{
	EditAccessCheck();
	TWeakObjectPtr<const UStateTree> WeakStateTree;
	if (ensureMsgf(AssetDebugId.IsValid(), TEXT("Expecting valid asset debug Id."))
		&& !GetAssetFromDebugId(AssetDebugId, WeakStateTree))
	{
		StateTreeAssets.Emplace(FStateTreeDebugIdPair(InStateTree, AssetDebugId));
	}
}

bool FStateTreeTraceProvider::GetAssetFromDebugId(const FStateTreeIndex16 AssetDebugId, TWeakObjectPtr<const UStateTree>& WeakStateTree) const
{
	ReadAccessCheck();
	verifyf(AssetDebugId.IsValid(), TEXT("Expecting valid asset debug Id."));
	const FStateTreeDebugIdPair* ExistingPair = StateTreeAssets.FindByPredicate([AssetDebugId](const FStateTreeDebugIdPair& Pair)
	{
		return Pair.Id == AssetDebugId;
	});

	WeakStateTree = ExistingPair ? ExistingPair->WeakStateTree : nullptr; 

	return ExistingPair != nullptr;
}

bool FStateTreeTraceProvider::GetAssetFromInstanceId(const FStateTreeInstanceDebugId InstanceId, TWeakObjectPtr<const UStateTree>& WeakStateTree) const
{
	ReadAccessCheck();
	if (const uint32* IndexPtr = InstanceIdToDebuggerEntryTimelines.Find(InstanceId))
	{
		check(Descriptors.Num() == EventsTimelines.Num());
		WeakStateTree = Descriptors[*IndexPtr].Get().StateTree;
		return true;
	}

	return false;
}

TSharedPtr<const UE::StateTreeDebugger::FInstanceDescriptor> FStateTreeTraceProvider::GetInstanceDescriptor(const FStateTreeInstanceDebugId InstanceId) const
{
	ReadAccessCheck();
	const TSharedRef<UE::StateTreeDebugger::FInstanceDescriptor>* Descriptor = Descriptors.FindByPredicate([InstanceId](const TSharedRef<const UE::StateTreeDebugger::FInstanceDescriptor>& Descriptor)
	{
		return Descriptor.Get().Id == InstanceId;
	});

	return Descriptor ? Descriptor->ToSharedPtr() : TSharedPtr<const UE::StateTreeDebugger::FInstanceDescriptor>{};
}

void FStateTreeTraceProvider::GetInstances(TArray<const TSharedRef<const UE::StateTreeDebugger::FInstanceDescriptor>>& OutInstances) const
{
	ReadAccessCheck();
	OutInstances.Reserve(Descriptors.Num());
	Algo::Transform(Descriptors, OutInstances, [](const TSharedRef<UE::StateTreeDebugger::FInstanceDescriptor>& Descriptor)
	{
		return Descriptor;
	});
}

#undef LOCTEXT_NAMESPACE

#endif // WITH_STATETREE_TRACE_DEBUGGER
