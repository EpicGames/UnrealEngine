// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debugger/StateTreeTraceTypes.h"


#include "StateTree.h"
#include "StateTreeModule.h"
#include "StateTreeNodeBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeTraceTypes)

#if WITH_STATETREE_TRACE_DEBUGGER

namespace UE::StateTreeTrace
{
	FString GetStateName(const UStateTree& StateTree, const FCompactStateTreeState* CompactState)
	{
		check(CompactState);

		if (const UStateTree* LinkedAsset = CompactState->LinkedAsset.Get())
		{
			return FString::Printf(TEXT("%s > [%s]"), *CompactState->Name.ToString(), *LinkedAsset->GetName());
		}

		if (const FCompactStateTreeState* LinkedState = StateTree.GetStateFromHandle(CompactState->LinkedState))
		{
			return FString::Printf(TEXT("%s > %s"), *CompactState->Name.ToString(), *LinkedState->Name.ToString());
		}

		return CompactState->Name.ToString();
	}
}

//----------------------------------------------------------------------//
// FStateTreeTracePhaseEvent
//----------------------------------------------------------------------//
FString FStateTreeTracePhaseEvent::ToFullString(const UStateTree& StateTree) const
{
	return GetValueString(StateTree);
}

FString FStateTreeTracePhaseEvent::GetValueString(const UStateTree& StateTree) const
{
	TStringBuilder<256> StrBuilder;

	// Override to display only selection behavior
	if (Phase == EStateTreeUpdatePhase::TrySelectBehavior
		&& StateHandle.IsValid())
	{
		if (const FCompactStateTreeState* CompactState = StateTree.GetStateFromHandle(StateHandle))
		{
			return UEnum::GetDisplayValueAsText(CompactState->SelectionBehavior).ToString();
		}

		return FString::Printf(TEXT("Invalid State Index %s for '%s'"), *StateHandle.Describe(), *StateTree.GetFullName());
	}

	// Otherwise build either: "<PhaseDescription> '<StateName>'" or "<StateName>"
	if (Phase != EStateTreeUpdatePhase::Unset)
	{
		StrBuilder.Append(UEnum::GetDisplayValueAsText(Phase).ToString());
	}

	const FCompactStateTreeState* CompactState = StateTree.GetStateFromHandle(StateHandle);
	if (CompactState != nullptr || StateHandle.IsValid())
	{
		if (StrBuilder.Len())
		{
			StrBuilder.Appendf(TEXT(" '%s'"),
				CompactState != nullptr ? *UE::StateTreeTrace::GetStateName(StateTree, CompactState) : *StateHandle.Describe());
		}
		else
		{
			StrBuilder.Appendf(TEXT("%s"),
				CompactState != nullptr ? *UE::StateTreeTrace::GetStateName(StateTree, CompactState) : *StateHandle.Describe());
		}
	}

	return FString(StrBuilder);
}

FString FStateTreeTracePhaseEvent::GetTypeString(const UStateTree& StateTree) const
{
	return *UEnum::GetDisplayValueAsText(EventType).ToString();
}


//----------------------------------------------------------------------//
// FStateTreeTraceLogEvent
//----------------------------------------------------------------------//
FString FStateTreeTraceLogEvent::ToFullString(const UStateTree& StateTree) const
{
	return FString::Printf(TEXT("%s: %s"), *GetTypeString(StateTree), *GetValueString(StateTree));
}

FString FStateTreeTraceLogEvent::GetValueString(const UStateTree& StateTree) const
{
	return (*Message);
}

FString FStateTreeTraceLogEvent::GetTypeString(const UStateTree& StateTree) const
{
	return TEXT("Log");
}


//----------------------------------------------------------------------//
// FStateTreeTracePropertyEvent
//----------------------------------------------------------------------//
FString FStateTreeTracePropertyEvent::ToFullString(const UStateTree& StateTree) const
{
	return FString::Printf(TEXT("%s: %s"), *GetTypeString(StateTree), *GetValueString(StateTree));
}

FString FStateTreeTracePropertyEvent::GetValueString(const UStateTree& StateTree) const
{
	return (*Message);
}

FString FStateTreeTracePropertyEvent::GetTypeString(const UStateTree& StateTree) const
{
	return TEXT("Property value");
}

//----------------------------------------------------------------------//
// FStateTreeTraceTransitionEvent
//----------------------------------------------------------------------//
FString FStateTreeTraceTransitionEvent::ToFullString(const UStateTree& StateTree) const
{
	return FString::Printf(TEXT("%s '%s': %s"), *UEnum::GetDisplayValueAsText(EventType).ToString(), *GetTypeString(StateTree), *GetValueString(StateTree));
}

FString FStateTreeTraceTransitionEvent::GetValueString(const UStateTree& StateTree) const
{
	const FCompactStateTreeState* CompactState = StateTree.GetStateFromHandle(TransitionSource.TargetState);
	TStringBuilder<256> StrBuilder;

	if (TransitionSource.SourceType == EStateTreeTransitionSourceType::Asset)
	{
		if (const FCompactStateTransition* Transition = StateTree.GetTransitionFromIndex(TransitionSource.TransitionIndex))
		{
			ensureAlways(Transition->Priority == TransitionSource.Priority);
			ensureAlways(Transition->State == TransitionSource.TargetState);

			const bool bHasTag = Transition->RequiredEvent.Tag.IsValid();
			const bool bHasPayload = Transition->RequiredEvent.PayloadStruct != nullptr;
			
			if (bHasTag || bHasPayload)
			{
				StrBuilder.Appendf(TEXT("("));
				
				if (bHasTag)
				{
					StrBuilder.Appendf(TEXT("Tag: '%s'"), *Transition->RequiredEvent.Tag.ToString()); 
				}
				if (bHasTag && bHasPayload)
				{
					StrBuilder.Appendf(TEXT(", "));
				}
				if (bHasPayload)
				{
					StrBuilder.Appendf(TEXT(" Payload: '%s'"), *Transition->RequiredEvent.PayloadStruct->GetName()); 
				}
				StrBuilder.Appendf(TEXT(") ")); 
			}
		}
		else
		{
			StrBuilder.Appendf(TEXT("[Invalid Transition Index %s for '%s']"), *LexToString(TransitionSource.TransitionIndex.Get()), *StateTree.GetFullName());
		}
	}
	
	StrBuilder.Appendf(TEXT("go to State '%s'"), CompactState != nullptr ? *UE::StateTreeTrace::GetStateName(StateTree, CompactState) : *TransitionSource.TargetState.Describe());

	if (TransitionSource.Priority != EStateTreeTransitionPriority::None)
	{
		StrBuilder.Appendf(TEXT(" (Priority: %s)"), *UEnum::GetDisplayValueAsText(TransitionSource.Priority).ToString()); 
	}


	return FString(StrBuilder);
}

FString FStateTreeTraceTransitionEvent::GetTypeString(const UStateTree& StateTree) const
{
	if (TransitionSource.SourceType == EStateTreeTransitionSourceType::Asset)
	{
		if (const FCompactStateTransition* Transition = StateTree.GetTransitionFromIndex(TransitionSource.TransitionIndex))
		{
			return *UEnum::GetDisplayValueAsText(Transition->Trigger).ToString();
		}

		return FString::Printf(TEXT("Invalid Transition Index %s for '%s'"), *LexToString(TransitionSource.TransitionIndex.Get()), *StateTree.GetFullName());
	}

	return *UEnum::GetDisplayValueAsText(TransitionSource.SourceType).ToString();
}


//----------------------------------------------------------------------//
// FStateTreeTraceNodeEvent
//----------------------------------------------------------------------//
FString FStateTreeTraceNodeEvent::ToFullString(const UStateTree& StateTree) const
{
	const FConstStructView NodeView = Index.IsValid() ? StateTree.GetNode(Index.Get()) : FConstStructView();
	const FStateTreeNodeBase* Node = NodeView.GetPtr<const FStateTreeNodeBase>();

	return FString::Printf(TEXT("%s '%s (%s)'"),
			*UEnum::GetDisplayValueAsText(EventType).ToString(),
			Node != nullptr ? *Node->Name.ToString() : *LexToString(Index.Get()),
			NodeView.IsValid() ? *NodeView.GetScriptStruct()->GetName() : TEXT("Invalid Node"));
}

FString FStateTreeTraceNodeEvent::GetValueString(const UStateTree& StateTree) const
{
	const FConstStructView NodeView = Index.IsValid() ? StateTree.GetNode(Index.Get()) : FConstStructView();
	const FStateTreeNodeBase* Node = NodeView.GetPtr<const FStateTreeNodeBase>();

	return FString::Printf(TEXT("%s '%s'"),
			*UEnum::GetDisplayValueAsText(EventType).ToString(),
			Node != nullptr ? *Node->Name.ToString() : *LexToString(Index.Get()));
}

FString FStateTreeTraceNodeEvent::GetTypeString(const UStateTree& StateTree) const
{
	const FConstStructView NodeView = Index.IsValid() ? StateTree.GetNode(Index.Get()) : FConstStructView();
	return FString::Printf(TEXT("%s"), NodeView.IsValid() ? *NodeView.GetScriptStruct()->GetName() : TEXT("Invalid Node"));
}


//----------------------------------------------------------------------//
// FStateTreeTraceStateEvent
//----------------------------------------------------------------------//
FStateTreeStateHandle FStateTreeTraceStateEvent::GetStateHandle() const
{
	return FStateTreeStateHandle(Index.Get());
}

FString FStateTreeTraceStateEvent::ToFullString(const UStateTree& StateTree) const
{
	const FStateTreeStateHandle StateHandle(Index.Get());
	if (const FCompactStateTreeState* CompactState = StateTree.GetStateFromHandle(StateHandle))
	{
		if (CompactState->SelectionBehavior != EStateTreeStateSelectionBehavior::None)
		{
			return FString::Printf(TEXT("%s '%s' (%s)"),
				*UEnum::GetDisplayValueAsText(EventType).ToString(),
				*GetValueString(StateTree),
				*UEnum::GetDisplayValueAsText(CompactState->SelectionBehavior).ToString());
		}
	}

	return FString::Printf(TEXT("%s '%s'"),
		*UEnum::GetDisplayValueAsText(EventType).ToString(),
		*GetValueString(StateTree));
}

FString FStateTreeTraceStateEvent::GetValueString(const UStateTree& StateTree) const
{
	const FStateTreeStateHandle StateHandle(Index.Get());
	if (const FCompactStateTreeState* CompactState = StateTree.GetStateFromHandle(StateHandle))
	{
		TArray<FString, TInlineAllocator<FStateTreeActiveStates::MaxStates>> StateHierarchy;
		StateHierarchy.Push(UE::StateTreeTrace::GetStateName(StateTree, CompactState));

		// Climb hierarchy
		for (FStateTreeStateHandle ParentHandle = CompactState->Parent; ParentHandle.IsValid(); ParentHandle = CompactState->Parent)
		{
			if (CompactState = StateTree.GetStateFromHandle(ParentHandle); CompactState != nullptr)
			{
				StateHierarchy.Push(UE::StateTreeTrace::GetStateName(StateTree, CompactState));
			}
			else
			{
				break;
			}
		}

		// Reverse to output from the root
		Algo::Reverse(StateHierarchy);

		return FString::Join(StateHierarchy, TEXT("."));
	}

	return FString::Printf(TEXT("Invalid State Index %s for '%s'"), *StateHandle.Describe(), *StateTree.GetFullName());
}

FString FStateTreeTraceStateEvent::GetTypeString(const UStateTree& StateTree) const
{
	return TEXT("State");
}


//----------------------------------------------------------------------//
// FStateTreeTraceTaskEvent
//----------------------------------------------------------------------//
FString FStateTreeTraceTaskEvent::ToFullString(const UStateTree& StateTree) const
{
	return FString::Printf(TEXT("%s -> %s"), *FStateTreeTraceNodeEvent::ToFullString(StateTree), *UEnum::GetDisplayValueAsText(Status).ToString());
}

FString FStateTreeTraceTaskEvent::GetValueString(const UStateTree& StateTree) const
{
	return IStateTreeModule::Get().GetDebugInfoProvider()->GetNodeDescription(&StateTree, Index).ToString();
}

FString FStateTreeTraceTaskEvent::GetTypeString(const UStateTree& StateTree) const
{
	return FStateTreeTraceNodeEvent::GetTypeString(StateTree);
}


//----------------------------------------------------------------------//
// FStateTreeTraceEvaluatorEvent
//----------------------------------------------------------------------//
FString FStateTreeTraceEvaluatorEvent::ToFullString(const UStateTree& StateTree) const
{
	return FStateTreeTraceNodeEvent::ToFullString(StateTree);
}

FString FStateTreeTraceEvaluatorEvent::GetValueString(const UStateTree& StateTree) const
{
	return IStateTreeModule::Get().GetDebugInfoProvider()->GetNodeDescription(&StateTree, Index).ToString();
}

FString FStateTreeTraceEvaluatorEvent::GetTypeString(const UStateTree& StateTree) const
{
	return FStateTreeTraceNodeEvent::GetTypeString(StateTree);
}


//----------------------------------------------------------------------//
// FStateTreeTraceConditionEvent
//----------------------------------------------------------------------//
FString FStateTreeTraceConditionEvent::ToFullString(const UStateTree& StateTree) const
{
	return FStateTreeTraceNodeEvent::ToFullString(StateTree);
}

FString FStateTreeTraceConditionEvent::GetValueString(const UStateTree& StateTree) const
{
	return IStateTreeModule::Get().GetDebugInfoProvider()->GetNodeDescription(&StateTree, Index).ToString();
}

FString FStateTreeTraceConditionEvent::GetTypeString(const UStateTree& StateTree) const
{
	return FStateTreeTraceNodeEvent::GetTypeString(StateTree);
}


//----------------------------------------------------------------------//
// FStateTreeTraceActiveStatesEvent
//----------------------------------------------------------------------//
FStateTreeTraceActiveStatesEvent::FStateTreeTraceActiveStatesEvent(const double RecordingWorldTime)
	: FStateTreeTraceBaseEvent(RecordingWorldTime, EStateTreeTraceEventType::Unset)
{
}

FString FStateTreeTraceActiveStatesEvent::ToFullString(const UStateTree& StateTree) const
{
	if (ActiveStates.PerAssetStates.Num() > 0)
	{
		return FString::Printf(TEXT("%s: %s"), *GetTypeString(StateTree), *GetValueString(StateTree));
	}
	return TEXT("No active states");
}

FString FStateTreeTraceActiveStatesEvent::GetValueString(const UStateTree&) const
{
	TStringBuilder<256> StatePath;
	for (const FStateTreeTraceActiveStates::FAssetActiveStates& PerAssetStates : ActiveStates.PerAssetStates)
	{
		if (const UStateTree* StateTree = PerAssetStates.WeakStateTree.Get())
		{
			for (const FStateTreeStateHandle Handle : PerAssetStates.ActiveStates)
			{
				const FCompactStateTreeState* State = StateTree->GetStateFromHandle(Handle);
				StatePath.Appendf(TEXT("%s%s"),
					StatePath.Len() == 0 ? TEXT("") : TEXT("."),
					State == nullptr ? TEXT("Invalid") : *UE::StateTreeTrace::GetStateName(*StateTree, State));
			}
		}
	}
	return FString(StatePath);
}

FString FStateTreeTraceActiveStatesEvent::GetTypeString(const UStateTree& StateTree) const
{
	return TEXT("New active states");
}


//----------------------------------------------------------------------//
// FStateTreeTraceInstanceFrameEvent
//----------------------------------------------------------------------//
FStateTreeTraceInstanceFrameEvent::FStateTreeTraceInstanceFrameEvent(const double RecordingWorldTime, const EStateTreeTraceEventType EventType, const UStateTree* StateTree)
	: FStateTreeTraceBaseEvent(RecordingWorldTime, EventType)
	, WeakStateTree(StateTree)
{
}

FString FStateTreeTraceInstanceFrameEvent::ToFullString(const UStateTree& StateTree) const
{
	return FString::Printf(TEXT("%s: %s"), *GetTypeString(StateTree), *GetValueString(StateTree));
}

FString FStateTreeTraceInstanceFrameEvent::GetValueString(const UStateTree& StateTree) const
{
	return GetNameSafe(WeakStateTree.Get());
}

FString FStateTreeTraceInstanceFrameEvent::GetTypeString(const UStateTree& StateTree) const
{
	return TEXT("New instance frame");
}

#endif // WITH_STATETREE_TRACE_DEBUGGER
