// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeBooleanAlgebraPropertyFunctions.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeNodeDescriptionHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeBooleanAlgebraPropertyFunctions)

#define LOCTEXT_NAMESPACE "StateTree"

namespace UE::StateTree::BooleanPropertyFunctions::Internal
{
#if WITH_EDITOR
	FText GetDescriptionForOperation(FText OperationText, const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting)
	{
		const FStateTreeBooleanOperationPropertyFunctionInstanceData& InstanceData = InstanceDataView.Get<FStateTreeBooleanOperationPropertyFunctionInstanceData>();

		FText LeftValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FStateTreeBooleanOperationPropertyFunctionInstanceData, bLeft)), Formatting);
		if (LeftValue.IsEmpty())
		{
			LeftValue = UE::StateTree::DescHelpers::GetBoolText(InstanceData.bLeft, Formatting);
		}

		FText RightValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FStateTreeBooleanOperationPropertyFunctionInstanceData, bRight)), Formatting);
		if (RightValue.IsEmpty())
		{
			RightValue = UE::StateTree::DescHelpers::GetBoolText(InstanceData.bRight, Formatting);
		}

		return UE::StateTree::DescHelpers::GetMathOperationText(OperationText, LeftValue, RightValue, Formatting);
	}
#endif // WITH_EDITOR
} // namespace UE::StateTree::BooleanPropertyFunctions::Internal

void FStateTreeBooleanAndPropertyFunction::Execute(FStateTreeExecutionContext& Context) const
{
	FStateTreeBooleanOperationPropertyFunctionInstanceData& InstanceData = Context.GetInstanceData(*this);
	InstanceData.bResult = InstanceData.bLeft && InstanceData.bRight;
}

#if WITH_EDITOR
FText FStateTreeBooleanAndPropertyFunction::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	return UE::StateTree::BooleanPropertyFunctions::Internal::GetDescriptionForOperation(LOCTEXT("BoolAnd", "and"), ID, InstanceDataView, BindingLookup, Formatting);
}
#endif // WITH_EDITOR

void FStateTreeBooleanOrPropertyFunction::Execute(FStateTreeExecutionContext& Context) const
{
	FStateTreeBooleanOperationPropertyFunctionInstanceData& InstanceData = Context.GetInstanceData(*this);
	InstanceData.bResult = InstanceData.bLeft || InstanceData.bRight;
}

#if WITH_EDITOR
FText FStateTreeBooleanOrPropertyFunction::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	return UE::StateTree::BooleanPropertyFunctions::Internal::GetDescriptionForOperation(LOCTEXT("BoolOr", "or"), ID, InstanceDataView, BindingLookup, Formatting);
}
#endif // WITH_EDITOR

void FStateTreeBooleanXOrPropertyFunction::Execute(FStateTreeExecutionContext& Context) const
{
	FStateTreeBooleanOperationPropertyFunctionInstanceData& InstanceData = Context.GetInstanceData(*this);
	InstanceData.bResult = InstanceData.bLeft ^ InstanceData.bRight;
}

#if WITH_EDITOR
FText FStateTreeBooleanXOrPropertyFunction::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	return UE::StateTree::BooleanPropertyFunctions::Internal::GetDescriptionForOperation(LOCTEXT("BoolXOr", "xor"), ID, InstanceDataView, BindingLookup, Formatting);
}
#endif // WITH_EDITOR

void FStateTreeBooleanNotPropertyFunction::Execute(FStateTreeExecutionContext& Context) const
{
	FStateTreeBooleanNotOperationPropertyFunctionInstanceData& InstanceData = Context.GetInstanceData(*this);
	InstanceData.bResult = !InstanceData.bInput;
}

#if WITH_EDITOR
FText FStateTreeBooleanNotPropertyFunction::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	const FStateTreeBooleanNotOperationPropertyFunctionInstanceData& InstanceData = InstanceDataView.Get<FStateTreeBooleanNotOperationPropertyFunctionInstanceData>();

	FText InputValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FStateTreeBooleanNotOperationPropertyFunctionInstanceData, bInput)), Formatting);
	if (InputValue.IsEmpty())
	{
		InputValue = UE::StateTree::DescHelpers::GetBoolText(InstanceData.bInput, Formatting);
	}

	const FText Format = (Formatting == EStateTreeNodeFormatting::RichText)
		? LOCTEXT("BoolNotFuncRich", "(<s>Not</> {Input})")
		: LOCTEXT("BoolNotFunc", "(Not {Input})");

	return FText::FormatNamed(Format, TEXT("Input"), InputValue);
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
