// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreePropertyFunctionBase.h"
#include "StateTreeBooleanAlgebraPropertyFunctions.generated.h"

#define UE_API STATETREEMODULE_API

struct FStateTreeExecutionContext;

USTRUCT()
struct FStateTreeBooleanOperationPropertyFunctionInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Param)
	bool bLeft = false;

	UPROPERTY(EditAnywhere, Category = Param)
	bool bRight = false;

	UPROPERTY(EditAnywhere, Category = Output)
	bool bResult = false;
};

/**
 * Performs 'And' operation on two booleans.
 */
USTRUCT(meta=(DisplayName = "And", Category="Logic"))
struct FStateTreeBooleanAndPropertyFunction : public FStateTreePropertyFunctionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeBooleanOperationPropertyFunctionInstanceData;

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	UE_API virtual void Execute(FStateTreeExecutionContext& Context) const override;

#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const;
#endif
};

/**
 * Performs 'Or' operation on two booleans.
 */
USTRUCT(meta=(DisplayName = "Or", Category="Logic"))
struct FStateTreeBooleanOrPropertyFunction : public FStateTreePropertyFunctionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeBooleanOperationPropertyFunctionInstanceData;

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	UE_API virtual void Execute(FStateTreeExecutionContext& Context) const override;

#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const;
#endif
};

/**
 * Performs 'Exclusive Or' operation on two booleans.
 */
USTRUCT(meta=(DisplayName = "XOr", Category="Logic"))
struct FStateTreeBooleanXOrPropertyFunction : public FStateTreePropertyFunctionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeBooleanOperationPropertyFunctionInstanceData;

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	UE_API virtual void Execute(FStateTreeExecutionContext& Context) const override;

#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const;
#endif
};

USTRUCT()
struct FStateTreeBooleanNotOperationPropertyFunctionInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Param)
	bool bInput = false;

	UPROPERTY(EditAnywhere, Category = Output)
	bool bResult = false;
};

/**
 * Performs 'Not' operation on a boolean.
 */
USTRUCT(meta=(DisplayName = "Not", Category="Logic"))
struct FStateTreeBooleanNotPropertyFunction : public FStateTreePropertyFunctionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeBooleanNotOperationPropertyFunctionInstanceData;

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	UE_API virtual void Execute(FStateTreeExecutionContext& Context) const override;

#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const;
#endif
};

#undef UE_API
