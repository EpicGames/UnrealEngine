// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyFunctions/StateTreeFloatPropertyFunctions.h"

#include "StateTreeExecutionContext.h"
#include "StateTreeNodeDescriptionHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeFloatPropertyFunctions)

#define LOCTEXT_NAMESPACE "StateTree"

void FStateTreeAddFloatPropertyFunction::Execute(FStateTreeExecutionContext& Context) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	InstanceData.Result = InstanceData.Left + InstanceData.Right;
}

void FStateTreeSubtractFloatPropertyFunction::Execute(FStateTreeExecutionContext& Context) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	InstanceData.Result = InstanceData.Left - InstanceData.Right;
}

void FStateTreeMultiplyFloatPropertyFunction::Execute(FStateTreeExecutionContext& Context) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	InstanceData.Result = InstanceData.Left * InstanceData.Right;
}

void FStateTreeDivideFloatPropertyFunction::Execute(FStateTreeExecutionContext& Context) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	if (InstanceData.Right != 0)
	{
		InstanceData.Result = InstanceData.Left / InstanceData.Right;
	}
	else
	{
		InstanceData.Result = 0;
	}
}

void FStateTreeInvertFloatPropertyFunction::Execute(FStateTreeExecutionContext& Context) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	InstanceData.Result = -InstanceData.Input;
}

void FStateTreeAbsoluteFloatPropertyFunction::Execute(FStateTreeExecutionContext& Context) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	InstanceData.Result = FMath::Abs(InstanceData.Input);
}

#if WITH_EDITOR
FText FStateTreeAddFloatPropertyFunction::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	return UE::StateTree::DescHelpers::GetDescriptionForMathOperation<FInstanceDataType>(LOCTEXT("FloatAdd", "+"), ID, InstanceDataView, BindingLookup, Formatting);
}

FText FStateTreeSubtractFloatPropertyFunction::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	return UE::StateTree::DescHelpers::GetDescriptionForMathOperation<FInstanceDataType>(LOCTEXT("FloatSubtract", "-"), ID, InstanceDataView, BindingLookup, Formatting);
}

FText FStateTreeMultiplyFloatPropertyFunction::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	return UE::StateTree::DescHelpers::GetDescriptionForMathOperation<FInstanceDataType>(LOCTEXT("FloatMultiply", "*"), ID, InstanceDataView, BindingLookup, Formatting);
}

FText FStateTreeDivideFloatPropertyFunction::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	return UE::StateTree::DescHelpers::GetDescriptionForMathOperation<FInstanceDataType>(LOCTEXT("TreeFloatDivide", "/"), ID, InstanceDataView, BindingLookup, Formatting);
}

FText FStateTreeInvertFloatPropertyFunction::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	return UE::StateTree::DescHelpers::GetDescriptionForSingleParameterFunc<FInstanceDataType>(LOCTEXT("FloatInvert", "-"), ID, InstanceDataView, BindingLookup, Formatting);
}

FText FStateTreeAbsoluteFloatPropertyFunction::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	return UE::StateTree::DescHelpers::GetDescriptionForSingleParameterFunc<FInstanceDataType>(LOCTEXT("FloatAbsolute", "Abs"), ID, InstanceDataView, BindingLookup, Formatting);
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
