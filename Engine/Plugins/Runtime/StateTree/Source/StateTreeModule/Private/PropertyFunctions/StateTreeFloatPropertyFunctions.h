// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreePropertyFunctionBase.h"
#include "StateTreeFloatPropertyFunctions.generated.h"

#define UE_API STATETREEMODULE_API

struct FStateTreeExecutionContext;

USTRUCT()
struct FStateTreeFloatCombinaisonPropertyFunctionInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Parameter)
	float Left = 0.f;

	UPROPERTY(EditAnywhere, Category = Parameter)
	float Right = 0.f;

	UPROPERTY(EditAnywhere, Category = Output)
	float Result = 0.f;
};

/**
 * Add two floats.
 */
USTRUCT(meta=(DisplayName = "Add", Category = "Math|Float"))
struct FStateTreeAddFloatPropertyFunction : public FStateTreePropertyFunctionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeFloatCombinaisonPropertyFunctionInstanceData;

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	UE_API virtual void Execute(FStateTreeExecutionContext& Context) const override;

#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const;
#endif
};

/**
 * Subtract right float from left float.
 */
USTRUCT(meta=(DisplayName = "Subtract", Category = "Math|Float"))
struct FStateTreeSubtractFloatPropertyFunction : public FStateTreePropertyFunctionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeFloatCombinaisonPropertyFunctionInstanceData;

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	UE_API virtual void Execute(FStateTreeExecutionContext& Context) const override;

#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const;
#endif
};

/**
 * Multiply the two given float.
 */
USTRUCT(meta=(DisplayName = "Multiply", Category = "Math|Float"))
struct FStateTreeMultiplyFloatPropertyFunction : public FStateTreePropertyFunctionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeFloatCombinaisonPropertyFunctionInstanceData;

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	UE_API virtual void Execute(FStateTreeExecutionContext& Context) const override;

#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const;
#endif
};

/**
 * Divide left float by right float.
 */
USTRUCT(meta=(DisplayName = "Divide", Category = "Math|Float"))
struct FStateTreeDivideFloatPropertyFunction : public FStateTreePropertyFunctionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeFloatCombinaisonPropertyFunctionInstanceData;

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	UE_API virtual void Execute(FStateTreeExecutionContext& Context) const override;

#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const;
#endif
};

USTRUCT()
struct FStateTreeSingleFloatPropertyFunctionInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Parameter)
	float Input = 0.f;

	UPROPERTY(EditAnywhere, Category = Output)
	float Result = 0.f;
};

/**
 * Invert the given float.
 */
USTRUCT(meta=(DisplayName = "Invert", Category = "Math|Float"))
struct FStateTreeInvertFloatPropertyFunction : public FStateTreePropertyFunctionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeSingleFloatPropertyFunctionInstanceData;

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	UE_API virtual void Execute(FStateTreeExecutionContext& Context) const override;

#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const;
#endif
};

/**
 * Gives the absolute value of the given float.
 */
USTRUCT(meta=(DisplayName = "Absolute", Category = "Math|Float"))
struct FStateTreeAbsoluteFloatPropertyFunction : public FStateTreePropertyFunctionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeSingleFloatPropertyFunctionInstanceData;

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	UE_API virtual void Execute(FStateTreeExecutionContext& Context) const override;

#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const;
#endif
};

#undef UE_API
