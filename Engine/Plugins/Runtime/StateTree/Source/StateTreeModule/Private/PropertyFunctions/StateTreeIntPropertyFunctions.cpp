// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyFunctions/StateTreeIntPropertyFunctions.h"

#include "StateTreeExecutionContext.h"
#include "StateTreeNodeDescriptionHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeIntPropertyFunctions)

#define LOCTEXT_NAMESPACE "StateTree"

void FStateTreeAddIntPropertyFunction::Execute(FStateTreeExecutionContext& Context) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	InstanceData.Result = InstanceData.Left + InstanceData.Right;
}

void FStateTreeSubtractIntPropertyFunction::Execute(FStateTreeExecutionContext& Context) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	InstanceData.Result = InstanceData.Left - InstanceData.Right;
}

void FStateTreeMultiplyIntPropertyFunction::Execute(FStateTreeExecutionContext& Context) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	InstanceData.Result = InstanceData.Left * InstanceData.Right;
}

void FStateTreeDivideIntPropertyFunction::Execute(FStateTreeExecutionContext& Context) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	if (InstanceData.Right != 0)
	{
		InstanceData.Result = InstanceData.Left / InstanceData.Right;
	}
	else
	{
		InstanceData.Result = 0;
	}
}

void FStateTreeInvertIntPropertyFunction::Execute(FStateTreeExecutionContext& Context) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	InstanceData.Result = -InstanceData.Input;
}

void FStateTreeAbsoluteIntPropertyFunction::Execute(FStateTreeExecutionContext& Context) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	InstanceData.Result = FMath::Abs(InstanceData.Input);
}

#if WITH_EDITOR
FText FStateTreeAddIntPropertyFunction::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	return UE::StateTree::DescHelpers::GetDescriptionForMathOperation<FInstanceDataType>(LOCTEXT("IntAdd", "+"), ID, InstanceDataView, BindingLookup, Formatting);
}

FText FStateTreeSubtractIntPropertyFunction::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	return UE::StateTree::DescHelpers::GetDescriptionForMathOperation<FInstanceDataType>(LOCTEXT("IntSubtract", "-"), ID, InstanceDataView, BindingLookup, Formatting);
}

FText FStateTreeMultiplyIntPropertyFunction::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	return UE::StateTree::DescHelpers::GetDescriptionForMathOperation<FInstanceDataType>(LOCTEXT("IntMultiply", "*"), ID, InstanceDataView, BindingLookup, Formatting);
}

FText FStateTreeDivideIntPropertyFunction::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	return UE::StateTree::DescHelpers::GetDescriptionForMathOperation<FInstanceDataType>(LOCTEXT("TreeIntDivide", "/"), ID, InstanceDataView, BindingLookup, Formatting);
}

FText FStateTreeInvertIntPropertyFunction::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	return UE::StateTree::DescHelpers::GetDescriptionForSingleParameterFunc<FInstanceDataType>(LOCTEXT("IntInvert", "-"), ID, InstanceDataView, BindingLookup, Formatting);
}

FText FStateTreeAbsoluteIntPropertyFunction::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	return UE::StateTree::DescHelpers::GetDescriptionForSingleParameterFunc<FInstanceDataType>(LOCTEXT("IntAbsolute", "Abs"), ID, InstanceDataView, BindingLookup, Formatting);
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
