// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreePropertyFunctionBase.h"
#include "StateTreeIntPropertyFunctions.generated.h"

#define UE_API STATETREEMODULE_API

struct FStateTreeExecutionContext;

USTRUCT()
struct FStateTreeIntCombinaisonPropertyFunctionInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Parameter)
	int32 Left = 0;

	UPROPERTY(EditAnywhere, Category = Parameter)
	int32 Right = 0;

	UPROPERTY(EditAnywhere, Category = Output)
	int32 Result = 0;
};

/**
 * Add two ints.
 */
USTRUCT(meta=(DisplayName = "Add", Category = "Math|Integer"))
struct FStateTreeAddIntPropertyFunction : public FStateTreePropertyFunctionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeIntCombinaisonPropertyFunctionInstanceData;

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	UE_API virtual void Execute(FStateTreeExecutionContext& Context) const override;

#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const;
#endif
};

/**
 * Subtract right int from left int.
 */
USTRUCT(meta=(DisplayName = "Subtract", Category = "Math|Integer"))
struct FStateTreeSubtractIntPropertyFunction : public FStateTreePropertyFunctionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeIntCombinaisonPropertyFunctionInstanceData;

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	UE_API virtual void Execute(FStateTreeExecutionContext& Context) const override;

#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const;
#endif
};

/**
 * Multiply the two given ints.
 */
USTRUCT(meta=(DisplayName = "Multiply", Category = "Math|Integer"))
struct FStateTreeMultiplyIntPropertyFunction : public FStateTreePropertyFunctionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeIntCombinaisonPropertyFunctionInstanceData;

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	UE_API virtual void Execute(FStateTreeExecutionContext& Context) const override;

#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const;
#endif
};

/**
 * Divide left int by right int.
 */
USTRUCT(meta=(DisplayName = "Divide", Category = "Math|Integer"))
struct FStateTreeDivideIntPropertyFunction : public FStateTreePropertyFunctionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeIntCombinaisonPropertyFunctionInstanceData;

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	UE_API virtual void Execute(FStateTreeExecutionContext& Context) const override;

#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const;
#endif
};

USTRUCT()
struct FStateTreeSingleIntPropertyFunctionInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Parameter)
	int32 Input = 0;

	UPROPERTY(EditAnywhere, Category = Output)
	int32 Result = 0;
};

/**
 * Invert the given int.
 */
USTRUCT(meta=(DisplayName = "Invert", Category = "Math|Integer"))
struct FStateTreeInvertIntPropertyFunction : public FStateTreePropertyFunctionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeSingleIntPropertyFunctionInstanceData;

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	UE_API virtual void Execute(FStateTreeExecutionContext& Context) const override;

#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const;
#endif
};

/**
 * Gives the absolute value of the given int.
 */
USTRUCT(meta=(DisplayName = "Absolute", Category = "Math|Integer"))
struct FStateTreeAbsoluteIntPropertyFunction : public FStateTreePropertyFunctionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeSingleIntPropertyFunctionInstanceData;

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	UE_API virtual void Execute(FStateTreeExecutionContext& Context) const override;

#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const;
#endif
};

#undef UE_API
