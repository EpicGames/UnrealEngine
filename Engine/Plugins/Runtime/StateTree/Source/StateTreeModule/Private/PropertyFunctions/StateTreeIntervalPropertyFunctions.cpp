// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyFunctions/StateTreeIntervalPropertyFunctions.h"

#include "StateTreeExecutionContext.h"
#include "StateTreeNodeDescriptionHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeIntervalPropertyFunctions)

#define LOCTEXT_NAMESPACE "StateTree"

void FStateTreeMakeIntervalPropertyFunction::Execute(FStateTreeExecutionContext& Context) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	InstanceData.Result = FFloatInterval(InstanceData.Min, InstanceData.Max);
}

#if WITH_EDITOR
FText FStateTreeMakeIntervalPropertyFunction::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	const FInstanceDataType& InstanceData = InstanceDataView.Get<FInstanceDataType>();
	
	FText MinValueText = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, Min)), Formatting);
	if (MinValueText.IsEmpty())
	{
		MinValueText = UE::StateTree::DescHelpers::GetText(InstanceData.Min, Formatting);
	}

	FText MaxValueText = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, Max)), Formatting);
	if (MaxValueText.IsEmpty())
	{
		MaxValueText = UE::StateTree::DescHelpers::GetText(InstanceData.Max, Formatting);
	}

	return UE::StateTree::DescHelpers::GetIntervalText(MinValueText, MaxValueText, Formatting);
}
#endif //WITH_EDITOR

#undef LOCTEXT_NAMESPACE
