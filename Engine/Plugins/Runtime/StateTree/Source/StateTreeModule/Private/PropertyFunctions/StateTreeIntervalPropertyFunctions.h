// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreePropertyFunctionBase.h"
#include "StateTreeIntervalPropertyFunctions.generated.h"

#define UE_API STATETREEMODULE_API

struct FStateTreeExecutionContext;

USTRUCT()
struct FStateTreeMakeIntervalPropertyFunctionInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Parameter)
	float Min = 0.f;

	UPROPERTY(EditAnywhere, Category = Parameter)
	float Max = 1.f;

	UPROPERTY(EditAnywhere, Category = Output)
	FFloatInterval Result = FFloatInterval(0.f, 1.f);
};

/**
 * Make an Interval from two floats.
 */
USTRUCT(DisplayName = "Make Interval")
struct FStateTreeMakeIntervalPropertyFunction : public FStateTreePropertyFunctionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeMakeIntervalPropertyFunctionInstanceData;

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	UE_API virtual void Execute(FStateTreeExecutionContext& Context) const override;

#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const;
#endif
};

#undef UE_API
