// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeTransformPropertyFunctions.h"
#include "StateTreeExecutionContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeTransformPropertyFunctions)

void FStateTreeMakeTransformPropertyFunction::Execute(FStateTreeExecutionContext& Context) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	InstanceData.OutTransform = FTransform(InstanceData.InRotation, InstanceData.InTranslation);
}

void FStateTreeBreakTransformPropertyFunction::Execute(FStateTreeExecutionContext& Context) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	InstanceData.OutTranslation = InstanceData.InTransform.GetTranslation();
	InstanceData.OutRotation = InstanceData.InTransform.GetRotation();
}
