// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreePropertyFunctionBase.h"
#include "StateTreeTransformPropertyFunctions.generated.h"

#define UE_API STATETREEMODULE_API

struct FStateTreeExecutionContext;

USTRUCT()
struct FStateTreeMakeTransformPropertyFunctionInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Parameter, meta = (DisplayName = "Translation"))
	FVector InTranslation = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, Category = Parameter, meta = (DisplayName = "Rotation"))
	FQuat InRotation = FQuat::Identity;

	UPROPERTY(EditAnywhere, Category = Output, meta = (DisplayName = "Transform"))
	FTransform OutTransform = FTransform::Identity;
};

/**
 * Make a transform from translation and rotation.
 */
USTRUCT(DisplayName = "Make Transform")
struct FStateTreeMakeTransformPropertyFunction : public FStateTreePropertyFunctionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeMakeTransformPropertyFunctionInstanceData;

	virtual const UStruct* GetInstanceDataType() const override
	{
		return FInstanceDataType::StaticStruct();
	}

	UE_API virtual void Execute(FStateTreeExecutionContext& Context) const override;

#if WITH_EDITOR
	virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const override
	{
		return NSLOCTEXT("StateTreeTransformPropertyFunction", "Make", "Make Transform");
	}
#endif
};

USTRUCT()
struct FStateTreeBreakTransformPropertyFunctionInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Parameter, meta = (DisplayName = "Transform"))
	FTransform InTransform = FTransform::Identity;

	UPROPERTY(EditAnywhere, Category = Output, meta = (DisplayName = "Translation"))
	FVector OutTranslation = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, Category = Output, meta = (DisplayName = "Rotation"))
	FQuat OutRotation = FQuat::Identity;
};

/**
 * Break a transform into position and rotation.
 */
USTRUCT(DisplayName = "Break Transform")
struct FStateTreeBreakTransformPropertyFunction : public FStateTreePropertyFunctionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeBreakTransformPropertyFunctionInstanceData;

	virtual const UStruct* GetInstanceDataType() const override
	{
		return FInstanceDataType::StaticStruct();
	}

	UE_API virtual void Execute(FStateTreeExecutionContext& Context) const override;

#if WITH_EDITOR
	virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const override
	{
		return NSLOCTEXT("StateTreeTransformPropertyFunction", "Break", "Break Transform");
	}
#endif
};

#undef UE_API
