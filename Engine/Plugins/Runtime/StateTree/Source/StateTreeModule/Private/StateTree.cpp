// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTree.h"
#include "Containers/RingBuffer.h"
#include "Misc/PackageName.h"
#include "StateTreeLinker.h"
#include "StateTreeTaskBase.h"
#include "StateTreeEvaluatorBase.h"
#include "StateTreeConditionBase.h"
#include "StateTreeConsiderationBase.h"
#include "StateTreeExtension.h"
#include "StateTreeModuleImpl.h"
#include "AssetRegistry/AssetData.h"
#include "Misc/ScopeRWLock.h"
#include "StateTreeDelegates.h"
#include "StateTreeDelegatesInternal.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Misc/DataValidation.h"
#include "Misc/EnumerateRange.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "StructUtils/UserDefinedStruct.h"
#include "StateTreePropertyFunctionBase.h"
#include "AutoRTFM.h"
#include "UObject/ObjectSaveContext.h"

#include <atomic>

#if WITH_EDITOR
#include "UObject/LinkerLoad.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTree)

PRAGMA_DISABLE_DEPRECATION_WARNINGS
const FGuid FStateTreeCustomVersion::GUID(0x28E21331, 0x501F4723, 0x8110FA64, 0xEA10DA1E);
namespace UE::StateTree::Private
{
FCustomVersionRegistration GRegisterStateTreeCustomVersion(FStateTreeCustomVersion::GUID, FStateTreeCustomVersion::LatestVersion, TEXT("StateTreeAsset"));
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR
// 0 default
// 1 with DefaultGlobalParameters
const int32 UStateTree::LatestCustomAssetSavedVersion = 1;
#endif

bool UStateTree::IsReadyToRun() const
{
#if WITH_EDITOR
	if (bCompilationPending)
	{
		// The asset can be in the compiler queue. Compile the asset sync.
		const_cast<UStateTree*>(this)->CompileIfChanged();
	}
#endif

	// Valid tree must have at least one state and valid instance data.
	return States.Num() > 0 && CompileStatus == ECompileStatus::Executable && PropertyBindings.IsValid();
}

FConstStructView UStateTree::GetNode(const int32 NodeIndex) const
{
	return Nodes.IsValidIndex(NodeIndex) ? Nodes[NodeIndex] : FConstStructView();
}

FStateTreeIndex16 UStateTree::GetNodeIndexFromId(const FGuid Id) const
{
	const FStateTreeNodeIdToIndex* Entry = IDToNodeMappings.FindByPredicate([Id](const FStateTreeNodeIdToIndex& Entry){ return Entry.Id == Id; });
	return Entry != nullptr ? Entry->Index : FStateTreeIndex16::Invalid;
}

FGuid UStateTree::GetNodeIdFromIndex(const FStateTreeIndex16 NodeIndex) const
{
	const FStateTreeNodeIdToIndex* Entry = NodeIndex.IsValid()
		? IDToNodeMappings.FindByPredicate([NodeIndex](const FStateTreeNodeIdToIndex& Entry){ return Entry.Index == NodeIndex; })
		: nullptr;
	return Entry != nullptr ? Entry->Id : FGuid();
}

const FCompactStateTreeFrame* UStateTree::GetFrameFromHandle(const FStateTreeStateHandle StateHandle) const
{
	return Frames.FindByPredicate([StateHandle](const FCompactStateTreeFrame& Frame)
		{
			return Frame.RootState == StateHandle;
		});
}

const FCompactStateTreeState* UStateTree::GetStateFromHandle(const FStateTreeStateHandle StateHandle) const
{
	return States.IsValidIndex(StateHandle.Index) ? &States[StateHandle.Index] : nullptr;
}

FStateTreeStateHandle UStateTree::GetStateHandleFromId(const FGuid Id) const
{
	const FStateTreeStateIdToHandle* Entry = IDToStateMappings.FindByPredicate([Id](const FStateTreeStateIdToHandle& Entry){ return Entry.Id == Id; });
	return Entry != nullptr ? Entry->Handle : FStateTreeStateHandle::Invalid;
}

FStateTreeStateHandle UStateTree::GetStateHandleFromGameplayTag(const FGameplayTag Tag, EStateGameplayTagQueryMethod QueryMethod) const
{
	TRingBuffer<uint16, TInlineAllocator<32>> HandleBFSQueue;

	auto AddRootStatesIndicesToQueue = [Self = this, &HandleBFSQueue]()
	{
		uint16 RootStateIndex = 0;
		while (RootStateIndex < Self->States.Num())
		{
			HandleBFSQueue.Add(RootStateIndex);
			RootStateIndex = Self->States[RootStateIndex].GetNextSibling();
		}
	};

	auto TagMatches = [TagToQuery = Tag, QueryMethod](const FGameplayTag StateTag)
		{
			if (QueryMethod == EStateGameplayTagQueryMethod::Includes)
			{
				return StateTag.MatchesTag(TagToQuery);
			}

			if (QueryMethod == EStateGameplayTagQueryMethod::MatchesExact)
			{
				return StateTag.MatchesTagExact(TagToQuery);
			}

			return false;
		};

	AddRootStatesIndicesToQueue();

	while (!HandleBFSQueue.IsEmpty())
	{
		const uint16 CurrentStateHandleIndex = HandleBFSQueue.PopFrontValue();
		const FCompactStateTreeState& CurrentState = States[CurrentStateHandleIndex];

		if (TagMatches(CurrentState.Tag))
		{
			return FStateTreeStateHandle(CurrentStateHandleIndex);
		}
		
		uint16 CurrentStateNextLevelChildIndex = CurrentState.ChildrenBegin;
		while (CurrentStateNextLevelChildIndex != CurrentState.ChildrenEnd)
		{
			HandleBFSQueue.Add(CurrentStateNextLevelChildIndex);
			CurrentStateNextLevelChildIndex = States[CurrentStateNextLevelChildIndex].GetNextSibling();
		}
	}

	return FStateTreeStateHandle::Invalid;
}

FGuid UStateTree::GetStateIdFromHandle(const FStateTreeStateHandle Handle) const
{
	const FStateTreeStateIdToHandle* Entry = IDToStateMappings.FindByPredicate([Handle](const FStateTreeStateIdToHandle& Entry){ return Entry.Handle == Handle; });
	return Entry != nullptr ? Entry->Id : FGuid();
}

const FCompactStateTransition* UStateTree::GetTransitionFromIndex(const FStateTreeIndex16 TransitionIndex) const
{
	return TransitionIndex.IsValid() && Transitions.IsValidIndex(TransitionIndex.Get()) ? &Transitions[TransitionIndex.Get()] : nullptr;
}

FStateTreeIndex16 UStateTree::GetTransitionIndexFromId(const FGuid Id) const
{
	const FStateTreeTransitionIdToIndex* Entry = IDToTransitionMappings.FindByPredicate([Id](const FStateTreeTransitionIdToIndex& Entry){ return Entry.Id == Id; });
	return Entry != nullptr ? Entry->Index : FStateTreeIndex16::Invalid;
}

FGuid UStateTree::GetTransitionIdFromIndex(const FStateTreeIndex16 Index) const
{
	const FStateTreeTransitionIdToIndex* Entry = IDToTransitionMappings.FindByPredicate([Index](const FStateTreeTransitionIdToIndex& Entry){ return Entry.Index == Index; });
	return Entry != nullptr ? Entry->Id : FGuid();
}

const UStateTreeExtension* UStateTree::K2_GetExtension(TSubclassOf<UStateTreeExtension> InExtensionType) const
{
	for (const UStateTreeExtension* Extension : Extensions)
	{
		if (Extension && Extension->IsA(InExtensionType))
		{
			return Extension;
		}
	}
	return nullptr;
}

UE_AUTORTFM_ALWAYS_OPEN
static int32 GetThreadIndexForSharedInstanceData()
{
	// Create a unique index for each thread.
	static std::atomic_int ThreadIndexCounter {0};
	static thread_local int32 ThreadIndex = INDEX_NONE; // Cannot init directly on WinRT
	if (ThreadIndex == INDEX_NONE)
	{
		ThreadIndex = ThreadIndexCounter.fetch_add(1);
	}

	return ThreadIndex;
}

TSharedPtr<FStateTreeInstanceData> UStateTree::GetSharedInstanceData() const
{
	if (SharedInstanceData.Num() == 0)
	{
		return TSharedPtr<FStateTreeInstanceData>();
	}

	int32 ThreadIndex = GetThreadIndexForSharedInstanceData();

	// If shared instance data for this thread exists, return it.
	{
		UE::TReadScopeLock ReadLock(PerThreadSharedInstanceDataLock);
		if (ThreadIndex < PerThreadSharedInstanceData.Num())
		{
			return PerThreadSharedInstanceData[ThreadIndex];
		}
	}

	// Not initialized yet, create new instances up to the index.
	UE::TWriteScopeLock WriteLock(PerThreadSharedInstanceDataLock);

	// It is possible that multiple threads are waiting for the write lock,
	// which means that execution may get here so that 'ThreadIndex' is already in valid range.
	// The loop below is organized to handle that too.
	
	const int32 NewNum = ThreadIndex + 1;
	PerThreadSharedInstanceData.Reserve(NewNum);
	UStateTree* NonConstThis = const_cast<UStateTree*>(this); 
	
	for (int32 Index = PerThreadSharedInstanceData.Num(); Index < NewNum; Index++)
	{
		TSharedPtr<FStateTreeInstanceData> SharedData = MakeShared<FStateTreeInstanceData>();
		SharedData->CopyFrom(*NonConstThis, SharedInstanceData);
		PerThreadSharedInstanceData.Add(SharedData);
	}

	return PerThreadSharedInstanceData[ThreadIndex];
}

bool UStateTree::HasCompatibleContextData(const UStateTree& Other) const
{
	return HasCompatibleContextData(&Other);
}

bool UStateTree::HasCompatibleContextData(TNotNull<const UStateTree*> Other) const
{
	if (ContextDataDescs.Num() != Other->ContextDataDescs.Num())
	{
		return false;
	}

	const int32 Num = ContextDataDescs.Num();
	for (int32 Index = 0; Index < Num; Index++)
	{
		const FStateTreeExternalDataDesc& Desc = ContextDataDescs[Index];
		const FStateTreeExternalDataDesc& OtherDesc = Other->ContextDataDescs[Index];
		
		if (!OtherDesc.Struct 
			|| !OtherDesc.Struct->IsChildOf(Desc.Struct))
		{
			return false;
		}
	}
	
	return true;
}

#if WITH_STATETREE_DEBUG
void UStateTree::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_STATETREE_DEBUG
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		PreGCHandle = FStateTreeModule::OnPreRuntimeValidationInstanceData.AddUObject(this, &UStateTree::HandleRuntimeValidationPreGC);
		PostGCHandle = FStateTreeModule::OnPostRuntimeValidationInstanceData.AddUObject(this, &UStateTree::HandleRuntimeValidationPostGC);
	}
#endif
}

void UStateTree::BeginDestroy()
{
#if WITH_STATETREE_DEBUG
	FStateTreeModule::OnPreRuntimeValidationInstanceData.Remove(PreGCHandle);
	FStateTreeModule::OnPostRuntimeValidationInstanceData.Remove(PostGCHandle);
#endif

	Super::BeginDestroy();
}
#endif //WITH_STATETREE_DEBUG

#if WITH_EDITOR
namespace UE::StateTree::Compiler
{
	void RenameObjectToTransientPackage(TNotNull<UObject*> ObjectToRename, FStringView Name)
	{
		const ERenameFlags RenFlags = REN_DoNotDirty | REN_DontCreateRedirectors;

		ObjectToRename->SetFlags(RF_Transient);
		ObjectToRename->ClearFlags(RF_Public | RF_Standalone | RF_ArchetypeObject);

		const FName UniqueName = MakeUniqueObjectName(GetTransientPackage(), ObjectToRename->GetClass(), FName(Name));

		// Rename will remove the renamed object's linker when moving to a new package so invalidate the export beforehand
		FLinkerLoad::InvalidateExport(ObjectToRename);
		ObjectToRename->Rename(*UniqueName.ToString(), GetTransientPackage(), RenFlags);
	}
}

void UStateTree::ResetCompiled()
{
	Schema = nullptr;
	Frames.Reset();
	States.Reset();
	Transitions.Reset();
	Nodes.Reset();
	DefaultInstanceData.Reset();
	DefaultEvaluationScopeInstanceData.Reset();
	DefaultExecutionRuntimeData.Reset();
	SharedInstanceData.Reset();
	ContextDataDescs.Reset();
	PropertyBindings.Reset();
	PropertyFunctionEvaluationScopeMemoryRequirements.Reset();
	TaskCompletionDispatchers.Reset();
	Extensions.Reset();
	Parameters.Reset();
	ParameterDataType = EStateTreeParameterDataType::GlobalParameterData;
	IDToStateMappings.Reset();
	IDToNodeMappings.Reset();
	IDToTransitionMappings.Reset();

	EvaluatorsBegin = 0;
	EvaluatorsNum = 0;

	GlobalTasksBegin = 0;
	GlobalTasksNum = 0;
	bHasGlobalTransitionTasks = false;
	bHasGlobalTickTasks = false;
	bHasGlobalTickTasksOnlyOnEvents = false;
	bCachedRequestGlobalTick = false;
	bCachedRequestGlobalTickOnlyOnEvents = false;
	bScheduledTickAllowed = false;
	
	ResetLinked();

	CompileStatus = ECompileStatus::Public;
	EditorDataDirtyStatus = EDirtyStatus::Public;
	LastCompiledEditorDataHash = 0;

	// Remove objects created from last compilation.
	{
		TArray<UObject*, TInlineAllocator<32>> Children;
		ForEachObjectWithOuter(this, [&Children, EditorData = EditorData.Get()](UObject* Child)
			{
				if (Child != EditorData)
				{
					Children.Add(Child);
				}
			}, EGetObjectsFlags::None);

		for (UObject* Child : Children)
		{
			UE::StateTree::Compiler::RenameObjectToTransientPackage(Child);
		}
	}
}

bool UStateTree::IsEditorDataDirty() const
{
	return EditorDataDirtyStatus != EDirtyStatus::None;
}

void UStateTree::MarkAsModified(const bool bPubliclyModified)
{
	const EDirtyStatus PreviousDirtyStatus = EditorDataDirtyStatus;
	if (bPubliclyModified)
	{
		EditorDataDirtyStatus = EDirtyStatus::Public;
	}
	else if (EditorDataDirtyStatus != EDirtyStatus::Public)
	{
		EditorDataDirtyStatus = EDirtyStatus::Internal;
	}

	if (PreviousDirtyStatus != EditorDataDirtyStatus)
	{
		UE::StateTree::Delegates::Private::OnStateTreeMarkedAsModified.ExecuteIfBound(this);
	}
}

void UStateTree::PreSave(FObjectPreSaveContext Context)
{
	const bool bIsUserObject = !HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject);
	const bool bIsEditorAutoSave = Context.IsFromAutoSave();
	if (bIsUserObject && !bIsEditorAutoSave)
	{
		CustomAssetSavedVersion = LatestCustomAssetSavedVersion;
	}
	Super::PreSave(Context);
}

void UStateTree::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);
	ResetCompiled();
}

void UStateTree::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	UE::StateTree::Delegates::Private::OnRequestAssetRegistryTags.ExecuteIfBound(this, Context);
	Super::GetAssetRegistryTags(Context);
}

void UStateTree::BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform)
{
	UE::StateTree::Delegates::Private::OnPreCookStateTreeAsset.ExecuteIfBound(this);
	Super::BeginCacheForCookedPlatformData(TargetPlatform);
}

void UStateTree::ThreadedPostLoadAssetRegistryTagsOverride(FPostLoadAssetRegistryTagsContext& Context) const
{
	Super::ThreadedPostLoadAssetRegistryTagsOverride(Context);

	static const FName SchemaTag(TEXT("Schema"));
	const FString SchemaTagValue = Context.GetAssetData().GetTagValueRef<FString>(SchemaTag);
	if (!SchemaTagValue.IsEmpty() && FPackageName::IsShortPackageName(SchemaTagValue))
	{
		const FTopLevelAssetPath SchemaTagClassPathName = UClass::TryConvertShortTypeNameToPathName<UStruct>(SchemaTagValue, ELogVerbosity::Warning, TEXT("UStateTree::ThreadedPostLoadAssetRegistryTagsOverride"));
		if (!SchemaTagClassPathName.IsNull())
		{
			Context.AddTagToUpdate(FAssetRegistryTag(SchemaTag, SchemaTagClassPathName.ToString(), FAssetRegistryTag::TT_Alphabetical));
		}
	}
}

EDataValidationResult UStateTree::IsDataValid(FDataValidationContext& Context) const
{
	// Don't warn user that the tree they just saved is not compiled. Only for submit or manual validation
	if (Context.GetValidationUsecase() != EDataValidationUsecase::Save)
	{
		if (UE::StateTree::Delegates::OnRequestEditorHash.IsBound())
		{
			const uint32 CurrentHash = UE::StateTree::Delegates::OnRequestEditorHash.Execute(*this);
			if (CurrentHash != LastCompiledEditorDataHash)
			{
				Context.AddWarning(FText::FromString(FString::Printf(TEXT("%s is not compiled. Please recompile the State Tree."), *GetPathName())));
				return EDataValidationResult::Invalid;
			}
		}

#if WITH_EDITOR
		if (bCompilationPending)
		{
			// The asset can be in the compiler queue. Compile the asset sync.
			const_cast<UStateTree*>(this)->CompileIfChanged();
		}
#endif

		if (!const_cast<UStateTree*>(this)->Link())
		{
			Context.AddError(FText::FromString(FString::Printf(TEXT("%s failed to link. Please recompile the State Tree for more details errors."), *GetPathName())));
			return EDataValidationResult::Invalid;
		}
	}

	return Super::IsDataValid(Context);
}

#endif // WITH_EDITOR

void UStateTree::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	const UStateTree* StateTree = Cast<UStateTree>(InThis);
	check(StateTree);
	
	UE::TReadScopeLock ReadLock(StateTree->PerThreadSharedInstanceDataLock);

	for (const TSharedPtr<FStateTreeInstanceData>& InstanceData : StateTree->PerThreadSharedInstanceData)
	{
		if (InstanceData.IsValid())
		{
			Collector.AddPropertyReferencesWithStructARO(FStateTreeInstanceData::StaticStruct(), InstanceData.Get(), StateTree);
		}
	}
}

void UStateTree::PostLoad()
{
	Super::PostLoad();

	for (int32 NodeIndex = 0; NodeIndex < Nodes.Num(); ++NodeIndex)
	{
		FStructView NodeView = Nodes[NodeIndex];
		if (FStateTreeNodeBase* Node = NodeView.GetPtr<FStateTreeNodeBase>())
		{
			auto PostLoadInstance = [&Node]<typename T>(T & Container, FStateTreeIndex16 Index)
				{
					if (Container.IsObject(Index.Get()))
					{
						Node->PostLoad(Container.GetMutableObject(Index.Get()));
					}
					else
					{
						Node->PostLoad(Container.GetMutableStruct(Index.Get()));
					}
				};
			if (Node->InstanceTemplateIndex.IsValid())
			{
				const bool bIsUsingSharedInstanceData = Node->InstanceDataHandle.GetSource() == EStateTreeDataSourceType::SharedInstanceData
					|| Node->InstanceDataHandle.GetSource() == EStateTreeDataSourceType::SharedInstanceDataObject;
				const bool bIsUsingEvaluationScopeInstanceData = Node->InstanceDataHandle.GetSource() == EStateTreeDataSourceType::EvaluationScopeInstanceData
					|| Node->InstanceDataHandle.GetSource() == EStateTreeDataSourceType::EvaluationScopeInstanceDataObject;
				if (bIsUsingEvaluationScopeInstanceData)
				{
					PostLoadInstance(DefaultEvaluationScopeInstanceData, Node->InstanceTemplateIndex);
				}
				else if (bIsUsingSharedInstanceData)
				{
					PostLoadInstance(SharedInstanceData, Node->InstanceTemplateIndex);
				}
				else
				{
					PostLoadInstance(DefaultInstanceData, Node->InstanceTemplateIndex);
				}
			}
		}
	}

	if (Schema)
	{
		Schema->ConditionalPostLoad();
	}

	for (UStateTreeExtension* Extension : Extensions)
	{
		if (Extension)
		{
			Extension->ConditionalPostLoad();
		}
	}

	// Request a link
	CompileStatus = ECompileStatus::Link;

#if WITH_EDITOR
	if (EditorData)
	{
		// Make sure all the fix up logic in the editor data has had chance to happen.
		EditorData->ConditionalPostLoad();
		ResetCompiled();
	}
#endif

	// Request a compilation (can be async or sync).
	UE::StateTree::Delegates::Private::OnStateTreeAssetLoaded.ExecuteIfBound(this);

	// The OnStateTreeAssetLoaded can trigger a compilation. That would also trigger a Link()
	// Only Link() if no compilation occurs.
	// LastCompiledEditorDataHash is only set if a compilation is valid. A valid compilation set CompileStatus to Executable.
	if (CompileStatus == ECompileStatus::Link && LastCompiledEditorDataHash != 0)
	{
		if (!Link())
		{
			UE_CLOGF(CompileStatus == ECompileStatus::Link, LogStateTree, Warning, "%ls failed to link. Asset will not be usable at runtime.", *GetPathName());
		}
	}
}

#if WITH_EDITORONLY_DATA
void UStateTree::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	TArray<UClass*> SchemaClasses;
	GetDerivedClasses(UStateTreeSchema::StaticClass(), SchemaClasses);
	for (UClass* SchemaClass : SchemaClasses)
	{
		if (!SchemaClass->HasAnyClassFlags(CLASS_Abstract | CLASS_Transient))
		{
			OutConstructClasses.Add(FTopLevelAssetPath(SchemaClass));
		}
	}
}

void UStateTree::AppendToClassSchema(FAppendToClassSchemaContext& Context)
{
	Super::AppendToClassSchema(Context);
	UE::StateTree::Delegates::Private::OnAppendToClassSchema.ExecuteIfBound(Context);
}
#endif

void UStateTree::Serialize(const FStructuredArchiveRecord Record)
{
	Super::Serialize(Record);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const FGuid StateTreeCustomVersion = FStateTreeCustomVersion::GUID;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	Record.GetUnderlyingArchive().UsingCustomVersion(StateTreeCustomVersion);
	
	// We need to link and rebind property bindings each time a BP is compiled,
	// because property bindings may get invalid, and instance data potentially needs refreshed.
	if (Record.GetUnderlyingArchive().IsModifyingWeakAndStrongReferences())
	{
		if (!Link() && !HasAnyFlags(RF_ClassDefaultObject))
		{
			UE_LOGF(LogStateTree, Log, "%ls failed to link. Asset will not be usable at runtime.", *GetName());	
		}
	}
}

void UStateTree::ResetLinked()
{
	if (CompileStatus == ECompileStatus::Executable)
	{
		CompileStatus = ECompileStatus::Link;
	}
#if WITH_EDITOR
	if (EditorDataDirtyStatus == EDirtyStatus::None)
	{
		EditorDataDirtyStatus = EDirtyStatus::Link;
	}
#endif

	ExternalDataDescs.Reset();

#if WITH_EDITOR
	OutOfDateStructs.Reset();
#endif

	UE::TWriteScopeLock WriteLock(PerThreadSharedInstanceDataLock);
	PerThreadSharedInstanceData.Reset();
}

bool UStateTree::ValidateInstanceData()
{
	bool bResult = true;
	for (int32 Index = 0; Index < Nodes.Num(); ++Index)
	{
		const FConstStructView& NodeView = Nodes[Index];
		const FStateTreeNodeBase* Node = NodeView.GetPtr<const FStateTreeNodeBase>();
		if (Node && Node->InstanceTemplateIndex.IsValid())
		{
			auto TestInstanceData = [this, Node, Index, &bResult](const UStruct* CurrentInstanceDataType, const UStruct* DesiredInstanceDataType)
				{
					if (CurrentInstanceDataType == nullptr)
					{
						UE_LOGF(LogStateTree, Error, "%ls: node (%d) '%ls' with name '%ls' failed. Missing instance value, possibly due to Blueprint class or C++ class/struct template deletion."
							, *GetPathName()
							, Index
							, *WriteToString<64>(Node->StaticStruct()->GetFName())
							, *WriteToString<128>(Node->Name)
							);

						bResult = false;
						return;
					}

					auto HasNewerVersionExists = [](TNotNull<const UStruct*> InstanceDataType)
					{
						// Is the class/scriptstruct a blueprint that got replaced by another class.
						bool bHasNewerVersionExists = InstanceDataType->HasAnyFlags(RF_NewerVersionExists);
						if (!bHasNewerVersionExists)
						{
							if (const UClass* InstanceDataClass = Cast<UClass>(InstanceDataType))
							{
								bHasNewerVersionExists = InstanceDataClass->HasAnyClassFlags(CLASS_NewerVersionExists);
							}
							else if (const UScriptStruct* InstanceDataStruct = Cast<UScriptStruct>(InstanceDataType))
							{
								bHasNewerVersionExists = (InstanceDataStruct->StructFlags & STRUCT_NewerVersionExists) != 0;
							}
						}
						return bHasNewerVersionExists;
					};

					if (HasNewerVersionExists(CurrentInstanceDataType))
					{
						bool bLogError = true;
#if WITH_EDITOR
						OutOfDateStructs.Add(CurrentInstanceDataType);
						bLogError = false;
#endif

						if (bLogError)
						{
							UE_LOGF(LogStateTree, Error, "%ls: node '%ls' failed. The source Instance Data type '%ls' has a newer version.", *GetPathName(), *WriteToString<64>(Node->StaticStruct()->GetFName()), *WriteToString<64>(CurrentInstanceDataType->GetFName()));
						}

						bResult = false;
					}

					{
						// The FMyInstance::StaticStruct doesn't get a notification like the other objects when reinstanced.
						const bool bDesiredHasNewerVersion = HasNewerVersionExists(DesiredInstanceDataType);

						// Use strict testing so that the users will have option to initialize data mismatch if the type changes (even if potentially compatible).
						if (CurrentInstanceDataType != DesiredInstanceDataType
							&& !bDesiredHasNewerVersion)
						{
							bool bLogError = true;
#if WITH_EDITOR
							const UClass* CurrentInstanceDataClass = Cast<UClass>(CurrentInstanceDataType);
							const UClass* DesiredInstanceDataClass = Cast<UClass>(DesiredInstanceDataType);
							if (CurrentInstanceDataClass && DesiredInstanceDataClass)
							{
								// Because of the loading order. It's possible that the OnObjectsReinstanced did not completed.
								if (CurrentInstanceDataClass->ClassGeneratedBy == DesiredInstanceDataClass->ClassGeneratedBy)
								{
									OutOfDateStructs.Add(CurrentInstanceDataType);
									bLogError = false;
								}
							}
#endif
							if (bLogError)
							{
								UE_LOGF(LogStateTree, Error, "%ls: node '%ls' failed. The source Instance Data type '%ls' does not match '%ls'", *GetPathName(), *WriteToString<64>(Node->StaticStruct()->GetFName()), *GetNameSafe(CurrentInstanceDataType), *GetNameSafe(DesiredInstanceDataType));
							}
							bResult = false;
						}
					}
				};

			{
				const UStruct* CurrentInstanceDataType = nullptr;
				{
					const bool bIsUsingSharedInstanceData = Node->InstanceDataHandle.GetSource() == EStateTreeDataSourceType::SharedInstanceData
						|| Node->InstanceDataHandle.GetSource() == EStateTreeDataSourceType::SharedInstanceDataObject;
					const bool bIsUsingEvaluationScopeInstanceData = Node->InstanceDataHandle.GetSource() == EStateTreeDataSourceType::EvaluationScopeInstanceData
						|| Node->InstanceDataHandle.GetSource() == EStateTreeDataSourceType::EvaluationScopeInstanceDataObject;
					if (bIsUsingEvaluationScopeInstanceData)
					{
						if (DefaultEvaluationScopeInstanceData.IsObject(Node->InstanceTemplateIndex.Get()))
						{
							const UObject* InstanceObject = DefaultEvaluationScopeInstanceData.GetObject(Node->InstanceTemplateIndex.Get());
							CurrentInstanceDataType = InstanceObject ? InstanceObject->GetClass() : nullptr;
						}
						else
						{
							CurrentInstanceDataType = DefaultEvaluationScopeInstanceData.GetStruct(Node->InstanceTemplateIndex.Get()).GetScriptStruct();
						}
					}
					else
					{
						const FStateTreeInstanceData& SourceInstanceData = bIsUsingSharedInstanceData ? SharedInstanceData : DefaultInstanceData;
						if (SourceInstanceData.IsObject(Node->InstanceTemplateIndex.Get()))
						{
							const UObject* InstanceObject = SourceInstanceData.GetObject(Node->InstanceTemplateIndex.Get());
							CurrentInstanceDataType = InstanceObject ? InstanceObject->GetClass() : nullptr;
						}
						else
						{
							CurrentInstanceDataType = SourceInstanceData.GetStruct(Node->InstanceTemplateIndex.Get()).GetScriptStruct();
						}
					}
				}
				TestInstanceData(CurrentInstanceDataType, Node->GetInstanceDataType());
			}

			if (Node->ExecutionRuntimeTemplateIndex.IsValid())
			{
				const UStruct* CurrentInstanceDataType = nullptr;
				if (GetDefaultExecutionRuntimeData().IsObject(Node->ExecutionRuntimeTemplateIndex.Get()))
				{
					const UObject* InstanceObject = GetDefaultExecutionRuntimeData().GetObject(Node->ExecutionRuntimeTemplateIndex.Get());
					CurrentInstanceDataType = InstanceObject ? InstanceObject->GetClass() : nullptr;
				}
				else
				{
					CurrentInstanceDataType = GetDefaultExecutionRuntimeData().GetStruct(Node->ExecutionRuntimeTemplateIndex.Get()).GetScriptStruct();
				}
				TestInstanceData(CurrentInstanceDataType, Node->GetExecutionRuntimeDataType());
			}
		}
	}

	return bResult;
}

bool UStateTree::Link()
{
	// Make sure the StateTreeCompilationManager is aware that the asset fails to compile, so that it can retry compilation when necessary (e.g. when PIE starts).
	auto Failure = [this]()
		{
#if WITH_EDITOR
			constexpr bool bPubliclyModified = false;
			MarkAsModified(bPubliclyModified);
#endif //WITH_EDITOR
			return false;
		};

	if (CompileStatus == ECompileStatus::Public || CompileStatus == ECompileStatus::Internal)
	{
		return Failure();
	}

	// Initialize the instance data default value.
	// This data will be used to allocate runtime instance on all StateTree users.
	ResetLinked();

	// Validate that all the source instance data types matches the node instance data types
	if (!ValidateInstanceData())
	{
		return Failure();
	}

	if (States.Num() > 0 && Nodes.Num() > 0)
	{
		// Check that all nodes are valid.
		for (FConstStructView Node : Nodes)
		{
			if (!Node.IsValid())
			{
				UE_LOGF(LogStateTree, Error, "%ls: State Tree asset was not properly loaded (missing node). See log for loading failures, or recompile the StateTree asset.", *GetPathName());
				return Failure();
			}
		}
	}

	// Resolves nodes references to other StateTree data
	{
		FStateTreeLinker Linker(this);

		for (int32 Index = 0; Index < Nodes.Num(); Index++)
		{
			FStructView Node = Nodes[Index];
			FStateTreeNodeBase* NodePtr = Node.GetPtr<FStateTreeNodeBase>();
			if (ensure(NodePtr))
			{
				const bool bLinkSucceeded = NodePtr->Link(Linker);
				if (!bLinkSucceeded || Linker.GetStatus() == EStateTreeLinkerStatus::Failed)
				{
					UE_LOGF(LogStateTree, Error, "%ls: node '%ls' failed to resolve its references.", *GetPathName(), *NodePtr->StaticStruct()->GetName());
					return Failure();
				}
			}
		}

		// Schema
		if (Schema)
		{
			const bool bSchemaLinkSucceeded = Schema->Link(Linker);
			if (!bSchemaLinkSucceeded || Linker.GetStatus() == EStateTreeLinkerStatus::Failed)
			{
				UE_LOGF(LogStateTree, Error, "%ls: schema failed to resolve its references.", *GetPathName());
				return Failure();
			}
		}

		// Extensions
		for (UStateTreeExtension* Extension : Extensions)
		{
			if (Extension)
			{
				const bool bLinkSucceeded = Extension->Link(Linker);
				if (!bLinkSucceeded || Linker.GetStatus() == EStateTreeLinkerStatus::Failed)
				{
					UE_LOGF(LogStateTree, Error, "%ls: extension failed to resolve its references.", *GetPathName());
					return Failure();
				}
			}
		}

		ExternalDataDescs = Linker.GetExternalDataDescs();
	}

	UpdateRuntimeFlags();

	if (!DefaultInstanceData.AreAllInstancesValid())
	{
		UE_LOGF(LogStateTree, Error, "%ls: State Tree asset was not properly loaded (missing instance data). See log for loading failures, or recompile the StateTree asset.", *GetPathName());
		return Failure();
	}

	if (!SharedInstanceData.AreAllInstancesValid())
	{
		UE_LOGF(LogStateTree, Error, "%ls: State Tree asset was not properly loaded (missing shared instance data). See log for loading failures, or recompile the StateTree asset.", *GetPathName());
		return Failure();
	}

	if (!DefaultEvaluationScopeInstanceData.AreAllInstancesValid())
	{
		UE_LOGF(LogStateTree, Error, "%ls: State Tree asset was not properly loaded (missing evaluation scope instance data). See log for loading failures, or recompile the StateTree asset.", *GetPathName());
		return Failure();
	}

	if (!GetDefaultExecutionRuntimeData().AreAllInstancesValid())
	{
		UE_LOGF(LogStateTree, Error, "%ls: State Tree asset was not properly loaded (missing execution runtime data). See log for loading failures, or recompile the StateTree asset.", *GetPathName());
		return Failure();
	}
	
	if (!PatchBindings())
	{
		return Failure();
	}

	// Resolves property paths used by bindings a store property pointers
	if (!PropertyBindings.ResolvePaths())
	{
		return Failure();
	}

	// Link succeeded, setup tree to be ready to run
	CompileStatus = ECompileStatus::Executable;
#if WITH_EDITOR
	if (EditorDataDirtyStatus == EDirtyStatus::Link)
	{
		EditorDataDirtyStatus = EDirtyStatus::None;
	}
#endif
	
	return true;
}

void UStateTree::UpdateRuntimeFlags()
{
	// Set the tick flags at runtime instead of compilation.
	//This is to support hotfix (when we only modify cpp code).

	for (FCompactStateTreeState& State : States)
	{
		// Update the state task flags.
		State.bHasTickTasks = false;
		State.bHasTickTasksOnlyOnEvents = false;
		State.bHasTransitionTasks = false;
		State.bCachedRequestTick = false;
		State.bCachedRequestTickOnlyOnEvents = false;
		for (int32 TaskIndex = State.TasksBegin; TaskIndex < (State.TasksBegin + State.TasksNum); ++TaskIndex)
		{
			const FStateTreeTaskBase& Task = Nodes[TaskIndex].Get<const FStateTreeTaskBase>();
			if (Task.bTaskEnabled)
			{
				State.bHasTickTasks |= Task.bShouldCallTick;
				State.bHasTickTasksOnlyOnEvents |= Task.bShouldCallTickOnlyOnEvents;
				State.bHasTransitionTasks |= Task.bShouldAffectTransitions;
				if (Task.bConsideredForScheduling)
				{
					State.bCachedRequestTick |= Task.bShouldCallTick || Task.bShouldAffectTransitions;
					State.bCachedRequestTickOnlyOnEvents |= Task.bShouldCallTickOnlyOnEvents;
				}
			}
		}

		// Cache the amount of memory needed to execute the conditions.
		{
			UE::StateTree::InstanceData::FEvaluationScopeInstanceContainer::FMemoryRequirementBuilder Requirement;
			for (int32 Index = 0; Index < State.EnterConditionsNum; ++Index)
			{
				const int32 NodeIndex = State.EnterConditionsBegin + Index;
				const FStateTreeConditionBase& Cond = Nodes[NodeIndex].Get<const FStateTreeConditionBase>();
				if (Cond.InstanceDataHandle.GetSource() == EStateTreeDataSourceType::EvaluationScopeInstanceData
					|| Cond.InstanceDataHandle.GetSource() == EStateTreeDataSourceType::EvaluationScopeInstanceDataObject)
				{
					const FConstStructView DefaultInstanceView = DefaultEvaluationScopeInstanceData.GetStruct(Cond.InstanceTemplateIndex.Get());
					Requirement.Add(DefaultInstanceView.GetScriptStruct());
				}
			}
			State.EnterConditionEvaluationScopeMemoryRequirement = Requirement.Build();
		}
		// Cache the amount of memory needed to execute the considerations.
		{
			UE::StateTree::InstanceData::FEvaluationScopeInstanceContainer::FMemoryRequirementBuilder Requirement;
			for (int32 Index = 0; Index < State.UtilityConsiderationsNum; ++Index)
			{
				const int32 NodeIndex = State.UtilityConsiderationsBegin + Index;
				const FStateTreeConsiderationBase& Consideration = Nodes[NodeIndex].Get<const FStateTreeConsiderationBase>();
				if (Consideration.InstanceDataHandle.GetSource() == EStateTreeDataSourceType::EvaluationScopeInstanceData
					|| Consideration.InstanceDataHandle.GetSource() == EStateTreeDataSourceType::EvaluationScopeInstanceDataObject)
				{
					const FConstStructView DefaultInstanceView = DefaultEvaluationScopeInstanceData.GetStruct(Consideration.InstanceTemplateIndex.Get());
					Requirement.Add(DefaultInstanceView.GetScriptStruct());
				}
			}
			State.ConsiderationEvaluationScopeMemoryRequirement = Requirement.Build();
		}
	}

	// Update the global task flags.
	{
		bHasGlobalTickTasks = false;
		bHasGlobalTickTasksOnlyOnEvents = false;
		bHasGlobalTransitionTasks = false;
		bCachedRequestGlobalTick = false;
		bCachedRequestGlobalTickOnlyOnEvents = false;
		for (int32 TaskIndex = GlobalTasksBegin; TaskIndex < (GlobalTasksBegin + GlobalTasksNum); ++TaskIndex)
		{
			const FStateTreeTaskBase& Task = Nodes[TaskIndex].Get<const FStateTreeTaskBase>();
			if (Task.bTaskEnabled)
			{
				bHasGlobalTickTasks |= Task.bShouldCallTick;
				bHasGlobalTickTasksOnlyOnEvents |= Task.bShouldCallTickOnlyOnEvents;
				bHasGlobalTransitionTasks |= Task.bShouldAffectTransitions;
				if (Task.bConsideredForScheduling)
				{
					bCachedRequestGlobalTick |= Task.bShouldCallTick || Task.bShouldAffectTransitions;
					bCachedRequestGlobalTickOnlyOnEvents |= Task.bShouldCallTickOnlyOnEvents;
				}
			}
		}
	}

	// Cache the amount of memory needed to execute the transition's condition.
	for (FCompactStateTransition& Transition : Transitions)
	{
		UE::StateTree::InstanceData::FEvaluationScopeInstanceContainer::FMemoryRequirementBuilder Requirement;
		for (int32 Index = 0; Index < Transition.ConditionsNum; ++Index)
		{
			const int32 NodeIndex = Transition.ConditionsBegin + Index;
			const FStateTreeConditionBase& Cond = Nodes[NodeIndex].Get<const FStateTreeConditionBase>();
			if (Cond.InstanceDataHandle.GetSource() == EStateTreeDataSourceType::EvaluationScopeInstanceData
				|| Cond.InstanceDataHandle.GetSource() == EStateTreeDataSourceType::EvaluationScopeInstanceDataObject)
			{
				const FConstStructView DefaultInstanceView = DefaultEvaluationScopeInstanceData.GetStruct(Cond.InstanceTemplateIndex.Get());
				Requirement.Add(DefaultInstanceView.GetScriptStruct());
			}
		}
		Transition.ConditionEvaluationScopeMemoryRequirement = Requirement.Build();
	}

	bScheduledTickAllowed = Schema ? Schema->IsScheduledTickAllowed() : false;
	StateSelectionRules = Schema ? Schema->GetStateSelectionRules() : EStateTreeStateSelectionRules::Default;
}

bool UStateTree::PatchBindings()
{
	const TArrayView<FStateTreeBindableStructDesc> SourceStructs = PropertyBindings.SourceStructs;
	TArrayView<FPropertyBindingCopyInfoBatch> CopyBatches = PropertyBindings.GetMutableCopyBatches();
	const TArrayView<FStateTreePropertyPathBinding> PropertyPathBindings = PropertyBindings.PropertyPathBindings;

	// Make mapping from data handle to source struct.
	TMap<FStateTreeDataHandle, int32> SourceStructByHandle;
	for (TConstEnumerateRef<FStateTreeBindableStructDesc> SourceStruct : EnumerateRange(SourceStructs))
	{
		SourceStructByHandle.Add(SourceStruct->DataHandle, SourceStruct.GetIndex());
	}

	auto GetSourceStructByHandle = [&SourceStructByHandle, &SourceStructs](const FStateTreeDataHandle DataHandle) -> FStateTreeBindableStructDesc*
	{
		if (int32* Index = SourceStructByHandle.Find(DataHandle))
		{
			return &SourceStructs[*Index];
		}
		return nullptr;
	};
	
	// Reconcile out of date classes.
	for (FStateTreeBindableStructDesc& SourceStruct : SourceStructs)
	{
		if (const UClass* SourceClass = Cast<UClass>(SourceStruct.Struct))
		{
			if (SourceClass->HasAnyClassFlags(CLASS_NewerVersionExists))
			{
				SourceStruct.Struct = SourceClass->GetAuthoritativeClass();
			}
		}
	}

	for (FPropertyBindingCopyInfoBatch& CopyBatch : CopyBatches)
	{
		if (const UClass* TargetClass = Cast<UClass>(CopyBatch.TargetStruct.Get().Struct))
		{
			if (TargetClass->HasAnyClassFlags(CLASS_NewerVersionExists))
			{
				CopyBatch.TargetStruct.GetMutable<>().Struct = TargetClass->GetAuthoritativeClass();
			}
		}
	}

	auto PatchPropertyPath = [](FPropertyBindingPath& PropertyPath)
	{
		for (FPropertyBindingPathSegment& Segment : PropertyPath.GetMutableSegments())
		{
			if (const UClass* InstanceStruct = Cast<UClass>(Segment.GetInstanceStruct()))
			{
				if (InstanceStruct->HasAnyClassFlags(CLASS_NewerVersionExists))
				{
					Segment.SetInstanceStruct(InstanceStruct->GetAuthoritativeClass());
				}
			}
		}
	};

	for (FStateTreePropertyPathBinding& PropertyPathBinding : PropertyPathBindings)
	{
		PatchPropertyPath(PropertyPathBinding.GetMutableSourcePath());
		PatchPropertyPath(PropertyPathBinding.GetMutableTargetPath());
	}

	// Update property bag structs before resolving binding.
	const EStateTreeDataSourceType GlobalParameterDataType = UE::StateTree::CastToDataSourceType(ParameterDataType);
	if (FStateTreeBindableStructDesc* RootParamsDesc = GetSourceStructByHandle(FStateTreeDataHandle(GlobalParameterDataType)))
	{
		RootParamsDesc->Struct = Parameters.GetPropertyBagStruct();
	}

	// Refresh state parameter descs and bindings batches.
	for (const FCompactStateTreeState& State : States)
	{
		// For subtrees and linked states, the parameters must exists.
		if (State.Type == EStateTreeStateType::Subtree
			|| State.Type == EStateTreeStateType::Linked
			|| State.Type == EStateTreeStateType::LinkedAsset)
		{
			if (!State.ParameterTemplateIndex.IsValid())
			{
				UE_LOGF(LogStateTree, Error, "%ls: Data for state '%ls' is malformed. Please recompile the StateTree asset.", *GetPathName(), *State.Name.ToString());
				return false;
			}
		}

		if (State.ParameterTemplateIndex.IsValid())
		{
			// Subtree is a bind source, update bag struct.
			const FCompactStateTreeParameters& Params = DefaultInstanceData.GetMutableStruct(State.ParameterTemplateIndex.Get()).Get<FCompactStateTreeParameters>();
			FStateTreeBindableStructDesc* Desc = GetSourceStructByHandle(State.ParameterDataHandle);
			if (!Desc)
			{
				UE_LOGF(LogStateTree, Error, "%ls: Data for state '%ls' is malformed. Please recompile the StateTree asset.", *GetPathName(), *State.Name.ToString());
				return false;
			}
			Desc->Struct = Params.GetValue().GetScriptStruct();

			if (State.ParameterBindingsBatch.IsValid())
			{
				FPropertyBindingCopyInfoBatch& Batch = CopyBatches[State.ParameterBindingsBatch.Get()];
				Batch.TargetStruct.GetMutable().Struct = Params.GetValue().GetScriptStruct();
			}
		}
	}

	// Check linked state property bags consistency
	for (const FCompactStateTreeState& State : States)
	{
		if (State.Type == EStateTreeStateType::Linked && State.LinkedState.IsValid())
		{
			const FCompactStateTreeState& LinkedState = States[State.LinkedState.Index];

			if (State.ParameterTemplateIndex.IsValid() == false
				|| LinkedState.ParameterTemplateIndex.IsValid() == false)
			{
				UE_LOGF(LogStateTree, Error, "%ls: Data for state '%ls' is malformed. Please recompile the StateTree asset.", *GetPathName(), *State.Name.ToString());
				return false;
			}

			// Check that the bag in linked state matches.
			const FCompactStateTreeParameters& Params = DefaultInstanceData.GetMutableStruct(State.ParameterTemplateIndex.Get()).Get<FCompactStateTreeParameters>();
			const FCompactStateTreeParameters& LinkedStateParams = DefaultInstanceData.GetMutableStruct(LinkedState.ParameterTemplateIndex.Get()).Get<FCompactStateTreeParameters>();

			if (LinkedStateParams.GetValue().GetScriptStruct() != Params.GetValue().GetScriptStruct())
			{
				UE_LOGF(LogStateTree, Error, "%ls: The parameters on state '%ls' does not match the linked state parameters in state '%ls'. Please recompile the StateTree asset.", *GetPathName(), *State.Name.ToString(), *LinkedState.Name.ToString());
				return false;
			}
		}
		else if (State.Type == EStateTreeStateType::LinkedAsset && State.LinkedAsset)
		{
			// Check that the bag in linked state matches.
			const FInstancedPropertyBag& TargetTreeParameters = State.LinkedAsset->Parameters;
			const FCompactStateTreeParameters& Params = DefaultInstanceData.GetMutableStruct(State.ParameterTemplateIndex.Get()).Get<FCompactStateTreeParameters>();
			const bool bIsValid = TargetTreeParameters.GetPropertyBagStruct() == Params.GetValue().GetScriptStruct();
			if (!bIsValid)
			{
				UE_LOGF(LogStateTree, Error, "%ls: The parameters on state '%ls' does not match the linked asset parameters '%ls'. Please recompile the StateTree asset.",
					*GetPathName(), *State.Name.ToString(), *State.LinkedAsset->GetPathName());
				return false;
			}
		}
	}

	TMap<FStateTreeDataHandle, FStateTreeDataView> DataViews;
	TMap<FStateTreeIndex16, FStateTreeDataView> BindingBatchDataView;

	// Tree parameters
	DataViews.Add(FStateTreeDataHandle(GlobalParameterDataType), Parameters.GetMutableValue());

	// Setup data views for context data. Since the external data is passed at runtime, we can only provide the type.
	for (const FStateTreeExternalDataDesc& DataDesc : ContextDataDescs)
	{
		DataViews.Add(DataDesc.Handle.DataHandle, FStateTreeDataView(DataDesc.Struct, nullptr));
	}
	
	// Setup data views for state parameters.
	for (FCompactStateTreeState& State : States)
	{
		if (State.ParameterDataHandle.IsValid())
		{
			FCompactStateTreeParameters& Params = DefaultInstanceData.GetMutableStruct(State.ParameterTemplateIndex.Get()).Get<FCompactStateTreeParameters>();
			DataViews.Add(State.ParameterDataHandle, Params.GetMutableValue());
			if (State.ParameterBindingsBatch.IsValid())
			{
				BindingBatchDataView.Add(State.ParameterBindingsBatch, Params.GetMutableValue());
			}
		}
	}

	// Setup data views for all nodes.
	for (FConstStructView NodeView : Nodes)
	{
		const FStateTreeNodeBase& Node = NodeView.Get<const FStateTreeNodeBase>();

		FStateTreeDataView NodeDataView;
		if (Node.InstanceDataHandle.GetSource() == EStateTreeDataSourceType::SharedInstanceData
			|| Node.InstanceDataHandle.GetSource() == EStateTreeDataSourceType::SharedInstanceDataObject)
		{
			NodeDataView = Node.InstanceDataHandle.IsObjectSource()
				? FStateTreeDataView(SharedInstanceData.GetMutableObject(Node.InstanceTemplateIndex.Get()))
				: FStateTreeDataView(SharedInstanceData.GetMutableStruct(Node.InstanceTemplateIndex.Get()));
		}
		else if (Node.InstanceDataHandle.GetSource() == EStateTreeDataSourceType::EvaluationScopeInstanceData
			|| Node.InstanceDataHandle.GetSource() == EStateTreeDataSourceType::EvaluationScopeInstanceDataObject)
		{
			NodeDataView = Node.InstanceDataHandle.IsObjectSource()
				? FStateTreeDataView(DefaultEvaluationScopeInstanceData.GetMutableObject(Node.InstanceTemplateIndex.Get()))
				: FStateTreeDataView(DefaultEvaluationScopeInstanceData.GetMutableStruct(Node.InstanceTemplateIndex.Get()));
		}
		else
		{
			NodeDataView = Node.InstanceDataHandle.IsObjectSource()
				? FStateTreeDataView(DefaultInstanceData.GetMutableObject(Node.InstanceTemplateIndex.Get()))
				: FStateTreeDataView(DefaultInstanceData.GetMutableStruct(Node.InstanceTemplateIndex.Get()));
		}
		DataViews.Add(Node.InstanceDataHandle, NodeDataView);

		if (Node.BindingsBatch.IsValid())
		{
			BindingBatchDataView.Add(Node.BindingsBatch, NodeDataView);
		}

		if (Node.OutputBindingsBatch.IsValid())
		{
			BindingBatchDataView.Add(Node.OutputBindingsBatch, NodeDataView);
		}
	}
	
	auto GetDataSourceView = [&DataViews](const FStateTreeDataHandle Handle) -> FStateTreeDataView
	{
		if (const FStateTreeDataView* ViewPtr = DataViews.Find(Handle))
		{
			return *ViewPtr;
		}
		return FStateTreeDataView();
	};

	auto GetBindingBatchDataView = [&BindingBatchDataView](const FStateTreeIndex16 Index) -> FStateTreeDataView
	{
		if (const FStateTreeDataView* ViewPtr = BindingBatchDataView.Find(Index))
		{
			return *ViewPtr;
		}
		return FStateTreeDataView();
	};


	for (int32 BatchIndex = 0; BatchIndex < CopyBatches.Num(); ++BatchIndex)
	{
		const FPropertyBindingCopyInfoBatch& Batch = CopyBatches[BatchIndex];

		// Find data view for the binding target.
		FStateTreeDataView TargetView = GetBindingBatchDataView(FStateTreeIndex16(BatchIndex));
		if (!TargetView.IsValid())
		{
			UE_LOGF(LogStateTree, Error, "%s: '%ls' Invalid target struct when trying to bind to '%ls'"
				, __FUNCTION__
				, *GetPathName()
				, *Batch.TargetStruct.Get().Name.ToString());
			return false;
		}

		FString ErrorMsg;
		for (int32 Index = Batch.BindingsBegin.Get(); Index != Batch.BindingsEnd.Get(); Index++)
		{
			FStateTreePropertyPathBinding& Binding = PropertyPathBindings[Index];

			const EStateTreeDataSourceType Source = Binding.GetSourceDataHandle().GetSource();
			const bool bIsSourceEvent = Source == EStateTreeDataSourceType::TransitionEvent || Source == EStateTreeDataSourceType::StateEvent;

			if(!bIsSourceEvent)
			{
				FStateTreeDataView SourceView = GetDataSourceView(Binding.GetSourceDataHandle());

				if (!Binding.GetMutableSourcePath().UpdateSegmentsFromValue(SourceView, &ErrorMsg))
				{
					UE_LOGF(LogStateTree, Error, "%s: '%ls' Failed to update source instance structs for property binding '%ls'. Reason: %ls"
						, __FUNCTION__
						, *GetPathName()
						, *Binding.GetTargetPath().ToString()
						, *ErrorMsg);
					return false;
				}
			}

			if (!Binding.GetMutableTargetPath().UpdateSegmentsFromValue(TargetView, &ErrorMsg))
			{
				UE_LOGF(LogStateTree, Error, "%s: '%ls' Failed to update target instance structs for property binding '%ls'. Reason: %ls"
					, __FUNCTION__
					, *GetPathName()
					, *Binding.GetTargetPath().ToString()
					, *ErrorMsg);
				return false;
			}
		}
	}

	// Cache the amount of memory needed to execute property functions for each batch.
	PropertyFunctionEvaluationScopeMemoryRequirements.Reset(CopyBatches.Num());
	for (FPropertyBindingCopyInfoBatch& CopyBatch : CopyBatches)
	{
		UE::StateTree::InstanceData::FEvaluationScopeInstanceContainer::FMemoryRequirementBuilder CopyBatchFunctionRequirement;

		if (CopyBatch.PropertyFunctionsBegin != CopyBatch.PropertyFunctionsEnd)
		{
			const int32 FuncsBegin = CopyBatch.PropertyFunctionsBegin.Get();
			const int32 FuncsEnd = CopyBatch.PropertyFunctionsEnd.Get();
			for (int32 FuncIndex = FuncsBegin; FuncIndex < FuncsEnd; ++FuncIndex)
			{
				const FStateTreePropertyFunctionBase& Func = Nodes[FuncIndex].Get<const FStateTreePropertyFunctionBase>();
				if (Func.InstanceDataHandle.GetSource() == EStateTreeDataSourceType::EvaluationScopeInstanceData
					|| Func.InstanceDataHandle.GetSource() == EStateTreeDataSourceType::EvaluationScopeInstanceDataObject)
				{
					const FConstStructView DefaultInstanceView = DefaultEvaluationScopeInstanceData.GetStruct(Func.InstanceTemplateIndex.Get());
					CopyBatchFunctionRequirement.Add(DefaultInstanceView.GetScriptStruct());
				}
			}
		}

		PropertyFunctionEvaluationScopeMemoryRequirements.Add(CopyBatchFunctionRequirement.Build());
	}
	check(CopyBatches.Num() == PropertyFunctionEvaluationScopeMemoryRequirements.Num());

	return true;
}

#if WITH_EDITOR

void FStateTreeMemoryUsage::AddUsage(const FConstStructView View)
{
	if (const UScriptStruct* ScriptStruct = View.GetScriptStruct())
	{
		EstimatedMemoryUsage = Align(EstimatedMemoryUsage, ScriptStruct->GetMinAlignment());
		EstimatedMemoryUsage += ScriptStruct->GetStructureSize();
	}
}

void FStateTreeMemoryUsage::AddUsage(const UObject* Object)
{
	if (Object != nullptr)
	{
		check(Object->GetClass());
		EstimatedMemoryUsage += Object->GetClass()->GetStructureSize();
	}
}

TArray<FStateTreeMemoryUsage> UStateTree::CalculateEstimatedMemoryUsage() const
{
	TArray<FStateTreeMemoryUsage> MemoryUsages;
	TArray<TPair<int32, int32>> StateLinks;

	if (CompileStatus != ECompileStatus::Executable
		|| States.IsEmpty()
		|| !Nodes.IsValid())
	{
		return MemoryUsages;
	}

	const int32 TreeMemUsageIndex = MemoryUsages.Emplace(TEXT("State Tree Max"));
	const int32 InstanceMemUsageIndex = MemoryUsages.Emplace(TEXT("Instance Overhead"));
	const int32 EvalMemUsageIndex = MemoryUsages.Emplace(TEXT("Evaluators"));
	const int32 GlobalTaskMemUsageIndex = MemoryUsages.Emplace(TEXT("GlobalTask"));
	const int32 SharedMemUsageIndex = MemoryUsages.Emplace(TEXT("Shared Data"));
	const int32 ExtensionMemUsageIndex = MemoryUsages.Emplace(TEXT("Extensions"));

	auto GetRootStateHandle = [this](const FStateTreeStateHandle InState) -> FStateTreeStateHandle
	{
		FStateTreeStateHandle Result = InState;
		while (Result.IsValid() && States[Result.Index].Parent.IsValid())
		{
			Result = States[Result.Index].Parent;
		}
		return Result;		
	};

	auto GetUsageIndexForState = [&MemoryUsages, this](const FStateTreeStateHandle InStateHandle) -> int32
	{
		check(InStateHandle.IsValid());
		
		const int32 FoundMemUsage = MemoryUsages.IndexOfByPredicate([InStateHandle](const FStateTreeMemoryUsage& MemUsage) { return MemUsage.Handle == InStateHandle; });
		if (FoundMemUsage != INDEX_NONE)
		{
			return FoundMemUsage;
		}

		const FCompactStateTreeState& CompactState = States[InStateHandle.Index];
		
		return MemoryUsages.Emplace(TEXT("State ") + CompactState.Name.ToString(), InStateHandle);
	};

	// Calculate memory usage per state.
	TArray<FStateTreeMemoryUsage> TempStateMemoryUsages;
	TempStateMemoryUsages.SetNum(States.Num());

	for (int32 Index = 0; Index < States.Num(); Index++)
	{
		const FStateTreeStateHandle StateHandle((uint16)Index);
		const FCompactStateTreeState& CompactState = States[Index];
		const FStateTreeStateHandle ParentHandle = GetRootStateHandle(StateHandle);
		const int32 ParentUsageIndex = GetUsageIndexForState(ParentHandle);
		
		FStateTreeMemoryUsage& MemUsage = CompactState.Parent.IsValid() ? TempStateMemoryUsages[Index] : MemoryUsages[GetUsageIndexForState(StateHandle)];
		
		MemUsage.NodeCount += CompactState.TasksNum;

		if (CompactState.Type == EStateTreeStateType::Linked)
		{
			const int32 LinkedUsageIndex = GetUsageIndexForState(CompactState.LinkedState);
			StateLinks.Emplace(ParentUsageIndex, LinkedUsageIndex);
		}
		
		if (CompactState.ParameterTemplateIndex.IsValid())
		{
			MemUsage.NodeCount++;
			MemUsage.AddUsage(DefaultInstanceData.GetStruct(CompactState.ParameterTemplateIndex.Get()));
		}
		
		for (int32 TaskIndex = CompactState.TasksBegin; TaskIndex < (CompactState.TasksBegin + CompactState.TasksNum); TaskIndex++)
		{
			if (const FStateTreeTaskBase* Task = Nodes[TaskIndex].GetPtr<const FStateTreeTaskBase>())
			{
				if (Task->InstanceDataHandle.IsObjectSource())
				{
					MemUsage.NodeCount++;
					MemUsage.AddUsage(DefaultInstanceData.GetObject(Task->InstanceTemplateIndex.Get()));
				}
				else
				{
					MemUsage.NodeCount++;
					MemUsage.AddUsage(DefaultInstanceData.GetStruct(Task->InstanceTemplateIndex.Get()));
				}
			}
		}
	}

	// Combine max child usage to parents. Iterate backwards to update children first.
	for (int32 Index = States.Num() - 1; Index >= 0; Index--)
	{
		const FStateTreeStateHandle StateHandle((uint16)Index);
		const FCompactStateTreeState& CompactState = States[Index];

		FStateTreeMemoryUsage& MemUsage = CompactState.Parent.IsValid() ? TempStateMemoryUsages[Index] : MemoryUsages[GetUsageIndexForState(StateHandle)];

		int32 MaxChildStateMem = 0;
		int32 MaxChildStateNodes = 0;
		
		for (uint16 ChildState = CompactState.ChildrenBegin; ChildState < CompactState.ChildrenEnd; ChildState = States[ChildState].GetNextSibling())
		{
			const FStateTreeMemoryUsage& ChildMemUsage = TempStateMemoryUsages[ChildState];
			if (ChildMemUsage.EstimatedMemoryUsage > MaxChildStateMem)
			{
				MaxChildStateMem = ChildMemUsage.EstimatedMemoryUsage;
				MaxChildStateNodes = ChildMemUsage.NodeCount;
			}
		}

		MemUsage.EstimatedMemoryUsage += MaxChildStateMem;
		MemUsage.NodeCount += MaxChildStateNodes;
	}

	// Accumulate linked states.
	for (int32 Index = StateLinks.Num() - 1; Index >= 0; Index--)
	{
		FStateTreeMemoryUsage& ParentUsage = MemoryUsages[StateLinks[Index].Get<0>()];
		const FStateTreeMemoryUsage& LinkedUsage = MemoryUsages[StateLinks[Index].Get<1>()];
		const int32 LinkedTotalUsage = LinkedUsage.EstimatedMemoryUsage + LinkedUsage.EstimatedChildMemoryUsage;
		if (LinkedTotalUsage > ParentUsage.EstimatedChildMemoryUsage)
		{
			ParentUsage.EstimatedChildMemoryUsage = LinkedTotalUsage;
			ParentUsage.ChildNodeCount = LinkedUsage.NodeCount + LinkedUsage.ChildNodeCount;
		}
	}

	// Evaluators
	FStateTreeMemoryUsage& EvalMemUsage = MemoryUsages[EvalMemUsageIndex];
	for (int32 EvalIndex = EvaluatorsBegin; EvalIndex < (EvaluatorsBegin + EvaluatorsNum); EvalIndex++)
	{
		const FStateTreeEvaluatorBase& Eval = Nodes[EvalIndex].Get<const FStateTreeEvaluatorBase>();
		if (Eval.InstanceDataHandle.IsObjectSource())
		{
			EvalMemUsage.AddUsage(DefaultInstanceData.GetObject(Eval.InstanceTemplateIndex.Get()));
		}
		else
		{
			EvalMemUsage.AddUsage(DefaultInstanceData.GetStruct(Eval.InstanceTemplateIndex.Get()));
		}
		EvalMemUsage.NodeCount++;
	}

	// Global Tasks
	FStateTreeMemoryUsage& GlobalTaskMemUsage = MemoryUsages[GlobalTaskMemUsageIndex];
	for (int32 TaskIndex = GlobalTasksBegin; TaskIndex < (GlobalTasksBegin + GlobalTasksNum); TaskIndex++)
	{
		const FStateTreeTaskBase& Task = Nodes[TaskIndex].Get<const FStateTreeTaskBase>();
		if (Task.InstanceDataHandle.IsObjectSource())
		{
			GlobalTaskMemUsage.AddUsage(DefaultInstanceData.GetObject(Task.InstanceTemplateIndex.Get()));
		}
		else
		{
			GlobalTaskMemUsage.AddUsage(DefaultInstanceData.GetStruct(Task.InstanceTemplateIndex.Get()));
		}
		GlobalTaskMemUsage.NodeCount++;
	}

	// Estimate highest combined usage.
	FStateTreeMemoryUsage& TreeMemUsage = MemoryUsages[TreeMemUsageIndex];

	// Exec state
	TreeMemUsage.AddUsage(DefaultInstanceData.GetStruct(0));
	TreeMemUsage.NodeCount++;

	TreeMemUsage.EstimatedMemoryUsage += EvalMemUsage.EstimatedMemoryUsage;
	TreeMemUsage.NodeCount += EvalMemUsage.NodeCount;

	TreeMemUsage.EstimatedMemoryUsage += GlobalTaskMemUsage.EstimatedMemoryUsage;
	TreeMemUsage.NodeCount += GlobalTaskMemUsage.NodeCount;

	FStateTreeMemoryUsage& InstanceMemUsage = MemoryUsages[InstanceMemUsageIndex];
	// FStateTreeInstanceData overhead.
	InstanceMemUsage.EstimatedMemoryUsage += sizeof(FStateTreeInstanceData);
	// FInstancedStructContainer overhead.
	InstanceMemUsage.EstimatedMemoryUsage += TreeMemUsage.NodeCount * FInstancedStructContainer::OverheadPerItem;

	TreeMemUsage.EstimatedMemoryUsage += InstanceMemUsage.EstimatedMemoryUsage;
	
	int32 MaxSubtreeUsage = 0;
	int32 MaxSubtreeNodeCount = 0;
	
	for (const FStateTreeMemoryUsage& MemUsage : MemoryUsages)
	{
		if (MemUsage.Handle.IsValid())
		{
			const int32 TotalUsage = MemUsage.EstimatedMemoryUsage + MemUsage.EstimatedChildMemoryUsage;
			if (TotalUsage > MaxSubtreeUsage)
			{
				MaxSubtreeUsage = TotalUsage;
				MaxSubtreeNodeCount = MemUsage.NodeCount + MemUsage.ChildNodeCount;
			}
		}
	}

	TreeMemUsage.EstimatedMemoryUsage += MaxSubtreeUsage;
	TreeMemUsage.NodeCount += MaxSubtreeNodeCount;

	FStateTreeMemoryUsage& SharedMemUsage = MemoryUsages[SharedMemUsageIndex];
	SharedMemUsage.NodeCount = SharedInstanceData.Num();
	SharedMemUsage.EstimatedMemoryUsage = SharedInstanceData.GetEstimatedMemoryUsage();

	// Extensions
	FStateTreeMemoryUsage& ExtensionMemUsage = MemoryUsages[ExtensionMemUsageIndex];
	for (UStateTreeExtension* Extension : Extensions)
	{
		if (Extension)
		{
			ExtensionMemUsage.AddUsage(Extension);
			++ExtensionMemUsage.NodeCount;
		}
	}

	return MemoryUsages;
}

void UStateTree::CompileIfChanged()
{
	UE::StateTree::Delegates::Private::OnCompileIfChanged.ExecuteIfBound(this);
}
#endif //WITH_EDITOR

#if WITH_EDITOR || WITH_STATETREE_DEBUG
FString UStateTree::DebugInternalLayoutAsString() const
{
	TStringBuilder<0> DebugString;
	DebugString << TEXT("StateTree (asset: '");
	GetFullName(DebugString);
	DebugString << TEXT("')\n");

	auto PrintObjectNameSafe = [&DebugString](int32 Index, const UObject* Obj)
		{
			DebugString << TEXT("  (");
			DebugString << Index;
			DebugString << TEXT(")");
			if (Obj)
			{
				DebugString << Obj->GetFName();
			}
			else
			{
				DebugString << TEXT("null");
			}
			DebugString << TEXT('\n');
		};
	auto PrintViewNameSafe = [&DebugString](int32 Index, const FConstStructView& View)
		{
			DebugString << TEXT("  (");
			DebugString << Index;
			DebugString << TEXT(")");
			if (View.IsValid())
			{
				DebugString << View.GetScriptStruct()->GetFName();
			}
			else
			{
				DebugString << TEXT("null");
			}
			DebugString << TEXT('\n');
		};

	if (Schema)
	{
		DebugString.Appendf(TEXT("Schema: %s\n"), *WriteToString<128>(Schema->GetFName()));
	}
	else
	{
		DebugString.Append(TEXT("Schema: [None]\n"));
	}

	// Tree items (e.g. tasks, evaluators, conditions)
	DebugString.Appendf(TEXT("\nNodes(%d)\n"), Nodes.Num());
	for (int32 Index = 0; Index < Nodes.Num(); Index++)
	{
		const FConstStructView Node = Nodes[Index];
		PrintViewNameSafe(Index, Node);
	}

	auto PrintInstanceData = [&DebugString, &PrintObjectNameSafe, &PrintViewNameSafe]<typename T>(T& Container, const FStringView Name)
		{
			DebugString.Appendf(TEXT("\n%s(%d)\n"), Name.GetData(), Container.Num());
			for (int32 Index = 0; Index < Container.Num(); Index++)
			{
				if (Container.IsObject(Index))
				{
					const UObject* Data = Container.GetObject(Index);
					PrintObjectNameSafe(Index, Data);
				}
				else
				{
					const FConstStructView Data = Container.GetStruct(Index);
					PrintViewNameSafe(Index, Data);
				}
			}
		};

	// Instance InstanceData data (e.g. tasks)
	PrintInstanceData(DefaultInstanceData, TEXT("Instance Data"));

	// Shared Instance data (e.g. conditions/evaluators)
	PrintInstanceData(SharedInstanceData, TEXT("Shared Instance Data"));

	// Evaluation Scope InstanceData data (e.g. conditions/evaluators)
	PrintInstanceData(DefaultEvaluationScopeInstanceData, TEXT("Evaluation Scope Instance Data"));

	// Execution Runtime InstanceData data (e.g. tasks)
	PrintInstanceData(GetDefaultExecutionRuntimeData(), TEXT("Execution Runtime Instance Data"));

	// External data (e.g. fragments, subsystems)
	DebugString.Appendf(TEXT("\nExternal Data(%d)\n"), ExternalDataDescs.Num());
	if (ExternalDataDescs.Num())
	{
		DebugString.Appendf(TEXT("  [ (Idx) | %-40s | %-8s | %15s ]\n"), TEXT("Name"), TEXT("Optional"), TEXT("Handle"));
		for (int32 DataDescIndex = 0; DataDescIndex < ExternalDataDescs.Num(); ++DataDescIndex)
		{
			const FStateTreeExternalDataDesc& Desc = ExternalDataDescs[DataDescIndex];
			DebugString.Appendf(TEXT("  | (%3d) | %-40s | %8s | %15s |\n"),
				DataDescIndex,
				Desc.Struct ? *Desc.Struct->GetName() : TEXT("null"),
				*UEnum::GetDisplayValueAsText(Desc.Requirement).ToString(),
				*Desc.Handle.DataHandle.Describe());
		}
	}

	// Bindings
#if WITH_PROPERTYBINDINGUTILS_DEBUG
	DebugString << PropertyBindings.DebugAsString();
#endif

	// Task Completion Dispatchers
	DebugString.Appendf(TEXT("\nTask Completion Dispatchers(%d)\n"), TaskCompletionDispatchers.Num());
	if (TaskCompletionDispatchers.Num())
	{
		DebugString.Appendf(TEXT("  [ (Idx) | %15s | %-30s ]\n"),
			TEXT("Task Node Idx"), TEXT("Condition"));
		for (int32 Index = 0; Index < TaskCompletionDispatchers.Num(); ++Index)
		{
			const UE::StateTree::FTaskCompletionDispatcher& Dispatcher = TaskCompletionDispatchers[Index];
			DebugString.Appendf(TEXT("  | (%3d) | %15d | %-30s |\n"),
				Index,
				Dispatcher.TaskNodeIndex.Get(),
				*UEnum::GetDisplayValueAsText(Dispatcher.Condition).ToString());
		}
	}

	// Frames
	DebugString.Appendf(TEXT("\nFrames(%d)\n"), Frames.Num());
	if (Frames.Num())
	{
		DebugString.Appendf(TEXT("  [ (Idx) | %-4s ]\n"), TEXT("Root"));
		for (int32 FrameIndex = 0; FrameIndex < Frames.Num(); ++FrameIndex)
		{
			const FCompactStateTreeFrame& Frame = Frames[FrameIndex];
			DebugString.Appendf(TEXT("  | (%3d) | %-4d |\n"),
				FrameIndex,
				Frame.RootState.Index
			);
		}
	}

	// States
	DebugString.Appendf(TEXT("\nStates(%d)\n"), States.Num());
	if (States.Num())
	{
		DebugString.Appendf(TEXT("  [ (Idx) | %-30s | %15s | %5s [%3s:%-3s[ | Begin Idx : %4s %4s %4s %4s | Num : %4s %4s %4s %4s ]\n"),
			TEXT("Name"), TEXT("Parent"), TEXT("Child"), TEXT("Beg"), TEXT("End"),
			TEXT("Cond"), TEXT("Tr"), TEXT("Tsk"), TEXT("Uti"), TEXT("Cond"), TEXT("Tr"), TEXT("Tsk"), TEXT("Uti"));
		for (int32 StateIndex = 0; StateIndex < States.Num(); ++StateIndex)
		{
			const FCompactStateTreeState& State = States[StateIndex];
			DebugString.Appendf(TEXT("  | (%3d) | %-30s | %15s | %5s [%3d:%-3d[ | %9s   %4d %4d %4d %4d | %3s   %4d %4d %4d %4d |\n"),
				StateIndex,
				*State.Name.ToString(),
				*State.Parent.Describe(),
				TEXT(" "), State.ChildrenBegin, State.ChildrenEnd,
				TEXT(" "), State.EnterConditionsBegin, State.TransitionsBegin, State.TasksBegin, State.UtilityConsiderationsBegin,
				TEXT(" "), State.EnterConditionsNum, State.TransitionsNum, State.TasksNum, State.UtilityConsiderationsNum
			);
		}

		auto AppendBinary = [&DebugString](const uint32 Mask)
			{
				for (int32 Index = (sizeof(uint32) * 8) - 1; Index >= 0; --Index)
				{
					const TCHAR BinaryValue = ((Mask >> Index) & 1U) ? TEXT('1') : TEXT('0');
					DebugString << BinaryValue;
				}
			};

		DebugString.Appendf(TEXT("\n  [ (Idx) | %32s | %8s ]\n"),
			TEXT("CompMask"), TEXT("CustTick"));
		for (int32 StateIndex = 0; StateIndex < States.Num(); ++StateIndex)
		{
			const FCompactStateTreeState& State = States[StateIndex];
			DebugString.Appendf(TEXT("  | (%3d) | "), StateIndex);
			AppendBinary(State.CompletionTasksMask);
			DebugString.Appendf(TEXT(" | %3f |\n"), State.CustomTickRate);
		}
	}

	// Transitions
	DebugString.Appendf(TEXT("\nTransitions(%d)\n"), Transitions.Num());
	if (Transitions.Num())
	{
		DebugString.Appendf(TEXT("  [ (Idx) | %-3s | %15s | %-20s | %-40s | %-40s | %-8s ]\n")
			, TEXT("Idx"), TEXT("State"), TEXT("Transition Trigger"), TEXT("Transition Event Tag"), TEXT("Transition Event Payload"), TEXT("Cond:Num"));
		for (int32 TransitionIndex = 0; TransitionIndex < Transitions.Num(); ++TransitionIndex)
		{
			const FCompactStateTransition& Transition = Transitions[TransitionIndex];
			DebugString.Appendf(TEXT("  | (%3d) | %3d | %15s | %-20s | %-40s | %-40s | %4d:%3d |\n"),
				TransitionIndex,
				Transition.ConditionsBegin,
				*Transition.State.Describe(),
				*UEnum::GetDisplayValueAsText(Transition.Trigger).ToString(),
				*Transition.RequiredEvent.Tag.ToString(),
				Transition.RequiredEvent.PayloadStruct ? *Transition.RequiredEvent.PayloadStruct->GetName() : TEXT("None"),
				Transition.ConditionsBegin,
				Transition.ConditionsNum);
		}
	}

	// Evaluators
	DebugString.Appendf(TEXT("\nEvaluators(%d)\n"), EvaluatorsNum);
	if (EvaluatorsNum)
	{
		DebugString.Appendf(TEXT("  [ (Idx) | %-30s | %12s | %12s | %14s ]\n"),
			TEXT("Name"), TEXT("In Bindings"), TEXT("Out Binding"), TEXT("Struct Idx"));
		for (int32 EvalIndex = EvaluatorsBegin; EvalIndex < (EvaluatorsBegin + EvaluatorsNum); EvalIndex++)
		{
			const FStateTreeEvaluatorBase& Eval = Nodes[EvalIndex].Get<const FStateTreeEvaluatorBase>();
			DebugString.Appendf(TEXT("  | (%3d) | %-30s | %12d | %12d | %14s |\n"),
				EvalIndex,
				*Eval.Name.ToString(),
				Eval.BindingsBatch.Get(),
				Eval.OutputBindingsBatch.Get(),
				*Eval.InstanceDataHandle.Describe());
		}
	}

	// Tasks
	int32 NumberOfTasks = GlobalTasksNum;
	for (const FCompactStateTreeState& State : States)
	{
		NumberOfTasks += State.TasksNum;
	}

	DebugString.Appendf(TEXT("\nTasks(%d)\n  [ (Idx) | %-30s | %-30s | %12s | %12s | %14s ]\n"),
		NumberOfTasks, TEXT("State"), TEXT("Name"), TEXT("In Bindings"), TEXT("Out Bindings"), TEXT("Struct Idx"));
	for (const FCompactStateTreeState& State : States)
	{
		if (State.TasksNum)
		{
			for (int32 TaskIndex = State.TasksBegin; TaskIndex < (State.TasksBegin + State.TasksNum); TaskIndex++)
			{
				const FStateTreeTaskBase& Task = Nodes[TaskIndex].Get<const FStateTreeTaskBase>();
				DebugString.Appendf(TEXT("  | (%3d) | %-30s | %-30s | %12d | %12d | %14s |\n"),
					TaskIndex,
					*State.Name.ToString(),
					*Task.Name.ToString(),
					Task.BindingsBatch.Get(),
					Task.OutputBindingsBatch.Get(),
					*Task.InstanceDataHandle.Describe());
			}
		}
	}
	for (int32 TaskIndex = GlobalTasksBegin; TaskIndex < (GlobalTasksBegin + GlobalTasksNum); TaskIndex++)
	{
		const FStateTreeTaskBase& Task = Nodes[TaskIndex].Get<const FStateTreeTaskBase>();
		DebugString.Appendf(TEXT("  | (%3d) | %-30s | %-30s | %8d | %12d | %14s |\n"),
			TaskIndex,
			TEXT("Global"),
			*Task.Name.ToString(),
			Task.BindingsBatch.Get(),
			Task.OutputBindingsBatch.Get(),
			*Task.InstanceDataHandle.Describe());
	}

	// Conditions
	DebugString.Appendf(TEXT("\nConditions\n  [ (Idx) | %-30s | %8s | %12s | %14s ]\n"),
		TEXT("Name"), TEXT("Operand"), TEXT("Evaluation"), TEXT("Struct Idx"));
	{
		for (int32 Index = 0; Index < Nodes.Num(); ++Index)
		{
			if(const FStateTreeConditionBase* Cond = Nodes[Index].GetPtr<const FStateTreeConditionBase>())
			{
				DebugString.Appendf(TEXT("  | (%3d) | %-30s | %8s | %12s | %14s |\n"),
					Index,
					*Cond->Name.ToString(),
					*UEnum::GetDisplayValueAsText(Cond->Operand).ToString(),
					*UEnum::GetDisplayValueAsText(Cond->EvaluationMode).ToString(),
					*Cond->InstanceDataHandle.Describe());
			}
		}
	}

	// Extensions
	DebugString.Appendf(TEXT("\nExtensions(%d)\n"), Extensions.Num());
	for (int32 Index = 0; Index < Extensions.Num(); ++Index)
	{
		if (const UStateTreeExtension* Extension = Extensions[Index])
		{
			if (Extension)
			{
				DebugString.Appendf(TEXT("  %s\n"), *WriteToString<128>(Extension->GetFName()));
			}
			else
			{
				DebugString.Append(TEXT("  [None]\n"));
			}
		}
	}

	return FString(DebugString);
}
#endif // WITH_EDITOR || WITH_STATETREE_DEBUG

#if WITH_STATETREE_DEBUG
namespace UE::StateTree::Debug::Private
{
	IConsoleVariable* FindCVarInstanceDataGC()
	{
		IConsoleVariable* FoundVariable = IConsoleManager::Get().FindConsoleVariable(TEXT("StateTree.RuntimeValidation.InstanceDataGC"));
		check(FoundVariable);
		return FoundVariable;
	}
}

void UStateTree::HandleRuntimeValidationPreGC()
{
	GCObjectDatas.Reset();
	auto AddInstanceObject = [this](const FStateTreeInstanceStorage& InstanceData, int32 PerThreadSharedIndex, FDebugInstanceData::EContainer Container)
		{
			const int32 NumObject = InstanceData.Num();
			for (int32 Index = 0; Index < NumObject; ++Index)
			{
				FConstStructView Instance = InstanceData.GetStruct(Index);
				if (!Instance.IsValid())
				{
					const TCHAR* Msg = (Container == FDebugInstanceData::EContainer::SharedInstance) ? TEXT("A shared") : TEXT("An");
					ensureAlwaysMsgf(false, TEXT("%s instance data is invalid. InstanceDataIndex:%d. PerThreadSharedIndex:%d."), Msg, Index, PerThreadSharedIndex);
					UE::StateTree::Debug::Private::FindCVarInstanceDataGC()->Set(false);
					GCObjectDatas.Reset();
					break;
				}

				FDebugInstanceData& Data = GCObjectDatas.AddDefaulted_GetRef();
				Data.InstanceDataStructIndex = Index;
				Data.SharedInstanceDataIndex = PerThreadSharedIndex;
				Data.Container = Container;

				if (const FStateTreeInstanceObjectWrapper* Wrapper = Instance.GetPtr<const FStateTreeInstanceObjectWrapper>())
				{
					if (!IsValid(Wrapper->InstanceObject))
					{
						const TCHAR* Msg = (Container == FDebugInstanceData::EContainer::SharedInstance) ? TEXT("A shared") : TEXT("An");
						ensureAlwaysMsgf(false, TEXT("%s instance data object is invalid. InstanceDataIndex:%d. PerThreadSharedIndex:%d."), Msg, Index, PerThreadSharedIndex);
						UE::StateTree::Debug::Private::FindCVarInstanceDataGC()->Set(false);
						GCObjectDatas.Reset();
						break;
					}

					Data.Object = FWeakObjectPtr(Wrapper->InstanceObject);
					Data.Type = FDebugInstanceData::EObjectType::ObjectInstance;
				}
				else
				{
					Data.Object = FWeakObjectPtr(Instance.GetScriptStruct());
					Data.Type = FDebugInstanceData::EObjectType::Struct;
				}
			}
		};

	AddInstanceObject(DefaultInstanceData.GetStorage(), INDEX_NONE, FDebugInstanceData::EContainer::DefaultInstance);
	AddInstanceObject(SharedInstanceData.GetStorage(), INDEX_NONE, FDebugInstanceData::EContainer::SharedInstance);

	{
		UE::TReadScopeLock ReadLock(PerThreadSharedInstanceDataLock);
		for (int32 Index = 0; Index < PerThreadSharedInstanceData.Num(); ++Index)
		{
			TSharedPtr<FStateTreeInstanceData>& SharedInstance = PerThreadSharedInstanceData[Index];
			if (ensureMsgf(SharedInstance, TEXT("The shared instance data is invalid. Index %d"), Index))
			{
				AddInstanceObject(SharedInstance->GetStorage(), Index, FDebugInstanceData::EContainer::SharedInstance);
			}
		}
	}
}

void UStateTree::HandleRuntimeValidationPostGC()
{
	for (const FDebugInstanceData& Data : GCObjectDatas)
	{
		if (Data.Object.IsStale())
		{
			if (Data.Type == FDebugInstanceData::EObjectType::ObjectInstance)
			{
				const TCHAR* Msg = (Data.Container == FDebugInstanceData::EContainer::SharedInstance) ? TEXT("A shared") : TEXT("An");
				ensureAlwaysMsgf(false, TEXT("%s instance data object is GCed. InstDataIndex:%d. SharedIndex:%d."), Msg, Data.InstanceDataStructIndex, Data.SharedInstanceDataIndex);
			}
			else
			{ //-V523 disabling identical branch warning
				const TCHAR* Msg = (Data.Container == FDebugInstanceData::EContainer::SharedInstance) ? TEXT("A shared") : TEXT("An");
				ensureAlwaysMsgf(false, TEXT("%s instance data struct type is GCed. InstDataIndex:%d. SharedIndex:%d."), Msg, Data.InstanceDataStructIndex, Data.SharedInstanceDataIndex);
			}
			UE::StateTree::Debug::Private::FindCVarInstanceDataGC()->Set(false);
			break;
		}
	}
	GCObjectDatas.Reset();
}
#endif
