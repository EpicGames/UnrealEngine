// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeAsyncExecutionContext.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeTaskBase.h"

namespace UE::StateTree::Async
{
	const FStateTreeNodeBase& FActivePathInfo::GetNode() const
	{
		check(IsValid() && Frame->StateTree && NodeIndex.IsValid());

		return *(Frame->StateTree->GetNode(NodeIndex.AsInt32()).GetPtr<const FStateTreeNodeBase>());
	}
}


template <bool bWithWriteAccess>
TStateTreeStrongExecutionContext<bWithWriteAccess>::TStateTreeStrongExecutionContext(const FStateTreeWeakExecutionContext& WeakContext)
	: Owner(WeakContext.Owner.Pin()),
	StateTree(WeakContext.StateTree.Pin()),
	Storage(WeakContext.Storage.Pin()),
	TemporaryStorage(WeakContext.TemporaryStorage.Pin()),
	FrameID(WeakContext.FrameID),
	StateID(WeakContext.StateID),
	NodeIndex(WeakContext.NodeIndex)
{
	if (Owner && StateTree && Storage)
	{
		bAccessAcquired = true;
		if constexpr (bWithWriteAccess)
		{
			Storage->AcquireWriteAccess();
		}
		else
		{
			Storage->AcquireReadAccess();
		}
	}
}

template <bool bWithWriteAccess>
TStateTreeStrongExecutionContext<bWithWriteAccess>::~TStateTreeStrongExecutionContext()
{
	if (bAccessAcquired)
	{
		check(IsValidInstanceStorage());

		if constexpr (bWithWriteAccess)
		{
			Storage->ReleaseWriteAccess();
		}
		else
		{
			Storage->ReleaseReadAccess();
		}
	}
}

template <bool bWithWriteAccess>
bool TStateTreeStrongExecutionContext<bWithWriteAccess>::SendEvent(const FGameplayTag Tag, const FConstStructView Payload, const FName Origin) const requires bWithWriteAccess
{
	static_assert(bWithWriteAccess);

	if (IsValid())
	{
		FStateTreeMinimalExecutionContext Context(Owner.Get(), StateTree.Get(), *Storage.Get());
		Context.SendEvent(Tag, Payload, Origin);
		return true;
	}

	return false;
}

template <bool bWithWriteAccess>
bool TStateTreeStrongExecutionContext<bWithWriteAccess>::RequestTransition(FStateTreeStateHandle TargetState,
	EStateTreeTransitionPriority Priority, EStateTreeSelectionFallback Fallback) const requires bWithWriteAccess
{
	static_assert(bWithWriteAccess);

	UE::StateTree::Async::FActivePathInfo ActivePath = GetActivePathInfo();
	if (ActivePath.IsValid())
	{
		FStateTreeTransitionRequest Request;
		Request.SourceFrameID = FrameID;
		Request.SourceStateID = StateID;
		Request.TargetState = TargetState;
		Request.Priority = Priority;
		Request.Fallback = Fallback;

		Storage->AddTransitionRequest(Owner.Get(), Request);
		ScheduleNextTick(Owner.Get(), StateTree.Get(), *Storage, UE::StateTree::ETickReason::TransitionRequest);

		return true;
	}

	return false;
}

template <bool bWithWriteAccess>
bool TStateTreeStrongExecutionContext<bWithWriteAccess>::BroadcastDelegate(const FStateTreeDelegateDispatcher& Dispatcher) const requires bWithWriteAccess
{
	static_assert(bWithWriteAccess);

	if (!Dispatcher.IsValid())
	{
		// Nothings binds to the delegate, not an error.
		return true;
	}

	UE::StateTree::Async::FActivePathInfo ActivePath = GetActivePathInfo();
	if (ActivePath.IsValid())
	{
		FStateTreeExecutionState& Exec = Storage->GetMutableExecutionState();
		Exec.DelegateActiveListeners.BroadcastDelegate(Dispatcher, Exec);
		if (UE::StateTree::ExecutionContext::MarkDelegateAsBroadcasted(Dispatcher, *ActivePath.Frame, *Storage))
		{
			ScheduleNextTick(Owner.Get(), StateTree.Get(), *Storage, UE::StateTree::ETickReason::Delegate);
		}

		return true;
	}

	return false;
}

template <bool bWithWriteAccess>
bool TStateTreeStrongExecutionContext<bWithWriteAccess>::BindDelegate(const FStateTreeDelegateListener& Listener, FSimpleDelegate Delegate) const requires bWithWriteAccess
{
	static_assert(bWithWriteAccess);

	if (!Listener.IsValid())
	{
		// Nothing binds to the delegate, not an error.
		return true;
	}

	UE::StateTree::Async::FActivePathInfo ActivePath = GetActivePathInfo();
	if (ActivePath.IsValid())
	{
		FStateTreeExecutionState& Exec = Storage->GetMutableExecutionState();
		if (ensure(ActivePath.Frame->StateTree))
		{
			Exec.DelegateActiveListeners.Add(Listener, MoveTemp(Delegate), FrameID, StateID, NodeIndex);
			return true;
		}
	}

	return false;
}

template <bool bWithWriteAccess>
bool TStateTreeStrongExecutionContext<bWithWriteAccess>::UnbindDelegate(const FStateTreeDelegateListener& Listener) const requires bWithWriteAccess
{
	static_assert(bWithWriteAccess);

	if (!Listener.IsValid())
	{
		// The listener is not bound to a dispatcher. It will never trigger the delegate. It is not an error.
		return true;
	}

	// Allow unbinding from context created outside the ExecContext loop
	if (IsValid())
	{
		FStateTreeExecutionState& Exec = Storage->GetMutableExecutionState();
		Exec.DelegateActiveListeners.Remove(Listener);

		return true;
	}

	return false;
}

template <bool bWithWriteAccess>
bool TStateTreeStrongExecutionContext<bWithWriteAccess>::FinishTask(EStateTreeFinishTaskType FinishType) const requires bWithWriteAccess
{
	static_assert(bWithWriteAccess);

	bool bSucceed = false;

	UE::StateTree::Async::FActivePathInfo ActivePath = GetActivePathInfo();
	if (ActivePath.IsValid())
	{
		FStateTreeExecutionState& Exec = Storage->GetMutableExecutionState();
		const UStateTree* FrameStateTree = ActivePath.Frame->StateTree;
		if (ensure(FrameStateTree))
		{
			using namespace UE::StateTree;
			using namespace UE::StateTree::ExecutionContext;

			const int32 AssetNodeIndex = ActivePath.NodeIndex.AsInt32();
			const int32 GlobalTaskBeginIndex = FrameStateTree->GlobalTasksBegin;
			const int32 GlobalTaskEndIndex = FrameStateTree->GlobalTasksBegin + FrameStateTree->GlobalTasksNum;
			const bool bIsGlobalTask = AssetNodeIndex >= GlobalTaskBeginIndex && AssetNodeIndex < GlobalTaskEndIndex;
			const ETaskCompletionStatus TaskStatus = ExecutionContext::CastToTaskStatus(FinishType);

			auto BroadcastDelegateHelper = [Self=this, FrameStateTree, AssetNodeIndex](const FTasksCompletionStatus::FSetStatusResult StatusResult)
				{
					// Broadcast task completion dispatcher only if status actually changed
					if (StatusResult.bChanged)
					{
						check(FrameStateTree->GetNodes().IsValidIndex(AssetNodeIndex));
						const FStateTreeTaskBase* TaskPtr = FrameStateTree->GetNode(AssetNodeIndex).GetPtr<const FStateTreeTaskBase>();
						check(TaskPtr);
						if (TaskPtr->bHasTaskCompletionDelegateDispatcher)
						{
							const FTaskCompletionDelegateDispatcherContainer Dispatchers = GetTaskCompletionDispatcher(FrameStateTree, AssetNodeIndex, StatusResult.CurrentStatus);
							for (const FStateTreeDelegateDispatcher& Dispatcher : Dispatchers)
							{
								Self->BroadcastDelegate(Dispatcher);
							}
						}
					}
				};

			bool bCompleted = false;
			if (ActivePath.Frame->bIsGlobalFrame && bIsGlobalTask)
			{
				check(!ActivePath.StateHandle.IsValid());
				const int32 FrameTaskIndex = AssetNodeIndex - GlobalTaskBeginIndex;
				FTasksCompletionStatus GlobalTasksStatus = ActivePath.Frame->ActiveTasksStatus.GetStatus(FrameStateTree);
				const FTasksCompletionStatus::FSetStatusResult GlobalStatusResult = GlobalTasksStatus.SetTaskStatusWithPriority(FrameTaskIndex, TaskStatus);
				bCompleted = GlobalTasksStatus.IsCompleted();
				BroadcastDelegateHelper(GlobalStatusResult);
				bSucceed = true;
			}
			else if (ensure(ActivePath.StateHandle.IsValid()))
			{
				const FCompactStateTreeState& State = FrameStateTree->States[ActivePath.StateHandle.Index];
				const int32 StateTaskBeginIndex = State.TasksBegin;
				const int32 StateTaskEndIndex = State.TasksBegin + State.TasksNum;
				const bool bIsStateTask = AssetNodeIndex >= StateTaskBeginIndex && AssetNodeIndex < StateTaskEndIndex;
				if (bIsStateTask)
				{
					const int32 StateTaskIndex = AssetNodeIndex - State.TasksBegin;
					FTasksCompletionStatus StateTasksStatus = ActivePath.Frame->ActiveTasksStatus.GetStatus(State);
					const FTasksCompletionStatus::FSetStatusResult StateStatusResult = StateTasksStatus.SetTaskStatusWithPriority(StateTaskIndex, TaskStatus);
					bCompleted = StateTasksStatus.IsCompleted();
					BroadcastDelegateHelper(StateStatusResult);
					bSucceed = true;
				}
			}

			if (bCompleted)
			{
				Exec.bHasPendingCompletedState = true;
				ScheduleNextTick(Owner.Get(), StateTree.Get(), *Storage, UE::StateTree::ETickReason::CompletedState);
			}
		}
	}

	return bSucceed;
}

template <bool bWithWriteAccess>
bool TStateTreeStrongExecutionContext<bWithWriteAccess>::UpdateScheduledTickRequest(UE::StateTree::FScheduledTickHandle Handle, FStateTreeScheduledTick ScheduledTick) const requires bWithWriteAccess
{
	static_assert(bWithWriteAccess);

	if (IsValid())
	{
		FStateTreeExecutionState& Exec = Storage->GetMutableExecutionState();
		if (Exec.UpdateScheduledTickRequest(Handle, ScheduledTick))
		{
			ScheduleNextTick(Owner.Get(), StateTree.Get(), *Storage, UE::StateTree::ETickReason::ScheduledTickRequest);
		}

		return true;
	}

	return false;
}

template <bool bWithWriteAccess>
bool TStateTreeStrongExecutionContext<bWithWriteAccess>::CopyInputBindings() const requires bWithWriteAccess
{
	static_assert(bWithWriteAccess);
	using namespace UE::StateTree;
	using namespace UE::StateTree::InstanceData;

	const Async::FActivePathInfo ActivePath = GetActivePathInfo();
	if (!ActivePath.IsValid())
	{
		return false;
	}

	const FStateTreeNodeBase& Node = ActivePath.GetNode();
	if (!Node.BindingsBatch.IsValid())
	{
		return true;
	}

	const FStateTreeDataView InstanceDataView = GetDataViewOrTemporary(*Storage, nullptr, *ActivePath.Frame, Node.InstanceDataHandle);
	if (!InstanceDataView.IsValid())
	{
		return false;
	}

	return CopyBindingBatchForAsync(ActivePath.Frame, InstanceDataView, Node.BindingsBatch);
}

template <bool bWithWriteAccess>
bool TStateTreeStrongExecutionContext<bWithWriteAccess>::CopyOutputBindings() const requires bWithWriteAccess
{
	static_assert(bWithWriteAccess);
	using namespace UE::StateTree;
	using namespace UE::StateTree::InstanceData;

	const Async::FActivePathInfo ActivePath = GetActivePathInfo();
	if (!ActivePath.IsValid())
	{
		return false;
	}

	const FStateTreeNodeBase& Node = ActivePath.GetNode();
	if (!Node.OutputBindingsBatch.IsValid())
	{
		return true;
	}

	const FStateTreeDataView InstanceDataView = GetDataViewOrTemporary(*Storage, nullptr, *ActivePath.Frame, Node.InstanceDataHandle);
	if (!InstanceDataView.IsValid())
	{
		return false;
	}

	return CopyBindingBatchForAsync(ActivePath.Frame, InstanceDataView, Node.OutputBindingsBatch);
}

template <bool bWithWriteAccess>
UE::StateTree::Async::FActivePathInfo TStateTreeStrongExecutionContext<bWithWriteAccess>::GetActivePathInfo() const
{
	if (!IsValidInstanceStorage())
	{
		return {};
	}

	UE::StateTree::Async::FActivePathInfo Result;
	FStateTreeExecutionState& Exec = Storage->GetMutableExecutionState();
	if (const int32 ActiveFrameIndex = Exec.IndexOfActiveFrame(FrameID); ActiveFrameIndex != INDEX_NONE)
	{
		Result.Frame = &Exec.ActiveFrames[ActiveFrameIndex];
		Result.ParentFrame = ActiveFrameIndex != 0 ? &Exec.ActiveFrames[ActiveFrameIndex - 1] : nullptr;
	}
	else if (TemporaryStorage)
	{
		using FFrameAndParent = UE::StateTree::ExecutionContext::ITemporaryStorage::FFrameAndParent;
		const FFrameAndParent FrameInfo = TemporaryStorage->GetExecutionFrame(FrameID);
		Result.Frame = FrameInfo.Frame;
		if (FrameInfo.ParentFrameID.IsValid())
		{
			if (const int32 ActiveParentFrameIndex = Exec.IndexOfActiveFrame(FrameInfo.ParentFrameID); ActiveParentFrameIndex != INDEX_NONE)
			{
				Result.ParentFrame = &Exec.ActiveFrames[ActiveParentFrameIndex];
			}
			else
			{
				const FFrameAndParent ParentFrameInfo = TemporaryStorage->GetExecutionFrame(FrameInfo.ParentFrameID);
				Result.ParentFrame = ParentFrameInfo.Frame;
			}

			ensureMsgf(Result.Frame, TEXT("The frame ID exist in the frame holder. It should exist either in the active of the holder."));
		}
	}

	if (Result.Frame == nullptr)
	{
		return {};
	}

	if (NodeIndex.AsInt32() > Result.Frame->ActiveNodeIndex.AsInt32())
	{
		return {};
	}
	Result.NodeIndex = NodeIndex;

	if (StateID.IsValid())
	{
		Result.StateHandle = Result.Frame->ActiveStates.FindStateHandle(StateID);
		if (!Result.StateHandle.IsValid() && TemporaryStorage)
		{
			Result.StateHandle = TemporaryStorage->GetStateHandle(StateID).GetStateHandle();
		}

		if (!Result.StateHandle.IsValid())
		{
			return {};
		}
	}

	return Result;
}

template <bool bWithWriteAccess>
bool TStateTreeStrongExecutionContext<bWithWriteAccess>::CopyBindingBatchForAsync(const FStateTreeExecutionFrame* CurrentFrame, const FStateTreeDataView TargetView, const FStateTreeIndex16 BindingsBatch) const requires bWithWriteAccess
{
	using namespace UE::StateTree::InstanceData;

	if (!BindingsBatch.IsValid())
	{
		return true;
	}

	if (CurrentFrame == nullptr || CurrentFrame->StateTree == nullptr)
	{
		return false;
	}

	const FStateTreePropertyBindings& Bindings = CurrentFrame->StateTree->GetPropertyBindings();
	const FPropertyBindingCopyInfoBatch& Batch = Bindings.GetBatch(BindingsBatch);
	check(TargetView.GetStruct() == Batch.TargetStruct.Get().Struct);

	// Can't have property function because the context doesn't exist to call FStateTreePropertyFunctionBase::Execute(FStateTreeExecutionContext& Context) const.
	if (Batch.PropertyFunctionsBegin != Batch.PropertyFunctionsEnd)
	{
		UE_LOGF(LogStateTree, Verbose, "Property functions cannot be evaluated in async context. %ls", *CurrentFrame->StateTree->GetPathName());
		return false;
	}

	bool bSucceed = false;
	TSharedPtr<FStateTreeInstanceData> SharedPtr = CurrentFrame->StateTree->GetSharedInstanceData();
	FStateTreeInstanceStorage* SharedInstanceDataStorage = SharedPtr ? &(SharedPtr->GetMutableStorage()) : nullptr;

	bSucceed = true;
	for (const FPropertyBindingCopyInfo& Copy : Bindings.GetBatchCopies(Batch))
	{
		const FStateTreeDataHandle SourceHandle = Copy.SourceDataHandle.Get<FStateTreeDataHandle>();

		// Some data source are not available.
		if (DoesRequireInstanceStorage(SourceHandle.GetSource()))
		{
			FStateTreeInstanceStorage* StoragePtr = Storage.Get();
			const FStateTreeDataView SourceView = GetDataViewOrTemporary(*StoragePtr, SharedInstanceDataStorage, *CurrentFrame, SourceHandle);
			bSucceed &= Bindings.CopyProperty(Copy, SourceView, TargetView);
		}
		else
		{
			UE_LOGF(LogStateTree, Verbose, "%ls cannot be evaluated in async context.", *SourceHandle.Describe());
			bSucceed = false;
		}
	}

	return bSucceed;
}

template <bool bWithWriteAccess>
FStateTreeDataView TStateTreeStrongExecutionContext<bWithWriteAccess>::GetInstanceDataPtrInternal() const
{
	using namespace UE::StateTree;

	Async::FActivePathInfo ActivePath = GetActivePathInfo();
	if (ActivePath.IsValid())
	{
		return InstanceData::GetDataViewOrTemporary(
			*Storage,
			/* SharedInstanceStorage */ nullptr,
			*ActivePath.Frame,
			ActivePath.GetNode().InstanceDataHandle);
	}

	return {};
}

template <bool bWithWriteAccess>
FStateTreeDataView TStateTreeStrongExecutionContext<bWithWriteAccess>::GetExecutionRuntimeDataPtrInternal() const
{
	using namespace UE::StateTree;

	Async::FActivePathInfo ActivePath = GetActivePathInfo();
	if (ActivePath.IsValid())
	{
		const FStateTreeDataHandle DataHandle = FStateTreeDataHandle(EStateTreeDataSourceType::ExecutionRuntimeDataAny, ActivePath.GetNode().ExecutionRuntimeTemplateIndex.Get());
		return InstanceData::GetDataViewOrTemporary(
			*Storage,
			/* SharedInstanceStorage */ nullptr,
			*ActivePath.Frame,
			DataHandle);
	}

	return {};
}

template <bool bWithWriteAccess>
void TStateTreeStrongExecutionContext<bWithWriteAccess>::ScheduleNextTick(TNotNull<UObject*> Owner, TNotNull<const UStateTree*> RootStateTree, FStateTreeInstanceStorage& Storage, UE::StateTree::ETickReason Reason) requires bWithWriteAccess
{
	static_assert(bWithWriteAccess);

	TInstancedStruct<FStateTreeExecutionExtension>& ExecutionExtension = Storage.GetMutableExecutionState().ExecutionExtension;
	if (RootStateTree->IsScheduledTickAllowed() && ExecutionExtension.IsValid())
	{
		ExecutionExtension.GetMutable().ScheduleNextTick(FStateTreeExecutionExtension::FContextParameters(*Owner, *RootStateTree, Storage), FStateTreeExecutionExtension::FNextTickArguments(Reason));
	}
}

template struct TStateTreeStrongExecutionContext<false>;
template struct TStateTreeStrongExecutionContext<true>;

FStateTreeWeakExecutionContext::FStateTreeWeakExecutionContext(const FStateTreeExecutionContext& Context)
	: Owner(Context.GetOwner())
	, StateTree(Context.GetStateTree())
	, Storage(Context.GetMutableInstanceData()->GetWeakMutableStorage())
{
	if (const FStateTreeExecutionFrame* Frame = Context.GetCurrentlyProcessedFrame())
	{
		FrameID = Frame->FrameID;
		StateID = Frame->ActiveStates.FindStateID(Context.GetCurrentlyProcessedState());
		NodeIndex = Context.GetCurrentlyProcessedNodeIndex();
	}
	TemporaryStorage = Context.GetCurrentlyProcessedTemporaryStorage();
}

bool FStateTreeWeakExecutionContext::SendEvent(const FGameplayTag Tag, const FConstStructView Payload, const FName Origin) const
{
	return MakeStrongExecutionContext().SendEvent(Tag, Payload, Origin);
}

bool FStateTreeWeakExecutionContext::RequestTransition(FStateTreeStateHandle TargetState, EStateTreeTransitionPriority Priority, const EStateTreeSelectionFallback Fallback) const
{
	return MakeStrongExecutionContext().RequestTransition(TargetState, Priority, Fallback);
}

bool FStateTreeWeakExecutionContext::BroadcastDelegate(const FStateTreeDelegateDispatcher& Dispatcher) const
{
	return MakeStrongExecutionContext().BroadcastDelegate(Dispatcher);
}

bool FStateTreeWeakExecutionContext::BindDelegate(const FStateTreeDelegateListener& Listener, FSimpleDelegate Delegate) const
{
	constexpr bool bWithWriteAccess = true;
	return MakeStrongExecutionContext().BindDelegate(Listener, Delegate);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool FStateTreeWeakExecutionContext::BindDelegate(const FStateTreeWeakTaskRef& Task, const FStateTreeDelegateListener& Listener, FSimpleDelegate Delegate) const
{
	// Deprecated. Use the version without the TaskRef.
	return BindDelegate(Listener, Delegate);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool FStateTreeWeakExecutionContext::RemoveDelegateListener(const FStateTreeDelegateListener& Listener) const
{
	return UnbindDelegate(Listener);
}

bool FStateTreeWeakExecutionContext::UnbindDelegate(const FStateTreeDelegateListener& Listener) const
{
	return MakeStrongExecutionContext().UnbindDelegate(Listener);
}

bool FStateTreeWeakExecutionContext::FinishTask(EStateTreeFinishTaskType FinishType) const
{
	return MakeStrongExecutionContext().FinishTask(FinishType);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool FStateTreeWeakExecutionContext::FinishTask(const FStateTreeWeakTaskRef& Task, EStateTreeFinishTaskType FinishType) const
{
	return FinishTask(FinishType);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool FStateTreeWeakExecutionContext::UpdateScheduledTickRequest(UE::StateTree::FScheduledTickHandle Handle, FStateTreeScheduledTick ScheduledTick) const
{
	return MakeStrongExecutionContext().UpdateScheduledTickRequest(Handle, ScheduledTick);
}
