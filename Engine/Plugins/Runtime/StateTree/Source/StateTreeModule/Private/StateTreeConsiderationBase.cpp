// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeConsiderationBase.h"
#include "StateTreeTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeConsiderationBase)

FStateTreeConsiderationBase::FStateTreeConsiderationBase()
	: Operand(EStateTreeExpressionOperand::And)
	, DeltaIndent(0)
{
}

float FStateTreeConsiderationBase::GetNormalizedScore(FStateTreeExecutionContext& Context) const
{
	return FMath::Clamp(GetScore(Context), 0.f, 1.f);
}
