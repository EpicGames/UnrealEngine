// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeDelegates.h"
#include "StateTreeDelegatesInternal.h"

namespace UE::StateTree::Delegates
{

#if WITH_EDITOR
	FOnIdentifierChanged OnIdentifierChanged;
	FOnSchemaChanged OnSchemaChanged;
	FOnParametersChanged OnParametersChanged;
	FOnGlobalDataChanged OnGlobalDataChanged;
	FOnVisualThemeChanged OnVisualThemeChanged;
	FOnStateParametersChanged OnStateParametersChanged;
	FOnBreakpointsChanged OnBreakpointsChanged;
	FOnPostCompile OnPostCompile;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	//Deprecated
	FOnRequestCompile OnRequestCompile;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	FOnRequestEditorHash OnRequestEditorHash;
#endif // WITH_EDITOR

#if WITH_STATETREE_TRACE
	FOnTracingStateChanged OnTracingStateChanged;
#endif // WITH_STATETREE_TRACE

#if WITH_STATETREE_TRACE_DEBUGGER
	FOnTraceAnalysisStateChanged OnTraceAnalysisStateChanged;
	FOnTracingTimelineScrubbed OnTracingTimelineScrubbed;
#endif // WITH_STATETREE_TRACE_DEBUGGER

}; // UE::StateTree::Delegates

namespace UE::StateTree::Delegates::Private
{
	FOnStateTreeEditorBindingUpdated OnStateTreeEditorBindingUpdated;
	FOnStateTreeAssetLoaded OnStateTreeAssetLoaded;
#if WITH_EDITOR
	FOnCompileIfChanged OnCompileIfChanged;
	FOnStateTreeMarkedDirty OnStateTreeMarkedAsModified;
	FOnStateTreeRequestAssetRegistryTags OnRequestAssetRegistryTags;
	FOnAppendToClassSchema OnAppendToClassSchema;
	FOnStateTreeAssetLoaded OnPreCookStateTreeAsset;
#endif
} //UE::StateTree::Delegates::Private
