// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeEvaluationScopeInstanceContainer.h"

#include "UObject/Class.h"
#include "UObject/Package.h"
#include <type_traits>

namespace UE::StateTree::InstanceData
{
namespace Private
{
	static constexpr uint32 DebugTableEndTag = 0x99AABBCC;
	static constexpr uint32 DebugStructEndTag = 0xFFEEDDCC;
	void Add(FEvaluationScopeInstanceContainer::FMemoryRequirement& MemoryRequirement, TNotNull<const UScriptStruct*> Struct)
	{
		const int32 StructSize = Struct->GetStructureSize();
		const int32 StructAlignment = Struct->GetMinAlignment();
		MemoryRequirement.Alignment = FMath::Max(StructAlignment, MemoryRequirement.Alignment);
		MemoryRequirement.Size = Align(MemoryRequirement.Size, StructAlignment);
		MemoryRequirement.Size += StructSize;
		MemoryRequirement.Size += sizeof(DebugStructEndTag); // for struct end tag
	}
} // namespace Private

void FEvaluationScopeInstanceContainer::FMemoryRequirementBuilder::Add(TNotNull<const UScriptStruct*> Struct)
{
	Private::Add(MemoryRequirement, Struct);
	if (FirstStructAlignment <= 0)
	{
		FirstStructAlignment = Struct->GetMinAlignment();
	}
	++MemoryRequirement.NumberOfElements;
}

FEvaluationScopeInstanceContainer::FMemoryRequirement FEvaluationScopeInstanceContainer::FMemoryRequirementBuilder::Build()
{
	if (MemoryRequirement.NumberOfElements == 0)
	{
		return FMemoryRequirement();
	}

	// There is no container memory for object wrapper. The instance is directly saved in the StructView.
	//In case all the structs are object wrapper.
	if (FirstStructAlignment <= 0)
	{
		MemoryRequirement.Alignment = alignof(FItem);
		FirstStructAlignment = alignof(FItem);
	}

	int32 ContainerSize = sizeof(FItem) * MemoryRequirement.NumberOfElements;
	ContainerSize += sizeof(Private::DebugTableEndTag); // table for end tag
	ContainerSize = Align(ContainerSize, FirstStructAlignment);
	FMemoryRequirement Result = MemoryRequirement;
	Result.Size += ContainerSize;
	return Result;
}

FEvaluationScopeInstanceContainer::FEvaluationScopeInstanceContainer(TNotNull<void*> InMemory, const FMemoryRequirement& InRequirement)
	: Memory(InMemory)
	, MemoryRequirement(InRequirement)
{
	AddTableDebugTag();
}

FEvaluationScopeInstanceContainer::~FEvaluationScopeInstanceContainer()
{
	Reset();
}

void FEvaluationScopeInstanceContainer::Add(FStateTreeDataHandle DataHandle, FConstStructView DefaultInstance)
{
	check(NumberOfElements < MemoryRequirement.NumberOfElements);

	FItem* ItemContainer = static_cast<FItem*>(Memory);
	const UScriptStruct* Struct = DefaultInstance.GetScriptStruct();

	void* MemoryForNewItem = (NumberOfElements > 0)
		? (uint8*)ItemContainer[NumberOfElements - 1].Instance.GetMemory()
			+ ItemContainer[NumberOfElements - 1].Instance.GetScriptStruct()->GetStructureSize()
			+ sizeof(Private::DebugStructEndTag)
		: (uint8*)Memory + (MemoryRequirement.NumberOfElements * sizeof(FItem))
			+ sizeof(Private::DebugTableEndTag);
	MemoryForNewItem = Align(MemoryForNewItem, Struct->GetMinAlignment());

	// Initialize the instance data
	constexpr int32 ArrayDim = 1;
	Struct->InitializeStruct(MemoryForNewItem);

	if (!Struct->IsChildOf(FStateTreeInstanceObjectWrapper::StaticStruct()))
	{
		Struct->CopyScriptStruct(MemoryForNewItem, DefaultInstance.GetMemory(), ArrayDim);
	}
	else
	{
		// Duplicate the instance object. Copy nullptr otherwise.
		const FStateTreeInstanceObjectWrapper& Wrapper = DefaultInstance.Get<const FStateTreeInstanceObjectWrapper>();
		FStateTreeInstanceObjectWrapper NewWrapperInstance;
		if (Wrapper.InstanceObject)
		{
			NewWrapperInstance.InstanceObject = ::DuplicateObject(Wrapper.InstanceObject, GetTransientPackage());
		}
		Struct->CopyScriptStruct(MemoryForNewItem, &NewWrapperInstance, ArrayDim);
	}

	// Initialize the item in the table
	new (ItemContainer + NumberOfElements)FItem();
	ItemContainer[NumberOfElements].Instance = FStructView(Struct, (uint8*)MemoryForNewItem);
	ItemContainer[NumberOfElements].DataHandle = DataHandle;

	AddStructDebugTag(NumberOfElements);

	if ((Struct->StructFlags & (STRUCT_IsPlainOldData | STRUCT_NoDestructor)) == 0)
	{
		bStructsHaveDestructor = true;
	}
	++NumberOfElements;
}

void FEvaluationScopeInstanceContainer::Reset()
{
	TestDebugTags();

	if (bStructsHaveDestructor)
	{
		FItem* ItemContainer = static_cast<FItem*>(Memory);
		for (; NumberOfElements > 0; --NumberOfElements)
		{
			const UStruct* InstanceStruct = ItemContainer[NumberOfElements - 1].Instance.GetScriptStruct();
			if (InstanceStruct && Cast<const UClass>(InstanceStruct) == nullptr)
			{
				FStructView& Instance = ItemContainer[NumberOfElements - 1].Instance;
				Instance.GetScriptStruct()->DestroyStruct(Instance.GetMemory());
			}
			if constexpr (!std::is_trivially_destructible_v<FItem>)
			{
				ItemContainer[NumberOfElements - 1].~FItem();
			}
		}
		check(NumberOfElements == 0);
	}
	else if constexpr (!std::is_trivially_destructible_v<FItem>)
	{
		FItem* ItemContainer = static_cast<FItem*>(Memory);
		for (; NumberOfElements > 0; --NumberOfElements)
		{
			ItemContainer[NumberOfElements - 1].~FItem();
		}
		check(NumberOfElements == 0);
	}
	NumberOfElements = 0;
}

FEvaluationScopeInstanceContainer::FItem* FEvaluationScopeInstanceContainer::Get(const FStateTreeDataHandle& DataHandle) const
{
	FItem* ItemContainer = static_cast<FItem*>(Memory);
	for (int32 Index = 0; Index < NumberOfElements; ++Index)
	{
		if (ItemContainer[Index].DataHandle == DataHandle)
		{
			return &ItemContainer[Index];
		}
	}
	return nullptr;
}

void FEvaluationScopeInstanceContainer::AddTableDebugTag()
{
#if WITH_STATETREE_DEBUG
	void* MemoryForDebugTag = (uint8*)Memory + (MemoryRequirement.NumberOfElements * sizeof(FItem));
	*reinterpret_cast<uint32*>(MemoryForDebugTag) = Private::DebugTableEndTag;
#endif
}

void FEvaluationScopeInstanceContainer::AddStructDebugTag(int32 Index)
{
#if WITH_STATETREE_DEBUG
	FItem* ItemContainer = static_cast<FItem*>(Memory);
	void* MemoryForDebugTag = (uint8*)ItemContainer[Index].Instance.GetMemory()
		+ ItemContainer[Index].Instance.GetScriptStruct()->GetStructureSize();
	*reinterpret_cast<uint32*>(MemoryForDebugTag) = Private::DebugStructEndTag;
#endif
}

void FEvaluationScopeInstanceContainer::TestDebugTags() const
{
#if WITH_STATETREE_DEBUG && DO_ENSURE
	CA_SUPPRESS(6269); // warning C6269: Possibly incorrect order of operations.
	if (Memory)
	{
		const void* MemoryForTableDebugTag = (const uint8*)Memory + (sizeof(FItem) * MemoryRequirement.NumberOfElements);
		const int32 TableEndTag = *reinterpret_cast<const uint32*>(MemoryForTableDebugTag);
		ensure(TableEndTag == Private::DebugTableEndTag);

		FItem* ItemContainer = static_cast<FItem*>(Memory);
		for (int32 Index = 0; Index < NumberOfElements; ++Index)
		{
			// Object wrapper do not have EndTag, they use the FItem directly.
			const UStruct* InstanceStruct = ItemContainer[Index].Instance.GetScriptStruct();
			if (InstanceStruct && Cast<const UClass>(InstanceStruct) == nullptr)
			{
				const void* MemoryForStructDebugTag = (const uint8*)ItemContainer[Index].Instance.GetMemory()
					+ InstanceStruct->GetStructureSize();
				const int32 StructEndTag = *reinterpret_cast<const uint32*>(MemoryForStructDebugTag);
				ensure(StructEndTag == Private::DebugStructEndTag);
			}
		}
	}
#endif
}

} // namespace UE::StateTree::InstanceData
