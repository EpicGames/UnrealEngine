// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeEvaluatorBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeEvaluatorBase)

#if WITH_GAMEPLAY_DEBUGGER
FString FStateTreeEvaluatorBase::GetDebugInfo(const FStateTreeReadOnlyExecutionContext& Context) const
{
	TStringBuilder<256> Builder;
	Builder << TEXT('[');
	Builder << Name;
	Builder << TEXT("]\n");
	return Builder.ToString();
}
#endif // WITH_GAMEPLAY_DEBUGGER

