// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeEvents.h"
#include "StateTreeTypes.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeEvents)

//----------------------------------------------------------------//
// FStateTreeSharedEvent
//----------------------------------------------------------------//

UE::StateTree::Event::FEventsPendingForNextTransitionProcessingScope::FEventsPendingForNextTransitionProcessingScope(TNotNull<FStateTreeEventQueue*> EventQueue): ScopedEventQueue(EventQueue)
{
	bSavedIsPendingForNextTransitionProcessingScope = ScopedEventQueue->bNewlyAddedEventsShouldBePendingForNextTransitionProcessing;
	ScopedEventQueue->bNewlyAddedEventsShouldBePendingForNextTransitionProcessing = true;
}

UE::StateTree::Event::FEventsPendingForNextTransitionProcessingScope::~FEventsPendingForNextTransitionProcessingScope()
{
	ScopedEventQueue->bNewlyAddedEventsShouldBePendingForNextTransitionProcessing = bSavedIsPendingForNextTransitionProcessingScope;
}

void FStateTreeSharedEvent::AddStructReferencedObjects(FReferenceCollector& Collector)
{
	if (IsValid())
	{
		Collector.AddPropertyReferencesWithStructARO(FStateTreeEvent::StaticStruct(), Event.Get());
	}
}


//----------------------------------------------------------------//
// FStateTreeEventQueue
//----------------------------------------------------------------//

bool FStateTreeEventQueue::SendEvent(const UObject* Owner, const FGameplayTag& Tag, const FConstStructView Payload, const FName Origin)
{
	if (!Tag.IsValid() && !Payload.IsValid())
	{
		UE_VLOG_UELOG(Owner, LogStateTree, Error, TEXT("%s: An event with an invalid tag and payload has been sent to '%s'. This is not allowed."), ANSI_TO_TCHAR(__FUNCTION__), *GetNameSafe(Owner));
		return false;
	}

	if (SharedEvents.Num() >= MaxActiveEvents)
	{
		UE_VLOG_UELOG(Owner, LogStateTree, Error, TEXT("%s: Too many events send on '%s'. Dropping event %s"), ANSI_TO_TCHAR(__FUNCTION__), *GetNameSafe(Owner), *Tag.ToString());
		return false;
	}

	FStateTreeSharedEvent& SharedEvent = SharedEvents.Emplace_GetRef(Tag, Payload, Origin);
	if (bNewlyAddedEventsShouldBePendingForNextTransitionProcessing)
	{
		SharedEvent->bIsPendingForNextTransitionProcessing = true;
	}

	return true;
}

bool FStateTreeEventQueue::ConsumeEvent(const FStateTreeSharedEvent& Event)
{
	const int32 Count = SharedEvents.RemoveAllSwap([&EventToRemove = Event](const FStateTreeSharedEvent& Event)
	{
		return Event == EventToRemove;
	});
	return Count > 0;
}

void FStateTreeEventQueue::ClearEventsForCurrentTransitionProcessingPhase()
{
	SharedEvents.RemoveAllSwap([](FStateTreeSharedEvent& Event)
	{
		if (!Event->bIsPendingForNextTransitionProcessing)
		{
			return true;
		}

		Event->bIsPendingForNextTransitionProcessing = false;
		return false;
	});
}
