// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeExecutionContext.h"
#include "StateTreeAsyncExecutionContext.h"
#include "StateTreeTaskBase.h"
#include "StateTreeEvaluatorBase.h"
#include "StateTreeConditionBase.h"
#include "StateTreeConsiderationBase.h"
#include "StateTreeDelegate.h"
#include "StateTreeInstanceDataHelpers.h"
#include "StateTreePropertyFunctionBase.h"
#include "StateTreeReference.h"
#include "AutoRTFM.h"
#include "ObjectTrace.h"
#include "Containers/StaticArray.h"
#include "CrashReporter/StateTreeCrashReporterHandler.h"
#include "Debugger/StateTreeDebug.h"
#include "Debugger/StateTreeTrace.h"
#include "Debugger/StateTreeTraceTypes.h"
#include "Misc/ScopeExit.h"
#include "VisualLogger/VisualLogger.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Logging/LogScopedVerbosityOverride.h"

#define STATETREE_LOG(Verbosity, Format, ...) UE_VLOG_UELOG(GetOwner(), LogStateTree, Verbosity, TEXT("%s: ") Format, *GetInstanceDescriptionInternal(), ##__VA_ARGS__)
#define STATETREE_CLOG(Condition, Verbosity, Format, ...) UE_CVLOG_UELOG((Condition), GetOwner(), LogStateTree, Verbosity, TEXT("%s: ") Format, *GetInstanceDescriptionInternal(), ##__VA_ARGS__)

CSV_DEFINE_CATEGORY(StateTree, false);

namespace UE::StateTree::Debug
{
	// Debug printing indent for hierarchical data.
	constexpr int32 IndentSize = 2;
};

namespace UE::StateTree::ExecutionContext::Private
{
	bool bCopyBoundPropertiesOnNonTickedTask = false;
	FAutoConsoleVariableRef CVarCopyBoundPropertiesOnNonTickedTask(
		TEXT("StateTree.CopyBoundPropertiesOnNonTickedTask"),
		bCopyBoundPropertiesOnNonTickedTask,
		TEXT("When ticking the tasks, copy the bindings when the tasks are not ticked because it disabled ticking or completed. The bindings are not updated on failure.")
	);

	bool bTickGlobalNodesFollowingTreeHierarchy = true;
	FAutoConsoleVariableRef CVarTickGlobalNodesFollowingTreeHierarchy(
		TEXT("StateTree.TickGlobalNodesFollowingTreeHierarchy"),
		bTickGlobalNodesFollowingTreeHierarchy,
		TEXT("If true, then the global evaluators and global tasks are ticked following the asset hierarchy.\n")
		TEXT("The order is (1)root evaluators, (2)root global tasks, (3)state tasks, (4)linked asset evaluators, (5)linked asset global tasks, (6)linked asset state tasks.\n")
		TEXT("If false (5.5 behavior), then all the global nodes, from all linked assets, are evaluated, then the state tasks are ticked.\n")
		TEXT("You should upgrade your asset. This option is to help migrate to the new behavior.")
	);

	bool bGlobalTasksCompleteOwningFrame = true;
	FAutoConsoleVariableRef CVarGlobalTasksCompleteFrame(
		TEXT("StateTree.GlobalTasksCompleteOwningFrame"),
		bGlobalTasksCompleteOwningFrame,
		TEXT("If true, the global tasks complete the tree they are part of. The root or the linked state. 5.6 behavior.\n")
		TEXT("If false, the global tasks complete the root tree execution (even for linked assets). 5.5 behavior.")
		TEXT("You should upgrade your asset. This option is to help migrate to the new behavior.")
	);

	bool bSetDeprecatedTransitionResultProperties = false;
	FAutoConsoleVariableRef CVarSetDeprecatedTransitionResultProperties(
		TEXT("StateTree.SetDeprecatedTransitionResultProperties"),
		bSetDeprecatedTransitionResultProperties,
		TEXT("If true, set the FStateTreeTransitionResult deprecated properties when a transition occurs.\n")
		TEXT("The information is not relevant and not valid but can be needed for backward compatibility.")
	);

	bool bTargetStateRequiresTheSameEventForStateSelectionAsTheRequestedTransition = false;
	FAutoConsoleVariableRef CVarTargetStateRequiresTheSameEventForStateSelectionAsTheRequestedTransition(
		TEXT("StateTree.TargetStateRequiresTheSameEventForStateSelectionAsTheRequestedTransition"),
		bTargetStateRequiresTheSameEventForStateSelectionAsTheRequestedTransition,
		TEXT("If true, when a transition is triggered by an event that occurs and the target state requires an event, \n")
		TEXT("then the state required event must match the transition required event.")
	);

	bool bCaptureStateEventPayloadForSustainedState = true;
	FAutoConsoleVariableRef CVarCaptureStateEventPayloadForSustainedState(
		TEXT("StateTree.CaptureStateEventPayloadForSustainedState"),
		bCaptureStateEventPayloadForSustainedState,
		TEXT(" If true, the event payload is captured even when the state sustained.\n")
		TEXT(" If false, the event payload is only captured when the state is changed.")
	);

	const FLazyName Name_Tick = "Tick";
	const FLazyName Name_Start = "Start";
	const FLazyName Name_Stop = "Stop";

	constexpr uint32 NumEStateTreeRunStatus()
	{
#ifdef UE_STATETREE_ESTATETREERUNSTATUS_PRIVATE
#error UE_STATETREE_ESTATETREERUNSTATUS_PRIVATE_ALREADY_DEFINED
#endif

#define UE_STATETREE_ESTATETREERUNSTATUS_PRIVATE(EnumValue) ++Number;

		int32 Number = 0;
		FOREACH_ENUM_ESTATETREERUNSTATUS(UE_STATETREE_ESTATETREERUNSTATUS_PRIVATE)

#undef UE_STATETREE_ESTATETREERUNSTATUS_PRIVATE
			return Number;
	}

	constexpr uint32 NumEStateTreeFinishTaskType()
	{
#ifdef UE_STATETREE_ESTATETREEFINISHTASKTYPE_PRIVATE
#error UE_STATETREE_ESTATETREEFINISHTASKTYPE_PRIVATE_ALREADY_DEFINED
#endif

#define UE_STATETREE_ESTATETREEFINISHTASKTYPE_PRIVATE(EnumValue) ++Number;

		int32 Number = 0;
		FOREACH_ENUM_ESTATETREEFINISHTASKTYPE(UE_STATETREE_ESTATETREEFINISHTASKTYPE_PRIVATE)

#undef UE_STATETREE_ESTATETREEFINISHTASKTYPE_PRIVATE
			return Number;
	}

	const FCompactStateTreeFrame* FindStateTreeFrame(const FExecutionFrameHandle& FrameHandle)
	{
		check(FrameHandle.IsValid());
		const FCompactStateTreeFrame* FrameInfo = FrameHandle.GetStateTree()->GetFrameFromHandle(FrameHandle.GetRootState());
		ensureAlwaysMsgf(FrameInfo, TEXT("The compiled data is invalid. It should contains the information for the new root frame."));
		return FrameInfo;
	}

	// From metric, paths are usually of length < 8. We put a large enough number to not have to worry about complex assets.
	static constexpr int32 ExpectedActiveStatePathLength = 24;
	using FActiveStateInlineArray = TArray<FActiveState, TInlineAllocator<ExpectedActiveStatePathLength, FNonconcurrentLinearArrayAllocator>>;
	void GetActiveStatePath(TArrayView<const FStateTreeExecutionFrame> Frames, FActiveStateInlineArray& OutResult)
	{
		if (Frames.Num() == 0 || Frames[0].StateTree == nullptr)
		{
			return;
		}

		for (const FStateTreeExecutionFrame& Frame : Frames)
		{
			for (int32 StateIndex = 0; StateIndex < Frame.ActiveStates.Num(); ++StateIndex)
			{
				OutResult.Emplace(Frame.FrameID, Frame.ActiveStates.StateIDs[StateIndex], Frame.ActiveStates.States[StateIndex]);
			}
		}
	}

	using FStateHandleContextInlineArray = TArray<FStateHandleContext, TInlineAllocator<FStateTreeActiveStates::MaxStates>>;
	bool GetStatesListToState(FStateHandleContext State, FStateHandleContextInlineArray& OutStates)
	{
		// Walk towards the root from the state.
		FStateHandleContext CurrentState = State;
		while (CurrentState.StateTree && CurrentState.StateHandle.IsValid())
		{
			if (OutStates.Num() == FStateTreeActiveStates::MaxStates)
			{
				return false;
			}
			// Store the states that are in between the 'NextState' and the common ancestor.
			OutStates.Add(CurrentState);
			CurrentState = FStateHandleContext(CurrentState.StateTree, CurrentState.StateTree->GetStates()[CurrentState.StateHandle.Index].Parent);
		}
		Algo::Reverse(OutStates);
		return true;
	}

	static constexpr int32 ExpectedAmountTransitionEvent = 8;
	using FSharedEventInlineArray = TArray<FStateTreeSharedEvent, TInlineAllocator<ExpectedAmountTransitionEvent, FNonconcurrentLinearArrayAllocator>>;
	void GetTriggerTransitionEvent(const FCompactStateTransition& Transition, TConstArrayView<FStateTreeDelegateDispatcher> BroadcastedDelegates, const FStateTreeSharedEvent& TransitionEvent, const TArrayView<const FStateTreeSharedEvent> EventsToProcess, FSharedEventInlineArray& OutTransitionEvents)
	{
		if (Transition.Trigger == EStateTreeTransitionTrigger::OnEvent)
		{
			check(Transition.RequiredEvent.IsValid());

			if (TransitionEvent.IsValid())
			{
				OutTransitionEvents.Add(TransitionEvent);
			}
			else
			{
				for (const FStateTreeSharedEvent& Event : EventsToProcess)
				{
					check(Event.IsValid());
					if (Transition.RequiredEvent.DoesEventMatchDesc(*Event))
					{
						OutTransitionEvents.Emplace(Event);
					}
				}
			}
		}
		else if (EnumHasAnyFlags(Transition.Trigger, EStateTreeTransitionTrigger::OnTick))
		{
			// Dummy event to make sure we iterate the transition loop once.
			OutTransitionEvents.Emplace(FStateTreeSharedEvent());
		}
		else if (EnumHasAnyFlags(Transition.Trigger, EStateTreeTransitionTrigger::OnDelegate))
		{
			if (BroadcastedDelegates.Contains(Transition.RequiredDelegateDispatcher))
			{
				// Dummy event to make sure we iterate the transition loop once.
				OutTransitionEvents.Emplace(FStateTreeSharedEvent());
			}
		}
		else
		{
			ensureMsgf(false, TEXT("The trigger type is not supported."));
		}
	}

	FString GetStatePathAsString(TNotNull<const UStateTree*> StateTree, const TArrayView<const UE::StateTree::FActiveState>& Path)
	{
		TValueOrError<FString, void> Result = UE::StateTree::FActiveStatePath::Describe(StateTree, Path);
		if (Result.HasValue())
		{
			return Result.StealValue();
		}
		return FString();
	}

	void CleanFrame(FStateTreeExecutionState& Exec, FActiveFrameID FrameID)
	{
		Exec.DelegateActiveListeners.RemoveAll(FrameID);
	}

	void CleanState(FStateTreeExecutionState& Exec, FActiveStateID StateID)
	{
		Exec.DelegateActiveListeners.RemoveAll(StateID);
		Exec.DelayedTransitions.RemoveAll([StateID](const FStateTreeTransitionDelayedState& DelayedState)
			{
				return DelayedState.StateID == StateID;
			});
	}

	bool UseEvaluationScopeIntanceData(const FStateTreeDataHandle& DataHandle)
	{
		return DataHandle.GetSource() == EStateTreeDataSourceType::EvaluationScopeInstanceData
			|| DataHandle.GetSource() == EStateTreeDataSourceType::EvaluationScopeInstanceDataObject;
	}

	void InitEvaluationScopeInstanceData(UE::StateTree::InstanceData::FEvaluationScopeInstanceContainer& Container, TNotNull<const UStateTree*> StateTree, const int32 NodesBegin, const int32 NodesEnd)
	{
		for (int32 NodeIndex = NodesBegin; NodeIndex != NodesEnd; ++NodeIndex)
		{
			const FStateTreeNodeBase& Node = StateTree->GetNodes()[NodeIndex].Get<const FStateTreeNodeBase>();
			if (UseEvaluationScopeIntanceData(Node.InstanceDataHandle))
			{
				Container.Add(Node.InstanceDataHandle, StateTree->GetDefaultEvaluationScopeInstanceData().GetStruct(Node.InstanceTemplateIndex.Get()));
			}
		}
	}

	struct FSelectStartStateInfo
	{
		FStateTreeStateHandle StartStateToSelectHandle;
		FStateTreeStateHandle FrameRootStateHandle;
	};
	FSelectStartStateInfo GetSelectStartStateInfo(TNotNull<const UStateTree*> InRootTree, const FStateTreeExecutionContext::FStartParameters& InStartParameters)
	{
		FSelectStartStateInfo SelectStartState = { FStateTreeStateHandle::Invalid, FStateTreeStateHandle::Invalid };

		if (const FStateTreeExecutionContext::FStartParameters::FStateToSelectOverrideArgs* SelectStateOverrideArgs = InStartParameters.SelectStateOverrideArgs.GetPtrOrNull())
		{
			if (SelectStateOverrideArgs->StateTag.IsValid())
			{
				FStateTreeStateHandle OverrideStateToSelectHandle = InRootTree->GetStateHandleFromGameplayTag(SelectStateOverrideArgs->StateTag, SelectStateOverrideArgs->TagQueryMethod);
				if (OverrideStateToSelectHandle.IsValid())
				{
					check(!OverrideStateToSelectHandle.IsCompletionState());

					const FCompactStateTreeState* OverrideStateToSelect = InRootTree->GetStateFromHandle(OverrideStateToSelectHandle);
					check(OverrideStateToSelect);

					SelectStartState.StartStateToSelectHandle = OverrideStateToSelectHandle;
					SelectStartState.FrameRootStateHandle = OverrideStateToSelectHandle;

					// A tree could have multiple roots and the selected state could be within a subtree(the subtree will be treated as standalone)
					FStateTreeStateHandle OverrideParentStateHandle = OverrideStateToSelect->Parent;
					while (const FCompactStateTreeState* OverrideParentState = InRootTree->GetStateFromHandle(OverrideParentStateHandle))
					{
						SelectStartState.FrameRootStateHandle = OverrideParentStateHandle;
						OverrideParentStateHandle = OverrideParentState->Parent;
					}
				}
			}
		}
		else
		{
			// No override info is provided, tree will start from root state by default
			SelectStartState = {FStateTreeStateHandle::Root, FStateTreeStateHandle::Root};
		}

		return SelectStartState;
	}
} // namespace UE::StateTree::ExecutionContext::Private

namespace UE::StateTree::ExecutionContext
{
	bool MarkDelegateAsBroadcasted(FStateTreeDelegateDispatcher Dispatcher, const FStateTreeExecutionFrame& CurrentFrame, FStateTreeInstanceStorage& Storage)
	{
		const UStateTree* StateTree = CurrentFrame.StateTree;
		check(StateTree);

		for (FStateTreeStateHandle ActiveState : CurrentFrame.ActiveStates)
		{
			const FCompactStateTreeState* State = StateTree->GetStateFromHandle(ActiveState);
			check(State);

			if (!State->bHasDelegateTriggerTransitions)
			{
				continue;
			}

			const int32 TransitionEnd = State->TransitionsBegin + State->TransitionsNum;
			for (int32 TransitionIndex = State->TransitionsBegin; TransitionIndex < TransitionEnd; ++TransitionIndex)
			{
				const FCompactStateTransition* Transition = StateTree->GetTransitionFromIndex(FStateTreeIndex16(TransitionIndex));
				check(Transition);
				if (Transition->RequiredDelegateDispatcher == Dispatcher)
				{
					ensureMsgf(EnumHasAnyFlags(Transition->Trigger, EStateTreeTransitionTrigger::OnDelegate), TEXT("The transition should have both (a valid dispatcher and the OnDelegate flag) or none."));
					Storage.MarkDelegateAsBroadcasted(Dispatcher);
					return true;
				}
			}
		}

		return false;
	}

	/** @return in order {Failed, Succeeded, Stopped, Running, Unset} */
	EStateTreeRunStatus GetPriorityRunStatus(EStateTreeRunStatus A, EStateTreeRunStatus B)
	{
		static_assert((int32)EStateTreeRunStatus::Running == 0);
		static_assert((int32)EStateTreeRunStatus::Stopped == 1);
		static_assert((int32)EStateTreeRunStatus::Succeeded == 2);
		static_assert((int32)EStateTreeRunStatus::Failed == 3);
		static_assert((int32)EStateTreeRunStatus::Unset == 4);
		static_assert(Private::NumEStateTreeRunStatus() == 5, "The number of entries in EStateTreeRunStatus changed. Update GetPriorityRunStatus.");

		static constexpr int32 PriorityMatrix[] = { 1, 2, 3, 4, 0 };
		return PriorityMatrix[(uint8)A] > PriorityMatrix[(uint8)B] ? A : B;
	}

	UE::StateTree::ETaskCompletionStatus CastToTaskStatus(EStateTreeFinishTaskType FinishTask)
	{
		static_assert(Private::NumEStateTreeFinishTaskType() == 2, "The number of entries in EStateTreeFinishTaskType changed. Update CastToTaskStatus.");

		return FinishTask == EStateTreeFinishTaskType::Succeeded ? UE::StateTree::ETaskCompletionStatus::Succeeded : UE::StateTree::ETaskCompletionStatus::Failed;
	}

	EStateTreeRunStatus CastToRunStatus(EStateTreeFinishTaskType FinishTask)
	{
		static_assert(Private::NumEStateTreeFinishTaskType() == 2, "The number of entries in EStateTreeFinishTaskType changed. Update CastToRunStatus.");

		return FinishTask == EStateTreeFinishTaskType::Succeeded ? EStateTreeRunStatus::Succeeded : EStateTreeRunStatus::Failed;
	}

	UE::StateTree::ETaskCompletionStatus CastToTaskStatus(EStateTreeRunStatus InStatus)
	{
		static_assert((int32)EStateTreeRunStatus::Running == (int32)UE::StateTree::ETaskCompletionStatus::Running);
		static_assert((int32)EStateTreeRunStatus::Stopped == (int32)UE::StateTree::ETaskCompletionStatus::Stopped);
		static_assert((int32)EStateTreeRunStatus::Succeeded == (int32)UE::StateTree::ETaskCompletionStatus::Succeeded);
		static_assert((int32)EStateTreeRunStatus::Failed == (int32)UE::StateTree::ETaskCompletionStatus::Failed);
		static_assert(Private::NumEStateTreeRunStatus() == 5, "The number of entries in EStateTreeRunStatus changed. Update CastToTaskStatus.");

		return InStatus != EStateTreeRunStatus::Unset ? (UE::StateTree::ETaskCompletionStatus)InStatus : UE::StateTree::ETaskCompletionStatus::Running;
	}

	EStateTreeRunStatus CastToRunStatus(UE::StateTree::ETaskCompletionStatus InStatus)
	{
		static_assert((int32)EStateTreeRunStatus::Running == (int32)UE::StateTree::ETaskCompletionStatus::Running);
		static_assert((int32)EStateTreeRunStatus::Stopped == (int32)UE::StateTree::ETaskCompletionStatus::Stopped);
		static_assert((int32)EStateTreeRunStatus::Succeeded == (int32)UE::StateTree::ETaskCompletionStatus::Succeeded);
		static_assert((int32)EStateTreeRunStatus::Failed == (int32)UE::StateTree::ETaskCompletionStatus::Failed);
		static_assert(UE::StateTree::NumberOfTaskStatus == 4, "The number of entries in EStateTreeRunStatus changed. Update CastToRunStatus.");

		return (EStateTreeRunStatus)InStatus;
	}
} // namespace UE::StateTree::ExecutionContext

/**
 * FStateTreeReadOnlyExecutionContext implementation
 */
FStateTreeReadOnlyExecutionContext::FStateTreeReadOnlyExecutionContext(TNotNull<UObject*> InOwner, TNotNull<const UStateTree*> InStateTree, FStateTreeInstanceData& InInstanceData)
	: FStateTreeReadOnlyExecutionContext(InOwner, InStateTree, InInstanceData.GetMutableStorage())
{
}

FStateTreeReadOnlyExecutionContext::FStateTreeReadOnlyExecutionContext(TNotNull<UObject*> InOwner, TNotNull<const UStateTree*> InStateTree, FStateTreeInstanceStorage& InStorage)
	: Owner(*InOwner)
	, RootStateTree(*InStateTree)
	, Storage(InStorage)
{
	Storage.AcquireReadAccess();

	if (IsValid())
	{
		constexpr bool bWriteAccessAcquired = false;
		Storage.GetRuntimeValidation().SetContext(&Owner, &RootStateTree, bWriteAccessAcquired);
	}
}

FStateTreeReadOnlyExecutionContext::~FStateTreeReadOnlyExecutionContext()
{
	Storage.ReleaseReadAccess();
}

FStateTreeScheduledTick FStateTreeReadOnlyExecutionContext::GetNextScheduledTick() const
{
	if (!IsValid())
	{
		STATETREE_LOG(Warning, TEXT("%hs: StateTree context is not initialized properly ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		return FStateTreeScheduledTick::MakeSleep();
	}

	const FStateTreeExecutionState& Exec = Storage.GetExecutionState();
	if (Exec.TreeRunStatus != EStateTreeRunStatus::Running)
	{
		return FStateTreeScheduledTick::MakeSleep();
	}

	// USchema::IsScheduleTickAllowed.
	//Used the state tree cached value to prevent runtime changes that could affect the behavior.
	for (int32 FrameIndex = 0; FrameIndex < Exec.ActiveFrames.Num(); ++FrameIndex)
	{
		const FStateTreeExecutionFrame& CurrentFrame = Exec.ActiveFrames[FrameIndex];
		if (!CurrentFrame.StateTree->IsScheduledTickAllowed())
		{
			return FStateTreeScheduledTick::MakeEveryFrames(UE::StateTree::ETickReason::Forced);
		}
	}

	const FStateTreeEventQueue& EventQueue = Storage.GetEventQueue();
	const bool bHasEvents = EventQueue.HasEvents();
	const bool bHasBroadcastedDelegates = Storage.HasBroadcastedDelegates();

	struct FCustomTickRate
	{
		FCustomTickRate(float InDeltaTime, UE::StateTree::ETickReason InReason)
			: DeltaTime(InDeltaTime)
			, Reason(InReason)
		{}

		bool operator<(const FCustomTickRate& Other) const
		{
			return DeltaTime < Other.DeltaTime;
		}

		float DeltaTime = 0.0f;
		UE::StateTree::ETickReason Reason = UE::StateTree::ETickReason::None;
	};


	TOptional<FCustomTickRate> CustomTickRate;

	// We wish to return in order: EveryFrames, then NextFrame, then CustomTickRate, then Sleep.
	// Do we have a state that requires a tick or is waiting for an event.
	{
		bool bHasTaskWithEveryFramesTick = false;
		UE::StateTree::ETickReason TaskTickingReason = UE::StateTree::ETickReason::None;
		for (int32 FrameIndex = 0; FrameIndex < Exec.ActiveFrames.Num(); ++FrameIndex)
		{
			const FStateTreeExecutionFrame& CurrentFrame = Exec.ActiveFrames[FrameIndex];
			const UStateTree* CurrentStateTree = CurrentFrame.StateTree;

			// Test global tasks.
			if (CurrentStateTree->DoesRequestTickGlobalTasks(bHasEvents))
			{
				bHasTaskWithEveryFramesTick = true;
			}

			// Test active states tasks.
			for (int32 StateIndex = 0; StateIndex < CurrentFrame.ActiveStates.Num(); ++StateIndex)
			{
				const FStateTreeStateHandle CurrentHandle = CurrentFrame.ActiveStates[StateIndex];
				const FCompactStateTreeState& State = CurrentStateTree->States[CurrentHandle.Index];
				checkf(State.bEnabled, TEXT("Only enabled states are in ActiveStates."));
				if (State.bHasCustomTickRate)
				{
					if (!CustomTickRate.IsSet() || CustomTickRate.GetValue().DeltaTime > State.CustomTickRate)
					{
						CustomTickRate = {State.CustomTickRate, UE::StateTree::ETickReason::StateCustomTickRate};
					}
				}
				else if (!CustomTickRate.IsSet())
				{
					if (State.DoesRequestTickTasks(bHasEvents))
					{
						bHasTaskWithEveryFramesTick = true;
						TaskTickingReason = UE::StateTree::ETickReason::TaskTicking;
					}
					else if (State.ShouldTickTransitions(bHasEvents, bHasBroadcastedDelegates))
					{
						// todo: ShouldTickTransitions has onevent or ontick. both can be already triggered and we are waiting for the delay
						bHasTaskWithEveryFramesTick = true;
						TaskTickingReason = UE::StateTree::ETickReason::TransitionTicking;
					}
				}
			}
		}

		if (!CustomTickRate.IsSet() && bHasTaskWithEveryFramesTick)
		{
			return FStateTreeScheduledTick::MakeEveryFrames(TaskTickingReason);
		}

		// If one state has a custom tick rate, then it overrides the tick rate for all states.
		//Only return the CustomTickRate if it's > than NextFrame, the custom tick rate will be processed at the end.
		if (const FCustomTickRate* CustomTickRateValue = CustomTickRate.GetPtrOrNull())
		{
			if (CustomTickRateValue->DeltaTime <= 0.0f)
			{
				// A state might override the custom tick rate with > 0, then another state overrides it again with 0 to tick back every frame.
				return FStateTreeScheduledTick::MakeEveryFrames(CustomTickRateValue->Reason);
			}
		}
	}

	// Requests
	if (Exec.HasScheduledTickRequests())
	{
		// The ScheduledTickRequests loop value is cached. Returns every frame or next frame. CustomTime needs to wait after the other tests.
		const FStateTreeScheduledTick ScheduledTickRequest = Exec.GetScheduledTickRequest();
		if (ScheduledTickRequest.ShouldTickEveryFrames() || ScheduledTickRequest.ShouldTickOnceNextFrame())
		{
			return ScheduledTickRequest;
		}
		const FCustomTickRate CachedTickRate(ScheduledTickRequest.GetTickRate(), ScheduledTickRequest.GetReason());

		CustomTickRate = CustomTickRate.IsSet() ? FMath::Min(CustomTickRate.GetValue(), CachedTickRate) : CachedTickRate;
	}

	// Transitions
	if (Storage.GetTransitionRequests().Num() > 0)
	{
		return FStateTreeScheduledTick::MakeNextFrame(UE::StateTree::ETickReason::TransitionRequest);
	}

	// Events are cleared every tick.
	if (bHasEvents && Storage.IsOwningEventQueue())
	{
		return FStateTreeScheduledTick::MakeNextFrame(UE::StateTree::ETickReason::Event);
	}

	// Completed task. For EnterState or for user that only called TickTasks and not TickTransitions.
	if (Exec.bHasPendingCompletedState)
	{
		return FStateTreeScheduledTick::MakeNextFrame(UE::StateTree::ETickReason::CompletedState);
	}

	// Min of all delayed transitions.
	if (Exec.DelayedTransitions.Num() > 0)
	{
		for (const FStateTreeTransitionDelayedState& Transition : Exec.DelayedTransitions)
		{
			const FCustomTickRate TransitionTickRate(Transition.TimeLeft, UE::StateTree::ETickReason::DelayedTransition);
			CustomTickRate = CustomTickRate.IsSet() ? FMath::Min(CustomTickRate.GetValue(), TransitionTickRate) : TransitionTickRate;
		}
	}

	// Custom tick rate for tasks and transitions.
	if (const FCustomTickRate* CustomTickRateValue = CustomTickRate.GetPtrOrNull())
	{
		return FStateTreeScheduledTick::MakeCustomTickRate(CustomTickRateValue->DeltaTime, CustomTickRateValue->Reason);
	}

	return FStateTreeScheduledTick::MakeSleep();
}

EStateTreeRunStatus FStateTreeReadOnlyExecutionContext::GetStateTreeRunStatus() const
{
	if (!IsValid())
	{
		STATETREE_LOG(Warning, TEXT("%hs: StateTree context is not initialized properly ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		return EStateTreeRunStatus::Failed;
	}

	return Storage.GetExecutionState().TreeRunStatus;
}

EStateTreeRunStatus FStateTreeReadOnlyExecutionContext::GetLastTickStatus() const
{
	if (!IsValid())
	{
		STATETREE_LOG(Warning, TEXT("%hs: StateTree context is not initialized properly ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		return EStateTreeRunStatus::Failed;
	}

	const FStateTreeExecutionState& Exec = Storage.GetExecutionState();
	return Exec.LastTickStatus;
}

TConstArrayView<FStateTreeExecutionFrame> FStateTreeReadOnlyExecutionContext::GetActiveFrames() const
{
	if (!IsValid())
	{
		STATETREE_LOG(Warning, TEXT("%hs: StateTree context is not initialized properly ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		return TConstArrayView<FStateTreeExecutionFrame>();
	}

	const FStateTreeExecutionState& Exec = Storage.GetExecutionState();
	return Exec.ActiveFrames;
}

FString FStateTreeReadOnlyExecutionContext::GetActiveStateName() const
{
	if (!IsValid())
	{
		STATETREE_LOG(Warning, TEXT("%hs: StateTree context is not initialized properly ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		return FString();
	}

	const FStateTreeExecutionState& Exec = Storage.GetExecutionState();

	TStringBuilder<1024> FullStateName;

	const UStateTree* LastStateTree = &RootStateTree;
	int32 Indent = 0;

	for (int32 FrameIndex = 0; FrameIndex < Exec.ActiveFrames.Num(); FrameIndex++)
	{
		const FStateTreeExecutionFrame& CurrentFrame = Exec.ActiveFrames[FrameIndex];
		const UStateTree* CurrentStateTree = CurrentFrame.StateTree;

		// Append linked state marker at the end of the previous line.
		if (Indent > 0)
		{
			FullStateName << TEXT(" >");
		}
		// If tree has changed, append that too.
		if (CurrentFrame.StateTree != LastStateTree)
		{
			FullStateName << TEXT(" [");
			FullStateName << CurrentFrame.StateTree.GetFName();
			FullStateName << TEXT(']');

			LastStateTree = CurrentFrame.StateTree;
		}

		for (int32 Index = 0; Index < CurrentFrame.ActiveStates.Num(); Index++)
		{
			const FStateTreeStateHandle Handle = CurrentFrame.ActiveStates[Index];
			if (Handle.IsValid())
			{
				const FCompactStateTreeState& State = CurrentStateTree->States[Handle.Index];
				if (Indent > 0)
				{
					FullStateName += TEXT("\n");
				}
				FullStateName.Appendf(TEXT("%*s-"), Indent * 3, TEXT("")); // Indent
				FullStateName << State.Name;
				Indent++;
			}
		}
	}

	switch (Exec.TreeRunStatus)
	{
	case EStateTreeRunStatus::Failed:
		FullStateName << TEXT(" FAILED\n");
		break;
	case EStateTreeRunStatus::Succeeded:
		FullStateName << TEXT(" SUCCEEDED\n");
		break;
	case EStateTreeRunStatus::Running:
		// Empty
		break;
	default:
		FullStateName << TEXT("--\n");
	}

	return FullStateName.ToString();
}

TArray<FName> FStateTreeReadOnlyExecutionContext::GetActiveStateNames() const
{
	if (!IsValid())
	{
		STATETREE_LOG(Warning, TEXT("%hs: StateTree context is not initialized properly ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		return TArray<FName>();
	}

	TArray<FName> Result;
	const FStateTreeExecutionState& Exec = Storage.GetExecutionState();

	// Active States
	for (const FStateTreeExecutionFrame& CurrentFrame : Exec.ActiveFrames)
	{
		const UStateTree* CurrentStateTree = CurrentFrame.StateTree;
		for (int32 Index = 0; Index < CurrentFrame.ActiveStates.Num(); Index++)
		{
			const FStateTreeStateHandle Handle = CurrentFrame.ActiveStates[Index];
			if (Handle.IsValid())
			{
				const FCompactStateTreeState& State = CurrentStateTree->States[Handle.Index];
				Result.Add(State.Name);
			}
		}
	}

	return Result;
}

#if WITH_GAMEPLAY_DEBUGGER
FString FStateTreeReadOnlyExecutionContext::GetDebugInfoString() const
{
	TStringBuilder<2048> DebugString;
	DebugString << TEXT("StateTree (asset: '");
	RootStateTree.GetFullName(DebugString);
	DebugString << TEXT("')");

	if (IsValid())
	{
		const FStateTreeExecutionState& Exec = Storage.GetExecutionState();

		DebugString << TEXT("Status: ");
		DebugString << UEnum::GetDisplayValueAsText(Exec.TreeRunStatus).ToString();
		DebugString << TEXT("\n");

		// Active States
		DebugString << TEXT("Current State:\n");
		for (const FStateTreeExecutionFrame& CurrentFrame : Exec.ActiveFrames)
		{
			const UStateTree* CurrentStateTree = CurrentFrame.StateTree;

			if (CurrentFrame.bIsGlobalFrame)
			{
				DebugString.Appendf(TEXT("\nEvaluators\n  [ %-30s | %8s | %8s | %15s ]\n"),
					TEXT("Name"), TEXT("Bindings"), TEXT("Output Bindings"), TEXT("Data Handle"));
				for (int32 EvalIndex = CurrentStateTree->EvaluatorsBegin; EvalIndex < (CurrentStateTree->EvaluatorsBegin + CurrentStateTree->EvaluatorsNum); EvalIndex++)
				{
					const FStateTreeEvaluatorBase& Eval = CurrentStateTree->Nodes[EvalIndex].Get<const FStateTreeEvaluatorBase>();
					DebugString.Appendf(TEXT("| %-30s | %8d | %8d | %15s |\n"),
						*Eval.Name.ToString(), Eval.BindingsBatch.Get(), Eval.OutputBindingsBatch.Get(), *Eval.InstanceDataHandle.Describe());
				}

				DebugString << TEXT("\nGlobal Tasks\n");
				for (int32 TaskIndex = CurrentStateTree->GlobalTasksBegin; TaskIndex < (CurrentStateTree->GlobalTasksBegin + CurrentStateTree->GlobalTasksNum); TaskIndex++)
				{
					const FStateTreeTaskBase& Task = CurrentStateTree->Nodes[TaskIndex].Get<const FStateTreeTaskBase>();
					if (Task.bTaskEnabled)
					{
						DebugString << Task.GetDebugInfo(*this);

					}
				}
			}

			for (int32 Index = 0; Index < CurrentFrame.ActiveStates.Num(); Index++)
			{
				FStateTreeStateHandle Handle = CurrentFrame.ActiveStates[Index];
				if (Handle.IsValid())
				{
					const FCompactStateTreeState& State = CurrentStateTree->States[Handle.Index];
					DebugString << TEXT('[');
					DebugString << State.Name;
					DebugString << TEXT("]\n");

					if (State.TasksNum > 0)
					{
						DebugString += TEXT("\nTasks:\n");
						for (int32 TaskIndex = State.TasksBegin; TaskIndex < (State.TasksBegin + State.TasksNum); TaskIndex++)
						{
							const FStateTreeTaskBase& Task = CurrentStateTree->Nodes[TaskIndex].Get<const FStateTreeTaskBase>();
							if (Task.bTaskEnabled)
							{
								DebugString << Task.GetDebugInfo(*this);
							}
						}
					}
				}
			}
		}
	}
	else
	{
		DebugString << TEXT("StateTree context is not initialized properly.");
	}

	return DebugString.ToString();
}
#endif // WITH_GAMEPLAY_DEBUGGER

#if WITH_STATETREE_DEBUG
int32 FStateTreeReadOnlyExecutionContext::GetStateChangeCount() const
{
	if (!IsValid())
	{
		STATETREE_LOG(Warning, TEXT("%hs: StateTree context is not initialized properly ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		return 0;
	}

	const FStateTreeExecutionState& Exec = Storage.GetExecutionState();
	return Exec.StateChangeCount;
}

void FStateTreeReadOnlyExecutionContext::DebugPrintInternalLayout()
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogStateTree, ELogVerbosity::Log);
	UE_LOGF(LogStateTree, Log, "%ls", *RootStateTree.DebugInternalLayoutAsString());
}
#endif // WITH_STATETREE_DEBUG

FString FStateTreeReadOnlyExecutionContext::GetInstanceDescriptionInternal() const
{
	const TInstancedStruct<FStateTreeExecutionExtension>& ExecutionExtension = Storage.GetExecutionState().ExecutionExtension;
	return ExecutionExtension.IsValid()
		? ExecutionExtension.Get().GetInstanceDescription(FStateTreeExecutionExtension::FContextParameters(Owner, RootStateTree, Storage))
		: Owner.GetName();
}

#if WITH_STATETREE_TRACE
FStateTreeInstanceDebugId FStateTreeReadOnlyExecutionContext::GetInstanceDebugId() const
{
	FStateTreeInstanceDebugId& InstanceDebugId = Storage.GetMutableExecutionState().InstanceDebugId;
	if (!InstanceDebugId.IsValid())
	{
		// Using an Id from the object trace pool to allow each instance to be a unique debuggable object in the Traces and RewindDebugger
#if OBJECT_TRACE_ENABLED
		InstanceDebugId = FStateTreeInstanceDebugId(FObjectTrace::AllocateInstanceId());
#else
		InstanceDebugId = FStateTreeInstanceDebugId(0);
#endif
	}
	return InstanceDebugId;
}
#endif // WITH_STATETREE_TRACE

/**
 * FStateTreeMinimalExecutionContext implementation
 */
 // Deprecated
FStateTreeMinimalExecutionContext::FStateTreeMinimalExecutionContext(UObject& InOwner, const UStateTree& InStateTree, FStateTreeInstanceData& InInstanceData)
	: FStateTreeMinimalExecutionContext(&InOwner, &InStateTree, InInstanceData.GetMutableStorage())
{
}

// Deprecated
FStateTreeMinimalExecutionContext::FStateTreeMinimalExecutionContext(UObject& InOwner, const UStateTree& InStateTree, FStateTreeInstanceStorage& InStorage)
	: FStateTreeMinimalExecutionContext(&InOwner, &InStateTree, InStorage)
{
}

FStateTreeMinimalExecutionContext::FStateTreeMinimalExecutionContext(TNotNull<UObject*> InOwner, TNotNull<const UStateTree*> InStateTree, FStateTreeInstanceData& InInstanceData)
	: FStateTreeMinimalExecutionContext(InOwner, InStateTree, InInstanceData.GetMutableStorage())
{
}

FStateTreeMinimalExecutionContext::FStateTreeMinimalExecutionContext(TNotNull<UObject*> InOwner, TNotNull<const UStateTree*> InStateTree, FStateTreeInstanceStorage& InStorage)
	: FStateTreeReadOnlyExecutionContext(InOwner, InStateTree, InStorage)
{
	Storage.AcquireWriteAccess();

	if (IsValid())
	{
		constexpr bool bWriteAccessAcquired = true;
		Storage.GetRuntimeValidation().SetContext(&Owner, &RootStateTree, bWriteAccessAcquired);
	}
}

FStateTreeMinimalExecutionContext::~FStateTreeMinimalExecutionContext()
{
	Storage.ReleaseWriteAccess();
}

UE::StateTree::FScheduledTickHandle FStateTreeMinimalExecutionContext::AddScheduledTickRequest(FStateTreeScheduledTick ScheduledTick)
{
	if (!IsValid())
	{
		STATETREE_LOG(Warning, TEXT("%hs: StateTree context is not initialized properly ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		return UE::StateTree::FScheduledTickHandle();
	}

	UE::StateTree::FScheduledTickHandle Result = Storage.GetMutableExecutionState().AddScheduledTickRequest(ScheduledTick);
	ScheduleNextTick(UE::StateTree::ETickReason::ScheduledTickRequest);
	return Result;
}

void FStateTreeMinimalExecutionContext::UpdateScheduledTickRequest(UE::StateTree::FScheduledTickHandle Handle, FStateTreeScheduledTick ScheduledTick)
{
	if (!IsValid())
	{
		STATETREE_LOG(Warning, TEXT("%hs: StateTree context is not initialized properly ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		return;
	}

	if (Storage.GetMutableExecutionState().UpdateScheduledTickRequest(Handle, ScheduledTick))
	{
		ScheduleNextTick(UE::StateTree::ETickReason::ScheduledTickRequest);
	}
}

void FStateTreeMinimalExecutionContext::RemoveScheduledTickRequest(UE::StateTree::FScheduledTickHandle Handle)
{
	if (!IsValid())
	{
		STATETREE_LOG(Warning, TEXT("%hs: StateTree context is not initialized properly ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		return;
	}

	if (Storage.GetMutableExecutionState().RemoveScheduledTickRequest(Handle))
	{
		ScheduleNextTick(UE::StateTree::ETickReason::ScheduledTickRequest);
	}
}

void FStateTreeMinimalExecutionContext::SendEvent(const FGameplayTag Tag, const FConstStructView Payload, const FName Origin)
{
	CSV_SCOPED_TIMING_STAT(StateTree, SendEvent);

	if (!IsValid())
	{
		STATETREE_LOG(Warning, TEXT("%hs: StateTree context is not initialized properly ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		return;
	}

	STATETREE_LOG(VeryVerbose, TEXT("Send Event '%s'"), *Tag.ToString());
	UE_STATETREE_DEBUG_LOG_EVENT(this, Log, TEXT("Send Event '%s'"), *Tag.ToString());

	FStateTreeEventQueue& LocalEventQueue = Storage.GetMutableEventQueue();
	if (LocalEventQueue.SendEvent(&Owner, Tag, Payload, Origin))
	{
		ScheduleNextTick(UE::StateTree::ETickReason::Event);
		UE_STATETREE_DEBUG_SEND_EVENT(this, &RootStateTree, Tag, Payload, Origin);
	}
}

FStateTreeExecutionExtension* FStateTreeMinimalExecutionContext::GetMutableExecutionExtension() const
{
	return Storage.GetMutableExecutionState().ExecutionExtension.GetMutablePtr();
}

void FStateTreeMinimalExecutionContext::ScheduleNextTick(UE::StateTree::ETickReason Reason)
{
	if (bAllowedToScheduleNextTick && RootStateTree.IsScheduledTickAllowed())
	{
		if (FStateTreeExecutionExtension* ExecExtension = GetMutableExecutionExtension())
		{
			ExecExtension->ScheduleNextTick(FStateTreeExecutionExtension::FContextParameters(Owner, RootStateTree, Storage), FStateTreeExecutionExtension::FNextTickArguments(Reason));
		}
	}
}

/**
 * FStateTreeExecutionContext::FSelectStateResult implementation
 */
FStateTreeExecutionFrame& FStateTreeExecutionContext::FSelectStateResult::MakeAndAddTemporaryFrame(
	const UE::StateTree::FActiveFrameID FrameID,
	const UE::StateTree::FExecutionFrameHandle& FrameHandle,
	const bool bIsGlobalFrame)
{
	check(FrameHandle.IsValid());

	FStateTreeExecutionFrame& ExecFrame = TemporaryFrames.AddDefaulted_GetRef();
	ExecFrame.FrameID = FrameID;
	ExecFrame.StateTree = FrameHandle.GetStateTree();
	ExecFrame.RootState = FrameHandle.GetRootState();
	ExecFrame.bIsGlobalFrame = bIsGlobalFrame;

	const FCompactStateTreeFrame* TargetFrame = UE::StateTree::ExecutionContext::Private::FindStateTreeFrame(FrameHandle);
	ensureMsgf(TargetFrame, TEXT("The compiled data is invalid. It should contains the information for the new root frame."));
	ExecFrame.ActiveTasksStatus = TargetFrame ? FStateTreeTasksCompletionStatus(*TargetFrame) : FStateTreeTasksCompletionStatus();

	return ExecFrame;
}

FStateTreeExecutionFrame& FStateTreeExecutionContext::FSelectStateResult::MakeAndAddTemporaryFrameWithNewRoot(
	const UE::StateTree::FActiveFrameID FrameID,
	const UE::StateTree::FExecutionFrameHandle& FrameHandle,
	FStateTreeExecutionFrame& OtherFrame)
{
	check(FrameHandle.IsValid());

	constexpr bool bIsGlobalFrame = true;
	FStateTreeExecutionFrame& ExecFrame = MakeAndAddTemporaryFrame(FrameID, FrameHandle, bIsGlobalFrame);
	check(ExecFrame.StateTree == OtherFrame.StateTree);

	ExecFrame.ExternalDataBaseIndex = OtherFrame.ExternalDataBaseIndex;

	// new frame is global frame
	ExecFrame.GlobalInstanceDataFrameID = ExecFrame.FrameID;
	ExecFrame.GlobalInstanceIndexBase = OtherFrame.GlobalInstanceIndexBase;

	ExecFrame.GlobalParameterDataHandle = OtherFrame.GlobalParameterDataHandle;
	if (ExecFrame.GlobalParameterDataHandle.HasValidIndex())
	{
		// Global Parameter is a state parameter living within parent frame with different state tree
		ExecFrame.GlobalParameterDataFrameID = OtherFrame.GlobalParameterDataFrameID;
	}
	else
	{
		// Global Parameter is root global parameter and new frame is global frame
		ExecFrame.GlobalParameterDataFrameID = ExecFrame.FrameID;
	}
	ExecFrame.GlobalParameterIndexBase = OtherFrame.GlobalParameterIndexBase;

	// new frame could be a subtree, but it's always standalone.
	ExecFrame.StateParameterDataHandle = ExecFrame.StateTree->GetStateFromHandle(ExecFrame.RootState)->ParameterDataHandle;
	ExecFrame.StateParameterDataFrameID = ExecFrame.FrameID;
	
	// Don't stop globals.
	ExecFrame.bHaveEntered = OtherFrame.bHaveEntered;
	OtherFrame.bHaveEntered = false;

	return ExecFrame;
}

FStateTreeExecutionContext::FSelectStateResult::FFrameAndParent FStateTreeExecutionContext::FSelectStateResult::GetExecutionFrame(UE::StateTree::FActiveFrameID ID)
{
	const int32 FoundIndex = SelectedFrames.IndexOfByKey(ID);
	FStateTreeExecutionFrame* Frame = FoundIndex != INDEX_NONE
		? TemporaryFrames.FindByPredicate([ID](const FStateTreeExecutionFrame& Other){ return Other.FrameID == ID; })
		: nullptr;
	const UE::StateTree::FActiveFrameID ParentFrameID = FoundIndex > 0 ? SelectedFrames[FoundIndex - 1] : UE::StateTree::FActiveFrameID();
	return FFrameAndParent{ .Frame = Frame, .ParentFrameID = ParentFrameID };
}

UE::StateTree::FActiveState FStateTreeExecutionContext::FSelectStateResult::GetStateHandle(UE::StateTree::FActiveStateID ID) const
{
	const int32 FoundIndex = UE::StateTree::FActiveStatePath::IndexOf(SelectedStates, ID);
	if (FoundIndex != INDEX_NONE)
	{
		return SelectedStates[FoundIndex];
	}
	return {};
}

TConstArrayView<FStateTreeExecutionFrame> FStateTreeExecutionContext::FSelectStateResult::GetTemporaryFrames() const
{
	return TemporaryFrames;
}

/**
 * FStateTreeExecutionContext::FCurrentlyProcessedFrameScope implementation
 */
FStateTreeExecutionContext::FCurrentlyProcessedFrameScope::FCurrentlyProcessedFrameScope(FStateTreeExecutionContext& InContext, const FStateTreeExecutionFrame* CurrentParentFrame, const FStateTreeExecutionFrame& CurrentFrame) : Context(InContext)
{
	check(CurrentFrame.StateTree);

	TSharedPtr<FStateTreeInstanceData> SharedPtr = CurrentFrame.StateTree->GetSharedInstanceData();
	FStateTreeInstanceStorage* SharedInstanceDataStorage = SharedPtr ? &(SharedPtr->GetMutableStorage()) : nullptr;

	SavedFrame = Context.CurrentlyProcessedFrame;
	SavedParentFrame = Context.CurrentlyProcessedParentFrame;
	SavedSharedInstanceDataStorage = Context.CurrentlyProcessedSharedInstanceStorage;
	Context.CurrentlyProcessedFrame = &CurrentFrame;
	Context.CurrentlyProcessedParentFrame = CurrentParentFrame;
	Context.CurrentlyProcessedSharedInstanceStorage = SharedInstanceDataStorage;

	UE_STATETREE_DEBUG_INSTANCE_FRAME_EVENT(&Context, Context.CurrentlyProcessedFrame);
}

FStateTreeExecutionContext::FCurrentlyProcessedFrameScope::~FCurrentlyProcessedFrameScope()
{
	Context.CurrentlyProcessedFrame = SavedFrame;
	Context.CurrentlyProcessedParentFrame = SavedParentFrame;
	Context.CurrentlyProcessedSharedInstanceStorage = SavedSharedInstanceDataStorage;

	if (Context.CurrentlyProcessedFrame)
	{
		UE_STATETREE_DEBUG_INSTANCE_FRAME_EVENT(&Context, Context.CurrentlyProcessedFrame);
	}
}

/**
 * FStateTreeExecutionContext::FNodeInstanceDataScope implementation
 */
FStateTreeExecutionContext::FNodeInstanceDataScope::FNodeInstanceDataScope(FStateTreeExecutionContext& InContext, const FStateTreeNodeBase* InNode, const int32 InNodeIndex, const FStateTreeDataHandle InNodeDataHandle, const FStateTreeDataView InNodeInstanceData)
	: Context(InContext)
{
	SavedNode = Context.CurrentNode;
	SavedNodeIndex = Context.CurrentNodeIndex;
	SavedNodeDataHandle = Context.CurrentNodeDataHandle;
	SavedNodeInstanceData = Context.CurrentNodeInstanceData;
	Context.CurrentNode = InNode;
	Context.CurrentNodeIndex = InNodeIndex;
	Context.CurrentNodeDataHandle = InNodeDataHandle;
	Context.CurrentNodeInstanceData = InNodeInstanceData;
}

FStateTreeExecutionContext::FNodeInstanceDataScope::~FNodeInstanceDataScope()
{
	Context.CurrentNodeDataHandle = SavedNodeDataHandle;
	Context.CurrentNodeInstanceData = SavedNodeInstanceData;
	Context.CurrentNodeIndex = SavedNodeIndex;
	Context.CurrentNode = SavedNode;
}

/**
 * FStateTreeExecutionContext implementation
 */
FStateTreeExecutionContext::FStateTreeExecutionContext(UObject& InOwner, const UStateTree& InStateTree, FStateTreeInstanceData& InInstanceData, const FOnCollectStateTreeExternalData& InCollectExternalDataDelegate, const EStateTreeRecordTransitions RecordTransitions)
	: FStateTreeExecutionContext(&InOwner, &InStateTree, InInstanceData, InCollectExternalDataDelegate, RecordTransitions)
{
}

FStateTreeExecutionContext::FStateTreeExecutionContext(TNotNull<UObject*> InOwner, TNotNull<const UStateTree*> InStateTree, FStateTreeInstanceData& InInstanceData, const FOnCollectStateTreeExternalData& InCollectExternalDataDelegate, const EStateTreeRecordTransitions RecordTransitions)
	: FStateTreeMinimalExecutionContext(InOwner, InStateTree, InInstanceData)
	, InstanceData(InInstanceData)
	, CollectExternalDataDelegate(InCollectExternalDataDelegate)
{
	if (IsValid())
	{
		// Initialize data views for all possible items.
		ContextAndExternalDataViews.SetNum(RootStateTree.GetNumContextDataViews());
		EventQueue = InstanceData.GetSharedMutableEventQueue();
		bRecordTransitions = RecordTransitions == EStateTreeRecordTransitions::Yes;
	}
	else
	{
		STATETREE_LOG(Warning, TEXT("%hs: StateTree asset is not valid ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
	}
}

FStateTreeExecutionContext::FStateTreeExecutionContext(const FStateTreeExecutionContext& InContextToCopy, const UStateTree& InStateTree, FStateTreeInstanceData& InInstanceData)
	: FStateTreeExecutionContext(InContextToCopy, &InStateTree, InInstanceData)
{
}

FStateTreeExecutionContext::FStateTreeExecutionContext(const FStateTreeExecutionContext& InContextToCopy, TNotNull<const UStateTree*> InStateTree, FStateTreeInstanceData& InInstanceData)
	: FStateTreeExecutionContext(&InContextToCopy.Owner, InStateTree, InInstanceData, InContextToCopy.CollectExternalDataDelegate)
{
	if (IsValid())
	{
		SetLinkedStateTreeOverrides(InContextToCopy.LinkedAssetStateTreeOverrides);
		const bool bIsSameSchema = RootStateTree.GetSchema()->GetClass() == InContextToCopy.GetStateTree()->GetSchema()->GetClass();
		if (bIsSameSchema)
		{
			for (const FStateTreeExternalDataDesc& TargetDataDesc : GetContextDataDescs())
			{
				const int32 TargetIndex = TargetDataDesc.Handle.DataHandle.GetIndex();
				const bool bViewsValidIndex = ContextAndExternalDataViews.IsValidIndex(TargetIndex);
				const bool bCopyViewsValidIndex = InContextToCopy.ContextAndExternalDataViews.IsValidIndex(TargetIndex);
				if (ensure(bViewsValidIndex && bCopyViewsValidIndex)) // same context, should share the same ContextAndExternalDataViews.
				{
					ContextAndExternalDataViews[TargetIndex] = InContextToCopy.ContextAndExternalDataViews[TargetIndex];
				}
			}
		}
		else
		{
			STATETREE_LOG(Error, TEXT("%hs: '%s' using StateTree '%s' trying to run subtree '%s' but their schemas don't match"),
				__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(InContextToCopy.GetStateTree()), *GetFullNameSafe(&RootStateTree));
		}
	}
	else
	{
		STATETREE_LOG(Warning, TEXT("%hs: StateTree asset is not valid ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FStateTreeExecutionContext::~FStateTreeExecutionContext()
{
	// Mark external data indices as invalid
	FStateTreeExecutionState& Exec = InstanceData.GetMutableStorage().GetMutableExecutionState();
	for (FStateTreeExecutionFrame& Frame : Exec.ActiveFrames)
	{
		Frame.ExternalDataBaseIndex = {};
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FStateTreeExecutionContext::SetCollectExternalDataCallback(const FOnCollectStateTreeExternalData& Callback)
{
	if (!IsValid())
	{
		STATETREE_LOG(Warning, TEXT("%hs: StateTree context is not initialized properly ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		return;
	}

	FStateTreeExecutionState& Exec = GetExecState();
	if (!ensureMsgf(Exec.CurrentPhase == EStateTreeUpdatePhase::Unset, TEXT("%hs can't be called while already in %s ('%s' using StateTree '%s')."),
		__FUNCTION__, *UEnum::GetDisplayValueAsText(Exec.CurrentPhase).ToString(), *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree)))
	{
		return;
	}

	CollectExternalDataDelegate = Callback;
}

void FStateTreeExecutionContext::SetLinkedStateTreeOverrides(const FStateTreeReferenceOverrides* InLinkedStateTreeOverrides)
{
	if (InLinkedStateTreeOverrides)
	{
		SetLinkedStateTreeOverrides(*InLinkedStateTreeOverrides);
	}
	else
	{
		SetLinkedStateTreeOverrides(FStateTreeReferenceOverrides());
	}
}

void FStateTreeExecutionContext::SetLinkedStateTreeOverrides(FStateTreeReferenceOverrides InLinkedStateTreeOverrides)
{
	if (!IsValid())
	{
		STATETREE_LOG(Warning, TEXT("%hs: StateTree context is not initialized properly ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		return;
	}

	bool bValid = true;

	// Confirms that the overrides schema matches.
	const TConstArrayView<FStateTreeReferenceOverrideItem> InOverrideItems = InLinkedStateTreeOverrides.GetOverrideItems();
	for (const FStateTreeReferenceOverrideItem& Item : InOverrideItems)
	{
		if (const UStateTree* ItemStateTree = Item.GetStateTreeReference().GetStateTree())
		{
			if (!ItemStateTree->IsReadyToRun())
			{
				STATETREE_LOG(Error, TEXT("%hs: '%s' using StateTree '%s' trying to set override '%s' but the tree is not initialized properly."),
					__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(GetStateTree()), *GetFullNameSafe(ItemStateTree));
				bValid = false;
				break;
			}

			if (!RootStateTree.HasCompatibleContextData(*ItemStateTree))
			{
				STATETREE_LOG(Error, TEXT("%hs: '%s' using StateTree '%s' trying to set override '%s' but the tree context data is not compatible."),
					__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(GetStateTree()), *GetFullNameSafe(ItemStateTree));
				bValid = false;
				break;
			}

			const UStateTreeSchema* OverrideSchema = ItemStateTree->GetSchema();
			if (ItemStateTree->GetSchema() == nullptr)
			{
				STATETREE_LOG(Error, TEXT("%hs: '%s' using StateTree '%s' trying to set override '%s' but the tree does not have a schema."),
					__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(GetStateTree()), *GetFullNameSafe(ItemStateTree));
				bValid = false;
				break;
			}

			const bool bIsSameSchema = RootStateTree.GetSchema()->GetClass() == OverrideSchema->GetClass();
			if (!bIsSameSchema)
			{
				STATETREE_LOG(Error, TEXT("%hs: '%s' using StateTree '%s' trying to set override '%s' but their schemas don't match."),
					__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(GetStateTree()), *GetFullNameSafe(Item.GetStateTreeReference().GetStateTree()));
				bValid = false;
				break;
			}
		}
	}

	bool bChanged = false;
	if (bValid)
	{
		LinkedAssetStateTreeOverrides = MoveTemp(InLinkedStateTreeOverrides);
		bChanged = LinkedAssetStateTreeOverrides.GetOverrideItems().Num() > 0;
	}
	else if (LinkedAssetStateTreeOverrides.GetOverrideItems().Num() > 0)
	{
		LinkedAssetStateTreeOverrides.Reset();
		bChanged = true;
	}

	if (bChanged)
	{
		TInstancedStruct<FStateTreeExecutionExtension>& ExecutionExtension = Storage.GetMutableExecutionState().ExecutionExtension;
		if (ExecutionExtension.IsValid())
		{
			ExecutionExtension.GetMutable().OnLinkedStateTreeOverridesSet(FStateTreeExecutionExtension::FContextParameters(Owner, RootStateTree, Storage), LinkedAssetStateTreeOverrides);
		}
	}
}

const FStateTreeReference* FStateTreeExecutionContext::GetLinkedStateTreeOverrideForTag(const FGameplayTag StateTag) const
{
	for (const FStateTreeReferenceOverrideItem& Item : LinkedAssetStateTreeOverrides.GetOverrideItems())
	{
		if (StateTag.MatchesTag(Item.GetStateTag()))
		{
			return &Item.GetStateTreeReference();
		}
	}

	return nullptr;
}

bool FStateTreeExecutionContext::FExternalGlobalParameters::Add(const FPropertyBindingCopyInfo& Copy, uint8* InParameterMemory)
{
	const int32 TypeHash = HashCombine(GetTypeHash(Copy.SourceLeafProperty), GetTypeHash(Copy.SourceIndirection));
	const int32 NumMappings = Mappings.Num();
	Mappings.Add(TypeHash, InParameterMemory);
	return Mappings.Num() > NumMappings;
}

uint8* FStateTreeExecutionContext::FExternalGlobalParameters::Find(const FPropertyBindingCopyInfo& Copy) const
{
	const int32 TypeHash = HashCombine(GetTypeHash(Copy.SourceLeafProperty), GetTypeHash(Copy.SourceIndirection));
	if (uint8* const* MappingPtr = Mappings.Find(TypeHash))
	{
		return *MappingPtr;
	}

	checkf(false, TEXT("Missing external parameter data"));
	return nullptr;
}

void FStateTreeExecutionContext::FExternalGlobalParameters::Reset()
{
	Mappings.Reset();
}

void FStateTreeExecutionContext::SetExternalGlobalParameters(const FExternalGlobalParameters* Parameters)
{
	ExternalGlobalParameters = Parameters;
}

bool FStateTreeExecutionContext::AreContextDataViewsValid() const
{
	if (!IsValid())
	{
		return false;
	}

	bool bResult = true;

	for (const FStateTreeExternalDataDesc& DataDesc : RootStateTree.GetContextDataDescs())
	{
		const FStateTreeDataView& DataView = ContextAndExternalDataViews[DataDesc.Handle.DataHandle.GetIndex()];

		// Required items must have valid pointer of the expected type.  
		if (DataDesc.Requirement == EStateTreeExternalDataRequirement::Required)
		{
			if (!DataView.IsValid() || !DataDesc.IsCompatibleWith(DataView))
			{
				bResult = false;
				break;
			}
		}
		else // Optional items must have the expected type if they are set.
		{
			if (DataView.IsValid() && !DataDesc.IsCompatibleWith(DataView))
			{
				bResult = false;
				break;
			}
		}
	}
	return bResult;
}

bool FStateTreeExecutionContext::SetContextDataByName(const FName Name, FStateTreeDataView DataView)
{
	const FStateTreeExternalDataDesc* Desc = RootStateTree.GetContextDataDescs().FindByPredicate([&Name](const FStateTreeExternalDataDesc& Desc)
		{
			return Desc.Name == Name;
		});
	if (Desc)
	{
		ContextAndExternalDataViews[Desc->Handle.DataHandle.GetIndex()] = DataView;
		return true;
	}
	return false;
}

FStateTreeDataView FStateTreeExecutionContext::GetContextDataByName(const FName Name) const
{
	const FStateTreeExternalDataDesc* Desc = RootStateTree.GetContextDataDescs().FindByPredicate([&Name](const FStateTreeExternalDataDesc& Desc)
		{
			return Desc.Name == Name;
		});
	if (Desc)
	{
		return ContextAndExternalDataViews[Desc->Handle.DataHandle.GetIndex()];
	}
	return FStateTreeDataView();
}

FStateTreeWeakExecutionContext FStateTreeExecutionContext::MakeWeakExecutionContext() const
{
	return FStateTreeWeakExecutionContext(*this);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FStateTreeWeakTaskRef FStateTreeExecutionContext::MakeWeakTaskRef(const FStateTreeTaskBase& Node) const
{
	// This function has been deprecated
	check(CurrentNode == &Node);
	return MakeWeakTaskRefInternal();
}

FStateTreeWeakTaskRef FStateTreeExecutionContext::MakeWeakTaskRefInternal() const
{
	// This function has been deprecated
	FStateTreeWeakTaskRef Result;
	if (const FStateTreeExecutionFrame* Frame = GetCurrentlyProcessedFrame())
	{
		if (Frame->StateTree->Nodes.IsValidIndex(CurrentNodeIndex)
			&& Frame->StateTree->Nodes[CurrentNodeIndex].GetPtr<const FStateTreeTaskBase>() != nullptr)
		{
			Result = FStateTreeWeakTaskRef(Frame->StateTree, FStateTreeIndex16(CurrentNodeIndex));
		}
	}
	return Result;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

// Deprecated
EStateTreeRunStatus FStateTreeExecutionContext::Start(const FInstancedPropertyBag* InitialParameters, int32 RandomSeed)
{
	const TOptional<int32> ParamRandomSeed = RandomSeed == -1 ? TOptional<int32>() : RandomSeed;
	return Start(FStartParameters
		{
			.InitialGlobalParameters = InitialParameters ? InitialParameters->GetValue() : FConstStructView(),
			.RandomSeed = ParamRandomSeed
		});
}

EStateTreeRunStatus FStateTreeExecutionContext::Start()
{
	return Start(FStartParameters());
}

EStateTreeRunStatus FStateTreeExecutionContext::Start(const FConstStructView InitialParameters)
{
	return Start(FStartParameters
		{
			.InitialGlobalParameters = InitialParameters
		});
}

void FStateTreeExecutionContext::SetUpdatePhaseInExecutionState(FStateTreeExecutionState& ExecutionState, const EStateTreeUpdatePhase UpdatePhase) const
{
	if (ExecutionState.CurrentPhase == UpdatePhase)
	{
		return;
	}

	if (ExecutionState.CurrentPhase != EStateTreeUpdatePhase::Unset)
	{
		UE_STATETREE_DEBUG_EXIT_PHASE(this, ExecutionState.CurrentPhase);
	}

	ExecutionState.CurrentPhase = UpdatePhase;

	if (ExecutionState.CurrentPhase != EStateTreeUpdatePhase::Unset)
	{
		UE_STATETREE_DEBUG_ENTER_PHASE(this, ExecutionState.CurrentPhase);
	}
}

EStateTreeRunStatus FStateTreeExecutionContext::Start(FStartParameters Parameters)
{
	CSV_SCOPED_TIMING_STAT(StateTree, Start);
	TRACE_CPUPROFILER_EVENT_SCOPE(FStateTreeExecutionContext::Start);
	UE_STATETREE_CRASH_REPORTER_SCOPE(&Owner, &RootStateTree, UE::StateTree::ExecutionContext::Private::Name_Start.Resolve());
	SCOPE_CYCLE_UOBJECT(StateTree, &Owner);

	using namespace UE::StateTree;
	using namespace UE::StateTree::ExecutionContext;
	using namespace UE::StateTree::ExecutionContext::Private;

	if (!IsValid())
	{
		STATETREE_LOG(Warning, TEXT("%hs: StateTree context is not initialized properly ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		return EStateTreeRunStatus::Failed;
	}

	FStateTreeExecutionState& Exec = GetExecState();
	if (!ensureMsgf(Exec.CurrentPhase == EStateTreeUpdatePhase::Unset, TEXT("%hs can't be called while already in %s ('%s' using StateTree '%s')."),
		__FUNCTION__, *UEnum::GetDisplayValueAsText(Exec.CurrentPhase).ToString(), *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree)))
	{
		return EStateTreeRunStatus::Failed;
	}

	// Stop if still running previous state.
	if (Exec.TreeRunStatus == EStateTreeRunStatus::Running)
	{
		Stop();
	}

	// Initialize instance data. No active states yet, so we'll initialize the evals and global tasks.
	InstanceData.Reset();

	constexpr bool bWriteAccessAcquired = true;
	Storage.GetRuntimeValidation().SetContext(&Owner, &RootStateTree, bWriteAccessAcquired);
	Exec.ExecutionExtension = MoveTemp(Parameters.ExecutionExtension);
	if (Parameters.SharedEventQueue)
	{
		InstanceData.SetSharedEventQueue(Parameters.SharedEventQueue.ToSharedRef());
	}

#if WITH_STATETREE_TRACE
	// Make sure the debug id is valid. We want to construct it with the current GetInstanceDescriptionInternal
	GetInstanceDebugId();
#endif

	if (!Parameters.InitialGlobalParameters.IsValid() || !SetGlobalParameters(Parameters.InitialGlobalParameters))
	{
		SetGlobalParameters(RootStateTree.GetDefaultParameters().GetValue());
	}

	Exec.RandomStream.Initialize(Parameters.RandomSeed.IsSet() ? Parameters.RandomSeed.GetValue() : FPlatformTime::Cycles());

	TGuardValue<bool> ScheduledNextTickScope(bAllowedToScheduleNextTick, false);
	ensure(Exec.ActiveFrames.Num() == 0);

	// Initialize for the init frame.
	TSharedRef<FSelectStateResult> SelectStateResult = MakeShared<FSelectStateResult>();
	check(CurrentlyProcessedTemporaryStorage == nullptr);
	TGuardValue<TSharedPtr<FSelectStateResult>> TemporaryStorageScope(CurrentlyProcessedTemporaryStorage, SelectStateResult);

	const FActiveFrameID InitFrameID = FActiveFrameID(Storage.GenerateUniqueId());

	const FSelectStartStateInfo SelectStartStateInfo = GetSelectStartStateInfo(&RootStateTree, Parameters);
	if (!SelectStartStateInfo.StartStateToSelectHandle.IsValid())
	{
		STATETREE_LOG(Warning, TEXT("%hs: The start state in tree %s could not be found. "
			"The overriding state tag is either invalid or no state corresponds to that tag."),
			__FUNCTION__, *GetFullNameSafe(&RootStateTree));

		return EStateTreeRunStatus::Failed;
	}

	const FExecutionFrameHandle InitFrameHandle = FExecutionFrameHandle(&RootStateTree, SelectStartStateInfo.FrameRootStateHandle);

	constexpr bool bIsGlobalFrame = true;
	FStateTreeExecutionFrame& InitFrame = SelectStateResult->MakeAndAddTemporaryFrame(InitFrameID, InitFrameHandle, bIsGlobalFrame);
	InitFrame.ExecutionRuntimeIndexBase = FStateTreeIndex16(Storage.AddExecutionRuntimeData(GetOwner(), InitFrameHandle));

	InitFrame.GlobalParameterDataHandle = FStateTreeDataHandle(EStateTreeDataSourceType::GlobalParameterData);
	InitFrame.GlobalParameterDataFrameID = InitFrameID;

	InitFrame.GlobalInstanceDataFrameID = InitFrameID;

	InitFrame.StateParameterDataHandle = InitFrame.StateTree->GetStateFromHandle(InitFrame.RootState)->ParameterDataHandle;
	InitFrame.StateParameterDataFrameID = InitFrameID;

	if (!CollectActiveExternalData(SelectStateResult->TemporaryFrames))
	{
		STATETREE_LOG(Warning, TEXT("%hs: Failed to collect external data ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		return EStateTreeRunStatus::Failed;
	}

	// Must sent instance creation event first 
	UE_STATETREE_DEBUG_INSTANCE_EVENT(this, EStateTreeTraceEventType::Push);

	STATETREE_LOG(VeryVerbose, TEXT("%hs: Starting State Tree %s on owner '%s'."),
		__FUNCTION__, *GetFullNameSafe(&RootStateTree), *GetNameSafe(&Owner));

	// From this point any calls to Stop should be deferred.
	SetUpdatePhaseInExecutionState(Exec, EStateTreeUpdatePhase::StartTree);

	// Start evaluators and global tasks. Fail the execution if any global task fails.
	constexpr FStateTreeExecutionFrame* ParentInitFrame = nullptr;
	SelectStateResult->SelectedFrames.Add(InitFrame.FrameID);
	EStateTreeRunStatus GlobalTasksRunStatus = StartTemporaryEvaluatorsAndGlobalTasks(ParentInitFrame, InitFrame);
	if (GlobalTasksRunStatus == EStateTreeRunStatus::Running)
	{
		// Exception with Start. Tick the evaluators.
		constexpr float DeltaTime = 0.0f;
		TickGlobalEvaluatorsForFrameWithValidation(DeltaTime, ParentInitFrame, InitFrame);

		// Initialize to unset running state.
		Exec.TreeRunStatus = EStateTreeRunStatus::Running;
		Exec.LastTickStatus = EStateTreeRunStatus::Unset;

		FSelectStateArguments SelectStateArgs;
		SelectStateArgs.SourceState = FActiveState(InitFrameID, FActiveStateID(), FStateTreeStateHandle());
		SelectStateArgs.TargetState = FStateHandleContext(InitFrameHandle.GetStateTree(), SelectStartStateInfo.StartStateToSelectHandle);
		SelectStateArgs.Behavior = ESelectStateBehavior::StateTransition;
		SelectStateArgs.SelectionRules = InitFrameHandle.GetStateTree()->StateSelectionRules;
		if (SelectState(SelectStateArgs, SelectStateResult))
		{
			check(SelectStateResult->SelectedStates.Num() > 0);
			const FStateTreeStateHandle LastSelectedStateHandle = SelectStateResult->SelectedStates.Last().GetStateHandle();
			if (LastSelectedStateHandle.IsCompletionState())
			{
				// Transition to a terminal state (succeeded/failed).
				STATETREE_LOG(Warning, TEXT("%hs: Tree %s at StateTree start on '%s' using StateTree '%s'."),
					__FUNCTION__,
					LastSelectedStateHandle == FStateTreeStateHandle::Succeeded ? TEXT("succeeded") : TEXT("failed"),
					*GetNameSafe(&Owner),
					*GetFullNameSafe(&RootStateTree)
				);
				Exec.TreeRunStatus = LastSelectedStateHandle.ToCompletionStatus();
			}
			else
			{
				// Enter state tasks can fail/succeed, treat it same as tick.
				FStateTreeTransitionResult Transition;
				Transition.TargetState = InitFrameHandle.GetRootState();
				Transition.CurrentRunStatus = Exec.LastTickStatus;
				const EStateTreeRunStatus LastTickStatus = EnterState(SelectStateResult, Transition);

				Exec.LastTickStatus = LastTickStatus;

				// Report state completed immediately.
				if (Exec.LastTickStatus != EStateTreeRunStatus::Running)
				{
					StateCompleted();
				}

				// Was not able to enter the root state. Fail selection.
				if (Exec.ActiveFrames.Num() == 0 || Exec.ActiveFrames[0].ActiveStates.Num() == 0)
				{
					GlobalTasksRunStatus = EStateTreeRunStatus::Failed;
					Exec.TreeRunStatus = EStateTreeRunStatus::Failed;
					STATETREE_LOG(Error, TEXT("%hs: Failed to enter the initial state on '%s' using StateTree '%s'. Check that the StateTree logic can always select a state at start."),
						__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
				}
			}
		}
		else
		{
			GlobalTasksRunStatus = EStateTreeRunStatus::Failed;
			Exec.TreeRunStatus = EStateTreeRunStatus::Failed;
			STATETREE_LOG(Error, TEXT("%hs: Failed to select initial state on '%s' using StateTree '%s'. Check that the StateTree logic can always select a state at start."),
				__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		}
	}
	else
	{
		Exec.TreeRunStatus = GlobalTasksRunStatus;
		STATETREE_LOG(VeryVerbose, TEXT("%hs: Start globals on '%s' using StateTree '%s' completes."),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
	}

	if (Exec.TreeRunStatus != EStateTreeRunStatus::Running)
	{
		StopTemporaryEvaluatorsAndGlobalTasks(ParentInitFrame, InitFrame, GlobalTasksRunStatus);

		STATETREE_LOG(VeryVerbose, TEXT("%hs: Global tasks completed on '%s' using StateTree '%s' on start in status '%s'."),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree), *UEnum::GetDisplayValueAsText(GlobalTasksRunStatus).ToString());

		// No active states or global tasks anymore, reset frames.
		Exec.ActiveFrames.Reset();
		RemoveAllDelegateListeners();

		SelectStateResult->SelectedFrames.Pop();

		// We are not considered as running yet so we only set the status without requiring a stop.
		Exec.TreeRunStatus = GlobalTasksRunStatus;
	}
	InstanceData.ResetTemporaryInstances();

	// Reset phase since we are now safe to stop and before potentially stopping the instance.
	SetUpdatePhaseInExecutionState(Exec, EStateTreeUpdatePhase::Unset);

	// Use local for resulting run state since Stop will reset the instance data.
	EStateTreeRunStatus Result = Exec.TreeRunStatus;

	if (Exec.RequestedStop != EStateTreeRunStatus::Unset
		&& Exec.TreeRunStatus == EStateTreeRunStatus::Running)
	{
		STATETREE_LOG(VeryVerbose, TEXT("Processing Deferred Stop"));
		UE_STATETREE_DEBUG_LOG_EVENT(this, Log, TEXT("Processing Deferred Stop"));
		Result = Stop(Exec.RequestedStop);
	}

	return Result;
}

EStateTreeRunStatus FStateTreeExecutionContext::Stop(EStateTreeRunStatus CompletionStatus)
{
	CSV_SCOPED_TIMING_STAT(StateTree, Stop);
	TRACE_CPUPROFILER_EVENT_SCOPE(FStateTreeExecutionContext::Stop);
	UE_STATETREE_CRASH_REPORTER_SCOPE(&Owner, &RootStateTree, UE::StateTree::ExecutionContext::Private::Name_Stop.Resolve());
	SCOPE_CYCLE_UOBJECT(StateTree, &Owner);

	if (!IsValid())
	{
		STATETREE_LOG(Warning, TEXT("%hs: StateTree context is not initialized properly ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		return EStateTreeRunStatus::Failed;
	}

	if (!CollectActiveExternalData())
	{
		STATETREE_LOG(Warning, TEXT("%hs: Failed to collect external data ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		return EStateTreeRunStatus::Failed;
	}

	TGuardValue<bool> ScheduledNextTickScope(bAllowedToScheduleNextTick, false);

	// Make sure that we return a valid completion status (i.e. Succeeded, Failed or Stopped)
	if (CompletionStatus == EStateTreeRunStatus::Unset
		|| CompletionStatus == EStateTreeRunStatus::Running)
	{
		CompletionStatus = EStateTreeRunStatus::Stopped;
	}

	FStateTreeExecutionState& Exec = GetExecState();

	// A reentrant call to Stop or a call from Start or Tick must be deferred.
	if (Exec.CurrentPhase != EStateTreeUpdatePhase::Unset)
	{
		STATETREE_LOG(VeryVerbose, TEXT("Deferring Stop at end of %s"), *UEnum::GetDisplayValueAsText(Exec.CurrentPhase).ToString());
		UE_STATETREE_DEBUG_LOG_EVENT(this, Log, TEXT("Deferring Stop at end of %s"), *UEnum::GetDisplayValueAsText(Exec.CurrentPhase).ToString());

		Exec.RequestedStop = CompletionStatus;
		return EStateTreeRunStatus::Running;
	}

	// No need to clear on exit since we reset all the instance data before leaving the function.
	SetUpdatePhaseInExecutionState(Exec, EStateTreeUpdatePhase::StopTree);

	EStateTreeRunStatus Result = Exec.TreeRunStatus;

	// Exit states if still in some valid state.
	if (Exec.TreeRunStatus == EStateTreeRunStatus::Running)
	{
		// Transition to Succeeded state.
		const TSharedPtr<FSelectStateResult> EmptySelectionResult;
		FStateTreeTransitionResult Transition;
		Transition.TargetState = FStateTreeStateHandle::FromCompletionStatus(CompletionStatus);
		Transition.CurrentRunStatus = CompletionStatus;
		ExitState(EmptySelectionResult, Transition);

		// No active states or global tasks anymore, reset frames.
		Exec.ActiveFrames.Reset();

		Result = CompletionStatus;
	}

	Exec.TreeRunStatus = CompletionStatus;

	// Trace before resetting the instance data since it is required to provide all the event information
	UE_STATETREE_DEBUG_ACTIVE_STATES_EVENT(this, {});
	UE_STATETREE_DEBUG_EXIT_PHASE(this, EStateTreeUpdatePhase::StopTree);
	UE_STATETREE_DEBUG_INSTANCE_EVENT(this, EStateTreeTraceEventType::Pop);

	// Destruct all allocated instance data (does not shrink the buffer). This will invalidate Exec too.
	InstanceData.Reset();

	// External data needs to be recollected if this exec context is reused.
	bActiveExternalDataCollected = false;

	return Result;
}

EStateTreeRunStatus FStateTreeExecutionContext::TickPrelude()
{
	if (!IsValid())
	{
		STATETREE_LOG(Warning, TEXT("%hs: StateTree context is not initialized properly ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		return EStateTreeRunStatus::Failed;
	}

	if (!CollectActiveExternalData())
	{
		STATETREE_LOG(Warning, TEXT("%hs: Failed to collect external data ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		return EStateTreeRunStatus::Failed;
	}

	FStateTreeExecutionState& Exec = GetExecState();

	// No ticking if the tree is done or stopped.
	if (Exec.TreeRunStatus != EStateTreeRunStatus::Running)
	{
		return Exec.TreeRunStatus;
	}

	if (!ensureMsgf(Exec.CurrentPhase == EStateTreeUpdatePhase::Unset, TEXT("%hs can't be called while already in %s ('%s' using StateTree '%s')."),
		__FUNCTION__, *UEnum::GetDisplayValueAsText(Exec.CurrentPhase).ToString(), *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree)))
	{
		return EStateTreeRunStatus::Failed;
	}

	// From this point any calls to Stop should be deferred.
	SetUpdatePhaseInExecutionState(Exec, EStateTreeUpdatePhase::TickStateTree);

	return EStateTreeRunStatus::Running;
}


EStateTreeRunStatus FStateTreeExecutionContext::TickPostlude()
{
	FStateTreeExecutionState& Exec = GetExecState();

	// Reset phase since we are now safe to stop.
	SetUpdatePhaseInExecutionState(Exec, EStateTreeUpdatePhase::Unset);

	// Use local for resulting run state since Stop will reset the instance data.
	EStateTreeRunStatus Result = Exec.TreeRunStatus;

	if (Exec.RequestedStop != EStateTreeRunStatus::Unset)
	{
		STATETREE_LOG(VeryVerbose, TEXT("Processing Deferred Stop"));
		UE_STATETREE_DEBUG_LOG_EVENT(this, Log, TEXT("Processing Deferred Stop"));

		Result = Stop(Exec.RequestedStop);
	}

	return Result;
}

EStateTreeRunStatus FStateTreeExecutionContext::Tick(const float DeltaTime)
{
	CSV_SCOPED_TIMING_STAT(StateTree, Tick);
	TRACE_CPUPROFILER_EVENT_SCOPE(FStateTreeExecutionContext::Tick);
	UE_STATETREE_CRASH_REPORTER_SCOPE(&Owner, &RootStateTree, UE::StateTree::ExecutionContext::Private::Name_Tick.Resolve());
	SCOPE_CYCLE_UOBJECT(StateTree, &Owner);

	TGuardValue<bool> ScheduledNextTickScope(bAllowedToScheduleNextTick, false);

	const EStateTreeRunStatus PreludeResult = TickPrelude();
	if (PreludeResult != EStateTreeRunStatus::Running)
	{
		return PreludeResult;
	}

	TickUpdateTasksInternal(DeltaTime);
	TickTriggerTransitionsInternal();

	return TickPostlude();
}

EStateTreeRunStatus FStateTreeExecutionContext::TickUpdateTasks(const float DeltaTime)
{
	CSV_SCOPED_TIMING_STAT(StateTree, Tick);
	TRACE_CPUPROFILER_EVENT_SCOPE(FStateTreeExecutionContext::TickUpdateTasks);
	UE_STATETREE_CRASH_REPORTER_SCOPE(&Owner, &RootStateTree, UE::StateTree::ExecutionContext::Private::Name_Tick.Resolve());
	SCOPE_CYCLE_UOBJECT(StateTree, &Owner);

	TGuardValue<bool> ScheduledNextTickScope(bAllowedToScheduleNextTick, false);

	const EStateTreeRunStatus PreludeResult = TickPrelude();
	if (PreludeResult != EStateTreeRunStatus::Running)
	{
		return PreludeResult;
	}

	TickUpdateTasksInternal(DeltaTime);

	return TickPostlude();
}

EStateTreeRunStatus FStateTreeExecutionContext::TickTriggerTransitions()
{
	CSV_SCOPED_TIMING_STAT(StateTree, Tick);
	TRACE_CPUPROFILER_EVENT_SCOPE(FStateTreeExecutionContext::TickTriggerTransitions);
	UE_STATETREE_CRASH_REPORTER_SCOPE(&Owner, &RootStateTree, UE::StateTree::ExecutionContext::Private::Name_Tick.Resolve());
	SCOPE_CYCLE_UOBJECT(StateTree, &Owner);

	TGuardValue<bool> ScheduledNextTickScope(bAllowedToScheduleNextTick, false);

	const EStateTreeRunStatus PreludeResult = TickPrelude();
	if (PreludeResult != EStateTreeRunStatus::Running)
	{
		return PreludeResult;
	}

	TickTriggerTransitionsInternal();

	return TickPostlude();
}

void FStateTreeExecutionContext::TickUpdateTasksInternal(float DeltaTime)
{
	FStateTreeExecutionState& Exec = GetExecState();

	// If stop is requested, do not try to tick tasks.
	if (Exec.RequestedStop != EStateTreeRunStatus::Unset)
	{
		return;
	}

	// Prevent wrong user input.
	DeltaTime = FMath::Max(0.f, DeltaTime);

	// Update the delayed transitions.
	for (FStateTreeTransitionDelayedState& DelayedState : Exec.DelayedTransitions)
	{
		DelayedState.TimeLeft -= DeltaTime;
	}

	const EStateTreeRunStatus PreviousTickStatus = Exec.LastTickStatus;
	auto LogRequestStop = [&Exec, this]()
		{
			if (Exec.RequestedStop != EStateTreeRunStatus::Unset) // -V547
			{
				UE_STATETREE_DEBUG_LOG_EVENT(this, Log, TEXT("Global tasks completed (%s), stopping the tree"), *UEnum::GetDisplayValueAsText(Exec.RequestedStop).ToString());
				STATETREE_LOG(Log, TEXT("Global tasks completed (%s), stopping the tree"), *UEnum::GetDisplayValueAsText(Exec.RequestedStop).ToString());
			}
		};
	auto TickTaskLogic = [&Exec, &LogRequestStop, PreviousTickStatus, this](float DeltaTime)
		{
			// Tick tasks on active states.
			Exec.LastTickStatus = TickTasks(DeltaTime);
			// Report state completed immediately (and no global task completes)
			if (Exec.LastTickStatus != EStateTreeRunStatus::Running && Exec.RequestedStop == EStateTreeRunStatus::Unset && PreviousTickStatus == EStateTreeRunStatus::Running)
			{
				StateCompleted();
			}

			LogRequestStop();
		};

	if (UE::StateTree::ExecutionContext::Private::bTickGlobalNodesFollowingTreeHierarchy)
	{
		TickTaskLogic(DeltaTime);
	}
	else
	{
		// Tick global evaluators and tasks.
		const bool bTickGlobalTasks = true;
		const EStateTreeRunStatus EvalAndGlobalTaskStatus = TickEvaluatorsAndGlobalTasks(DeltaTime, bTickGlobalTasks);
		if (EvalAndGlobalTaskStatus == EStateTreeRunStatus::Running)
		{
			if (Exec.LastTickStatus == EStateTreeRunStatus::Running)
			{
				TickTaskLogic(DeltaTime);
			}
		}
		else
		{
			using namespace UE::StateTree;
			if (ExecutionContext::Private::bGlobalTasksCompleteOwningFrame)
			{
				// Only set RequestStop if it's the first frame (root)
				if (Exec.ActiveFrames.Num() > 0)
				{
					const UStateTree* StateTree = Exec.ActiveFrames[0].StateTree;
					check(StateTree == &RootStateTree);
					const ETaskCompletionStatus GlobalTaskStatus = Exec.ActiveFrames[0].ActiveTasksStatus.GetStatus(StateTree).GetCompletionStatus();
					const EStateTreeRunStatus GlobalRunStatus = ExecutionContext::CastToRunStatus(GlobalTaskStatus);
					if (GlobalRunStatus != EStateTreeRunStatus::Running)
					{
						Exec.RequestedStop = ExecutionContext::GetPriorityRunStatus(Exec.RequestedStop, GlobalRunStatus);
						LogRequestStop();
					}
				}
				else
				{
					// Start failed and the user called Tick anyway.
					Exec.RequestedStop = EStateTreeRunStatus::Failed;
					LogRequestStop();
				}
			}
			else
			{
				// Any completion stops the tree execution.
				Exec.RequestedStop = ExecutionContext::GetPriorityRunStatus(Exec.RequestedStop, EvalAndGlobalTaskStatus);
				LogRequestStop();
			}
		}
	}
}

void FStateTreeExecutionContext::TickTriggerTransitionsInternal()
{
	UE_STATETREE_DEBUG_SCOPED_PHASE(this, EStateTreeUpdatePhase::TickTransitions);

	FStateTreeExecutionState& Exec = GetExecState();

	// If stop is requested, do not try to trigger transitions.
	if (Exec.RequestedStop != EStateTreeRunStatus::Unset)
	{
		return;
	}

	// The state selection is repeated up to MaxIteration time. This allows failed EnterState() to potentially find a new state immediately.
	// This helps event driven StateTrees to not require another event/tick to find a suitable state.
	static constexpr int32 MaxIterations = 5;
	for (int32 Iter = 0; Iter < MaxIterations; Iter++)
	{
		ON_SCOPE_EXIT{ InstanceData.ResetTemporaryInstances(); };

		if (TriggerTransitions())
		{
			check(RequestedTransition.IsValid());
			UE_STATETREE_DEBUG_SCOPED_PHASE(this, EStateTreeUpdatePhase::ApplyTransitions);
			UE_STATETREE_DEBUG_TRANSITION_EVENT(this, RequestedTransition->Source, EStateTreeTraceEventType::OnTransition);

			BeginApplyTransition(RequestedTransition->Transition);

			ExitState(RequestedTransition->Selection, RequestedTransition->Transition);

			// Tree succeeded or failed.
			if (RequestedTransition->Transition.TargetState.IsCompletionState())
			{
				// Transition to a terminal state (succeeded/failed), or default transition failed.
				Exec.TreeRunStatus = RequestedTransition->Transition.TargetState.ToCompletionStatus();

				// Stop evaluators and global tasks (handled in ExitState)
				if (!ensure(Exec.ActiveFrames.Num() == 0))
				{
					StopEvaluatorsAndGlobalTasks(Exec.TreeRunStatus);

					// No active states or global tasks anymore, reset frames.
					Exec.ActiveFrames.Reset();
					RemoveAllDelegateListeners();
				}

				break;
			}

			// Enter state tasks can fail/succeed, treat it same as tick.
			const EStateTreeRunStatus LastTickStatus = EnterState(RequestedTransition->Selection, RequestedTransition->Transition);

			RequestedTransition.Reset();

			Exec.LastTickStatus = LastTickStatus;

			// Report state completed immediately.
			if (Exec.LastTickStatus != EStateTreeRunStatus::Running)
			{
				StateCompleted();
			}
		}

		// Stop as soon as have found a running state.
		if (Exec.LastTickStatus == EStateTreeRunStatus::Running && !Storage.HasBroadcastedDelegates())
		{
			break;
		}
	}
}

void FStateTreeExecutionContext::BeginApplyTransition(const FStateTreeTransitionResult& InTransitionResult)
{
	if (FStateTreeExecutionExtension* ExecExtension = GetMutableExecutionExtension())
	{
		ExecExtension->OnBeginApplyTransition({ Owner, RootStateTree, Storage }, InTransitionResult);
	}
}

void FStateTreeExecutionContext::BroadcastDelegate(const FStateTreeDelegateDispatcher& Dispatcher)
{
	if (!Dispatcher.IsValid())
	{
		return;
	}

	if (!IsValid())
	{
		STATETREE_LOG(Warning, TEXT("%hs: StateTree context is not initialized properly ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		return;
	}

	const FStateTreeExecutionFrame* CurrentFrame = GetCurrentlyProcessedFrame();
	check(CurrentFrame);

	GetExecState().DelegateActiveListeners.BroadcastDelegate(Dispatcher, GetExecState());
	if (UE::StateTree::ExecutionContext::MarkDelegateAsBroadcasted(Dispatcher, *CurrentFrame, GetMutableInstanceData()->GetMutableStorage()))
	{
		ScheduleNextTick(UE::StateTree::ETickReason::Delegate);
	}
}

namespace UE::StateTree::ExecutionContext
{
	FTaskCompletionDelegateDispatcherContainer GetTaskCompletionDispatcher(TNotNull<const UStateTree*> StateTree, int32 NodeIndex, UE::StateTree::ETaskCompletionStatus TaskStatus)
	{
		using namespace UE::StateTree;

		// FTaskCompletionDelegateDispatcherContainer is of size 3 by default.
		//There is a max of 3 dispatchers per node (Completes, Succeeds, Fails).

		if (TaskStatus == ETaskCompletionStatus::Running)
		{
			ensureMsgf(false, TEXT("Task completion dispatchers should only be searched for completed tasks."));
			return {};
		}

		FTaskCompletionDelegateDispatcherContainer Result;
		for (const FTaskCompletionDispatcher& TaskCompletionDispatcher : StateTree->GetTaskCompletionDispatchers())
		{
			if (TaskCompletionDispatcher.TaskNodeIndex.Get() == NodeIndex)
			{
				const bool bConditionMatches = (TaskCompletionDispatcher.Condition == ETaskCompletionCondition::Completes)
					|| (TaskCompletionDispatcher.Condition == ETaskCompletionCondition::Succeeds && TaskStatus == ETaskCompletionStatus::Succeeded)
					|| (TaskCompletionDispatcher.Condition == ETaskCompletionCondition::Fails && TaskStatus == ETaskCompletionStatus::Failed)
					|| (TaskCompletionDispatcher.Condition == ETaskCompletionCondition::Fails && TaskStatus == ETaskCompletionStatus::Stopped);
				if (bConditionMatches)
				{
					Result.Add(TaskCompletionDispatcher.Dispatcher);
				}
			}
		}

		return Result;
	}
}

void FStateTreeExecutionContext::BroadcastTaskCompletionDispatchers(TNotNull<const UStateTree*> StateTree, int32 NodeIndex, UE::StateTree::ETaskCompletionStatus TaskStatus)
{
	using namespace UE::StateTree::ExecutionContext;

	const FTaskCompletionDelegateDispatcherContainer Dispatchers = GetTaskCompletionDispatcher(StateTree, NodeIndex, TaskStatus);
	for (const FStateTreeDelegateDispatcher& Dispatcher : Dispatchers)
	{
		BroadcastDelegate(Dispatcher);
	}
}

// Deprecated
bool FStateTreeExecutionContext::AddDelegateListener(const FStateTreeDelegateListener& Listener, FSimpleDelegate Delegate)
{
	BindDelegate(Listener, MoveTemp(Delegate));
	return true;
}

void FStateTreeExecutionContext::BindDelegate(const FStateTreeDelegateListener& Listener, FSimpleDelegate Delegate)
{
	if (!Listener.IsValid())
	{
		// The listener is not bound to a dispatcher. It will never trigger the delegate.
		return;
	}

	if (!IsValid())
	{
		STATETREE_LOG(Warning, TEXT("%hs: StateTree context is not initialized properly ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		return;
	}

	const FStateTreeExecutionFrame* CurrentFrame = GetCurrentlyProcessedFrame();
	if (CurrentFrame == nullptr)
	{
		return;
	}

	const UE::StateTree::FActiveStateID CurrentlyProcessedStateID = CurrentFrame->ActiveStates.FindStateID(CurrentlyProcessedState);
	GetExecState().DelegateActiveListeners.Add(Listener, MoveTemp(Delegate), CurrentFrame->FrameID, CurrentlyProcessedStateID, FStateTreeIndex16(CurrentNodeIndex));
}

// Deprecated
void FStateTreeExecutionContext::RemoveDelegateListener(const FStateTreeDelegateListener& Listener)
{
	UnbindDelegate(Listener);
}

void FStateTreeExecutionContext::UnbindDelegate(const FStateTreeDelegateListener& Listener)
{
	if (!Listener.IsValid())
	{
		return;
	}

	if (!IsValid())
	{
		STATETREE_LOG(Warning, TEXT("%hs: StateTree context is not initialized properly ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		return;
	}

	GetExecState().DelegateActiveListeners.Remove(Listener);
}

void FStateTreeExecutionContext::ConsumeEvent(const FStateTreeSharedEvent& Event)
{
	if (EventQueue && EventQueue->ConsumeEvent(Event))
	{
		UE_STATETREE_DEBUG_EVENT_CONSUMED(this, Event);
	}
}

void FStateTreeExecutionContext::RequestTransition(const FStateTreeTransitionRequest& Request)
{
	using namespace UE::StateTree;
	using namespace UE::StateTree::ExecutionContext;

	CSV_SCOPED_TIMING_STAT(StateTree, RequestTransition);

	if (!IsValid())
	{
		STATETREE_LOG(Warning, TEXT("%hs: StateTree context is not initialized properly ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		return;
	}

	FStateTreeExecutionState& Exec = GetExecState();

	if (bAllowDirectTransitions)
	{
		checkf(CurrentlyProcessedFrame, TEXT("Expecting CurrentlyProcessedFrame to be valid when called during TriggerTransitions()."));

		STATETREE_LOG(Verbose, TEXT("Request transition to '%s' at priority %s"), *GetSafeStateName(*CurrentlyProcessedFrame, Request.TargetState), *UEnum::GetDisplayValueAsText(Request.Priority).ToString());

		const FActiveStateID CurrentlyProcessedStateID = CurrentlyProcessedFrame->ActiveStates.FindStateID(CurrentlyProcessedState);
		const FStateHandleContext Target = FStateHandleContext(CurrentlyProcessedFrame->StateTree, Request.TargetState);
		const bool bRequested = RequestTransitionInternal(*CurrentlyProcessedFrame, CurrentlyProcessedStateID, Target, FTransitionArguments{ .Priority = Request.Priority, .Fallback = Request.Fallback });
		if (bRequested)
		{
			check(RequestedTransition.IsValid());
			RequestedTransition->Source = FStateTreeTransitionSource(CurrentlyProcessedFrame->StateTree, EStateTreeTransitionSourceType::ExternalRequest, Request.TargetState, Request.Priority);
		}
	}
	else if (Exec.ActiveFrames.Num() > 0)
	{
		const FStateTreeExecutionFrame* RootFrame = &Exec.ActiveFrames[0];
		if (CurrentlyProcessedFrame)
		{
			RootFrame = CurrentlyProcessedFrame;
		}

		if (RootFrame->ActiveStates.Num() == 0)
		{
			STATETREE_LOG(Warning, TEXT("%hs: RequestTransition called on %s using StateTree %s without an active state. Start() must be called before requesting a transition."),
				__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
			return;
		}

		STATETREE_LOG(Verbose, TEXT("Request transition to '%s' at priority %s"), *GetSafeStateName(*RootFrame, Request.TargetState), *UEnum::GetDisplayValueAsText(Request.Priority).ToString());

		FStateTreeTransitionRequest RequestWithSource = Request;
		RequestWithSource.SourceFrameID = RootFrame->FrameID;

		const int32 ActiveStateIndex = RootFrame->ActiveStates.IndexOfReverse(CurrentlyProcessedState);
		RequestWithSource.SourceStateID = ActiveStateIndex != INDEX_NONE ? RootFrame->ActiveStates.StateIDs[ActiveStateIndex] : RootFrame->ActiveStates.StateIDs[0];

		InstanceData.AddTransitionRequest(&Owner, RequestWithSource);
		ScheduleNextTick(UE::StateTree::ETickReason::TransitionRequest);
	}
	else
	{
		STATETREE_LOG(Warning, TEXT("%hs: RequestTransition called on %s using StateTree %s without an active frame. Start() must be called before requesting a transition."),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		return;
	}
}

void FStateTreeExecutionContext::RequestTransition(FStateTreeStateHandle InTargetState, EStateTreeTransitionPriority InPriority, EStateTreeSelectionFallback InFallback)
{
	RequestTransition(FStateTreeTransitionRequest(InTargetState, InPriority, InFallback));
}

void FStateTreeExecutionContext::FinishTask(const FStateTreeTaskBase& Task, EStateTreeFinishTaskType FinishType)
{
	using namespace UE::StateTree;

	if (!IsValid())
	{
		STATETREE_LOG(Warning, TEXT("%hs: StateTree context is not initialized properly ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		return;
	}

	// Like GetInstanceData, only accept task if we are currently processing.
	if (!ensure(CurrentNode == &Task))
	{
		return;
	}
	check(CurrentlyProcessedFrame);
	check(CurrentNodeIndex >= 0);

	FStateTreeExecutionState& Exec = GetExecState();

	const UStateTree* CurrentStateTree = CurrentlyProcessedFrame->StateTree;
	const ETaskCompletionStatus TaskStatus = ExecutionContext::CastToTaskStatus(FinishType);

	if (CurrentlyProcessedState.IsValid())
	{
		check(CurrentStateTree->States.IsValidIndex(CurrentlyProcessedState.Index));
		const FCompactStateTreeState& State = CurrentStateTree->States[CurrentlyProcessedState.Index];

		const int32 ActiveStateIndex = CurrentlyProcessedFrame->ActiveStates.IndexOfReverse(CurrentlyProcessedState);
		check(ActiveStateIndex != INDEX_NONE);

		const int32 StateTaskIndex = CurrentNodeIndex - State.TasksBegin;
		check(StateTaskIndex >= 0);

		FTasksCompletionStatus StateTasksStatus = const_cast<FStateTreeExecutionFrame*>(CurrentlyProcessedFrame)->ActiveTasksStatus.GetStatus(State);
		const FTasksCompletionStatus::FSetStatusResult StateStatusResult = StateTasksStatus.SetTaskStatusWithPriority(StateTaskIndex, TaskStatus);
		Exec.bHasPendingCompletedState = Exec.bHasPendingCompletedState || StateTasksStatus.IsCompleted();

		// Broadcast task completion dispatcher only if status actually changed
		if (StateStatusResult.bChanged && Task.bHasTaskCompletionDelegateDispatcher)
		{
			BroadcastTaskCompletionDispatchers(CurrentStateTree, CurrentNodeIndex, StateStatusResult.CurrentStatus);
		}
	}
	else
	{
		// global frame
		const int32 FrameTaskIndex = CurrentNodeIndex - CurrentStateTree->GlobalTasksBegin;
		check(FrameTaskIndex >= 0);
		FTasksCompletionStatus GlobalTasksStatus = const_cast<FStateTreeExecutionFrame*>(CurrentlyProcessedFrame)->ActiveTasksStatus.GetStatus(CurrentStateTree);
		FTasksCompletionStatus::FSetStatusResult GlobalStatusResult = GlobalTasksStatus.SetTaskStatusWithPriority(FrameTaskIndex, TaskStatus);
		Exec.bHasPendingCompletedState = Exec.bHasPendingCompletedState || GlobalTasksStatus.IsCompleted();

		// Broadcast task completion dispatcher only if status actually changed
		if (GlobalStatusResult.bChanged && Task.bHasTaskCompletionDelegateDispatcher)
		{
			BroadcastTaskCompletionDispatchers(CurrentStateTree, CurrentNodeIndex, GlobalStatusResult.CurrentStatus);
		}
	}
}

// Deprecated
PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FStateTreeExecutionContext::FinishTask(const UE::StateTree::FFinishedTask& Task, EStateTreeFinishTaskType FinishType)
{
	FStateTreeExecutionState& Exec = GetExecState();
	FStateTreeExecutionFrame* Frame = Exec.FindActiveFrame(Task.FrameID);
	if (Frame == nullptr)
	{
		return;
	}

	using namespace UE::StateTree;

	const UE::StateTree::ETaskCompletionStatus Status = ExecutionContext::CastToTaskStatus(Task.RunStatus);
	if (Task.Reason == FFinishedTask::EReasonType::GlobalTask)
	{
		if (Frame->bIsGlobalFrame)
		{
			Frame->ActiveTasksStatus.GetStatus(Frame->StateTree).SetStatusWithPriority(Task.TaskIndex.AsInt32(), Status);
		}
	}
	else
	{
		const int32 FoundIndex = Frame->ActiveStates.IndexOfReverse(Task.StateID);
		if (FoundIndex != INDEX_NONE)
		{
			const FStateTreeStateHandle StateHandle = Frame->ActiveStates[FoundIndex];
			const FCompactStateTreeState* State = Frame->StateTree->GetStateFromHandle(StateHandle);
			if (State != nullptr)
			{
				if (Task.Reason == FFinishedTask::EReasonType::InternalTransition)
				{
					Frame->ActiveTasksStatus.GetStatus(*State).SetCompletionStatus(Status);
				}
				else
				{
					check(Task.Reason == FFinishedTask::EReasonType::StateTask);
					Frame->ActiveTasksStatus.GetStatus(*State).SetStatusWithPriority(Task.TaskIndex.AsInt32(), Status);
				}
			}
		}
	}
}

// Deprecated
bool FStateTreeExecutionContext::IsFinishedTaskValid(const UE::StateTree::FFinishedTask& Task) const
{
	return false;
}

// Deprecated
void FStateTreeExecutionContext::UpdateCompletedStateList()
{
}

// Deprecated
void FStateTreeExecutionContext::MarkStateCompleted(UE::StateTree::FFinishedTask& NewFinishedTask)
{
}

// Deprecated
void FStateTreeExecutionContext::UpdateInstanceData(TConstArrayView<FStateTreeExecutionFrame> CurrentActiveFrames, TArrayView<FStateTreeExecutionFrame> NextActiveFrames)
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FStateTreeExecutionContext::UpdateInstanceData(const TSharedPtr<FSelectStateResult>& Args)
{
	// Go though all the state and frame (includes unchanged, sustained, changed).
	//Keep 2 buffers. InstanceStructs for unchanged and sustained. TempInstanceStructs for changed.
	//1. Set the frame GlobalParameterDataHandle, GlobalInstanceIndexBase and ActiveInstanceIndexBase
	//2. Add instances for global frame.
	// Note that the global parameters are in a different buffer or it's last state parameter or the previous frame (see Start, SelectStateInternal_Linked, SelectStateInternal_LinkedAsset).
	//3. Add the instances for state parameters (Set the state StateParameterDataHandle).
	//4. Add instances for state tasks.
	//5. Remove the previous instances (that are not needed) and copy/move the new TempInstanceStructs.

	using namespace UE::StateTree;
	using namespace UE::StateTree::ExecutionContext;
	using namespace UE::StateTree::ExecutionContext::Private;

	FStateTreeExecutionState& Exec = GetExecState();

	// Estimate how many new instance data items we might have.
	int32 EstimatedNumStructs = 0;
	if (Args)
	{
		FActiveFrameID CurrentFrameID;
		const UStateTree* CurrentStateTree = nullptr;
		for (const FActiveState& SelectedState : Args->SelectedStates)
		{
			// Global
			if (SelectedState.GetFrameID() != CurrentFrameID)
			{
				CurrentFrameID = SelectedState.GetFrameID();

				const FStateTreeExecutionFrame* Frame = FindExecutionFrame(CurrentFrameID, MakeConstArrayView(Exec.ActiveFrames), MakeConstArrayView(Args->TemporaryFrames));
				if (ensure(Frame) && Frame->bIsGlobalFrame)
				{
					check(Frame->StateTree);

					CurrentStateTree = Frame->StateTree;
					EstimatedNumStructs += CurrentStateTree->NumGlobalInstanceData;
				}
			}

			// State
			if (ensure(CurrentStateTree))
			{
				const FCompactStateTreeState* CurrentState = CurrentStateTree->GetStateFromHandle(SelectedState.GetStateHandle());
				if (ensure(CurrentState))
				{
					EstimatedNumStructs += CurrentState->InstanceDataNum;
				}
			}
		}
	}

	TArray<FConstStructView, FNonconcurrentLinearArrayAllocator> InstanceStructs;
	InstanceStructs.Reserve(EstimatedNumStructs);

	// Temporaries that are moved to the new instance data buffer.
	// Node instance data are moved in EnterState when the ActiveNodeIndex is incremented.
	// The array is clear at the end of EnterState.
	TArray<TOptional<FInstancedStruct*>, FNonconcurrentLinearArrayAllocator> TemporaryInstanceStructs;
	TemporaryInstanceStructs.Reserve(EstimatedNumStructs);

	using FTemporaryInstanceKey = TTuple<FActiveFrameID, FStateTreeIndex16, FStateTreeDataHandle>;
	TArray<FTemporaryInstanceKey, TInlineAllocator<8, FNonconcurrentLinearArrayAllocator>> TemporaryInstanceStructsToConsume;

	constexpr int32 ExpectedAmountOfFrames = 8;
	TArray<FCompactStateTreeParameters, TInlineAllocator<ExpectedAmountOfFrames, FNonconcurrentLinearArrayAllocator>> TempParams;

	TArrayView<FStateTreeTemporaryInstanceData> TemporaryInstances = Storage.GetMutableTemporaryInstances();
	auto FindInstanceTempData = [&TemporaryInstances](FActiveFrameID FrameID, FStateTreeIndex16 OwnerNodeIndex, FStateTreeDataHandle DataHandle)
		{
			FStateTreeTemporaryInstanceData* TempData = TemporaryInstances.FindByPredicate([&FrameID, &OwnerNodeIndex, &DataHandle](const FStateTreeTemporaryInstanceData& Data)
				{
					return Data.FrameID == FrameID
						&& Data.OwnerNodeIndex == OwnerNodeIndex
						&& Data.DataHandle == DataHandle;
				});
				return TempData ? &TempData->Instance : nullptr;
		};

	auto ConsumeTemporaryInstanceData = [&FindInstanceTempData, &TemporaryInstanceStructsToConsume](FActiveFrameID FrameID, FStateTreeIndex16 OwnerNodeIndex, FStateTreeDataHandle DataHandle)
		{
			FInstancedStruct* FoundInstance = FindInstanceTempData(FrameID, OwnerNodeIndex, DataHandle);
			if (FoundInstance)
			{
				TemporaryInstanceStructsToConsume.Emplace(FrameID, OwnerNodeIndex, DataHandle);
			}
			return FoundInstance;
		};

	constexpr TOptional<FInstancedStruct*> NoExistingTemporaryDuplicateWrapper = TOptional<FInstancedStruct*>(nullptr);
	constexpr TOptional<FInstancedStruct*> TemporaryToBeMovedLater = TOptional<FInstancedStruct*>();

	// Find next instance data sources and find common/existing section of instance data at start.
	int32 CurrentGlobalInstanceIndexBase = 0;
	int32 NumCommonInstanceData = 0;

	const UStruct* NextStateParameterDataStruct = nullptr;
	FStateTreeDataHandle NextStateParameterDataHandle = FStateTreeDataHandle::Invalid;

	// start from Root Global Parameter. Update to state parameter if we encounter any linked asset state.
	FStateTreeIndex16 CurrentGlobalParameterIndexBase = FStateTreeIndex16::Invalid;
	if (Args)
	{
		FActiveFrameID CurrentFrameID;
		FStateTreeExecutionFrame* CurrentFrame = nullptr;
		const UStateTree* CurrentStateTree = nullptr;
		int32 CurrentFrameBaseIndex = 0;
		for (int32 SelectedStateIndex = 0; SelectedStateIndex < Args->SelectedStates.Num(); ++SelectedStateIndex)
		{
			const FActiveState& SelectedState = Args->SelectedStates[SelectedStateIndex];
			const FStateTreeExecutionFrame* ParentFrame = nullptr;
			// Global
			if (SelectedState.GetFrameID() != CurrentFrameID)
			{
				ParentFrame = FindExecutionFrame(CurrentFrameID, MakeConstArrayView(Exec.ActiveFrames), MakeConstArrayView(Args->TemporaryFrames));
				CurrentFrameID = SelectedState.GetFrameID();
				CurrentFrame = Exec.FindActiveFrame(CurrentFrameID);

				// The frame/globals are common (before the transition or sustained from the transition)
				const bool bIsFrameCommon = CurrentFrame != nullptr;

				if (CurrentFrame == nullptr)
				{
					CurrentFrame = Args->FindTemporaryFrame(SelectedState.GetFrameID());
				}
				check(CurrentFrame);
				CurrentStateTree = CurrentFrame->StateTree;

				// Global Nodes
				if (CurrentFrame->bIsGlobalFrame)
				{
					// Handle global tree parameters
					if (NextStateParameterDataHandle.IsValid())
					{
						// Point to the parameter block set by linked state.
						check(NextStateParameterDataStruct == CurrentStateTree->GetDefaultParameters().GetPropertyBagStruct());
						NextStateParameterDataHandle = FStateTreeDataHandle::Invalid; // Mark as used.

						check(ParentFrame);
						CurrentGlobalParameterIndexBase = ParentFrame->ActiveInstanceIndexBase;
					}

					const int32 BaseIndex = InstanceStructs.Num();
					CurrentGlobalInstanceIndexBase = BaseIndex;

					InstanceStructs.AddDefaulted(CurrentStateTree->NumGlobalInstanceData);
					TemporaryInstanceStructs.AddDefaulted(CurrentStateTree->NumGlobalInstanceData);

					// Global Evals
					for (int32 EvalIndex = CurrentStateTree->EvaluatorsBegin; EvalIndex < (CurrentStateTree->EvaluatorsBegin + CurrentStateTree->EvaluatorsNum); EvalIndex++)
					{
						const FStateTreeEvaluatorBase& Eval = CurrentStateTree->Nodes[EvalIndex].Get<const FStateTreeEvaluatorBase>();
						const FConstStructView EvalInstanceData = CurrentStateTree->DefaultInstanceData.GetStruct(Eval.InstanceTemplateIndex.Get());
						const int32 EvalInstanceDataIndex = BaseIndex + Eval.InstanceDataHandle.GetIndex();
						InstanceStructs[EvalInstanceDataIndex] = EvalInstanceData;
						TemporaryInstanceStructs[EvalInstanceDataIndex] = TemporaryToBeMovedLater;
					}

					// Global tasks
					for (int32 TaskIndex = CurrentStateTree->GlobalTasksBegin; TaskIndex < (CurrentStateTree->GlobalTasksBegin + CurrentStateTree->GlobalTasksNum); TaskIndex++)
					{
						const FStateTreeTaskBase& Task = CurrentStateTree->Nodes[TaskIndex].Get<const FStateTreeTaskBase>();
						const FConstStructView TaskInstanceData = CurrentStateTree->DefaultInstanceData.GetStruct(Task.InstanceTemplateIndex.Get());
						const int32 TaskInstanceDataIndex = BaseIndex + Task.InstanceDataHandle.GetIndex();
						InstanceStructs[TaskInstanceDataIndex] = TaskInstanceData;
						TemporaryInstanceStructs[TaskInstanceDataIndex] = TemporaryToBeMovedLater;
					}

					if (bIsFrameCommon)
					{
						NumCommonInstanceData = InstanceStructs.Num();
					}
				}

				CurrentFrameBaseIndex = InstanceStructs.Num();

				CurrentFrame->ActiveInstanceIndexBase = FStateTreeIndex16(CurrentFrameBaseIndex);

				CurrentFrame->GlobalInstanceIndexBase = FStateTreeIndex16(CurrentGlobalInstanceIndexBase);
				CurrentFrame->GlobalParameterIndexBase = CurrentGlobalParameterIndexBase;
				CurrentFrame->StateParameterIndexBase = CurrentFrame->ActiveInstanceIndexBase;
			}

			// State Params and Nodes
			if (ensure(CurrentStateTree))
			{
				const FStateTreeStateHandle CurrentStateHandle = SelectedState.GetStateHandle();
				const FCompactStateTreeState* CurrentState = CurrentStateTree->GetStateFromHandle(CurrentStateHandle);
				if (ensure(CurrentState))
				{
					// Find if the state is common (before the transition or sustained from the transition).
					//The frame may contain valid states (sustained or not). SelectedState will be added later.
					const bool bIsStateCommon = CurrentFrame->ActiveStates.Contains(SelectedState.GetStateID());

					InstanceStructs.AddDefaulted(CurrentState->InstanceDataNum);
					TemporaryInstanceStructs.AddDefaulted(CurrentState->InstanceDataNum);

					const int32 CurrentStateParameterInstanceDataIndex = CurrentFrameBaseIndex + CurrentState->ParameterDataHandle.GetIndex();

					bool bCanHaveTempData = false;
					if (CurrentState->Type == EStateTreeStateType::Subtree)
					{
						check(CurrentState->ParameterDataHandle.IsValid());
						check(CurrentState->ParameterTemplateIndex.IsValid());
						const FConstStructView ParamsInstanceData = CurrentStateTree->DefaultInstanceData.GetStruct(CurrentState->ParameterTemplateIndex.Get());
						if (!NextStateParameterDataHandle.IsValid())
						{
							// Parameters are not set by a linked state, create instance data.
							InstanceStructs[CurrentStateParameterInstanceDataIndex] = ParamsInstanceData;
							bCanHaveTempData = true;
						}
						else
						{
							// Point to the parameter block set by linked state.
							const FCompactStateTreeParameters* Params = ParamsInstanceData.GetPtr<const FCompactStateTreeParameters>();
							const UStruct* StateParameterDataStruct = Params ? Params->GetValue().GetScriptStruct() : nullptr;
							check(NextStateParameterDataStruct == StateParameterDataStruct);

							check(ParentFrame);
							CurrentFrame->StateParameterIndexBase = ParentFrame->ActiveInstanceIndexBase;

							NextStateParameterDataHandle = FStateTreeDataHandle::Invalid; // Mark as used.

							// This state will not instantiate parameter data, so we don't care about the temp data either.
							bCanHaveTempData = false;
						}
					}
					else
					{
						if (CurrentState->ParameterTemplateIndex.IsValid())
						{
							// Linked state's instance data is the parameters.
							check(CurrentState->ParameterDataHandle.IsValid());

							const FCompactStateTreeParameters* Params = nullptr;
							if (FInstancedStruct* TempParamsInstanceData = FindInstanceTempData(CurrentFrameID, FStateTreeIndex16::Invalid, CurrentState->ParameterDataHandle))
							{
								// If we have temp data for the parameters, then setup the instance data with just a type, so that we can steal the temp data below (TempInstanceStructs).
								// We expect overridden linked assets to hit this code path. 
								InstanceStructs[CurrentStateParameterInstanceDataIndex] = FConstStructView(TempParamsInstanceData->GetScriptStruct());
								Params = TempParamsInstanceData->GetPtr<const FCompactStateTreeParameters>();
								bCanHaveTempData = true;
							}
							else
							{
								// If not temp data, use the states or linked assets default values.
								FConstStructView ParamsInstanceData;
								if (CurrentState->Type == EStateTreeStateType::LinkedAsset)
								{
									// This state is a container for the linked state tree.
									// Its instance data matches the linked state tree parameters.
									// The linked state tree asset is the next frame.
									const bool bIsLastFrame = SelectedStateIndex == Args->SelectedStates.Num() - 1;
									if (!bIsLastFrame)
									{
										const FActiveState& FollowingSelectedState = Args->SelectedStates[SelectedStateIndex + 1];
										const FStateTreeExecutionFrame* FollowingNextFrame = UE::StateTree::ExecutionContext::FindExecutionFrame(FollowingSelectedState.GetFrameID(), MakeConstArrayView(Exec.ActiveFrames), MakeConstArrayView(Args->TemporaryFrames));
										if (ensure(FollowingNextFrame))
										{
											ParamsInstanceData = FConstStructView::Make(TempParams.Emplace_GetRef(FollowingNextFrame->StateTree->GetDefaultParameters()));
										}
									}
								}
								if (!ParamsInstanceData.IsValid())
								{
									ParamsInstanceData = CurrentStateTree->DefaultInstanceData.GetStruct(CurrentState->ParameterTemplateIndex.Get());
								}
								InstanceStructs[CurrentStateParameterInstanceDataIndex] = ParamsInstanceData;
								Params = ParamsInstanceData.GetPtr<const FCompactStateTreeParameters>();
								bCanHaveTempData = true;
							}

							if (CurrentState->Type == EStateTreeStateType::Linked
								|| CurrentState->Type == EStateTreeStateType::LinkedAsset)
							{
								// Store the index of the parameter data, so that we can point the linked state to it.
								check(CurrentState->ParameterDataHandle.GetSource() == EStateTreeDataSourceType::StateParameterData);
								checkf(!NextStateParameterDataHandle.IsValid(), TEXT("NextStateParameterDataIndex not should be set yet when we encounter a linked state."));
								NextStateParameterDataHandle = CurrentState->ParameterDataHandle;
								NextStateParameterDataStruct = Params ? Params->GetValue().GetScriptStruct() : nullptr;
							}
						}
					}

					if (!bIsStateCommon)
					{
						if (bCanHaveTempData)
						{
							TemporaryInstanceStructs[CurrentStateParameterInstanceDataIndex] = ConsumeTemporaryInstanceData(CurrentFrameID, FStateTreeIndex16::Invalid, CurrentState->ParameterDataHandle);
						}
						else
						{
							TemporaryInstanceStructs[CurrentStateParameterInstanceDataIndex] = NoExistingTemporaryDuplicateWrapper;
						}
					}

					if (CurrentState->EventDataIndex.IsValid())
					{
						const int32 EventDataInstanceDataIndex = CurrentFrameBaseIndex + CurrentState->EventDataIndex.Get();
						InstanceStructs[EventDataInstanceDataIndex] = FConstStructView(FStateTreeSharedEvent::StaticStruct());
						TemporaryInstanceStructs[EventDataInstanceDataIndex] = NoExistingTemporaryDuplicateWrapper;
					}

					for (int32 TaskIndex = CurrentState->TasksBegin; TaskIndex < (CurrentState->TasksBegin + CurrentState->TasksNum); TaskIndex++)
					{
						const FStateTreeTaskBase& Task = CurrentStateTree->Nodes[TaskIndex].Get<const FStateTreeTaskBase>();
						const FConstStructView TaskInstanceData = CurrentStateTree->DefaultInstanceData.GetStruct(Task.InstanceTemplateIndex.Get());
						const int32 TaskInstanceDataIndex = CurrentFrameBaseIndex + Task.InstanceDataHandle.GetIndex();
						InstanceStructs[TaskInstanceDataIndex] = TaskInstanceData;
						TemporaryInstanceStructs[TaskInstanceDataIndex] = NoExistingTemporaryDuplicateWrapper;
					}

					if (bIsStateCommon)
					{
						NumCommonInstanceData = InstanceStructs.Num();
					}
				}
			}
		}
	}

	// Common section should match.
#if WITH_STATETREE_DEBUG
	for (int32 Index = 0; Index < NumCommonInstanceData; ++Index)
	{
		check(Index < InstanceData.Num());

		FConstStructView ExistingInstanceDataView = InstanceData.GetStruct(Index);
		FConstStructView NewInstanceDataView = InstanceStructs[Index];

		check(NewInstanceDataView.GetScriptStruct() == ExistingInstanceDataView.GetScriptStruct());

		const FStateTreeInstanceObjectWrapper* ExistingWrapper = ExistingInstanceDataView.GetPtr<const FStateTreeInstanceObjectWrapper>();
		const FStateTreeInstanceObjectWrapper* NewWrapper = ExistingInstanceDataView.GetPtr<const FStateTreeInstanceObjectWrapper>();
		if (ExistingWrapper && NewWrapper)
		{
			check(ExistingWrapper->InstanceObject && NewWrapper->InstanceObject);
			check(ExistingWrapper->InstanceObject->GetClass() == NewWrapper->InstanceObject->GetClass());
		}
	}
#endif

	// Remove instance data that was not common.
	InstanceData.ShrinkTo(NumCommonInstanceData);

	// Add new instance data.
	InstanceData.Append(Owner,
		MakeArrayView(InstanceStructs.GetData() + NumCommonInstanceData, InstanceStructs.Num() - NumCommonInstanceData),
		MakeArrayView(TemporaryInstanceStructs.GetData() + NumCommonInstanceData, TemporaryInstanceStructs.Num() - NumCommonInstanceData));

	// Remove consumed temporary instances. We need to postpone it after the Append because it uses a direct pointer to the memory address.
	for (const FTemporaryInstanceKey& Key : TemporaryInstanceStructsToConsume)
	{
		Storage.RemoveTemporaryInstance(Key.Get<0>(), Key.Get<1>(), Key.Get<2>());
	}
}

FStateTreeDataView FStateTreeExecutionContext::MoveTemporaryToInstance(const FStateTreeExecutionFrame& CurrentFrame, const FStateTreeIndex16 NodeIndex, const FStateTreeDataHandle DataHandle)
{
	check(DataHandle.GetSource() == EStateTreeDataSourceType::ActiveInstanceData
		|| DataHandle.GetSource() == EStateTreeDataSourceType::ActiveInstanceDataObject
		|| DataHandle.GetSource() == EStateTreeDataSourceType::GlobalInstanceData
		|| DataHandle.GetSource() == EStateTreeDataSourceType::GlobalInstanceDataObject);

	using namespace UE::StateTree;
	using namespace UE::StateTree::ExecutionContext;
	using namespace UE::StateTree::ExecutionContext::Private;

	TArrayView<FStateTreeTemporaryInstanceData> TempInstances = Storage.GetMutableTemporaryInstances();
	auto FindInstanceTempData = [&TempInstances](FActiveFrameID FrameID, FStateTreeDataHandle DataHandle)
		{
			FStateTreeTemporaryInstanceData* TempData = TempInstances.FindByPredicate([&FrameID, &DataHandle](const FStateTreeTemporaryInstanceData& Data)
				{
					return Data.FrameID == FrameID && Data.DataHandle == DataHandle;
				});
			return TempData ? &TempData->Instance : nullptr;
		};

	// Use ToStructSource to get the FStateTreeInstanceObjectWrapper
	const FStateTreeDataHandle StructDataHandle = DataHandle.ToStructSource();
	FStateTreeDataView DataView = UE::StateTree::InstanceData::GetDataView(Storage, nullptr, CurrentFrame, StructDataHandle);
	FInstancedStruct* TemporarySource = FindInstanceTempData(CurrentFrame.FrameID, DataHandle);

	// If we found a temporary data, move it from the temporary to the instance data.
	if (DataView.IsValid())
	{
		// ToStructSource should have converted it to FStateTreeInstanceObjectWrapper
		check(DataView.GetStruct() && Cast<const UClass>(DataView.GetStruct()) == nullptr);

		if (TemporarySource)
		{
			if (ensure(TemporarySource->IsValid()))
			{
				check(DataView.GetStruct() == TemporarySource->GetScriptStruct());
				FMemory::Memswap(DataView.GetMutableMemory(), TemporarySource->GetMutableMemory(), DataView.GetStruct()->GetStructureSize());
				Storage.RemoveTemporaryInstance(CurrentFrame.FrameID, NodeIndex, DataHandle);
			}
		}
		else if (StructDataHandle != DataHandle)
		{
			// if there is no temporary, get the correct data from the input handle
			DataView = UE::StateTree::InstanceData::GetDataView(Storage, nullptr, CurrentFrame, DataHandle);
		}
	}

	return DataView;
}

FStateTreeDataView FStateTreeExecutionContext::GetDataView(const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& CurrentFrame, const FStateTreeDataHandle Handle)
{
	switch (Handle.GetSource())
	{
	case EStateTreeDataSourceType::ContextData:
		check(!ContextAndExternalDataViews.IsEmpty());
		return ContextAndExternalDataViews[Handle.GetIndex()];

	case EStateTreeDataSourceType::EvaluationScopeInstanceData:
	case EStateTreeDataSourceType::EvaluationScopeInstanceDataObject:
	{
		// The data can be accessed in any of the caches (depends on how the binding was constructed), but it is most likely on top of the stack.
		for (int32 Index = EvaluationScopeInstanceCaches.Num() - 1; Index >= 0; --Index)
		{
			if (EvaluationScopeInstanceCaches[Index].StateTree == CurrentFrame.StateTree)
			{
				if (FStructView* Result = EvaluationScopeInstanceCaches[Index].Container->GetDataViewPtr(Handle))
				{
					if (Handle.IsObjectSource())
					{
						const FStateTreeInstanceObjectWrapper& Wrapper = Result->Get<FStateTreeInstanceObjectWrapper>();
						return FStateTreeDataView(Wrapper.InstanceObject);
					};
					return FStateTreeDataView(*Result);
				}
			}
		}
		checkf(false, TEXT("The evaluation scope instance data needs to be constructed before you can access it."));
		return nullptr;
	}

	case EStateTreeDataSourceType::ExternalData:
		check(!ContextAndExternalDataViews.IsEmpty());
		return ContextAndExternalDataViews[CurrentFrame.ExternalDataBaseIndex.Get() + Handle.GetIndex()];

	case EStateTreeDataSourceType::TransitionEvent:
	{
		if (CurrentlyProcessedTransitionEvent)
		{
			// const_cast because events are read only, but we cannot express that in FStateTreeDataView.
			return FStateTreeDataView(FStructView::Make(*const_cast<FStateTreeEvent*>(CurrentlyProcessedTransitionEvent)));
		}

		return nullptr;
	}

	case EStateTreeDataSourceType::StateEvent:
	{
		// If state selection is going, return FStateTreeEvent of the event currently captured by the state selection.
		if (CurrentlyProcessedStateSelectionResult)
		{
			FSelectionEventWithID* FoundEvent = CurrentlyProcessedStateSelectionResult->SelectionEvents.FindByPredicate(
				[FrameID = CurrentFrame.FrameID, StateHandle = Handle.GetState()]
				(const FSelectionEventWithID& Event)
				{
					return Event.State.GetStateHandle() == StateHandle && Event.State.GetFrameID() == FrameID;
				});
			if (FoundEvent)
			{
				// Events are read only, but we cannot express that in FStateTreeDataView.
				return FStateTreeDataView(FStructView::Make(*FoundEvent->Event.GetMutable()));
			}
		}

		if (UE::StateTree::InstanceData::Private::IsActiveInstanceHandleSourceValid(Storage, CurrentFrame, Handle))
		{
			return UE::StateTree::InstanceData::GetDataView(Storage, CurrentlyProcessedSharedInstanceStorage, CurrentFrame, Handle);
		}
	}

	case EStateTreeDataSourceType::ExternalGlobalParameterData:
		checkf(false, TEXT("External global parameter data currently not supported for linked state-trees"));
		return nullptr;

	default:
		return UE::StateTree::InstanceData::GetDataView(Storage, CurrentlyProcessedSharedInstanceStorage, CurrentFrame, Handle);
	}
}

FStateTreeDataView FStateTreeExecutionContext::GetDataView(const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& CurrentFrame, const FPropertyBindingCopyInfo& CopyInfo)
{
	const FStateTreeDataHandle Handle = CopyInfo.SourceDataHandle.Get<FStateTreeDataHandle>();
	if (Handle.GetSource() == EStateTreeDataSourceType::ExternalGlobalParameterData)
	{
		return GetDataViewOrTemporary(ParentFrame, CurrentFrame, CopyInfo);
	}

	return GetDataView(ParentFrame, CurrentFrame, Handle);
}

FStateTreeDataView FStateTreeExecutionContext::GetExecutionRuntimeDataView() const
{
	check(CurrentNode && CurrentlyProcessedFrame);
	check(CurrentlyProcessedFrame->ExecutionRuntimeIndexBase.IsValid());

	if (!ensureMsgf(CurrentNode->ExecutionRuntimeTemplateIndex.IsValid(), TEXT("The node doesn't support execution runtime data.")))
	{
		return FStateTreeDataView();
	}

	const FStateTreeDataHandle DataHandle = FStateTreeDataHandle(EStateTreeDataSourceType::ExecutionRuntimeDataAny, CurrentNode->ExecutionRuntimeTemplateIndex.Get());
	return UE::StateTree::InstanceData::GetDataView(Storage, CurrentlyProcessedSharedInstanceStorage, *CurrentlyProcessedFrame, DataHandle);
}

EStateTreeRunStatus FStateTreeExecutionContext::ForceTransition(const FRecordedStateTreeTransitionResult& RecordedTransition)
{
	using namespace UE::StateTree;
	using namespace UE::StateTree::ExecutionContext;
	using namespace UE::StateTree::ExecutionContext::Private;

	TArray<FStateHandleContext, FNonconcurrentLinearArrayAllocator> StateContexts;
	StateContexts.Reserve(RecordedTransition.States.Num());
	for (const FRecordedActiveState& RecordedActiveState : RecordedTransition.States)
	{
		if (RecordedActiveState.StateTree == nullptr)
		{
			return EStateTreeRunStatus::Unset;
		}
		StateContexts.Add(FStateHandleContext(RecordedActiveState.StateTree, RecordedActiveState.State));
	}

	TSharedRef<FSelectStateResult> SelectStateResult = MakeShared<FSelectStateResult>();
	TGuardValue<TSharedPtr<FSelectStateResult>> TemporaryStorageScope(CurrentlyProcessedTemporaryStorage, SelectStateResult);
	if (!ForceTransitionInternal(StateContexts, SelectStateResult))
	{
		UE_STATETREE_DEBUG_LOG_EVENT(this, Verbose, TEXT("The force transition failed."));
		return EStateTreeRunStatus::Unset;
	}

	if (SelectStateResult->SelectedStates.Num() != StateContexts.Num())
	{
		UE_STATETREE_DEBUG_LOG_EVENT(this, Verbose, TEXT("The force transition failed to find the target."));
		return EStateTreeRunStatus::Unset;
	}

	TArray<FStateTreeSharedEvent, FNonconcurrentLinearArrayAllocator> SharedEvents;
	SharedEvents.Reserve(RecordedTransition.Events.Num());
	for (const FStateTreeEvent& SelectedEvent : RecordedTransition.Events)
	{
		SharedEvents.Add(FStateTreeSharedEvent(SelectedEvent));
	}
	check(RecordedTransition.Events.Num() == SharedEvents.Num());
	for (int32 RecordedStateIndex = 0; RecordedStateIndex < RecordedTransition.States.Num(); ++RecordedStateIndex)
	{
		const FRecordedActiveState& RecordedActiveState = RecordedTransition.States[RecordedStateIndex];
		if (SharedEvents.IsValidIndex(RecordedActiveState.EventIndex))
		{
			const FActiveState& SelectedState = SelectStateResult->SelectedStates[RecordedStateIndex];
			SelectStateResult->SelectionEvents.Add(FSelectionEventWithID{.State = SelectedState, .Event= SharedEvents[RecordedActiveState.EventIndex]});
		}
	}

	FStateTreeTransitionResult TransitionResult;
	// The SourceFrameID and SourceStateID are left uninitialized on purpose.
	TransitionResult.TargetState = StateContexts.Last().StateHandle;
	TransitionResult.CurrentState = FStateTreeStateHandle::Invalid;
	TransitionResult.CurrentRunStatus = GetExecState().LastTickStatus;
	TransitionResult.ChangeType = EStateTreeStateChangeType::Changed;
	TransitionResult.Priority = RecordedTransition.Priority;

	BeginApplyTransition(TransitionResult);

	ExitState(SelectStateResult, TransitionResult);
	return EnterState(SelectStateResult, TransitionResult);
}

namespace UE::StateTree::ExecutionContext::Private
{
	bool TestStateContextPath(TNotNull<const FStateTreeExecutionContext*> ExecutionContext, const TArrayView<const FStateHandleContext> StateContexts)
	{
		const UStateTree* PreviousStateTree = ExecutionContext->GetStateTree();
		bool bNewTree = true;
		bool bNewFrame = true;
		for (int32 Index = 0; Index < StateContexts.Num(); ++Index)
		{
			const FStateHandleContext& StateContext = StateContexts[Index];
			if (StateContext.StateTree == nullptr || StateContext.StateTree != PreviousStateTree)
			{
				// Child state has to have the same state tree.
				return false;
			}

			const FCompactStateTreeState* State = StateContext.StateTree->GetStateFromHandle(StateContext.StateHandle);
			if (State == nullptr)
			{
				// The handle do not exist in the asset.
				return false;
			}
			if (bNewFrame || bNewTree)
			{
				if (StateContext.StateTree->GetFrameFromHandle(StateContext.StateHandle) == nullptr)
				{
					// The state is a new frame (from a linked or linked asset) but is not compiled as a frame.
					return false;
				}
			}
			if (!bNewFrame)
			{
				if (State->Parent != StateContexts[Index - 1].StateHandle)
				{
					// Current state is not a child of previous state.
					return false;
				}
			}
			bNewFrame = false;
			bNewTree = false;

			if (State->Type == EStateTreeStateType::LinkedAsset)
			{
				bNewTree = true;
				bNewFrame = true;
				if (Index + 1 >= StateContexts.Num())
				{
					// The path cannot end with a linked asset state.
					return false;
				}
				const FStateHandleContext& NextStateContext = StateContexts[Index + 1];
				if (NextStateContext.StateTree == nullptr
					|| !NextStateContext.StateTree->IsReadyToRun()
					|| !NextStateContext.StateTree->HasCompatibleContextData(ExecutionContext->GetStateTree())
					|| NextStateContext.StateTree->GetSchema()->GetClass() != ExecutionContext->GetStateTree()->GetSchema()->GetClass())
				{
					// The trees have to be compatible.
					return false;
				}
				PreviousStateTree = NextStateContext.StateTree;
			}
			if (State->Type == EStateTreeStateType::Linked)
			{
				bNewFrame = true;
			}
		}
		if (bNewFrame || bNewTree)
		{
			// The path cannot end with a linked or linked asset state.
			return false;
		}

		return true;
	}
}

bool FStateTreeExecutionContext::ForceTransitionInternal(const TArrayView<const UE::StateTree::ExecutionContext::FStateHandleContext> StateContexts, const TSharedRef<FSelectStateResult>& OutSelectionResult)
{
	using namespace UE::StateTree;
	using namespace UE::StateTree::ExecutionContext;
	using namespace UE::StateTree::ExecutionContext::Private;

	if (!IsValid())
	{
		STATETREE_LOG(Warning, TEXT("%hs: StateTree context is not initialized properly ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		return false;
	}

	if (StateContexts.IsEmpty())
	{
		UE_STATETREE_DEBUG_LOG_EVENT(this, Verbose, TEXT("There are no states in the transition."));
		return false;
	}

	if (!TestStateContextPath(this, StateContexts))
	{
		UE_STATETREE_DEBUG_LOG_EVENT(this, Verbose, TEXT("The StateContexts is invalid."));
		return false;
	}

	const FStateTreeExecutionState& Exec = GetExecState();

	// A reentrant call to ForceTransition or a call from Start, Tick or Stop must be deferred.
	if (Exec.CurrentPhase != EStateTreeUpdatePhase::Unset)
	{
		UE_STATETREE_DEBUG_LOG_EVENT(this, Warning, TEXT("Can't force a transition while %s"), *UEnum::GetDisplayValueAsText(Exec.CurrentPhase).ToString());
		return false;
	}

	if (!CollectActiveExternalData())
	{
		STATETREE_LOG(Warning, TEXT("%hs: Failed to collect external data ('%s' using StateTree '%s')"),
			__FUNCTION__, *GetNameSafe(&Owner), *GetFullNameSafe(&RootStateTree));
		return false;
	}

	FActiveStateInlineArray CurrentActiveStatePath;
	GetActiveStatePath(Exec.ActiveFrames, CurrentActiveStatePath);
	if (CurrentActiveStatePath.Num() == 0)
	{
		UE_STATETREE_DEBUG_LOG_EVENT(this, Verbose, TEXT("The tree needs active states to transition from."));
		return false;
	}

	FSelectStateArguments SelectStateArgs;
	SelectStateArgs.ActiveStates = MakeConstArrayView(CurrentActiveStatePath);
	SelectStateArgs.SourceState = SelectStateArgs.ActiveStates[0];
	SelectStateArgs.TargetState = StateContexts.Last();
	SelectStateArgs.Behavior = ESelectStateBehavior::Forced;
	SelectStateArgs.SelectionRules = RootStateTree.StateSelectionRules;

	FSelectStateInternalArguments InternalArgs;
	InternalArgs.MissingActiveStates = MakeConstArrayView(CurrentActiveStatePath);
	InternalArgs.MissingSourceFrameID = Exec.ActiveFrames[0].FrameID;
	InternalArgs.MissingSourceStates = MakeConstArrayView(CurrentActiveStatePath).Left(1);
	InternalArgs.MissingStatesToReachTarget = MakeConstArrayView(StateContexts);

	FSelectStateResult SelectStateResult;
	if (!SelectStateInternal(SelectStateArgs, InternalArgs, OutSelectionResult))
	{
		UE_STATETREE_DEBUG_LOG_EVENT(this, Verbose, TEXT("The force selection to target failed."));
		return false;
	}
	return true;
}

const FStateTreeExecutionFrame* FStateTreeExecutionContext::FindFrame(const UStateTree* StateTree, FStateTreeStateHandle RootState, TConstArrayView<FStateTreeExecutionFrame> Frames, const FStateTreeExecutionFrame*& OutParentFrame)
{
	const int32 FrameIndex = Frames.IndexOfByPredicate([StateTree, RootState](const FStateTreeExecutionFrame& Frame)
		{
			return Frame.HasRoot(StateTree, RootState);
		});

	if (FrameIndex == INDEX_NONE)
	{
		OutParentFrame = nullptr;
		return nullptr;
	}

	if (FrameIndex > 0)
	{
		OutParentFrame = &Frames[FrameIndex - 1];
	}

	return &Frames[FrameIndex];
}

FStateTreeExecutionFrame* FStateTreeExecutionContext::FindMutableExecutionFrame(UE::StateTree::FActiveFrameID InFrameID, TArrayView<FStateTreeExecutionFrame> InActiveFrames, TArrayView<FStateTreeExecutionFrame> InTemporaryFrames)
{
	return const_cast<FStateTreeExecutionFrame*>(UE::StateTree::ExecutionContext::FindExecutionFrame(InFrameID, InActiveFrames, InTemporaryFrames));
}

bool FStateTreeExecutionContext::IsHandleSourceValid(const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& CurrentFrame, const FStateTreeDataHandle Handle) const
{
	switch (Handle.GetSource())
	{
	case EStateTreeDataSourceType::None:
		return true;

	case EStateTreeDataSourceType::ContextData:
		return true;

	case EStateTreeDataSourceType::EvaluationScopeInstanceData:
	case EStateTreeDataSourceType::EvaluationScopeInstanceDataObject:
		// The data can be accessed in any of the caches (depends on how the binding was constructed), but it is most likely on top of the stack.
		for (int32 Index = EvaluationScopeInstanceCaches.Num() - 1; Index >= 0; --Index)
		{
			if (EvaluationScopeInstanceCaches[Index].StateTree == CurrentFrame.StateTree)
			{
				if (const FStructView* Result = EvaluationScopeInstanceCaches[Index].Container->GetDataViewPtr(Handle))
				{
					return true;
				}
			}
		}
		return false;

	case EStateTreeDataSourceType::ExternalData:
		return CurrentFrame.ExternalDataBaseIndex.IsValid()
			&& ContextAndExternalDataViews.IsValidIndex(CurrentFrame.ExternalDataBaseIndex.Get() + Handle.GetIndex());

	case EStateTreeDataSourceType::TransitionEvent:
		return CurrentlyProcessedTransitionEvent != nullptr;

	case EStateTreeDataSourceType::StateEvent:
		return CurrentlyProcessedStateSelectionResult != nullptr
			|| UE::StateTree::InstanceData::Private::IsActiveInstanceHandleSourceValid(Storage, CurrentFrame, Handle);

	case EStateTreeDataSourceType::ExternalGlobalParameterData:
	{
		checkf(false, TEXT("External global parameter data currently not supported for linked state-trees"));
		break;
	}

	default:
		return UE::StateTree::InstanceData::Private::IsHandleSourceValid(Storage, CurrentFrame, Handle);
	}

	return false;
}

bool FStateTreeExecutionContext::IsHandleSourceValid(const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& CurrentFrame, const FPropertyBindingCopyInfo& CopyInfo) const
{
	const FStateTreeDataHandle Handle = CopyInfo.SourceDataHandle.Get<FStateTreeDataHandle>();
	if (Handle.GetSource() == EStateTreeDataSourceType::ExternalGlobalParameterData)
	{
		return ExternalGlobalParameters ? ExternalGlobalParameters->Find(CopyInfo) != nullptr : false;
	}

	return IsHandleSourceValid(ParentFrame, CurrentFrame, Handle);
}

FStateTreeDataView FStateTreeExecutionContext::GetDataViewOrTemporary(const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& CurrentFrame, const FStateTreeDataHandle Handle)
{
	if (IsHandleSourceValid(ParentFrame, CurrentFrame, Handle))
	{
		return GetDataView(ParentFrame, CurrentFrame, Handle);
	}

	return GetTemporaryDataView(ParentFrame, CurrentFrame, Handle);
}

FStateTreeDataView FStateTreeExecutionContext::GetDataViewOrTemporary(const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& CurrentFrame, const FPropertyBindingCopyInfo& CopyInfo)
{
	const FStateTreeDataHandle Handle = CopyInfo.SourceDataHandle.Get<FStateTreeDataHandle>();
	if (Handle.GetSource() == EStateTreeDataSourceType::ExternalGlobalParameterData)
	{
		ensureMsgf(ExternalGlobalParameters != nullptr, TEXT("User requested external global parameters without setting it. See SetExternalGlobalParameters."));
		uint8* MemoryPtr = ExternalGlobalParameters ? ExternalGlobalParameters->Find(CopyInfo) : nullptr;
		return FStateTreeDataView(CopyInfo.SourceStructType, MemoryPtr);
	}

	return GetDataViewOrTemporary(ParentFrame, CurrentFrame, Handle);
}

FStateTreeDataView FStateTreeExecutionContext::GetTemporaryDataView(const FStateTreeExecutionFrame* ParentFrame,
	const FStateTreeExecutionFrame& CurrentFrame, const FStateTreeDataHandle Handle)
{
	switch (Handle.GetSource())
	{
	case EStateTreeDataSourceType::ExternalGlobalParameterData:
		checkf(false, TEXT("External global parameter data currently not supported for linked state-trees"));
		return {};
	case EStateTreeDataSourceType::EvaluationScopeInstanceData:
	case EStateTreeDataSourceType::EvaluationScopeInstanceDataObject:
		ensureMsgf(false, TEXT("The evaluation scope instance data needs to be constructed before you can access it."));
		return {};

	default:
		return UE::StateTree::InstanceData::Private::GetTemporaryDataView(Storage, CurrentFrame, Handle);
	}
}

FStateTreeDataView FStateTreeExecutionContext::AddTemporaryInstance(const FStateTreeExecutionFrame& Frame, const FStateTreeIndex16 OwnerNodeIndex, const FStateTreeDataHandle DataHandle, FConstStructView NewInstanceData)
{
	const FStructView NewInstance = Storage.AddTemporaryInstance(Owner, Frame, OwnerNodeIndex, DataHandle, NewInstanceData);
	if (FStateTreeInstanceObjectWrapper* Wrapper = NewInstance.GetPtr<FStateTreeInstanceObjectWrapper>())
	{
		return FStateTreeDataView(Wrapper->InstanceObject);
	}
	return NewInstance;
}

void FStateTreeExecutionContext::PushEvaluationScopeInstanceContainer(UE::StateTree::InstanceData::FEvaluationScopeInstanceContainer& Container, const FStateTreeExecutionFrame& Frame)
{
	EvaluationScopeInstanceCaches.Emplace(&Container, Frame.StateTree);
}

void FStateTreeExecutionContext::PopEvaluationScopeInstanceContainer(UE::StateTree::InstanceData::FEvaluationScopeInstanceContainer& Container)
{
	if (ensure(EvaluationScopeInstanceCaches.Num() > 0 && EvaluationScopeInstanceCaches.Last().Container == &Container))
	{
		EvaluationScopeInstanceCaches.Pop();
	}
}

void FStateTreeExecutionContext::CopyAllBindingsOnActiveInstances(const ECopyBindings CopyType)
{
	FStateTreeExecutionState& Exec = GetExecState();
	for (int32 FrameIndex = 0; FrameIndex < Exec.ActiveFrames.Num(); FrameIndex++)
	{
		FStateTreeExecutionFrame* CurrentParentFrame = FrameIndex > 0 ? &Exec.ActiveFrames[FrameIndex - 1] : nullptr;
		FStateTreeExecutionFrame& CurrentFrame = Exec.ActiveFrames[FrameIndex];
		const UStateTree* CurrentStateTree = CurrentFrame.StateTree;
		const int32 CurrentActiveNodeIndex = CurrentFrame.ActiveNodeIndex.AsInt32();

		FCurrentlyProcessedFrameScope FrameScope(*this, CurrentParentFrame, CurrentFrame);

		const bool bShouldCallOnEvaluatorsAndGlobalTasks = CurrentFrame.bIsGlobalFrame;
		if (bShouldCallOnEvaluatorsAndGlobalTasks)
		{
			const int32 EvaluatorEnd = CurrentStateTree->EvaluatorsBegin + CurrentStateTree->EvaluatorsNum;
			for (int32 EvalIndex = CurrentStateTree->EvaluatorsBegin; EvalIndex < EvaluatorEnd; ++EvalIndex)
			{
				if (EvalIndex <= CurrentActiveNodeIndex)
				{
					const FStateTreeEvaluatorBase& Eval = CurrentStateTree->Nodes[EvalIndex].Get<const FStateTreeEvaluatorBase>();
					if (Eval.BindingsBatch.IsValid())
					{
						const FStateTreeDataView EvalInstanceView = GetDataView(CurrentParentFrame, CurrentFrame, Eval.InstanceDataHandle);
						FNodeInstanceDataScope DataScope(*this, &Eval, EvalIndex, Eval.InstanceDataHandle, EvalInstanceView);
						CopyBatchOnActiveInstances(CurrentParentFrame, CurrentFrame, EvalInstanceView, Eval.BindingsBatch);
					}
				}
			}

			const int32 GlobalTasksEnd = CurrentStateTree->GlobalTasksBegin + CurrentStateTree->GlobalTasksNum;
			for (int32 TaskIndex = CurrentStateTree->GlobalTasksBegin; TaskIndex < GlobalTasksEnd; ++TaskIndex)
			{
				if (TaskIndex <= CurrentActiveNodeIndex)
				{
					const FStateTreeTaskBase& Task = CurrentStateTree->Nodes[TaskIndex].Get<const FStateTreeTaskBase>();
					const bool bTaskRequestCopy = (CopyType == ECopyBindings::ExitState && Task.bShouldCopyBoundPropertiesOnExitState)
						|| (CopyType == ECopyBindings::EnterState)
						|| (CopyType == ECopyBindings::Tick && Task.bShouldCopyBoundPropertiesOnTick);
					if (bTaskRequestCopy && Task.BindingsBatch.IsValid())
					{
						const FStateTreeDataView TaskInstanceView = GetDataView(CurrentParentFrame, CurrentFrame, Task.InstanceDataHandle);
						FNodeInstanceDataScope DataScope(*this, &Task, TaskIndex, Task.InstanceDataHandle, TaskInstanceView);
						CopyBatchOnActiveInstances(CurrentParentFrame, CurrentFrame, TaskInstanceView, Task.BindingsBatch);
					}
				}
			}
		}

		for (int32 StateIndex = 0; StateIndex < CurrentFrame.ActiveStates.Num(); ++StateIndex)
		{
			const FStateTreeStateHandle CurrentStateHandle = CurrentFrame.ActiveStates.States[StateIndex];
			const UE::StateTree::FActiveStateID CurrentStateID = CurrentFrame.ActiveStates.StateIDs[StateIndex];
			const FCompactStateTreeState& CurrentState = CurrentStateTree->States[CurrentStateHandle.Index];

			FCurrentlyProcessedStateScope StateScope(*this, CurrentStateHandle);

			if (CurrentState.Type == EStateTreeStateType::Linked
				|| CurrentState.Type == EStateTreeStateType::LinkedAsset)
			{
				if (CurrentState.ParameterDataHandle.IsValid()
					&& CurrentState.ParameterBindingsBatch.IsValid())
				{
					const FStateTreeDataView StateParamsDataView = GetDataView(CurrentParentFrame, CurrentFrame, CurrentState.ParameterDataHandle);
					CopyBatchOnActiveInstances(CurrentParentFrame, CurrentFrame, StateParamsDataView, CurrentState.ParameterBindingsBatch);
				}
			}

			const int32 TasksEnd = CurrentState.TasksBegin + CurrentState.TasksNum;
			for (int32 TaskIndex = CurrentState.TasksBegin; TaskIndex < TasksEnd; ++TaskIndex)
			{
				if (TaskIndex <= CurrentActiveNodeIndex)
				{
					const FStateTreeTaskBase& Task = CurrentStateTree->Nodes[TaskIndex].Get<const FStateTreeTaskBase>();

					const bool bTaskRequestCopy = (CopyType == ECopyBindings::ExitState && Task.bShouldCopyBoundPropertiesOnExitState)
						|| (CopyType == ECopyBindings::EnterState)
						|| (CopyType == ECopyBindings::Tick && Task.bShouldCopyBoundPropertiesOnTick);
					if (bTaskRequestCopy && Task.BindingsBatch.IsValid())
					{
						const FStateTreeDataView TaskInstanceView = GetDataView(CurrentParentFrame, CurrentFrame, Task.InstanceDataHandle);
						CopyBatchOnActiveInstances(CurrentParentFrame, CurrentFrame, TaskInstanceView, Task.BindingsBatch);
					}
				}
			}
		}
	}
}

bool FStateTreeExecutionContext::CopyBatchOnActiveInstances(const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& CurrentFrame, const FStateTreeDataView TargetView, const FStateTreeIndex16 BindingsBatch)
{
	constexpr bool bOnActiveInstances = true;
	return CopyBatchInternal<bOnActiveInstances>(ParentFrame, CurrentFrame, TargetView, BindingsBatch);
}

bool FStateTreeExecutionContext::CopyBatchWithValidation(const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& CurrentFrame, const FStateTreeDataView TargetView, const FStateTreeIndex16 BindingsBatch)
{
	constexpr bool bOnActiveInstances = false;
	return CopyBatchInternal<bOnActiveInstances>(ParentFrame, CurrentFrame, TargetView, BindingsBatch);
}

template<bool bOnActiveInstances>
bool FStateTreeExecutionContext::CopyBatchInternal(const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& CurrentFrame, const FStateTreeDataView TargetView, const FStateTreeIndex16 BindingsBatch)
{
	using namespace UE::StateTree::InstanceData;
	using namespace UE::StateTree::ExecutionContext::Private;

	const FPropertyBindingCopyInfoBatch& Batch = CurrentFrame.StateTree->PropertyBindings.Super::GetBatch(BindingsBatch);
	check(TargetView.GetStruct() == Batch.TargetStruct.Get().Struct);

	FEvaluationScopeInstanceContainer EvaluationScopeContainer;
	bool bEvaluationScopeContainerPushed = false;
	if (Batch.PropertyFunctionsBegin != Batch.PropertyFunctionsEnd)
	{
		check(Batch.PropertyFunctionsBegin.IsValid() && Batch.PropertyFunctionsEnd.IsValid());

		const FEvaluationScopeInstanceContainer::FMemoryRequirement& MemoryRequirement = CurrentFrame.StateTree->PropertyFunctionEvaluationScopeMemoryRequirements[BindingsBatch.Get()];
		if (MemoryRequirement.HasMemory())
		{
			void* FunctionBindingMemory = FMemory_Alloca_Aligned(MemoryRequirement.Size, MemoryRequirement.Alignment);
			EvaluationScopeContainer = FEvaluationScopeInstanceContainer(FunctionBindingMemory, MemoryRequirement);
			PushEvaluationScopeInstanceContainer(EvaluationScopeContainer, CurrentFrame);
			InitEvaluationScopeInstanceData(EvaluationScopeContainer, CurrentFrame.StateTree, Batch.PropertyFunctionsBegin.Get(), Batch.PropertyFunctionsEnd.Get());
			bEvaluationScopeContainerPushed = true;
		}

		if constexpr (bOnActiveInstances)
		{
			EvaluatePropertyFunctionsOnActiveInstances(ParentFrame, CurrentFrame, FStateTreeIndex16(Batch.PropertyFunctionsBegin), Batch.PropertyFunctionsEnd.Get() - Batch.PropertyFunctionsBegin.Get());
		}
		else
		{
			EvaluatePropertyFunctionsWithValidation(ParentFrame, CurrentFrame, FStateTreeIndex16(Batch.PropertyFunctionsBegin), Batch.PropertyFunctionsEnd.Get() - Batch.PropertyFunctionsBegin.Get());
		}
	}

	bool bSucceed = true;
	for (const FPropertyBindingCopyInfo& Copy : CurrentFrame.StateTree->PropertyBindings.Super::GetBatchCopies(Batch))
	{
		if constexpr (bOnActiveInstances)
		{
			const FStateTreeDataView SourceView = GetDataView(ParentFrame, CurrentFrame, Copy);
			bSucceed &= CurrentFrame.StateTree->PropertyBindings.Super::CopyProperty(Copy, SourceView, TargetView);
		}
		else
		{
			const FStateTreeDataView SourceView = GetDataViewOrTemporary(ParentFrame, CurrentFrame, Copy);
			if (!SourceView.IsValid())
			{
				bSucceed = false;
				break;
			}

			bSucceed &= CurrentFrame.StateTree->PropertyBindings.Super::CopyProperty(Copy, SourceView, TargetView);
		}
	}

	if (bEvaluationScopeContainerPushed)
	{
		PopEvaluationScopeInstanceContainer(EvaluationScopeContainer);
	}

	return bSucceed;
}

bool FStateTreeExecutionContext::CollectActiveExternalData()
{
	return CollectActiveExternalData(GetExecState().ActiveFrames);
}

bool FStateTreeExecutionContext::CollectActiveExternalData(const TArrayView<FStateTreeExecutionFrame> Frames)
{
	if (bActiveExternalDataCollected)
	{
		return true;
	}

	bool bAllExternalDataValid = true;
	const FStateTreeExecutionFrame* PrevFrame = nullptr;

	for (FStateTreeExecutionFrame& Frame : Frames)
	{
		if (PrevFrame && PrevFrame->StateTree == Frame.StateTree)
		{
			Frame.ExternalDataBaseIndex = PrevFrame->ExternalDataBaseIndex;
		}
		else
		{
			Frame.ExternalDataBaseIndex = CollectExternalData(Frame.StateTree);
		}

		if (!Frame.ExternalDataBaseIndex.IsValid())
		{
			bAllExternalDataValid = false;
		}

		PrevFrame = &Frame;
	}

	if (bAllExternalDataValid)
	{
		bActiveExternalDataCollected = true;
	}

	return bAllExternalDataValid;
}

FStateTreeIndex16 FStateTreeExecutionContext::CollectExternalData(const UStateTree* StateTree)
{
	if (!StateTree)
	{
		return FStateTreeIndex16::Invalid;
	}

	// If one of the active states share the same state tree, get the external data from there.
	for (const FCollectedExternalDataCache& Cache : CollectedExternalCache)
	{
		if (Cache.StateTree == StateTree)
		{
			return Cache.BaseIndex;
		}
	}

	const TConstArrayView<FStateTreeExternalDataDesc> ExternalDataDescs = StateTree->GetExternalDataDescs();
	const int32 BaseIndex = ContextAndExternalDataViews.Num();
	const int32 NumDescs = ExternalDataDescs.Num();
	FStateTreeIndex16 Result(BaseIndex);

	if (NumDescs > 0)
	{
		ContextAndExternalDataViews.AddDefaulted(NumDescs);
		const TArrayView<FStateTreeDataView> DataViews = MakeArrayView(ContextAndExternalDataViews.GetData() + BaseIndex, NumDescs);

		if (ensureMsgf(CollectExternalDataDelegate.IsBound(), TEXT("The StateTree asset has external data, expecting CollectExternalData delegate to be provided.")))
		{
			if (!CollectExternalDataDelegate.Execute(*this, StateTree, StateTree->GetExternalDataDescs(), DataViews))
			{
				// The caller is responsible for error reporting. 
				return FStateTreeIndex16::Invalid;
			}
		}

		// Check that the data is valid and present.
		for (int32 Index = 0; Index < NumDescs; Index++)
		{
			const FStateTreeExternalDataDesc& DataDesc = ExternalDataDescs[Index];
			const FStateTreeDataView& DataView = ContextAndExternalDataViews[BaseIndex + Index];

			if (DataDesc.Requirement == EStateTreeExternalDataRequirement::Required)
			{
				// Required items must have valid pointer of the expected type.  
				if (!DataView.IsValid() || !DataDesc.IsCompatibleWith(DataView))
				{
					Result = FStateTreeIndex16::Invalid;
					break;
				}
			}
			else
			{
				// Optional items must have same type if they are set.
				if (DataView.IsValid() && !DataDesc.IsCompatibleWith(DataView))
				{
					Result = FStateTreeIndex16::Invalid;
					break;
				}
			}
		}
	}

	if (!Result.IsValid())
	{
		// Rollback
		ContextAndExternalDataViews.SetNum(BaseIndex);
	}

	// Cached both succeeded and failed attempts.
	CollectedExternalCache.Add({ StateTree, Result });

	return FStateTreeIndex16(Result);
}

bool FStateTreeExecutionContext::SetGlobalParameters(const FConstStructView Parameters)
{
	if (ensureMsgf(RootStateTree.GetDefaultParameters().GetPropertyBagStruct() == Parameters.GetScriptStruct(),
		TEXT("Parameters must be of the same struct type. Make sure to migrate the provided parameters to the same type as the StateTree default parameters.")))
	{
		Storage.SetGlobalParameters(Parameters);
		return true;
	}

	return false;
}

// Deprecated
PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FStateTreeExecutionContext::CaptureNewStateEvents(TConstArrayView<FStateTreeExecutionFrame> PrevFrames, TConstArrayView<FStateTreeExecutionFrame> NewFrames, TArrayView<FStateTreeFrameStateSelectionEvents> FramesStateSelectionEvents)
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FStateTreeExecutionContext::CaptureNewStateEvents(const TSharedRef<const FSelectStateResult>& Args)
{
	using namespace UE::StateTree;
	using namespace UE::StateTree::ExecutionContext::Private;

	if (Args->SelectedStates.Num() == 0)
	{
		return;
	}

	FStateTreeExecutionState& Exec = GetExecState();

	// Mark the events from delayed transitions as in use, so that each State will receive unique copy of the event struct. 
	TArray<FStateTreeSharedEvent, TInlineAllocator<16>> EventsInUse;
	for (const FStateTreeTransitionDelayedState& DelayedTransition : Exec.DelayedTransitions)
	{
		if (DelayedTransition.CapturedEvent.IsValid())
		{
			EventsInUse.Add(DelayedTransition.CapturedEvent);
		}
	}

	//  A state can be sustained, changed, or not part of the transition.
	//It captures the event payload for sustained and changed states (even when it's not needed by a binding).

	// For each state that are changed/sustained
	bool bTargetReached = false;
	FActiveFrameID CurrentFrameID;
	const FStateTreeExecutionFrame* CurrentFrame = nullptr;
	for (int32 SelectedStateIndex = 0; SelectedStateIndex < Args->SelectedStates.Num(); ++SelectedStateIndex)
	{
		const FActiveState& SelectedState = Args->SelectedStates[SelectedStateIndex];

		// Global
		if (SelectedState.GetFrameID() != CurrentFrameID)
		{
			CurrentFrameID = SelectedState.GetFrameID();
			CurrentFrame = UE::StateTree::ExecutionContext::FindExecutionFrame(SelectedState.GetFrameID(), MakeConstArrayView(Exec.ActiveFrames), MakeConstArrayView(Args->TemporaryFrames));
		}

		check(CurrentFrame);
		const int32 ActiveStateIndex = CurrentFrame->ActiveStates.IndexOfReverse(SelectedState.GetStateID());

		const bool bIsStateCommon = ActiveStateIndex != INDEX_NONE;
		// Once we reach the target state, all remaining common states below it are "sustained" or new states
		bTargetReached = bTargetReached || Args->TargetState == SelectedState;

		const bool bSustainedState = bIsStateCommon && bTargetReached && bCaptureStateEventPayloadForSustainedState;
		const bool bChangedState = !bIsStateCommon;
		if (!(bSustainedState || bChangedState))
		{
			continue;
		}

		if (const FCompactStateTreeState* State = CurrentFrame->StateTree->GetStateFromHandle(SelectedState.GetStateHandle()))
		{
			if (State->EventDataIndex.IsValid())
			{
				FStateTreeSharedEvent& StateTreeEvent = Storage.GetMutableStruct(CurrentFrame->ActiveInstanceIndexBase.Get() + State->EventDataIndex.Get()).Get<FStateTreeSharedEvent>();
				const FSelectionEventWithID* EventToCapturePtr = Args->SelectionEvents.FindByPredicate([SelectedState](const FSelectionEventWithID& EventID)
					{
						return EventID.State == SelectedState;
					});
				if (ensureAlways(EventToCapturePtr))
				{
					const FStateTreeSharedEvent& EventToCapture = EventToCapturePtr->Event;
					if (EventsInUse.Contains(EventToCapture))
					{
						// Event is already spoken for, make a copy.
						StateTreeEvent = FStateTreeSharedEvent(*EventToCapture);
					}
					else
					{
						// Event not in use, steal it.
						StateTreeEvent = EventToCapture;
						EventsInUse.Add(EventToCapture);
					}
				}
			}
		}
	}
}

// Deprecated
EStateTreeRunStatus FStateTreeExecutionContext::EnterState(FStateTreeTransitionResult& Transition)
{
	return EStateTreeRunStatus::Failed;
}

EStateTreeRunStatus FStateTreeExecutionContext::EnterState(const TSharedPtr<FSelectStateResult>& ArgsPtr, const FStateTreeTransitionResult& Transition)
{
	// 1. Update the data instance data of all frames and selected states.
	// The data won't be available until the node is reached or the state is added to the FStateTreeExecutionFrame::ActiveState.
	// 2. Capture StateEvents
	// 3. Call Enter state for each selected state from Target.
	//Note, no need to update the bindings for previously active state, the bindings are updated in ExitState or there was no binding before (See Start).

	using namespace UE::StateTree;
	using namespace UE::StateTree::ExecutionContext;
	using namespace UE::StateTree::ExecutionContext::Private;

	CSV_SCOPED_TIMING_STAT(StateTree, EnterState);

	ON_SCOPE_EXIT{ InstanceData.ResetTemporaryInstances(); };

	if (!ArgsPtr.IsValid())
	{
		TSharedPtr<FSelectStateResult> EmptySelectStateResult;
		UpdateInstanceData(EmptySelectStateResult);
		return EStateTreeRunStatus::Failed;
	}

	FSelectStateResult& Args = *ArgsPtr;
	FStateTreeExecutionState& Exec = GetExecState();

	if (bRecordTransitions)
	{
		RecordedTransitions.Add(MakeRecordedTransitionResult(ArgsPtr.ToSharedRef(), Transition));
	}

	UpdateInstanceData(ArgsPtr);
	if (Args.SelectedStates.IsEmpty())
	{
		return EStateTreeRunStatus::Failed;
	}
	CaptureNewStateEvents(ArgsPtr.ToSharedRef());

	++Exec.StateChangeCount;

	FStateTreeTransitionResult CurrentTransition = Transition;
	EStateTreeRunStatus Result = EStateTreeRunStatus::Running;

	STATETREE_LOG(Log, TEXT("Enter state '%s' (%d)")
		, *UE::StateTree::ExecutionContext::Private::GetStatePathAsString(&RootStateTree, Args.SelectedStates)
		, Exec.StateChangeCount
	);
	UE_STATETREE_DEBUG_ENTER_PHASE(this, EStateTreeUpdatePhase::EnterStates);

	bool bTargetReached = false;
	FActiveFrameID CurrentFrameID;
	const TArrayView<const UE::StateTree::FActiveState> SelectedStates = Args.SelectedStates;
	for (int32 SelectedStateIndex = 0; SelectedStateIndex < SelectedStates.Num(); ++SelectedStateIndex)
	{
		const UE::StateTree::FActiveState& SelectedState = SelectedStates[SelectedStateIndex];

		const bool bIsCurrentFrameNew = CurrentFrameID != SelectedState.GetFrameID();
		CurrentFrameID = SelectedState.GetFrameID();

		bool bFrameAdded = false;
		int32 FrameIndex = Exec.IndexOfActiveFrame(SelectedState.GetFrameID());
		if (FrameIndex == INDEX_NONE)
		{
			check(bIsCurrentFrameNew);
			FStateTreeExecutionFrame* FoundNewFrame = Args.FindTemporaryFrame(SelectedState.GetFrameID());
			if (!ensure(FoundNewFrame != nullptr))
			{
				return EStateTreeRunStatus::Failed;
			}

			// Add it to the active list
			FrameIndex = Exec.ActiveFrames.Add(MoveTemp(*FoundNewFrame));
			bFrameAdded = true;
		}

		FStateTreeExecutionFrame& CurrentFrame = Exec.ActiveFrames[FrameIndex];

		int32 ActiveStateIndex = CurrentFrame.ActiveStates.IndexOfReverse(SelectedState.GetStateID());
		// The state is the target of current transition or is a sustained state
		const bool bAreCommon = ActiveStateIndex != INDEX_NONE;
		bTargetReached = bTargetReached || SelectedState == Args.TargetState;
		const bool bSustainedState = bAreCommon && bTargetReached;

		// States which were active before and will remain active, but are not on target branch will not get
		// EnterState called. That is, a transition is handled as "replan from this state".
		if (bAreCommon && !bSustainedState)
		{
			// Is already in the active and we do not need to call EnterState with Sustained
			continue;
		}

		FStateTreeExecutionFrame* CurrentParentFrame = FrameIndex > 0 ? &Exec.ActiveFrames[FrameIndex - 1] : nullptr;
		const UStateTree* CurrentStateTree = CurrentFrame.StateTree;

		// Create the status. We lost the previous tasks' completion status.
		if (!ensureMsgf(CurrentFrame.ActiveTasksStatus.IsValid(), TEXT("Frame is not formed correct.")))
		{
			const FCompactStateTreeFrame* FrameInfo = CurrentFrame.StateTree->GetFrameFromHandle(CurrentFrame.RootState);
			ensureAlwaysMsgf(FrameInfo, TEXT("The compiled data is invalid. It should contains the information for the root frame."));
			CurrentFrame.ActiveTasksStatus = FrameInfo ? FStateTreeTasksCompletionStatus(*FrameInfo) : FStateTreeTasksCompletionStatus();
		}

		if (bIsCurrentFrameNew)
		{
			if (!bFrameAdded && !CurrentFrame.bHaveEntered)
			{
				// EnterState on Changed global was called during selection with StartTemporaryEvaluatorsAndGlobalTasks
				CurrentTransition.CurrentState = FStateTreeStateHandle::Invalid;
				CurrentTransition.ChangeType = EStateTreeStateChangeType::Sustained;

				constexpr bool bOnActiveInstances = true;
				constexpr bool bMoveFromTemporary = true;
				EStateTreeRunStatus GlobalTaskRunStatus = StartGlobalsForFrameInternal<bOnActiveInstances, bMoveFromTemporary>(CurrentParentFrame, CurrentFrame, CurrentTransition);
				Result = GetPriorityRunStatus(Result, GlobalTaskRunStatus);
				if (Result == EStateTreeRunStatus::Failed)
				{
					break;
				}
			}
			else
			{
				// New frame might have temporaries that need to move.
				MoveGlobalTemporariesToInstances(CurrentFrame);
			}
		}

		FCurrentlyProcessedFrameScope FrameScope(*this, CurrentParentFrame, CurrentFrame);
		{
			const FStateTreeStateHandle CurrentStateHandle = SelectedState.GetStateHandle();
			const FCompactStateTreeState& CurrentState = CurrentStateTree->States[CurrentStateHandle.Index];
			const UE::StateTree::FActiveStateID CurrentStateID = SelectedState.GetStateID();

			checkf(CurrentState.bEnabled, TEXT("Only enabled states are in SelectedStates."));

			// New state. Add it
			if (!bSustainedState)
			{
				CurrentFrame.ActiveTasksStatus.Push(CurrentState);
				if (!CurrentFrame.ActiveStates.Push(CurrentStateHandle, CurrentStateID))
				{
					// The ActiveStates array supports a max of 8 states. The depth is verified at compilation.
					ensureMsgf(false, TEXT("Reached max execution depth when trying to enter state '%s'. '%s' using StateTree '%s'."),
						*GetStateStatusString(Exec),
						*GetNameSafe(&Owner),
						*GetFullNameSafe(&RootStateTree)
					);
					break;
				}
				ActiveStateIndex = CurrentFrame.ActiveStates.Num() - 1;
			}

			FCurrentlyProcessedStateScope StateScope(*this, CurrentStateHandle);

			if (CurrentState.Type == EStateTreeStateType::Linked
				|| CurrentState.Type == EStateTreeStateType::LinkedAsset)
			{
				if (CurrentState.ParameterDataHandle.IsValid()
					&& CurrentState.ParameterBindingsBatch.IsValid())
				{
					const FStateTreeDataView StateParamsDataView = GetDataView(CurrentParentFrame, CurrentFrame, CurrentState.ParameterDataHandle);
					CopyBatchOnActiveInstances(CurrentParentFrame, CurrentFrame, StateParamsDataView, CurrentState.ParameterBindingsBatch);
				}
			}

			const EStateTreeStateChangeType ChangeType = bSustainedState ? EStateTreeStateChangeType::Sustained : EStateTreeStateChangeType::Changed;
			CurrentTransition.CurrentState = CurrentStateHandle;
			CurrentTransition.ChangeType = ChangeType;

			UE_STATETREE_DEBUG_STATE_EVENT(this, CurrentStateHandle, EStateTreeTraceEventType::OnEntering);
			STATETREE_LOG(Log, TEXT("%*sState '%s' (%s)"),
				(FrameIndex + ActiveStateIndex + 1) * UE::StateTree::Debug::IndentSize, TEXT(""),
				*GetSafeStateName(CurrentStateTree, CurrentStateHandle),
				*UEnum::GetDisplayValueAsText(CurrentTransition.ChangeType).ToString()
			);

			// Call state change events on conditions if needed.
			constexpr bool bOnActiveInstances = true;
			constexpr EConditionFunction ConditionFunction = EConditionFunction::EnterState;
			RunAllConditionsInternal<bOnActiveInstances, ConditionFunction>(CurrentParentFrame, CurrentFrame, CurrentState, CurrentTransition);

			// Activate tasks on current state.
			UE::StateTree::FTasksCompletionStatus CurrentStateTasksStatus = CurrentFrame.ActiveTasksStatus.GetStatus(CurrentState);
			for (int32 StateTaskIndex = 0; StateTaskIndex < CurrentState.TasksNum; ++StateTaskIndex)
			{
				const int32 AssetTaskIndex = CurrentState.TasksBegin + StateTaskIndex;
				const FStateTreeTaskBase& Task = CurrentStateTree->Nodes[AssetTaskIndex].Get<const FStateTreeTaskBase>();
				const FStateTreeDataView TaskInstanceView = MoveTemporaryToInstance(CurrentFrame, FStateTreeIndex16(AssetTaskIndex), Task.InstanceDataHandle);

				// Ignore disabled task
				if (Task.bTaskEnabled == false)
				{
					STATETREE_LOG(VeryVerbose, TEXT("%*sSkipped 'EnterState' for disabled Task: '%s'"), UE::StateTree::Debug::IndentSize, TEXT(""), *Task.Name.ToString());
					continue;
				}

				FNodeInstanceDataScope DataScope(*this, &Task, AssetTaskIndex, Task.InstanceDataHandle, TaskInstanceView);

				CurrentFrame.ActiveNodeIndex = FStateTreeIndex16(AssetTaskIndex);

				// Copy bound properties.
				if (Task.BindingsBatch.IsValid())
				{
					CopyBatchOnActiveInstances(CurrentParentFrame, CurrentFrame, TaskInstanceView, Task.BindingsBatch);
				}

				const bool bShouldCallEnterState = CurrentTransition.ChangeType == EStateTreeStateChangeType::Changed
					|| (CurrentTransition.ChangeType == EStateTreeStateChangeType::Sustained && Task.bShouldStateChangeOnReselect);

				if (bShouldCallEnterState)
				{
					STATETREE_LOG(Verbose, TEXT("%*sTask '%s'.EnterState()"), (FrameIndex + ActiveStateIndex + 1) * UE::StateTree::Debug::IndentSize, TEXT(""), *Task.Name.ToString());
					UE_STATETREE_DEBUG_TASK_ENTER_STATE(this, CurrentStateTree, FStateTreeIndex16(AssetTaskIndex));

					EStateTreeRunStatus TaskRunStatus = EStateTreeRunStatus::Unset;
					{
						QUICK_SCOPE_CYCLE_COUNTER(StateTree_Task_EnterState);
						CSV_SCOPED_TIMING_STAT(StateTree, Task_EnterState);

						TaskRunStatus = Task.EnterState(*this, CurrentTransition);
					}

					UE::StateTree::ETaskCompletionStatus TaskStatus = CastToTaskStatus(TaskRunStatus);
					const UE::StateTree::FTasksCompletionStatus::FSetStatusResult TaskStatusResult = CurrentStateTasksStatus.SetTaskStatusWithPriority(StateTaskIndex, TaskStatus);
					TaskStatus = TaskStatusResult.CurrentStatus;

					TaskRunStatus = CastToRunStatus(TaskStatus);
					if (TaskRunStatus != EStateTreeRunStatus::Failed && Task.OutputBindingsBatch.IsValid())
					{
						CopyBatchOnActiveInstances(CurrentlyProcessedParentFrame, *CurrentlyProcessedFrame, TaskInstanceView, Task.OutputBindingsBatch);
					}

					UE_STATETREE_DEBUG_TASK_EVENT(this, AssetTaskIndex, TaskInstanceView, EStateTreeTraceEventType::OnEntered, TaskRunStatus);

					// Broadcast task completion dispatcher only if status actually changed
					if (TaskStatusResult.bChanged && Task.bHasTaskCompletionDelegateDispatcher)
					{
						BroadcastTaskCompletionDispatchers(CurrentStateTree, AssetTaskIndex, TaskStatus);
					}

					if (CurrentStateTasksStatus.IsConsideredForCompletion(StateTaskIndex))
					{
						Result = GetPriorityRunStatus(Result, TaskRunStatus);
						if (Result == EStateTreeRunStatus::Failed)
						{
							break;
						}
					}
				}
			}
			UE_STATETREE_DEBUG_STATE_EVENT(this, CurrentStateHandle, EStateTreeTraceEventType::OnEntered);
		}

		if (Result == EStateTreeRunStatus::Failed)
		{
			break;
		}
	}

	UE_STATETREE_DEBUG_EXIT_PHASE(this, EStateTreeUpdatePhase::EnterStates);
	UE_STATETREE_DEBUG_ACTIVE_STATES_EVENT(this, Exec.ActiveFrames);

	Exec.bHasPendingCompletedState = Result != EStateTreeRunStatus::Running;
	return Result;
}

// Deprecated
void FStateTreeExecutionContext::ExitState(const FStateTreeTransitionResult& Transition)
{
}

void FStateTreeExecutionContext::ExitState(const TSharedPtr<const FSelectStateResult>& Args, const FStateTreeTransitionResult& Transition)
{
	// 1. Copy all bindings on all active nodes.
	// 2. Call ExitState on all active global tasks, global evaluators, state tasks and state condition that are affected by the transition.
	//  If the frame/state is before the target
	//    then it is not affected.
	//  Else
	//    Excluding the target, if the active state is not in the new selection, then call ExitState with EStateTreeStateChangeType::Changed.
	//    Including the target, if the active state is in the new selection then call ExitState with EStateTreeStateChangeType::Sustained.
	// 3. If the state/frame is not in the new selection, clean up memory, delegate...
	//  If the owning frame receive an ExitState, then frame globals will also receive an ExitState.
	//  It will be sustained if the owning frame is sustained.

	using namespace UE::StateTree;
	using namespace UE::StateTree::ExecutionContext;
	using namespace UE::StateTree::ExecutionContext::Private;

	CSV_SCOPED_TIMING_STAT(StateTree, ExitState);

	FStateTreeExecutionState& Exec = GetExecState();
	if (Exec.ActiveFrames.IsEmpty())
	{
		return;
	}

	CopyAllBindingsOnActiveInstances(ECopyBindings::ExitState);

	STATETREE_LOG(Log, TEXT("Exit state '%s' (%d)"), *DebugGetStatePath(Exec.ActiveFrames), Exec.StateChangeCount);
	UE_STATETREE_DEBUG_SCOPED_PHASE(this, EStateTreeUpdatePhase::ExitStates);

	FActiveStateInlineArray ActiveStates;
	GetActiveStatePath(GetExecState().ActiveFrames, ActiveStates);

	const FActiveState ArgsTargetState = Args ? Args->TargetState : FActiveState();
	const TArrayView<const FActiveState> CommonStates = Args != nullptr
		? FActiveStatePath::Intersect(MakeConstArrayView(ActiveStates), MakeConstArrayView(Args->SelectedStates))
		: TArrayView<const FActiveState>();
	const TArrayView<const FActiveFrameID> ArgsSelectedFrames = Args != nullptr
		? MakeConstArrayView(Args->SelectedFrames)
		: TArrayView<const FActiveFrameID>();
	const TArrayView<const FActiveState> ChangedStates = MakeConstArrayView(ActiveStates).Mid(CommonStates.Num());
	const bool bHasSustained = CommonStates.Contains(ArgsTargetState);

	bool bContinue = true;
	FStateTreeTransitionResult CurrentTransition = Transition;

	auto ExitPreviousFrame = [this, &Exec, &ArgsSelectedFrames, &CurrentTransition](const int32 FrameIndex, const bool bCallExit)
		{
			FStateTreeExecutionFrame& Frame = Exec.ActiveFrames[FrameIndex];
			const bool bIsFrameCommon = ArgsSelectedFrames.Contains(Frame.FrameID);
			if (Frame.bIsGlobalFrame && (bCallExit || !bIsFrameCommon))
			{
				const bool bIsFrameSustained = bIsFrameCommon;
				FStateTreeExecutionFrame* ParentFrame = FrameIndex > 0 ? &Exec.ActiveFrames[FrameIndex - 1] : nullptr;
				CurrentTransition.CurrentState = FStateTreeStateHandle::Invalid;
				CurrentTransition.ChangeType = bIsFrameSustained ? EStateTreeStateChangeType::Sustained : EStateTreeStateChangeType::Changed;
				StopGlobalsForFrameOnActiveInstances(ParentFrame, Frame, CurrentTransition);
			}

			if (!bIsFrameCommon)
			{
				checkf(Frame.ActiveStates.Num() == 0, TEXT("All states must received ExitState first."));
				CleanFrame(Exec, Frame.FrameID);
				Exec.ActiveFrames.RemoveAt(FrameIndex, EAllowShrinking::No);
			}
		};

	int32 FrameIndex = Exec.ActiveFrames.Num() - 1;
	for (; bContinue && FrameIndex >= 0; --FrameIndex)
	{
		FStateTreeExecutionFrame* CurrentParentFrame = FrameIndex > 0 ? &Exec.ActiveFrames[FrameIndex - 1] : nullptr;
		FStateTreeExecutionFrame& CurrentFrame = Exec.ActiveFrames[FrameIndex];
		const UStateTree* CurrentStateTree = CurrentFrame.StateTree;

		FCurrentlyProcessedFrameScope FrameScope(*this, CurrentParentFrame, CurrentFrame);
		for (int32 StateIndex = CurrentFrame.ActiveStates.Num() - 1; bContinue && StateIndex >= 0; --StateIndex)
		{
			const FStateTreeStateHandle CurrentHandle = CurrentFrame.ActiveStates[StateIndex];
			const FActiveStateID CurrentStateID = CurrentFrame.ActiveStates.StateIDs[StateIndex];
			const FCompactStateTreeState& CurrentState = CurrentStateTree->States[CurrentHandle.Index];

			const FActiveState CurrentActiveState(CurrentFrame.FrameID, CurrentStateID, CurrentHandle);
			const bool bIsStateCommon = !ChangedStates.Contains(CurrentActiveState);

			// It is in the common list and there is no "target" (everything else is new, a new state was created).
			if (bIsStateCommon && !bHasSustained)
			{
				// It will also stop the frame loop.
				bContinue = false;
				break;
			}

			// It is the target.
			if (bIsStateCommon && bHasSustained && CurrentActiveState == ArgsTargetState)
			{
				// this is the last sustained
				bContinue = false;
			}

			const bool bIsStateSustained = bIsStateCommon;

			CurrentTransition.CurrentState = CurrentHandle;
			CurrentTransition.ChangeType = bIsStateSustained ? EStateTreeStateChangeType::Sustained : EStateTreeStateChangeType::Changed;

			STATETREE_LOG(Log, TEXT("%*sState '%s' (%s)"), (FrameIndex + StateIndex + 1) * UE::StateTree::Debug::IndentSize, TEXT("")
				, *GetSafeStateName(CurrentFrame, CurrentHandle)
				, *UEnum::GetDisplayValueAsText(CurrentTransition.ChangeType).ToString());

			FCurrentlyProcessedStateScope StateScope(*this, CurrentHandle);
			UE_STATETREE_DEBUG_STATE_EVENT(this, CurrentHandle, EStateTreeTraceEventType::OnExiting);

			// Call task completed only if EnterState() was called.
			// The task order in the tree (BF) allows us to use the comparison.
			const int32 TaskEnd = FMath::Min(CurrentFrame.ActiveNodeIndex.AsInt32() + 1, CurrentState.TasksBegin + CurrentState.TasksNum);
			for (int32 TaskIndex = TaskEnd - 1; TaskIndex >= CurrentState.TasksBegin; --TaskIndex)
			{
				const FStateTreeTaskBase& Task = CurrentStateTree->Nodes[TaskIndex].Get<const FStateTreeTaskBase>();

				// Ignore disabled task
				if (Task.bTaskEnabled)
				{
					const FStateTreeDataView TaskInstanceView = GetDataView(CurrentParentFrame, CurrentFrame, Task.InstanceDataHandle);
					FNodeInstanceDataScope DataScope(*this, &Task, TaskIndex, Task.InstanceDataHandle, TaskInstanceView);

					const bool bShouldCallStateChange = CurrentTransition.ChangeType == EStateTreeStateChangeType::Changed
						|| (CurrentTransition.ChangeType == EStateTreeStateChangeType::Sustained && Task.bShouldStateChangeOnReselect);

					if (bShouldCallStateChange)
					{
						STATETREE_LOG(Verbose, TEXT("%*sTask '%s'.ExitState()"), (FrameIndex + StateIndex + 1) * UE::StateTree::Debug::IndentSize, TEXT(""), *Task.Name.ToString());
						UE_STATETREE_DEBUG_TASK_EXIT_STATE(this, CurrentStateTree, FStateTreeIndex16(TaskIndex));
						{
							QUICK_SCOPE_CYCLE_COUNTER(StateTree_Task_ExitState);
							CSV_SCOPED_TIMING_STAT(StateTree, Task_ExitState);
							Task.ExitState(*this, CurrentTransition);
						}

						if (Task.OutputBindingsBatch.IsValid())
						{
							CopyBatchOnActiveInstances(CurrentlyProcessedParentFrame, CurrentFrame, TaskInstanceView, Task.OutputBindingsBatch);
						}

						UE_STATETREE_DEBUG_TASK_EVENT(this, TaskIndex, TaskInstanceView, EStateTreeTraceEventType::OnExited, Transition.CurrentRunStatus);
					}
				}
				else
				{
					STATETREE_LOG(VeryVerbose, TEXT("%*sSkipped 'ExitState' for disabled Task: '%s'"), UE::StateTree::Debug::IndentSize, TEXT(""), *Task.Name.ToString());
				}
				CurrentFrame.ActiveNodeIndex = FStateTreeIndex16(TaskIndex - 1);
			}

			// Call state change events on conditions if needed.
			constexpr bool bOnActiveInstances = true;
			constexpr EConditionFunction ConditionFunction = EConditionFunction::ExitState;
			RunAllConditionsInternal<bOnActiveInstances, ConditionFunction>(CurrentParentFrame, CurrentFrame, CurrentState, CurrentTransition);

			// Reset the completed state. This is to keep the wrong UE5.6 behavior.
			if (!EnumHasAnyFlags(CurrentFrame.StateTree->GetStateSelectionRules(), EStateTreeStateSelectionRules::CompletedTransitionStatesCreateNewStates)
				&& bIsStateSustained)
			{
				CurrentFrame.ActiveTasksStatus.GetStatus(CurrentState).ResetStatus(CurrentState.TasksNum);
			}

			// Remove the state from the active states if it "changed".
			if (!bIsStateSustained)
			{
				CleanState(Exec, CurrentStateID);

				// Remove state
				const FStateTreeStateHandle PoppedState = CurrentFrame.ActiveStates.Pop();
				check(PoppedState == CurrentHandle);
			}

			UE_STATETREE_DEBUG_STATE_EVENT(this, CurrentHandle, EStateTreeTraceEventType::OnExited);
		}

		// The previous frame is not in the new
		ExitPreviousFrame(FrameIndex, bContinue);
	}
}

void FStateTreeExecutionContext::RemoveAllDelegateListeners()
{
	GetExecState().DelegateActiveListeners = FStateTreeDelegateActiveListeners();
}

void FStateTreeExecutionContext::StateCompleted()
{
	CSV_SCOPED_TIMING_STAT(StateTree, StateCompleted);

	const FStateTreeExecutionState& Exec = GetExecState();

	if (Exec.ActiveFrames.IsEmpty())
	{
		return;
	}

	STATETREE_LOG(Verbose, TEXT("State Completed %s (%d)"), *UEnum::GetDisplayValueAsText(Exec.LastTickStatus).ToString(), Exec.StateChangeCount);
	UE_STATETREE_DEBUG_SCOPED_PHASE(this, EStateTreeUpdatePhase::StateCompleted);

	// Call from child towards root to allow to pass results back.
	// Note: Completed is assumed to be called immediately after tick or enter state, we want to preserve the status of instance data for tasks.
	for (int32 FrameIndex = Exec.ActiveFrames.Num() - 1; FrameIndex >= 0; FrameIndex--)
	{
		const FStateTreeExecutionFrame* CurrentParentFrame = FrameIndex > 0 ? &Exec.ActiveFrames[FrameIndex - 1] : nullptr;
		const FStateTreeExecutionFrame& CurrentFrame = Exec.ActiveFrames[FrameIndex];
		const UStateTree* CurrentStateTree = CurrentFrame.StateTree;
		const int32 CurrentActiveNodeIndex = CurrentFrame.ActiveNodeIndex.AsInt32();

		FCurrentlyProcessedFrameScope FrameScope(*this, CurrentParentFrame, CurrentFrame);

		for (int32 StateIndex = CurrentFrame.ActiveStates.Num() - 1; StateIndex >= 0; StateIndex--)
		{
			const FStateTreeStateHandle CurrentHandle = CurrentFrame.ActiveStates[StateIndex];
			const FCompactStateTreeState& State = CurrentStateTree->States[CurrentHandle.Index];

			FCurrentlyProcessedStateScope StateScope(*this, CurrentHandle);
			UE_STATETREE_DEBUG_STATE_EVENT(this, CurrentHandle, EStateTreeTraceEventType::OnStateCompleted);
			STATETREE_LOG(Verbose, TEXT("%*sState '%s'"), (FrameIndex + StateIndex + 1) * UE::StateTree::Debug::IndentSize, TEXT(""), *GetSafeStateName(CurrentFrame, CurrentHandle));

			// Notify Tasks
			for (int32 TaskIndex = (State.TasksBegin + State.TasksNum) - 1; TaskIndex >= State.TasksBegin; TaskIndex--)
			{
				// Call task completed only if EnterState() was called.
				if (TaskIndex <= CurrentActiveNodeIndex)
				{
					const FStateTreeTaskBase& Task = CurrentStateTree->Nodes[TaskIndex].Get<const FStateTreeTaskBase>();

					// Ignore disabled task
					if (Task.bTaskEnabled == false)
					{
						STATETREE_LOG(VeryVerbose, TEXT("%*sSkipped 'StateCompleted' for disabled Task: '%s'"), UE::StateTree::Debug::IndentSize, TEXT(""), *Task.Name.ToString());
						continue;
					}

					const FStateTreeDataView TaskInstanceView = GetDataView(CurrentParentFrame, CurrentFrame, Task.InstanceDataHandle);
					FNodeInstanceDataScope DataScope(*this, &Task, TaskIndex, Task.InstanceDataHandle, TaskInstanceView);

					STATETREE_LOG(Verbose, TEXT("%*sTask '%s'.StateCompleted()"), (FrameIndex + StateIndex + 1) * UE::StateTree::Debug::IndentSize, TEXT(""), *Task.Name.ToString());
					Task.StateCompleted(*this, Exec.LastTickStatus, CurrentFrame.ActiveStates);
				}
			}

			// Call state change events on conditions if needed.
			constexpr bool bOnActiveInstances = true;
			constexpr EConditionFunction ConditionFunction = EConditionFunction::StateCompleted;
			RunAllConditionsInternal<bOnActiveInstances, ConditionFunction>(CurrentParentFrame, CurrentFrame, State, FStateTreeTransitionResult());
		}
	}
}

void FStateTreeExecutionContext::TickGlobalEvaluatorsForFrameOnActiveInstances(const float DeltaTime, const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& Frame)
{
	constexpr bool bOnActiveInstances = true;
	TickGlobalEvaluatorsForFrameInternal<bOnActiveInstances>(DeltaTime, ParentFrame, Frame);
}

void FStateTreeExecutionContext::TickGlobalEvaluatorsForFrameWithValidation(const float DeltaTime, const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& Frame)
{
	constexpr bool bOnActiveInstances = false;
	TickGlobalEvaluatorsForFrameInternal<bOnActiveInstances>(DeltaTime, ParentFrame, Frame);
}

template<bool bOnActiveInstances>
void FStateTreeExecutionContext::TickGlobalEvaluatorsForFrameInternal(const float DeltaTime, const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& Frame)
{
	check(Frame.bIsGlobalFrame);

	const UStateTree* CurrentStateTree = Frame.StateTree;
	const int32 EvaluatorEnd = CurrentStateTree->EvaluatorsBegin + CurrentStateTree->EvaluatorsNum;
	if (CurrentStateTree->EvaluatorsBegin < EvaluatorEnd)
	{
		FCurrentlyProcessedFrameScope FrameScope(*this, ParentFrame, Frame);

		for (int32 EvalIndex = CurrentStateTree->EvaluatorsBegin; EvalIndex < EvaluatorEnd; ++EvalIndex)
		{
			const FStateTreeEvaluatorBase& Eval = CurrentStateTree->Nodes[EvalIndex].Get<const FStateTreeEvaluatorBase>();
			FStateTreeDataView EvalInstanceView = bOnActiveInstances
				? GetDataView(ParentFrame, Frame, Eval.InstanceDataHandle)
				: GetDataViewOrTemporary(ParentFrame, Frame, Eval.InstanceDataHandle);
			FNodeInstanceDataScope DataScope(*this, &Eval, EvalIndex, Eval.InstanceDataHandle, EvalInstanceView);

			// Copy bound properties.
			if (Eval.BindingsBatch.IsValid())
			{
				if constexpr (bOnActiveInstances)
				{
					CopyBatchOnActiveInstances(ParentFrame, Frame, EvalInstanceView, Eval.BindingsBatch);
				}
				else
				{
					CopyBatchWithValidation(ParentFrame, Frame, EvalInstanceView, Eval.BindingsBatch);
				}
			}

			STATETREE_LOG(VeryVerbose, TEXT("  Tick: '%s'"), *Eval.Name.ToString());
			UE_STATETREE_DEBUG_EVALUATOR_TICK(this, CurrentStateTree, EvalIndex);
			{
				QUICK_SCOPE_CYCLE_COUNTER(StateTree_Eval_Tick);
				Eval.Tick(*this, DeltaTime);
				UE_STATETREE_DEBUG_EVALUATOR_EVENT(this, EvalIndex, EvalInstanceView, EStateTreeTraceEventType::OnTicked);
			}

			// Copy bound properties.
			if (Eval.OutputBindingsBatch.IsValid())
			{
				if constexpr (bOnActiveInstances)
				{
					CopyBatchOnActiveInstances(ParentFrame, Frame, EvalInstanceView, Eval.OutputBindingsBatch);
				}
				else
				{
					CopyBatchWithValidation(ParentFrame, Frame, EvalInstanceView, Eval.OutputBindingsBatch);
				}
			}
		}
	}
}

EStateTreeRunStatus FStateTreeExecutionContext::TickEvaluatorsAndGlobalTasks(const float DeltaTime, const bool bTickGlobalTasks)
{
	// When a global task is completed it completes the tree execution.
	// A global task can complete async. See CompletedStates.
	// When a global task fails, stop ticking the following tasks.

	CSV_SCOPED_TIMING_STAT(StateTree, TickEvaluators);
	UE_STATETREE_DEBUG_SCOPED_PHASE(this, EStateTreeUpdatePhase::TickingGlobalTasks);

	STATETREE_LOG(VeryVerbose, TEXT("Ticking Evaluators & Global Tasks"));

	FStateTreeExecutionState& Exec = GetExecState();

	EStateTreeRunStatus Result = EStateTreeRunStatus::Running;

	for (int32 FrameIndex = 0; FrameIndex < Exec.ActiveFrames.Num(); ++FrameIndex)
	{
		FStateTreeExecutionFrame* CurrentParentFrame = FrameIndex > 0 ? &Exec.ActiveFrames[FrameIndex - 1] : nullptr;
		FStateTreeExecutionFrame& CurrentFrame = Exec.ActiveFrames[FrameIndex];
		if (CurrentFrame.bIsGlobalFrame)
		{
			FCurrentlyProcessedFrameScope FrameScope(*this, CurrentParentFrame, CurrentFrame);

			const EStateTreeRunStatus FrameResult = TickEvaluatorsAndGlobalTasksForFrame(DeltaTime, bTickGlobalTasks, FrameIndex, CurrentParentFrame, &CurrentFrame);
			Result = UE::StateTree::ExecutionContext::GetPriorityRunStatus(Result, FrameResult);

			if (Result == EStateTreeRunStatus::Failed)
			{
				break;
			}
		}
	}

	Exec.bHasPendingCompletedState = Exec.bHasPendingCompletedState || Result != EStateTreeRunStatus::Running;
	return Result;
}

EStateTreeRunStatus FStateTreeExecutionContext::TickEvaluatorsAndGlobalTasksForFrame(const float DeltaTime, const bool bTickGlobalTasks, const int32 FrameIndex, const FStateTreeExecutionFrame* CurrentParentFrame, const TNotNull<FStateTreeExecutionFrame*> CurrentFrame)
{
	check(CurrentFrame->bIsGlobalFrame);

	EStateTreeRunStatus Result = EStateTreeRunStatus::Running;

	// Tick evaluators
	TickGlobalEvaluatorsForFrameOnActiveInstances(DeltaTime, CurrentParentFrame, *CurrentFrame);

	if (bTickGlobalTasks)
	{
		using namespace UE::StateTree;

		const UStateTree* CurrentStateTree = CurrentFrame->StateTree;
		FTasksCompletionStatus CurrentGlobalTasksStatus = CurrentFrame->ActiveTasksStatus.GetStatus(CurrentStateTree);
		if (!CurrentGlobalTasksStatus.HasAnyFailed())
		{
			const bool bHasEvents = EventQueue && EventQueue->HasEvents();
			if (ExecutionContext::Private::bCopyBoundPropertiesOnNonTickedTask || CurrentStateTree->ShouldTickGlobalTasks(bHasEvents))
			{
				// Update Tasks data and tick if possible (ie. if no task has yet failed and bShouldTickTasks is true)
				FTickTaskArguments TickArgs;
				TickArgs.DeltaTime = DeltaTime;
				TickArgs.TasksBegin = CurrentStateTree->GlobalTasksBegin;
				TickArgs.TasksNum = CurrentStateTree->GlobalTasksNum;
				TickArgs.Indent = FrameIndex + 1;
				TickArgs.ParentFrame = CurrentParentFrame;
				TickArgs.Frame = CurrentFrame;
				TickArgs.TasksCompletionStatus = &CurrentGlobalTasksStatus;
				TickArgs.bIsGlobalTasks = true;
				TickArgs.bShouldTickTasks = true;
				TickTasks(TickArgs);
			}
		}

		// Completed global task stops the frame execution.
		const ETaskCompletionStatus GlobalTaskStatus = CurrentGlobalTasksStatus.GetCompletionStatus();
		Result = ExecutionContext::CastToRunStatus(GlobalTaskStatus);
	}

	return Result;
}

TSharedPtr<FStateTreeExecutionContext::FSelectStateResult> FStateTreeExecutionContext::GetCurrentlyProcessedStateSelection() const
{
	// A transition requests can be in progress or already succeeded.
	//Use the current transition request first (for linked frame).
	return CurrentlyProcessedTemporaryStorage ? CurrentlyProcessedTemporaryStorage : RequestedTransition ? RequestedTransition->Selection : nullptr;
}

void FStateTreeExecutionContext::MoveGlobalTemporariesToInstances(FStateTreeExecutionFrame& Frame)
{
	TNotNull<const UStateTree*> CurrentStateTree = Frame.StateTree;

	// Move evaluators
	const int32 EvaluatorsEnd = CurrentStateTree->EvaluatorsBegin + CurrentStateTree->EvaluatorsNum;
	for (int32 EvalIndex = CurrentStateTree->EvaluatorsBegin; EvalIndex < EvaluatorsEnd; ++EvalIndex)
	{
		const FStateTreeEvaluatorBase& Eval = CurrentStateTree->Nodes[EvalIndex].Get<const FStateTreeEvaluatorBase>();
		MoveTemporaryToInstance(Frame, FStateTreeIndex16(EvalIndex), Eval.InstanceDataHandle);
	}

	// Start Global tasks
	// Only move data for tasks that successfully create their EnterState.
	const int32 ActiveNodeIndexMax = Frame.ActiveNodeIndex.IsValid() ? Frame.ActiveNodeIndex.AsInt32() + 1 : 0;
	const int32 AssetTaskMax = FMath::Min(ActiveNodeIndexMax, static_cast<int32>(CurrentStateTree->GlobalTasksBegin + CurrentStateTree->GlobalTasksNum));
	for (int32 AssetTaskIndex = CurrentStateTree->GlobalTasksBegin; AssetTaskIndex < AssetTaskMax; ++AssetTaskIndex)
	{
		const FStateTreeTaskBase& Task = CurrentStateTree->Nodes[AssetTaskIndex].Get<const FStateTreeTaskBase>();
		MoveTemporaryToInstance(Frame, FStateTreeIndex16(AssetTaskIndex), Task.InstanceDataHandle);
	}
}

EStateTreeRunStatus FStateTreeExecutionContext::StartGlobalsForFrameOnActiveInstances(const FStateTreeExecutionFrame* ParentFrame, FStateTreeExecutionFrame& Frame, FStateTreeTransitionResult& Transition)
{
	constexpr bool bOnActiveInstances = true;
	constexpr bool bMoveFromTemporary = false;
	return StartGlobalsForFrameInternal<bOnActiveInstances, bMoveFromTemporary>(ParentFrame, Frame, Transition);
}

EStateTreeRunStatus FStateTreeExecutionContext::StartGlobalsForFrameWithValidation(const FStateTreeExecutionFrame* ParentFrame, FStateTreeExecutionFrame& Frame, FStateTreeTransitionResult& Transition)
{
	constexpr bool bOnActiveInstances = false;
	constexpr bool bMoveFromTemporary = false;
	return StartGlobalsForFrameInternal<bOnActiveInstances, bMoveFromTemporary>(ParentFrame, Frame, Transition);
}

template<bool bOnActiveInstances, bool bMoveFromTemporary>
EStateTreeRunStatus FStateTreeExecutionContext::StartGlobalsForFrameInternal(const FStateTreeExecutionFrame* CurrentParentFrame, FStateTreeExecutionFrame& CurrentFrame, FStateTreeTransitionResult& Transition)
{
	if (!CurrentFrame.bIsGlobalFrame)
	{
		return EStateTreeRunStatus::Running;
	}

	CSV_SCOPED_TIMING_STAT(StateTree, StartEvaluators);

	FCurrentlyProcessedFrameScope FrameScope(*this, CurrentParentFrame, CurrentFrame);
	
	constexpr EStateTreeUpdatePhase StartGlobalTasksPhase = bOnActiveInstances ? EStateTreeUpdatePhase::StartGlobalTasks : EStateTreeUpdatePhase::StartGlobalTasksForSelection;

	UE_STATETREE_DEBUG_ENTER_PHASE(this, StartGlobalTasksPhase);

	CurrentFrame.bHaveEntered = true;

	EStateTreeRunStatus Result = EStateTreeRunStatus::Running;
	const UE::StateTree::FActiveFrameID CurrentFrameID = CurrentFrame.FrameID;
	const UStateTree* CurrentStateTree = CurrentFrame.StateTree;
	UE::StateTree::FTasksCompletionStatus CurrentTasksStatus = CurrentFrame.ActiveTasksStatus.GetStatus(CurrentStateTree);

	// Start evaluators
	const int32 EvaluatorsEnd = CurrentStateTree->EvaluatorsBegin + CurrentStateTree->EvaluatorsNum;
	for (int32 EvalIndex = CurrentStateTree->EvaluatorsBegin; EvalIndex < EvaluatorsEnd; ++EvalIndex)
	{
		const FStateTreeEvaluatorBase& Eval = CurrentStateTree->Nodes[EvalIndex].Get<const FStateTreeEvaluatorBase>();
		FStateTreeDataView EvalInstanceView;
		if constexpr (bMoveFromTemporary)
		{
			EvalInstanceView = MoveTemporaryToInstance(CurrentFrame, FStateTreeIndex16(EvalIndex), Eval.InstanceDataHandle);
		}
		else if constexpr (bOnActiveInstances)
		{
			EvalInstanceView = GetDataView(CurrentParentFrame, CurrentFrame, Eval.InstanceDataHandle);
		}
		else
		{
			EvalInstanceView = GetDataViewOrTemporary(CurrentParentFrame, CurrentFrame, Eval.InstanceDataHandle);
		}

		if constexpr (!bOnActiveInstances)
		{
			if (!EvalInstanceView.IsValid())
			{
				EvalInstanceView = AddTemporaryInstance(CurrentFrame, FStateTreeIndex16(EvalIndex), Eval.InstanceDataHandle, CurrentFrame.StateTree->DefaultInstanceData.GetStruct(Eval.InstanceTemplateIndex.Get()));
				check(EvalInstanceView.IsValid());
			}
		}

		FNodeInstanceDataScope DataScope(*this, &Eval, EvalIndex, Eval.InstanceDataHandle, EvalInstanceView);
		CurrentFrame.ActiveNodeIndex = FStateTreeIndex16(EvalIndex);

		// Copy bound properties.
		if (Eval.BindingsBatch.IsValid())
		{
			if constexpr (bOnActiveInstances)
			{
				CopyBatchOnActiveInstances(CurrentParentFrame, CurrentFrame, EvalInstanceView, Eval.BindingsBatch);
			}
			else
			{
				CopyBatchWithValidation(CurrentParentFrame, CurrentFrame, EvalInstanceView, Eval.BindingsBatch);
			}
		}

		STATETREE_LOG(Verbose, TEXT("  Start: '%s'"), *Eval.Name.ToString());
		UE_STATETREE_DEBUG_EVALUATOR_ENTER_TREE(this, CurrentStateTree, FStateTreeIndex16(EvalIndex));
		{
			QUICK_SCOPE_CYCLE_COUNTER(StateTree_Eval_TreeStart);
			Eval.TreeStart(*this);

			UE_STATETREE_DEBUG_EVALUATOR_EVENT(this, EvalIndex, EvalInstanceView, EStateTreeTraceEventType::OnTreeStarted);
		}

		// Copy output bound properties.
		if (Eval.OutputBindingsBatch.IsValid())
		{
			if constexpr (bOnActiveInstances)
			{
				CopyBatchOnActiveInstances(CurrentParentFrame, CurrentFrame, EvalInstanceView, Eval.OutputBindingsBatch);
			}
			else
			{
				CopyBatchWithValidation(CurrentParentFrame, CurrentFrame, EvalInstanceView, Eval.OutputBindingsBatch);
			}
		}
	}

	// Start Global tasks
	// Even if we call Enter/ExitState() on global tasks, they do not enter any specific state.
	for (int32 GlobalTaskIndex = 0; GlobalTaskIndex < CurrentStateTree->GlobalTasksNum; ++GlobalTaskIndex)
	{
		const int32 AssetTaskIndex = CurrentStateTree->GlobalTasksBegin + GlobalTaskIndex;
		const FStateTreeTaskBase& Task = CurrentStateTree->Nodes[AssetTaskIndex].Get<const FStateTreeTaskBase>();

		FStateTreeDataView TaskDataView;
		if constexpr (bMoveFromTemporary)
		{
			TaskDataView = MoveTemporaryToInstance(CurrentFrame, FStateTreeIndex16(AssetTaskIndex), Task.InstanceDataHandle);
		}

		// Ignore disabled task
		if (Task.bTaskEnabled == false)
		{
			STATETREE_LOG(VeryVerbose, TEXT("%*sSkipped 'EnterState' for disabled Task: '%s'"), UE::StateTree::Debug::IndentSize, TEXT(""), *Task.Name.ToString());
			continue;
		}

		if constexpr (!bMoveFromTemporary)
		{
			TaskDataView = bOnActiveInstances
				? GetDataView(CurrentParentFrame, CurrentFrame, Task.InstanceDataHandle)
				: GetDataViewOrTemporary(CurrentParentFrame, CurrentFrame, Task.InstanceDataHandle);
		}
		if constexpr (!bOnActiveInstances)
		{
			if (!TaskDataView.IsValid())
			{
				TaskDataView = AddTemporaryInstance(CurrentFrame, FStateTreeIndex16(AssetTaskIndex), Task.InstanceDataHandle, CurrentFrame.StateTree->DefaultInstanceData.GetStruct(Task.InstanceTemplateIndex.Get()));
				check(TaskDataView.IsValid())
			}
		}

		FNodeInstanceDataScope DataScope(*this, &Task, AssetTaskIndex, Task.InstanceDataHandle, TaskDataView);
		CurrentFrame.ActiveNodeIndex = FStateTreeIndex16(AssetTaskIndex);

		// Copy bound properties.
		if (Task.BindingsBatch.IsValid())
		{
			if constexpr (bOnActiveInstances)
			{
				CopyBatchOnActiveInstances(CurrentParentFrame, CurrentFrame, TaskDataView, Task.BindingsBatch);
			}
			else
			{
				CopyBatchWithValidation(CurrentParentFrame, CurrentFrame, TaskDataView, Task.BindingsBatch);
			}
		}

		STATETREE_LOG(Verbose, TEXT("  Start: '%s'"), *Task.Name.ToString());
		UE_STATETREE_DEBUG_TASK_ENTER_STATE(this, CurrentStateTree, FStateTreeIndex16(AssetTaskIndex));

		EStateTreeRunStatus TaskRunStatus = EStateTreeRunStatus::Unset;
		{
			QUICK_SCOPE_CYCLE_COUNTER(StateTree_Task_TreeStart);
			TaskRunStatus = Task.EnterState(*this, Transition);
		}

		UE::StateTree::ETaskCompletionStatus TaskStatus = UE::StateTree::ExecutionContext::CastToTaskStatus(TaskRunStatus);
		const UE::StateTree::FTasksCompletionStatus::FSetStatusResult TaskStatusResult = CurrentTasksStatus.SetTaskStatusWithPriority(GlobalTaskIndex, TaskStatus);
		TaskStatus = TaskStatusResult.CurrentStatus;

		TaskRunStatus = UE::StateTree::ExecutionContext::CastToRunStatus(TaskStatus);
		UE_STATETREE_DEBUG_TASK_EVENT(this, AssetTaskIndex, TaskDataView, EStateTreeTraceEventType::OnEntered, TaskRunStatus);

		// Copy output bound properties if the task didn't fail
		if (TaskRunStatus != EStateTreeRunStatus::Failed && Task.OutputBindingsBatch.IsValid())
		{
			if constexpr (bOnActiveInstances)
			{
				CopyBatchOnActiveInstances(CurrentParentFrame, CurrentFrame, TaskDataView, Task.OutputBindingsBatch);
			}
			else
			{
				CopyBatchWithValidation(CurrentParentFrame, CurrentFrame, TaskDataView, Task.OutputBindingsBatch);
			}
		}

		// Broadcast task completion dispatcher only if status actually changed
		if (TaskStatusResult.bChanged && Task.bHasTaskCompletionDelegateDispatcher)
		{
			BroadcastTaskCompletionDispatchers(CurrentStateTree, AssetTaskIndex, TaskStatus);
		}

		if (CurrentTasksStatus.IsConsideredForCompletion(GlobalTaskIndex))
		{
			Result = UE::StateTree::ExecutionContext::GetPriorityRunStatus(Result, TaskRunStatus);
			if (Result == EStateTreeRunStatus::Failed)
			{
				break;
			}
		}
	}

	UE_STATETREE_DEBUG_EXIT_PHASE(this, StartGlobalTasksPhase);

	return Result;
}

// Deprecated
EStateTreeRunStatus FStateTreeExecutionContext::StartEvaluatorsAndGlobalTasks(FStateTreeIndex16& OutLastInitializedTaskIndex)
{
	return StartEvaluatorsAndGlobalTasks();
}

EStateTreeRunStatus FStateTreeExecutionContext::StartEvaluatorsAndGlobalTasks()
{
	CSV_SCOPED_TIMING_STAT(StateTree, StartEvaluatorsAndGlobalTasks);

	STATETREE_LOG(Verbose, TEXT("Start Evaluators & Global tasks"));

	FStateTreeExecutionState& Exec = GetExecState();

	EStateTreeRunStatus Result = EStateTreeRunStatus::Running;
	FStateTreeTransitionResult Transition{};
	Transition.TargetState = FStateTreeStateHandle::Root;
	Transition.CurrentRunStatus = EStateTreeRunStatus::Running;

	for (int32 FrameIndex = 0; FrameIndex < Exec.ActiveFrames.Num(); FrameIndex++)
	{
		const FStateTreeExecutionFrame* CurrentParentFrame = FrameIndex > 0 ? &Exec.ActiveFrames[FrameIndex - 1] : nullptr;
		FStateTreeExecutionFrame& CurrentFrame = Exec.ActiveFrames[FrameIndex];
		EStateTreeRunStatus FrameResult = StartGlobalsForFrameOnActiveInstances(CurrentParentFrame, CurrentFrame, Transition);
		Result = UE::StateTree::ExecutionContext::GetPriorityRunStatus(Result, FrameResult);
		if (FrameResult == EStateTreeRunStatus::Failed)
		{
			break;
		}
	}

	return Result;
}

EStateTreeRunStatus FStateTreeExecutionContext::StartTemporaryEvaluatorsAndGlobalTasks(const FStateTreeExecutionFrame* CurrentParentFrame, FStateTreeExecutionFrame& CurrentFrame)
{
	STATETREE_LOG(Verbose, TEXT("Start Temporary Evaluators & Global tasks while trying to select linked asset: %s"), *GetNameSafe(CurrentFrame.StateTree));

	FStateTreeTransitionResult Transition;
	Transition.ChangeType = EStateTreeStateChangeType::Changed;
	Transition.CurrentRunStatus = EStateTreeRunStatus::Running;
	return StartGlobalsForFrameWithValidation(CurrentParentFrame, CurrentFrame, Transition);
}

// Deprecated
void FStateTreeExecutionContext::StopEvaluatorsAndGlobalTasks(const EStateTreeRunStatus CompletionStatus, const FStateTreeIndex16 LastInitializedTaskIndex)
{
	StopEvaluatorsAndGlobalTasks(CompletionStatus);
}

void FStateTreeExecutionContext::StopEvaluatorsAndGlobalTasks(const EStateTreeRunStatus CompletionStatus)
{
	CSV_SCOPED_TIMING_STAT(StateTree, StopEvaluatorsAndGlobalTasks);

	STATETREE_LOG(Verbose, TEXT("Stop Evaluators & Global Tasks"));

	FStateTreeExecutionState& Exec = GetExecState();

	// Update bindings
	CopyAllBindingsOnActiveInstances(ECopyBindings::ExitState);

	// Call in reverse order.
	FStateTreeTransitionResult Transition;
	Transition.TargetState = FStateTreeStateHandle::FromCompletionStatus(CompletionStatus);
	Transition.CurrentRunStatus = CompletionStatus;

	for (int32 FrameIndex = Exec.ActiveFrames.Num() - 1; FrameIndex >= 0; FrameIndex--)
	{
		const FStateTreeExecutionFrame* CurrentParentFrame = FrameIndex > 0 ? &Exec.ActiveFrames[FrameIndex - 1] : nullptr;
		FStateTreeExecutionFrame& CurrentFrame = Exec.ActiveFrames[FrameIndex];
		if (CurrentFrame.bIsGlobalFrame)
		{
			StopGlobalsForFrameOnActiveInstances(CurrentParentFrame, CurrentFrame, Transition);
		}
	}
}

void FStateTreeExecutionContext::StopGlobalsForFrameOnActiveInstances(const FStateTreeExecutionFrame* ParentFrame, FStateTreeExecutionFrame& Frame, const FStateTreeTransitionResult& Transition)
{
	constexpr bool bOnActiveInstances = true;
	StopGlobalsForFrameInternal<bOnActiveInstances>(ParentFrame, Frame, Transition);
}

void FStateTreeExecutionContext::StopGlobalsForFrameWithValidation(const FStateTreeExecutionFrame* ParentFrame, FStateTreeExecutionFrame& Frame, const FStateTreeTransitionResult& Transition)
{
	constexpr bool bOnActiveInstances = false;
	StopGlobalsForFrameInternal<bOnActiveInstances>(ParentFrame, Frame, Transition);
}

template<bool bOnActiveInstances>
void FStateTreeExecutionContext::StopGlobalsForFrameInternal(const FStateTreeExecutionFrame* ParentFrame, FStateTreeExecutionFrame& Frame, const FStateTreeTransitionResult& Transition)
{
	// Special case when we select a new root. See SelectState. We don't want to stop the globals.
	if (!Frame.bIsGlobalFrame || !Frame.bHaveEntered)
	{
		return;
	}

	FCurrentlyProcessedFrameScope FrameScope(*this, ParentFrame, Frame);

	constexpr EStateTreeUpdatePhase StopGlobalTasksPhase = bOnActiveInstances ? EStateTreeUpdatePhase::StopGlobalTasks : EStateTreeUpdatePhase::StopGlobalTasksForSelection;
	UE_STATETREE_DEBUG_ENTER_PHASE(this, StopGlobalTasksPhase);

	const UStateTree* CurrentStateTree = Frame.StateTree;

	const int32 GlobalTasksEnd = FMath::Min(Frame.ActiveNodeIndex.AsInt32() + 1, CurrentStateTree->GlobalTasksBegin + CurrentStateTree->GlobalTasksNum);
	for (int32 TaskIndex = GlobalTasksEnd - 1; TaskIndex >= CurrentStateTree->GlobalTasksBegin; --TaskIndex)
	{
		const FStateTreeTaskBase& Task = CurrentStateTree->Nodes[TaskIndex].Get<const FStateTreeTaskBase>();
		const FStateTreeDataView TaskInstanceView = bOnActiveInstances
			? GetDataView(ParentFrame, Frame, Task.InstanceDataHandle)
			: GetDataViewOrTemporary(ParentFrame, Frame, Task.InstanceDataHandle);
		FNodeInstanceDataScope DataScope(*this, &Task, TaskIndex, Task.InstanceDataHandle, TaskInstanceView);

		// Ignore disabled task
		if (Task.bTaskEnabled == false)
		{
			STATETREE_LOG(VeryVerbose, TEXT("%*sSkipped 'ExitState' for disabled Task: '%s'"), UE::StateTree::Debug::IndentSize, TEXT(""), *Task.Name.ToString());
		}
		else
		{
			STATETREE_LOG(Verbose, TEXT("  Stop: '%s'"), *Task.Name.ToString());
			UE_STATETREE_DEBUG_TASK_EXIT_STATE(this, CurrentStateTree, FStateTreeIndex16(TaskIndex));
			{
				QUICK_SCOPE_CYCLE_COUNTER(StateTree_Task_TreeStop);
				Task.ExitState(*this, Transition);
			}

			if (Task.OutputBindingsBatch.IsValid())
			{
				if constexpr (bOnActiveInstances)
				{
					CopyBatchOnActiveInstances(ParentFrame, Frame, TaskInstanceView, Task.OutputBindingsBatch);
				}
				else
				{
					CopyBatchWithValidation(ParentFrame, Frame, TaskInstanceView, Task.OutputBindingsBatch);
				}
			}

			UE_STATETREE_DEBUG_TASK_EVENT(this, TaskIndex, TaskInstanceView, EStateTreeTraceEventType::OnExited, Transition.CurrentRunStatus);
		}
		Frame.ActiveNodeIndex = FStateTreeIndex16(TaskIndex - 1);
	}

	const int32 EvaluatorsEnd = FMath::Min(Frame.ActiveNodeIndex.AsInt32() + 1, CurrentStateTree->EvaluatorsBegin + CurrentStateTree->EvaluatorsNum);
	for (int32 EvalIndex = EvaluatorsEnd - 1; EvalIndex >= CurrentStateTree->EvaluatorsBegin; --EvalIndex)
	{
		const FStateTreeEvaluatorBase& Eval = CurrentStateTree->Nodes[EvalIndex].Get<const FStateTreeEvaluatorBase>();
		const FStateTreeDataView EvalInstanceView = bOnActiveInstances
			? GetDataView(ParentFrame, Frame, Eval.InstanceDataHandle)
			: GetDataViewOrTemporary(ParentFrame, Frame, Eval.InstanceDataHandle);
		FNodeInstanceDataScope DataScope(*this, &Eval, EvalIndex, Eval.InstanceDataHandle, EvalInstanceView);

		STATETREE_LOG(Verbose, TEXT("  Stop: '%s'"), *Eval.Name.ToString());
		UE_STATETREE_DEBUG_EVALUATOR_EXIT_TREE(this, CurrentStateTree, FStateTreeIndex16(EvalIndex));
		{
			QUICK_SCOPE_CYCLE_COUNTER(StateTree_Eval_TreeStop);
			Eval.TreeStop(*this);

			if (Eval.OutputBindingsBatch.IsValid())
			{
				if constexpr (bOnActiveInstances)
				{
					CopyBatchOnActiveInstances(ParentFrame, Frame, EvalInstanceView, Eval.OutputBindingsBatch);
				}
				else
				{
					CopyBatchWithValidation(ParentFrame, Frame, EvalInstanceView, Eval.OutputBindingsBatch);
				}
			}

			UE_STATETREE_DEBUG_EVALUATOR_EVENT(this, EvalIndex, EvalInstanceView, EStateTreeTraceEventType::OnTreeStopped);
		}
		Frame.ActiveNodeIndex = FStateTreeIndex16(EvalIndex - 1);
	}

	Frame.ActiveNodeIndex = FStateTreeIndex16::Invalid;
	Frame.bHaveEntered = false;

	UE_STATETREE_DEBUG_EXIT_PHASE(this, StopGlobalTasksPhase);
}

// Deprecated
void FStateTreeExecutionContext::CallStopOnEvaluatorsAndGlobalTasks(const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& Frame, const FStateTreeTransitionResult& Transition, const FStateTreeIndex16 LastInitializedTaskIndex /*= FStateTreeIndex16()*/)
{
	constexpr bool bOnActiveInstances = true;
	StopGlobalsForFrameInternal<bOnActiveInstances>(ParentFrame, const_cast<FStateTreeExecutionFrame&>(Frame), Transition);
}

// Deprecated
void FStateTreeExecutionContext::StopTemporaryEvaluatorsAndGlobalTasks(const FStateTreeExecutionFrame* CurrentParentFrame, const FStateTreeExecutionFrame& CurrentFrame)
{
	StopTemporaryEvaluatorsAndGlobalTasks(CurrentParentFrame, const_cast<FStateTreeExecutionFrame&>(CurrentFrame), EStateTreeRunStatus::Running);
}

void FStateTreeExecutionContext::StopTemporaryEvaluatorsAndGlobalTasks(const FStateTreeExecutionFrame* CurrentParentFrame, FStateTreeExecutionFrame& CurrentFrame, EStateTreeRunStatus StartResult)
{
	STATETREE_LOG(Verbose, TEXT("Stop Temporary Evaluators & Global tasks"));

	// Create temporary transition to stop the unused global tasks and evaluators.
	FStateTreeTransitionResult Transition;
	Transition.TargetState = FStateTreeStateHandle::FromCompletionStatus(StartResult);
	Transition.CurrentRunStatus = StartResult;
	Transition.ChangeType = EStateTreeStateChangeType::Changed;
	StopGlobalsForFrameWithValidation(CurrentParentFrame, CurrentFrame, Transition);
}

EStateTreeRunStatus FStateTreeExecutionContext::TickTasks(const float DeltaTime)
{
	// When a task is completed it also completes the state and triggers the completion transition (because LastTickStatus is set).
	// A task can complete async.
	// When a task fails, stop ticking the following tasks.
	// When no task ticks, then the leaf completes.

	CSV_SCOPED_TIMING_STAT(StateTree, TickTasks);
	UE_STATETREE_DEBUG_SCOPED_PHASE(this, EStateTreeUpdatePhase::TickingTasks);

	using namespace UE::StateTree;

	FStateTreeExecutionState& Exec = GetExecState();
	Exec.bHasPendingCompletedState = false;

	if (Exec.ActiveFrames.IsEmpty())
	{
		return EStateTreeRunStatus::Failed;
	}

	int32 NumTotalEnabledTasks = 0;
	const bool bCopyBoundPropertiesOnNonTickedTask = ExecutionContext::Private::bCopyBoundPropertiesOnNonTickedTask;

	FTickTaskArguments TickArgs;
	TickArgs.DeltaTime = DeltaTime;
	TickArgs.bIsGlobalTasks = false;
	TickArgs.bShouldTickTasks = true;

	STATETREE_CLOG(Exec.ActiveFrames.Num() > 0, VeryVerbose, TEXT("Ticking Tasks"));

	for (int32 FrameIndex = 0; FrameIndex < Exec.ActiveFrames.Num(); ++FrameIndex)
	{
		TickArgs.ParentFrame = FrameIndex > 0 ? &Exec.ActiveFrames[FrameIndex - 1] : nullptr;
		TickArgs.Frame = &Exec.ActiveFrames[FrameIndex];
		const UStateTree* CurrentStateTree = TickArgs.Frame->StateTree;

		FCurrentlyProcessedFrameScope FrameScope(*this, TickArgs.ParentFrame, *TickArgs.Frame);

		if (ExecutionContext::Private::bTickGlobalNodesFollowingTreeHierarchy)
		{
			if (TickArgs.Frame->bIsGlobalFrame)
			{
				constexpr bool bTickGlobalTasks = true;
				const EStateTreeRunStatus FrameResult = TickEvaluatorsAndGlobalTasksForFrame(DeltaTime, bTickGlobalTasks, FrameIndex, TickArgs.ParentFrame, TickArgs.Frame);
				if (FrameResult != EStateTreeRunStatus::Running)
				{
					if (ExecutionContext::Private::bGlobalTasksCompleteOwningFrame == false || FrameIndex == 0)
					{
						// Stop the tree execution when it's the root frame or if the previous behavior is desired.
						Exec.RequestedStop = ExecutionContext::GetPriorityRunStatus(Exec.RequestedStop, FrameResult);
					}
					TickArgs.bShouldTickTasks = false;
					break;
				}
			}
		}

		for (int32 StateIndex = 0; StateIndex < TickArgs.Frame->ActiveStates.Num(); ++StateIndex)
		{
			const FStateTreeStateHandle CurrentHandle = TickArgs.Frame->ActiveStates[StateIndex];
			const FCompactStateTreeState& CurrentState = CurrentStateTree->States[CurrentHandle.Index];
			FTasksCompletionStatus CurrentCompletionStatus = TickArgs.Frame->ActiveTasksStatus.GetStatus(CurrentState);

			TickArgs.StateID = TickArgs.Frame->ActiveStates.StateIDs[StateIndex];
			TickArgs.TasksCompletionStatus = &CurrentCompletionStatus;

			FCurrentlyProcessedStateScope StateScope(*this, CurrentHandle);
			UE_STATETREE_DEBUG_SCOPED_STATE(this, CurrentHandle);

			STATETREE_CLOG(CurrentState.TasksNum > 0, VeryVerbose, TEXT("%*sState '%s'")
				, (FrameIndex + StateIndex + 1) * Debug::IndentSize, TEXT("")
				, *DebugGetStatePath(Exec.ActiveFrames, TickArgs.Frame, StateIndex));

			if (CurrentState.bCopyParameterBindingsOnTick
				&& CurrentState.ParameterDataHandle.IsValid()
				&& CurrentState.ParameterBindingsBatch.IsValid())
			{
				const FStateTreeDataView StateParamsDataView = GetDataView(TickArgs.ParentFrame, *TickArgs.Frame, CurrentState.ParameterDataHandle);
				CopyBatchOnActiveInstances(TickArgs.ParentFrame, *TickArgs.Frame, StateParamsDataView, CurrentState.ParameterBindingsBatch);
			}

			const bool bHasEvents = EventQueue && EventQueue->HasEvents();
			bool bRequestLoopStop = false;
			if (bCopyBoundPropertiesOnNonTickedTask || CurrentState.ShouldTickTasks(bHasEvents))
			{
				// Update Tasks data and tick if possible (ie. if no task has yet failed and bShouldTickTasks is true)
				TickArgs.TasksBegin = CurrentState.TasksBegin;
				TickArgs.TasksNum = CurrentState.TasksNum;
				TickArgs.Indent = (FrameIndex + StateIndex + 1);
				const FTickTaskResult TickTasksResult = TickTasks(TickArgs);

				// Keep updating the binding but do not call tick on tasks if there's a failure.
				TickArgs.bShouldTickTasks = TickTasksResult.bShouldTickTasks
					&& !CurrentCompletionStatus.HasAnyFailed();
				// If a failure and we do not copy then bindings, then we can stop.
				bRequestLoopStop = !bCopyBoundPropertiesOnNonTickedTask && !TickTasksResult.bShouldTickTasks;
			}

			NumTotalEnabledTasks += CurrentState.EnabledTasksNum;

			if (bRequestLoopStop)
			{
				break;
			}
		}
	}

	// Collect the result after every tasks has the chance to tick.
	//An async or delegate might complete a global or "previous" task (in a different order).
	EStateTreeRunStatus FirstFrameResult = EStateTreeRunStatus::Running;
	EStateTreeRunStatus FrameResult = EStateTreeRunStatus::Running;
	EStateTreeRunStatus StateResult = EStateTreeRunStatus::Running;
	for (int32 FrameIndex = 0; FrameIndex < Exec.ActiveFrames.Num(); ++FrameIndex)
	{
		using namespace UE::StateTree::ExecutionContext;
		using namespace UE::StateTree::ExecutionContext::Private;

		const FStateTreeExecutionFrame& CurrentFrame = Exec.ActiveFrames[FrameIndex];
		const UStateTree* CurrentStateTree = CurrentFrame.StateTree;
		if (CurrentFrame.bIsGlobalFrame)
		{
			const ETaskCompletionStatus GlobalTasksStatus = CurrentFrame.ActiveTasksStatus.GetStatus(CurrentStateTree).GetCompletionStatus();
			if (FrameIndex == 0)
			{
				FirstFrameResult = CastToRunStatus(GlobalTasksStatus);
			}
			FrameResult = GetPriorityRunStatus(FrameResult, CastToRunStatus(GlobalTasksStatus));
		}

		for (int32 StateIndex = 0; StateIndex < CurrentFrame.ActiveStates.Num() && StateResult != EStateTreeRunStatus::Failed; ++StateIndex)
		{
			const FStateTreeStateHandle CurrentHandle = CurrentFrame.ActiveStates[StateIndex];
			const FCompactStateTreeState& State = CurrentStateTree->States[CurrentHandle.Index];
			const ETaskCompletionStatus StateTasksStatus = CurrentFrame.ActiveTasksStatus.GetStatus(State).GetCompletionStatus();
			StateResult = GetPriorityRunStatus(StateResult, CastToRunStatus(StateTasksStatus));
		}
	}

	if (ExecutionContext::Private::bGlobalTasksCompleteOwningFrame && FirstFrameResult != EStateTreeRunStatus::Running)
	{
		Exec.RequestedStop = ExecutionContext::GetPriorityRunStatus(Exec.RequestedStop, FrameResult);
	}
	else if (ExecutionContext::Private::bGlobalTasksCompleteOwningFrame == false && FrameResult != EStateTreeRunStatus::Running)
	{
		Exec.RequestedStop = ExecutionContext::GetPriorityRunStatus(Exec.RequestedStop, FrameResult);
	}
	else if (NumTotalEnabledTasks == 0 && StateResult == EStateTreeRunStatus::Running && FrameResult == EStateTreeRunStatus::Running)
	{
		// No enabled tasks, done ticking.
		//Complete the the bottom state in the bottom frame (to trigger the completion transitions).
		if (Exec.ActiveFrames.Num() > 0)
		{
			FStateTreeExecutionFrame& LastFrame = Exec.ActiveFrames.Last();
			const int32 NumberOfActiveState = LastFrame.ActiveStates.Num();
			if (ensureMsgf(NumberOfActiveState != 0, TEXT("No task is allowed to clear/stop/transition. Those action should be delayed inside the execution context.")))
			{
				const FStateTreeStateHandle CurrentHandle = LastFrame.ActiveStates[NumberOfActiveState - 1];
				const FCompactStateTreeState& State = LastFrame.StateTree->States[CurrentHandle.Index];
				LastFrame.ActiveTasksStatus.GetStatus(State).SetCompletionStatus(ETaskCompletionStatus::Succeeded);
			}
			else
			{
				LastFrame.ActiveTasksStatus.GetStatus(LastFrame.StateTree).SetCompletionStatus(ETaskCompletionStatus::Succeeded);
			}
		}
		else
		{
			Exec.RequestedStop = ExecutionContext::GetPriorityRunStatus(Exec.RequestedStop, EStateTreeRunStatus::Stopped);
		}
		StateResult = EStateTreeRunStatus::Succeeded;
	}

	Exec.bHasPendingCompletedState = StateResult != EStateTreeRunStatus::Running || FrameResult != EStateTreeRunStatus::Running;
	return StateResult;
}

FStateTreeExecutionContext::FTickTaskResult FStateTreeExecutionContext::TickTasks(const FTickTaskArguments& Args)
{
	using namespace UE::StateTree;

	check(Args.Frame);
	check(Args.TasksCompletionStatus);

	bool bShouldTickTasks = Args.bShouldTickTasks;

	FStateTreeExecutionState& Exec = GetExecState();
	const bool bCopyBoundPropertiesOnNonTickedTask = ExecutionContext::Private::bCopyBoundPropertiesOnNonTickedTask;
	const UStateTree* CurrentStateTree = Args.Frame->StateTree;
	const FActiveFrameID CurrentFrameID = Args.Frame->FrameID;
	const int32 CurrentActiveNodeIndex = Args.Frame->ActiveNodeIndex.AsInt32();
	check(CurrentStateTree);

	for (int32 OwnerTaskIndex = 0; OwnerTaskIndex < Args.TasksNum; ++OwnerTaskIndex)
	{
		const int32 AssetTaskIndex = Args.TasksBegin + OwnerTaskIndex;
		const FStateTreeTaskBase& Task = CurrentStateTree->Nodes[AssetTaskIndex].Get<const FStateTreeTaskBase>();

		// Ignore disabled task
		if (Task.bTaskEnabled == false)
		{
			STATETREE_LOG(VeryVerbose, TEXT("%*sSkipped 'Tick' for disabled Task: '%s'"), Debug::IndentSize, TEXT(""), *Task.Name.ToString());
			continue;
		}

		if (AssetTaskIndex > CurrentActiveNodeIndex)
		{
			STATETREE_LOG(VeryVerbose, TEXT("%*sSkipped 'Tick' for task that didn't get the EnterState Task: '%s'"), Debug::IndentSize, TEXT(""), *Task.Name.ToString());
			bShouldTickTasks = false;
			break;
		}

		const FStateTreeDataView TaskInstanceView = GetDataView(Args.ParentFrame, *Args.Frame, Task.InstanceDataHandle);
		FNodeInstanceDataScope DataScope(*this, &Task, AssetTaskIndex, Task.InstanceDataHandle, TaskInstanceView);

		const bool bHasEvents = EventQueue && EventQueue->HasEvents();
		const bool bIsTaskRunning = Args.TasksCompletionStatus->IsRunning(OwnerTaskIndex);
		const bool bNeedsTick = bShouldTickTasks
			&& bIsTaskRunning
			&& (Task.bShouldCallTick || (bHasEvents && Task.bShouldCallTickOnlyOnEvents));
		STATETREE_LOG(VeryVerbose, TEXT("%*s  Tick: '%s' %s"), Args.Indent * Debug::IndentSize, TEXT("")
			, *Task.Name.ToString()
			, !bNeedsTick ? TEXT("[not ticked]") : TEXT(""));

		// Copy bound properties.
		// Only copy properties when the task is actually ticked, and copy properties at tick is requested.
		const bool bCopyBatch = (bCopyBoundPropertiesOnNonTickedTask || bNeedsTick)
			&& Task.BindingsBatch.IsValid()
			&& Task.bShouldCopyBoundPropertiesOnTick;
		if (bCopyBatch)
		{
			CopyBatchOnActiveInstances(Args.ParentFrame, *Args.Frame, TaskInstanceView, Task.BindingsBatch);
		}

		if (!bNeedsTick)
		{
			// Task didn't tick because it failed.
			//The following tasks should not tick but we might still need to update their bindings.
			if (!bIsTaskRunning && bShouldTickTasks && Args.TasksCompletionStatus->HasAnyFailed())
			{
				bShouldTickTasks = false;
			}
			continue;
		}

		//UE_STATETREE_DEBUG_TASK_EVENT(this, AssetTaskIndex, TaskDataView, EStateTreeTraceEventType::OnTickingTask, EStateTreeRunStatus::Running);
		UE_STATETREE_DEBUG_TASK_TICK(this, CurrentStateTree, AssetTaskIndex);

		EStateTreeRunStatus TaskRunStatus = EStateTreeRunStatus::Unset;
		{
			QUICK_SCOPE_CYCLE_COUNTER(StateTree_Task_Tick);
			CSV_SCOPED_TIMING_STAT(StateTree, Task_Tick);

			TaskRunStatus = Task.Tick(*this, Args.DeltaTime);
		}

		// Set the new status and fetch back the status with priority.
		//In case an async task completes the same task.
		//Or in case FinishTask() inside the Task.Tick()
		ETaskCompletionStatus TaskStatus = UE::StateTree::ExecutionContext::CastToTaskStatus(TaskRunStatus);
		const UE::StateTree::FTasksCompletionStatus::FSetStatusResult TaskStatusResult = Args.TasksCompletionStatus->SetTaskStatusWithPriority(OwnerTaskIndex, TaskStatus);
		TaskStatus = TaskStatusResult.CurrentStatus;
		TaskRunStatus = ExecutionContext::CastToRunStatus(TaskStatus);

		// Only copy output bound properties if the task wasn't failed already
		if (TaskRunStatus != EStateTreeRunStatus::Failed && Task.OutputBindingsBatch.IsValid())
		{
			CopyBatchOnActiveInstances(Args.ParentFrame, *Args.Frame, TaskInstanceView, Task.OutputBindingsBatch);
		}

		UE_STATETREE_DEBUG_TASK_EVENT(this, AssetTaskIndex, TaskInstanceView,
			TaskRunStatus != EStateTreeRunStatus::Running ? EStateTreeTraceEventType::OnTaskCompleted : EStateTreeTraceEventType::OnTicked,
			TaskRunStatus);

		// Broadcast task completion dispatcher only if status actually changed
		if (TaskStatusResult.bChanged && Task.bHasTaskCompletionDelegateDispatcher)
		{
			BroadcastTaskCompletionDispatchers(CurrentStateTree, AssetTaskIndex, TaskStatus);
		}

		if (Args.TasksCompletionStatus->IsConsideredForCompletion(OwnerTaskIndex))
		{
			if (TaskRunStatus == EStateTreeRunStatus::Failed)
			{
				bShouldTickTasks = false;
				break; // Stop copy binding.
			}
		}
	}

	return FTickTaskResult{ bShouldTickTasks };
}

// Deprecated
bool FStateTreeExecutionContext::TestAllConditions(const FStateTreeExecutionFrame* CurrentParentFrame, const FStateTreeExecutionFrame& CurrentFrame, const int32 ConditionsOffset, const int32 ConditionsNum)
{
	return TestAllConditionsInternal<false>(CurrentParentFrame, CurrentFrame, FStateTreeStateHandle(), FMemoryRequirement(), ConditionsOffset, ConditionsNum, EStateTreeUpdatePhase::EnterConditions);
}

bool FStateTreeExecutionContext::TestAllConditionsOnActiveInstances(const FStateTreeExecutionFrame* CurrentParentFrame, const FStateTreeExecutionFrame& CurrentFrame, FStateTreeStateHandle CurrentStateHandle, const FMemoryRequirement& MemoryRequirement, const int32 ConditionsOffset, const int32 ConditionsNum, EStateTreeUpdatePhase Phase)
{
	return TestAllConditionsInternal<true>(CurrentParentFrame, CurrentFrame, CurrentStateHandle, MemoryRequirement, ConditionsOffset, ConditionsNum, Phase);
}

bool FStateTreeExecutionContext::TestAllConditionsWithValidation(const FStateTreeExecutionFrame* CurrentParentFrame, const FStateTreeExecutionFrame& CurrentFrame, FStateTreeStateHandle CurrentStateHandle, const FMemoryRequirement& MemoryRequirement, const int32 ConditionsOffset, const int32 ConditionsNum, EStateTreeUpdatePhase Phase)
{
	return TestAllConditionsInternal<false>(CurrentParentFrame, CurrentFrame, CurrentStateHandle, MemoryRequirement, ConditionsOffset, ConditionsNum, Phase);
}

template<bool bOnActiveInstances>
bool FStateTreeExecutionContext::TestAllConditionsInternal(const FStateTreeExecutionFrame* CurrentParentFrame, const FStateTreeExecutionFrame& CurrentFrame, FStateTreeStateHandle CurrentStateHandle, const FMemoryRequirement& MemoryRequirement, const int32 ConditionsOffset, const int32 ConditionsNum, EStateTreeUpdatePhase Phase)
{
	CSV_SCOPED_TIMING_STAT(StateTree, TestConditions);

	using namespace UE::StateTree::InstanceData;
	using namespace UE::StateTree::ExecutionContext::Private;

	if (ConditionsNum == 0)
	{
		return true;
	}

	FCurrentlyProcessedFrameScope FrameScope(*this, CurrentParentFrame, CurrentFrame);
	FCurrentlyProcessedStateScope StateScope(*this, CurrentStateHandle);

	UE_STATETREE_DEBUG_SCOPED_PHASE(this, Phase);

	TStaticArray<EStateTreeExpressionOperand, UE::StateTree::MaxExpressionIndent + 1> Operands(InPlace, EStateTreeExpressionOperand::Copy);
	TStaticArray<bool, UE::StateTree::MaxExpressionIndent + 1> Values(InPlace, false);

	FEvaluationScopeInstanceContainer EvaluationScopeContainer;
	bool bEvaluationScopeContainerPushed = false;

	if (MemoryRequirement.HasMemory())
	{
		void* FunctionBindingMemory = FMemory_Alloca_Aligned(MemoryRequirement.Size, MemoryRequirement.Alignment);
		EvaluationScopeContainer = FEvaluationScopeInstanceContainer(FunctionBindingMemory, MemoryRequirement);
		PushEvaluationScopeInstanceContainer(EvaluationScopeContainer, CurrentFrame);
		InitEvaluationScopeInstanceData(EvaluationScopeContainer, CurrentFrame.StateTree, ConditionsOffset, ConditionsOffset + ConditionsNum);
		bEvaluationScopeContainerPushed = true;
	}

	int32 Level = 0;

	for (int32 Index = 0; Index < ConditionsNum; Index++)
	{
		const int32 ConditionIndex = ConditionsOffset + Index;
		const FStateTreeConditionBase& Cond = CurrentFrame.StateTree->Nodes[ConditionIndex].Get<const FStateTreeConditionBase>();

		const FStateTreeDataView ConditionInstanceView = GetDataView(CurrentParentFrame, CurrentFrame, Cond.InstanceDataHandle);
		FNodeInstanceDataScope DataScope(*this, &Cond, ConditionIndex, Cond.InstanceDataHandle, ConditionInstanceView);

		bool bValue = false;
		if (Cond.EvaluationMode == EStateTreeConditionEvaluationMode::Evaluated)
		{
			// Copy bound properties.
			if (Cond.BindingsBatch.IsValid())
			{
				const bool bBatchCopied = bOnActiveInstances
					? CopyBatchOnActiveInstances(CurrentParentFrame, CurrentFrame, ConditionInstanceView, Cond.BindingsBatch)
					: CopyBatchWithValidation(CurrentParentFrame, CurrentFrame, ConditionInstanceView, Cond.BindingsBatch);
				if (!bBatchCopied)
				{
					// If the source data cannot be accessed, the whole expression evaluates to false.
					static constexpr TCHAR Message[] = TEXT("Evaluation forced to false: source data cannot be accessed (e.g. enter conditions trying to access inactive parent state)");
					UE_STATETREE_DEBUG_CONDITION_EVENT(this, ConditionIndex, ConditionInstanceView, EStateTreeTraceEventType::InternalForcedFailure);
					UE_STATETREE_DEBUG_LOG_EVENT(this, Warning, Message);
					STATETREE_LOG(Warning, TEXT("%s"), Message);
					Values[0] = false;
					break;
				}
			}

			UE_STATETREE_DEBUG_CONDITION_TEST_CONDITION(this, CurrentFrame.StateTree, Index);

			bValue = Cond.TestCondition(*this);
			UE_STATETREE_DEBUG_CONDITION_EVENT(this, ConditionIndex, ConditionInstanceView, bValue ? EStateTreeTraceEventType::Passed : EStateTreeTraceEventType::Failed);

			// Reset copied properties that might contain object references.
			if (Cond.BindingsBatch.IsValid() && !UseEvaluationScopeIntanceData(Cond.InstanceDataHandle))
			{
				CurrentFrame.StateTree->PropertyBindings.Super::ResetObjects(Cond.BindingsBatch, ConditionInstanceView);
			}
		}
		else
		{
			bValue = Cond.EvaluationMode == EStateTreeConditionEvaluationMode::ForcedTrue;
			UE_STATETREE_DEBUG_CONDITION_EVENT(this, ConditionIndex, FStateTreeDataView{}, bValue ? EStateTreeTraceEventType::ForcedSuccess : EStateTreeTraceEventType::ForcedFailure);
		}

		const int32 DeltaIndent = Cond.DeltaIndent;
		const int32 OpenParens = FMath::Max(0, DeltaIndent) + 1;	// +1 for the current value that is stored at the empty slot at the top of the value stack.
		const int32 ClosedParens = FMath::Max(0, -DeltaIndent) + 1;

		// Store the operand to apply when merging higher level down when returning to this level.
		const EStateTreeExpressionOperand Operand = Index == 0 ? EStateTreeExpressionOperand::Copy : Cond.Operand;
		Operands[Level] = Operand;

		// Store current value at the top of the stack.
		Level += OpenParens;
		Values[Level] = bValue;

		// Evaluate and merge down values based on closed braces.
		// The current value is placed in parens (see +1 above), which makes merging down and applying the new value consistent.
		// The default operand is copy, so if the value is needed immediately, it is just copied down, or if we're on the same level,
		// the operand storing above gives handles with the right logic.
		for (int32 Paren = 0; Paren < ClosedParens; Paren++)
		{
			Level--;
			switch (Operands[Level])
			{
			case EStateTreeExpressionOperand::Copy:
				Values[Level] = Values[Level + 1];
				break;
			case EStateTreeExpressionOperand::And:
				Values[Level] &= Values[Level + 1];
				break;
			case EStateTreeExpressionOperand::Or:
				Values[Level] |= Values[Level + 1];
				break;
			case EStateTreeExpressionOperand::Multiply:
			default:
				checkf(false, TEXT("Unhandled operand %s"), *UEnum::GetValueAsString(Operand));
				break;
			}
			Operands[Level] = EStateTreeExpressionOperand::Copy;
		}
	}

	if (bEvaluationScopeContainerPushed)
	{
		PopEvaluationScopeInstanceContainer(EvaluationScopeContainer);
	}

	return Values[0];
}

template<bool bOnActiveInstances, FStateTreeExecutionContext::EConditionFunction ConditionFunction>
void FStateTreeExecutionContext::RunAllConditionsInternal(const FStateTreeExecutionFrame* CurrentParentFrame, const FStateTreeExecutionFrame& CurrentFrame, const FCompactStateTreeState& CurrentState, const FStateTreeTransitionResult& CurrentTransition)
{
	using namespace UE::StateTree;
	using namespace UE::StateTree::ExecutionContext;
	using namespace UE::StateTree::ExecutionContext::Private;
	using namespace UE::StateTree::InstanceData;

	if (CurrentState.bHasStateChangeConditions)
	{
		FStateTreeExecutionState& Exec = GetExecState();

		FEvaluationScopeInstanceContainer EvaluationScopeContainer;
		bool bEvaluationScopeContainerPushed = false;
		if (CurrentState.EnterConditionEvaluationScopeMemoryRequirement.HasMemory())
		{
			void* ScopedConditionMemory = FMemory_Alloca_Aligned(CurrentState.EnterConditionEvaluationScopeMemoryRequirement.Size, CurrentState.EnterConditionEvaluationScopeMemoryRequirement.Alignment);
			EvaluationScopeContainer = FEvaluationScopeInstanceContainer(ScopedConditionMemory, CurrentState.EnterConditionEvaluationScopeMemoryRequirement);
			PushEvaluationScopeInstanceContainer(EvaluationScopeContainer, CurrentFrame);
			InitEvaluationScopeInstanceData(EvaluationScopeContainer, CurrentFrame.StateTree, CurrentState.EnterConditionsBegin, CurrentState.EnterConditionsBegin + CurrentState.EnterConditionsNum);
			bEvaluationScopeContainerPushed = true;
		}

		const int32 EnterConditionsEnd = CurrentState.EnterConditionsBegin + CurrentState.EnterConditionsNum;
		constexpr bool bForwardIteration = ConditionFunction == EConditionFunction::EnterState;

		int32 ConditionIndex = bForwardIteration ? CurrentState.EnterConditionsBegin : EnterConditionsEnd - 1;
		bool bRunLoop = bForwardIteration ? ConditionIndex < EnterConditionsEnd : ConditionIndex >= CurrentState.EnterConditionsBegin;
		while (bRunLoop)
		{
			const FStateTreeConditionBase& Cond = CurrentFrame.StateTree->Nodes[ConditionIndex].Get<const FStateTreeConditionBase>();
			if (Cond.bHasShouldCallStateChangeEvents)
			{
				constexpr bool bAlwaysCallEnterState = ConditionFunction == EConditionFunction::StateCompleted;
				const bool bShouldCallEnterState = CurrentTransition.ChangeType == EStateTreeStateChangeType::Changed
					|| (CurrentTransition.ChangeType == EStateTreeStateChangeType::Sustained && Cond.bShouldStateChangeOnReselect);

				if (bAlwaysCallEnterState || bShouldCallEnterState)
				{
					const FStateTreeDataView ConditionInstanceView = GetDataView(CurrentParentFrame, CurrentFrame, Cond.InstanceDataHandle);
					FNodeInstanceDataScope DataScope(*this, &Cond, ConditionIndex, Cond.InstanceDataHandle, ConditionInstanceView);

					if (Cond.BindingsBatch.IsValid())
					{
						if constexpr (bOnActiveInstances)
						{
							CopyBatchOnActiveInstances(CurrentParentFrame, CurrentFrame, ConditionInstanceView, Cond.BindingsBatch);
						}
						else
						{
							CopyBatchWithValidation(CurrentParentFrame, CurrentFrame, ConditionInstanceView, Cond.BindingsBatch);
						}
					}

					if constexpr (ConditionFunction == EConditionFunction::EnterState)
					{
						UE_STATETREE_DEBUG_CONDITION_ENTER_STATE(this, CurrentFrame.StateTree, FStateTreeIndex16(ConditionIndex));
						Cond.EnterState(*this, CurrentTransition);
					}
					else if constexpr (ConditionFunction == EConditionFunction::ExitState)
					{
						UE_STATETREE_DEBUG_CONDITION_EXIT_STATE(this, CurrentFrame.StateTree, FStateTreeIndex16(ConditionIndex));
						Cond.ExitState(*this, CurrentTransition);
					}
					else if constexpr (ConditionFunction == EConditionFunction::StateCompleted)
					{
						Cond.StateCompleted(*this, Exec.LastTickStatus, CurrentFrame.ActiveStates);
					}

					// Reset copied properties that might contain object references.
					if (Cond.BindingsBatch.IsValid() && !UseEvaluationScopeIntanceData(Cond.InstanceDataHandle))
					{
						CurrentFrame.StateTree->PropertyBindings.Super::ResetObjects(Cond.BindingsBatch, ConditionInstanceView);
					}
				}
			}

			ConditionIndex += bForwardIteration ? 1 : -1;
			bRunLoop = bForwardIteration ? ConditionIndex < EnterConditionsEnd : ConditionIndex >= CurrentState.EnterConditionsBegin;
		}

		if (bEvaluationScopeContainerPushed)
		{
			PopEvaluationScopeInstanceContainer(EvaluationScopeContainer);
		}
	}
}

//Deprecated
float FStateTreeExecutionContext::EvaluateUtility(const FStateTreeExecutionFrame* CurrentParentFrame, const FStateTreeExecutionFrame& CurrentFrame, const int32 ConsiderationsBegin, const int32 ConsiderationsNum, const float StateWeight)
{
	const FStateTreeStateHandle StateHandle = FStateTreeStateHandle::Invalid;
	return EvaluateUtilityWithValidation(CurrentParentFrame, CurrentFrame, StateHandle, FMemoryRequirement(), ConsiderationsBegin, ConsiderationsNum, StateWeight);
}

float FStateTreeExecutionContext::EvaluateUtilityWithValidation(const FStateTreeExecutionFrame* CurrentParentFrame, const FStateTreeExecutionFrame& CurrentFrame, FStateTreeStateHandle CurrentStateHandle, const FMemoryRequirement& MemoryRequirement, const int32 ConsiderationsBegin, const int32 ConsiderationsNum, const float StateWeight)
{
	// @todo: Tracing support
	CSV_SCOPED_TIMING_STAT(StateTree, EvaluateUtility);

	using namespace UE::StateTree::InstanceData;
	using namespace UE::StateTree::ExecutionContext::Private;

	if (ConsiderationsNum == 0)
	{
		return 0.0f;
	}

	FCurrentlyProcessedFrameScope FrameScope(*this, CurrentParentFrame, CurrentFrame);
	FCurrentlyProcessedStateScope NextStateScope(*this, CurrentStateHandle);

	UE_STATETREE_DEBUG_SCOPED_STATE_PHASE(this, CurrentStateHandle, EStateTreeUpdatePhase::EvaluateUtility);

	TStaticArray<EStateTreeExpressionOperand, UE::StateTree::MaxExpressionIndent + 1> Operands(InPlace, EStateTreeExpressionOperand::Copy);
	TStaticArray<float, UE::StateTree::MaxExpressionIndent + 1> Values(InPlace, false);

	FEvaluationScopeInstanceContainer EvaluationScopeContainer;
	bool bEvaluationScopeContainerPushed = false;

	if (MemoryRequirement.HasMemory())
	{
		void* FunctionBindingMemory = FMemory_Alloca_Aligned(MemoryRequirement.Size, MemoryRequirement.Alignment);
		EvaluationScopeContainer = FEvaluationScopeInstanceContainer(FunctionBindingMemory, MemoryRequirement);
		PushEvaluationScopeInstanceContainer(EvaluationScopeContainer, CurrentFrame);
		InitEvaluationScopeInstanceData(EvaluationScopeContainer, CurrentFrame.StateTree, ConsiderationsBegin, ConsiderationsBegin + ConsiderationsNum);
		bEvaluationScopeContainerPushed = true;
	}

	int32 Level = 0;
	float Value = 0.0f;
	for (int32 Index = 0; Index < ConsiderationsNum; Index++)
	{
		const int32 ConsiderationIndex = ConsiderationsBegin + Index;
		const FStateTreeConsiderationBase& Consideration = CurrentFrame.StateTree->Nodes[ConsiderationIndex].Get<const FStateTreeConsiderationBase>();

		const FStateTreeDataView ConsiderationInstanceView = GetDataView(CurrentParentFrame, CurrentFrame, Consideration.InstanceDataHandle);
		FNodeInstanceDataScope DataScope(*this, &Consideration, ConsiderationIndex, Consideration.InstanceDataHandle, ConsiderationInstanceView);

		// Copy bound properties.
		if (Consideration.BindingsBatch.IsValid())
		{
			// Use validated copy, since we test in situations where the sources are not always valid (e.g. considerations may try to access inactive parent state). 
			if (!CopyBatchWithValidation(CurrentParentFrame, CurrentFrame, ConsiderationInstanceView, Consideration.BindingsBatch))
			{
				// If the source data cannot be accessed, the whole expression evaluates to zero.
				Values[0] = 0.0f;
				break;
			}
		}

		Value = Consideration.GetNormalizedScore(*this);

		// Reset copied properties that might contain object references.
		if (Consideration.BindingsBatch.IsValid() && !UseEvaluationScopeIntanceData(Consideration.InstanceDataHandle))
		{
			CurrentFrame.StateTree->PropertyBindings.Super::ResetObjects(Consideration.BindingsBatch, ConsiderationInstanceView);
		}

		const int32 DeltaIndent = Consideration.DeltaIndent;
		const int32 OpenParens = FMath::Max(0, DeltaIndent) + 1;	// +1 for the current value that is stored at the empty slot at the top of the value stack.
		const int32 ClosedParens = FMath::Max(0, -DeltaIndent) + 1;

		// Store the operand to apply when merging higher level down when returning to this level.
		const EStateTreeExpressionOperand Operand = Index == 0 ? EStateTreeExpressionOperand::Copy : Consideration.Operand;
		Operands[Level] = Operand;

		// Store current value at the top of the stack.
		Level += OpenParens;
		Values[Level] = Value;

		// Evaluate and merge down values based on closed braces.
		// The current value is placed in parens (see +1 above), which makes merging down and applying the new value consistent.
		// The default operand is copy, so if the value is needed immediately, it is just copied down, or if we're on the same level,
		// the operand storing above gives handles with the right logic.
		for (int32 Paren = 0; Paren < ClosedParens; Paren++)
		{
			Level--;
			switch (Operands[Level])
			{
			case EStateTreeExpressionOperand::Copy:
				Values[Level] = Values[Level + 1];
				break;
			case EStateTreeExpressionOperand::And:
				Values[Level] = FMath::Min(Values[Level], Values[Level + 1]);
				break;
			case EStateTreeExpressionOperand::Or:
				Values[Level] = FMath::Max(Values[Level], Values[Level + 1]);
				break;
			case EStateTreeExpressionOperand::Multiply:
				Values[Level] = Values[Level] * Values[Level + 1];
				break;
			default:
				checkf(false, TEXT("Unhandled operand %s"), *UEnum::GetValueAsString(Operand));
				break;
			}
			Operands[Level] = EStateTreeExpressionOperand::Copy;
		}
	}

	if (bEvaluationScopeContainerPushed)
	{
		PopEvaluationScopeInstanceContainer(EvaluationScopeContainer);
	}

	const float Score = StateWeight * Values[0];

	UE_STATETREE_DEBUG_STATE_UTILITY_EVALUATED(this, CurrentStateHandle, Score);

	return Score;
}

void FStateTreeExecutionContext::EvaluatePropertyFunctionsOnActiveInstances(const FStateTreeExecutionFrame* CurrentParentFrame, const FStateTreeExecutionFrame& CurrentFrame, FStateTreeIndex16 FuncsBegin, uint16 FuncsNum)
{
	constexpr bool bOnActiveInstances = true;
	EvaluatePropertyFunctionsInternal<bOnActiveInstances>(CurrentParentFrame, CurrentFrame, FuncsBegin, FuncsNum);
}

void FStateTreeExecutionContext::EvaluatePropertyFunctionsWithValidation(const FStateTreeExecutionFrame* CurrentParentFrame, const FStateTreeExecutionFrame& CurrentFrame, FStateTreeIndex16 FuncsBegin, uint16 FuncsNum)
{
	constexpr bool bOnActiveInstances = false;
	EvaluatePropertyFunctionsInternal<bOnActiveInstances>(CurrentParentFrame, CurrentFrame, FuncsBegin, FuncsNum);
}

template<bool bOnActiveInstances>
void FStateTreeExecutionContext::EvaluatePropertyFunctionsInternal(const FStateTreeExecutionFrame* CurrentParentFrame, const FStateTreeExecutionFrame& CurrentFrame, FStateTreeIndex16 FuncsBegin, uint16 FuncsNum)
{
	const int32 FuncsEnd = FuncsBegin.AsInt32() + FuncsNum;
	for (int32 FuncIndex = FuncsBegin.AsInt32(); FuncIndex < FuncsEnd; ++FuncIndex)
	{
		const FStateTreePropertyFunctionBase& Func = CurrentFrame.StateTree->Nodes[FuncIndex].Get<const FStateTreePropertyFunctionBase>();
		const FStateTreeDataView FuncInstanceView = GetDataView(CurrentParentFrame, CurrentFrame, Func.InstanceDataHandle);
		FNodeInstanceDataScope DataScope(*this, &Func, FuncIndex, Func.InstanceDataHandle, FuncInstanceView);

		// Copy bound properties.
		if (Func.BindingsBatch.IsValid())
		{
			if constexpr (bOnActiveInstances)
			{
				CopyBatchOnActiveInstances(CurrentParentFrame, CurrentFrame, FuncInstanceView, Func.BindingsBatch);
			}
			else
			{
				CopyBatchWithValidation(CurrentParentFrame, CurrentFrame, FuncInstanceView, Func.BindingsBatch);
			}
		}

		Func.Execute(*this);
	}
}

FString FStateTreeExecutionContext::DebugGetEventsAsString() const
{
	TStringBuilder<512> StrBuilder;

	if (EventQueue)
	{
		for (const FStateTreeSharedEvent& Event : EventQueue->GetEventsView())
		{
			if (Event.IsValid())
			{
				if (StrBuilder.Len() > 0)
				{
					StrBuilder << TEXT(", ");
				}

				const bool bHasTag = Event->Tag.IsValid();
				const bool bHasPayload = Event->Payload.GetScriptStruct() != nullptr;

				if (bHasTag || bHasPayload)
				{
					StrBuilder << (TEXT('('));

					if (bHasTag)
					{
						StrBuilder << TEXT("Tag: '");
						StrBuilder << Event->Tag.GetTagName();
						StrBuilder << TEXT('\'');
					}
					if (bHasTag && bHasPayload)
					{
						StrBuilder << TEXT(", ");
					}
					if (bHasPayload)
					{
						StrBuilder << TEXT(" Payload: '");
						StrBuilder << Event->Payload.GetScriptStruct()->GetFName();
						StrBuilder << TEXT('\'');
					}
					StrBuilder << TEXT(") ");
				}
			}
		}
	}

	return StrBuilder.ToString();
}

// Deprecated
bool FStateTreeExecutionContext::RequestTransition(
	const FStateTreeExecutionFrame& CurrentFrame,
	const FStateTreeStateHandle NextState,
	const EStateTreeTransitionPriority Priority,
	const FStateTreeSharedEvent* TransitionEvent,
	const EStateTreeSelectionFallback Fallback)
{
	using namespace UE::StateTree;
	using namespace UE::StateTree::ExecutionContext;

	const FStateHandleContext TargetState = FStateHandleContext(CurrentFrame.StateTree, NextState);
	FTransitionArguments TransitionArgs = FTransitionArguments{ .Priority = Priority, .Fallback = Fallback };
	if (TransitionEvent)
	{
		TransitionArgs.TransitionEvent = *TransitionEvent;
	}
	
	return RequestTransitionInternal(CurrentFrame, FActiveStateID(), TargetState, TransitionArgs);
}

bool FStateTreeExecutionContext::RequestTransitionInternal(
	const FStateTreeExecutionFrame& SourceFrame,
	const UE::StateTree::FActiveStateID SourceStateID,
	const UE::StateTree::ExecutionContext::FStateHandleContext TargetState,
	const FTransitionArguments& Args)
{
	using namespace UE::StateTree;
	using namespace UE::StateTree::ExecutionContext;
	using namespace UE::StateTree::ExecutionContext::Private;

	// Skip lower priority transitions.
	if (RequestedTransition.IsValid() && RequestedTransition->Transition.Priority >= Args.Priority)
	{
		return false;
	}

	if (TargetState.StateHandle.IsCompletionState())
	{
		SetupNextTransition(SourceFrame, SourceStateID, TargetState, Args);
		STATETREE_LOG(Verbose, TEXT("Transition on state '%s' -> state '%s'"),
			*GetSafeStateName(SourceFrame, SourceFrame.ActiveStates.Last()), *TargetState.StateHandle.Describe());
		return true;
	}
	if (!TargetState.StateHandle.IsValid() || TargetState.StateTree == nullptr)
	{
		// NotSet is no-operation, but can be used to mask a transition at parent state. Returning unset keeps updating current state.
		SetupNextTransition(SourceFrame, SourceStateID, FStateHandleContext(), Args);
		return true;
	}

	FActiveStateInlineArray CurrentActiveStatePath;
	GetActiveStatePath(GetExecState().ActiveFrames, CurrentActiveStatePath);

	FSelectStateArguments SelectStateArgs;
	SelectStateArgs.ActiveStates = MakeConstArrayView(CurrentActiveStatePath);

	// Source can be a GlobalTask (not in active state)
	FStateTreeStateHandle SourceStateHandle;
	if (SourceStateID.IsValid())
	{
		SourceStateHandle = SourceFrame.ActiveStates.FindStateHandle(SourceStateID);
		if (!ensure(SourceStateHandle.IsValid()))
		{
			return false;
		}
		SelectStateArgs.SourceState = FActiveState(SourceFrame.FrameID, SourceStateID, SourceStateHandle);
	}
	else
	{
		SelectStateArgs.SourceState = FActiveState(SourceFrame.FrameID, FActiveStateID(), FStateTreeStateHandle());
	}
	SelectStateArgs.TargetState = TargetState;
	SelectStateArgs.TransitionEvent = Args.TransitionEvent;
	SelectStateArgs.Fallback = Args.Fallback;
	SelectStateArgs.Behavior = ESelectStateBehavior::StateTransition;
	SelectStateArgs.SelectionRules = SourceFrame.StateTree->StateSelectionRules | Args.SelectionRules;

	TSharedRef<FSelectStateResult> StateSelectionResult = MakeShared<FSelectStateResult>();
	TGuardValue<TSharedPtr<FSelectStateResult>> TemporaryStorageScope(CurrentlyProcessedTemporaryStorage, StateSelectionResult);
	if (SelectState(SelectStateArgs, StateSelectionResult))
	{
		if (RequestedTransition)
		{
			// If we have a previous selection(i.e. we succeeded a selection already from a previous transition), we need to clean up temporary frames if any.
			if (FSelectStateResult* PrevSelectStateResult = RequestedTransition->Selection.Get())
			{
				TGuardValue<TSharedPtr<FSelectStateResult>> PrevTemporaryStorageScope(CurrentlyProcessedTemporaryStorage, RequestedTransition->Selection);

				for (int32 Idx = PrevSelectStateResult->SelectedFrames.Num() - 1; Idx >= 0; --Idx)
				{
					const FActiveFrameID TemporaryFrameID = PrevSelectStateResult->SelectedFrames[Idx];

					FSelectStateResult::FFrameAndParent TemporaryFrameAndParent = PrevSelectStateResult->GetExecutionFrame(TemporaryFrameID);
					FStateTreeExecutionFrame* TemporaryFrame = TemporaryFrameAndParent.Frame;

					// stop as soon as the frame is active already: no more temporary frames forward
					if (!TemporaryFrame)
					{
						break;
					}

					FStateTreeExecutionState& Exec = Storage.GetMutableExecutionState();
					const FStateTreeExecutionFrame* ParentFrame = ExecutionContext::FindExecutionFrame(TemporaryFrameAndParent.ParentFrameID, MakeConstArrayView(Exec.ActiveFrames), MakeConstArrayView(PrevSelectStateResult->TemporaryFrames));

					if (TemporaryFrame->bIsGlobalFrame && TemporaryFrame->bHaveEntered)
					{
						constexpr EStateTreeRunStatus Status = EStateTreeRunStatus::Stopped;
						StopTemporaryEvaluatorsAndGlobalTasks(ParentFrame, *TemporaryFrame, Status);
					}

					CleanFrame(Exec, TemporaryFrameID);
				}
			}
		}

		const FStateTreeExecutionState& Exec = GetExecState();

		SetupNextTransition(SourceFrame, SourceStateID, TargetState, Args);
		RequestedTransition->Selection = StateSelectionResult; // It will keep the temporary storage alive until the end of the EnterState

		// Fill NextActiveFrames & NextActiveFrameEvents for backward compatibility
		if (bSetDeprecatedTransitionResultProperties)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			RequestedTransition->Transition.NextActiveFrames.Reset();
			RequestedTransition->Transition.NextActiveFrameEvents.Reset();
			FActiveFrameID PreviousFrameID;
			FStateTreeExecutionFrame* CurrentFrame = nullptr;
			FStateTreeFrameStateSelectionEvents* CurrentSelectionEvents = nullptr;
			int32 CurrentSelectionEventsIndex = 0;
			for (const FActiveState& SelectedState : RequestedTransition->Selection->SelectedStates)
			{
				if (SelectedState.GetFrameID() != PreviousFrameID)
				{
					PreviousFrameID = SelectedState.GetFrameID();
					const FStateTreeExecutionFrame* Frame = UE::StateTree::ExecutionContext::FindExecutionFrame(SelectedState.GetFrameID(), MakeConstArrayView(Exec.ActiveFrames), MakeConstArrayView(RequestedTransition->Selection->TemporaryFrames));
					if (ensure(Frame))
					{
						RequestedTransition->Transition.NextActiveFrames.Add(*Frame);
						CurrentFrame = &RequestedTransition->Transition.NextActiveFrames.Last();

						RequestedTransition->Transition.NextActiveFrameEvents.AddDefaulted();
						CurrentSelectionEvents = &RequestedTransition->Transition.NextActiveFrameEvents.Last();
						CurrentSelectionEventsIndex = 0;

						CurrentFrame->ActiveStates = FStateTreeActiveStates();
					}
				}
				check(CurrentFrame);
				check(CurrentSelectionEvents);
				CurrentFrame->ActiveStates.Push(SelectedState.GetStateHandle(), SelectedState.GetStateID());

				const FSelectionEventWithID* FoundEvent = RequestedTransition->Selection->SelectionEvents.FindByPredicate(
					[SelectedState](const FSelectionEventWithID& Event)
					{
						return Event.State == SelectedState;
					});
				if (FoundEvent)
				{
					CurrentSelectionEvents->Events[CurrentSelectionEventsIndex] = FoundEvent->Event;
				}
			}
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		//@todo In TriggerTransitions, the events are processed bottom up. If an event with a higher priority is process after, the event will already be consumed.
		// Consume events from states, if required. The transition might also want to consume the event. (Not a bug.)
		for (FSelectionEventWithID& Event : RequestedTransition->Selection->SelectionEvents)
		{
			const FStateTreeExecutionFrame* Frame = UE::StateTree::ExecutionContext::FindExecutionFrame(Event.State.GetFrameID(), MakeConstArrayView(Exec.ActiveFrames), MakeConstArrayView(RequestedTransition->Selection->TemporaryFrames));
			if (ensure(Frame))
			{
				const FCompactStateTreeState* State = Frame->StateTree->GetStateFromHandle(Event.State.GetStateHandle());
				if (ensure(State) && State->bConsumeEventOnSelect)
				{
					ConsumeEvent(Event.Event);
				}
			}
		}

		UE_SUPPRESS(LogStateTree, Verbose,
		{
			check(RequestedTransition->Selection->SelectedStates.Num() > 0);
			const FActiveState& LastSelectedState = RequestedTransition->Selection->SelectedStates.Last();
			const FStateTreeExecutionFrame* SelectedFrame = UE::StateTree::ExecutionContext::FindExecutionFrame(LastSelectedState.GetFrameID(), MakeConstArrayView(Exec.ActiveFrames), MakeConstArrayView(RequestedTransition->Selection->TemporaryFrames));
			check(SelectedFrame);
			STATETREE_LOG(Verbose, TEXT("Transition Request. Source:'%s'. Target:'%s'. Selected:'%s'"),
				*GetSafeStateName(SourceFrame, SourceStateHandle),
				*GetSafeStateName(TargetState.StateTree, TargetState.StateHandle),
				*GetSafeStateName(*SelectedFrame, LastSelectedState.GetStateHandle())
			);
		})

		return true;
	}

	return false;
}

void FStateTreeExecutionContext::SetupNextTransition(
	const FStateTreeExecutionFrame& SourceFrame,
	const UE::StateTree::FActiveStateID SourceStateID,
	const UE::StateTree::ExecutionContext::FStateHandleContext TargetState,
	const FTransitionArguments& Args)
{
	RequestedTransition = MakeUnique<FRequestTransitionResult>();
	SetupNextTransition(SourceFrame, SourceStateID, TargetState, Args, RequestedTransition->Transition);
}

void FStateTreeExecutionContext::SetupNextTransition(
	const FStateTreeExecutionFrame& SourceFrame,
	const UE::StateTree::FActiveStateID SourceStateID,
	const UE::StateTree::ExecutionContext::FStateHandleContext TargetState,
	const FTransitionArguments& Args,
	FStateTreeTransitionResult& OutTransitionResult)
{
	const FStateTreeExecutionState& Exec = GetExecState();

	OutTransitionResult.SourceFrameID = SourceFrame.FrameID;
	OutTransitionResult.SourceStateID = SourceStateID;
	OutTransitionResult.TargetState = TargetState.StateHandle;
	OutTransitionResult.CurrentState = FStateTreeStateHandle::Invalid;
	OutTransitionResult.CurrentRunStatus = Exec.LastTickStatus;
	OutTransitionResult.ChangeType = EStateTreeStateChangeType::Changed;
	OutTransitionResult.Priority = Args.Priority;

	// Fill NextActiveFrames & NextActiveFrameEvents for backward compatibility
	if (UE::StateTree::ExecutionContext::Private::bSetDeprecatedTransitionResultProperties)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (SourceStateID.IsValid())
		{
			OutTransitionResult.SourceState = SourceFrame.ActiveStates.FindStateHandle(SourceStateID);
		}
		OutTransitionResult.SourceStateTree = SourceFrame.StateTree;
		OutTransitionResult.SourceRootState = SourceFrame.RootState;

		FStateTreeExecutionFrame& NewFrame = OutTransitionResult.NextActiveFrames.AddDefaulted_GetRef();
		NewFrame.StateTree = SourceFrame.StateTree;
		NewFrame.RootState = SourceFrame.RootState;
		NewFrame.ActiveTasksStatus = SourceFrame.ActiveTasksStatus;
		if (TargetState.StateHandle == FStateTreeStateHandle::Invalid)
		{
			NewFrame.ActiveStates = {};
		}
		else
		{
			NewFrame.ActiveStates = FStateTreeActiveStates(TargetState.StateHandle, UE::StateTree::FActiveStateID::Invalid);
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool FStateTreeExecutionContext::TriggerTransitions(const FTriggerTransitionsInternalArgs Args)
{
	//1. Process transition requests. Keep the single request with the highest priority.
	//2. Process tick/event/delegate transitions and tasks. TriggerTransitions, from bottom to top.
	// If delayed,
	//	If delayed completed, then process.
	//	Else add them to the delayed transition list.
	//3. If no transition, Process completion transitions, from bottom to top.
	//4. If transition occurs, check if there are any frame (sub-tree) that completed.

	CSV_SCOPED_TIMING_STAT(StateTree, TriggerTransition);
	UE_STATETREE_DEBUG_SCOPED_PHASE(this, EStateTreeUpdatePhase::TriggerTransitions);

	// Set flag for the scope of this function to allow direct transitions without buffering.
	FAllowDirectTransitionsScope AllowDirectTransitionsScope(*this);

	FStateTreeExecutionState& Exec = GetExecState();

	if (EventQueue && EventQueue->HasEvents())
	{
		STATETREE_LOG(Verbose, TEXT("Trigger transitions with events: %s"), *DebugGetEventsAsString());
		UE_STATETREE_DEBUG_LOG_EVENT(this, Log, TEXT("Trigger transitions with events: %s"), *DebugGetEventsAsString());
	}

	RequestedTransition.Reset();

	if (!Args.CompletedSubtreeParentFrameIndex.IsSet())
	{
		check(CurrentlyProcessedBroadcastedDelegates.IsEmpty());
		CurrentlyProcessedBroadcastedDelegates = Storage.StealBroadcastedDelegates();
	}

	bool bTransitionsTriggeredFromSubtreeCompletion = false;
	ON_SCOPE_EXIT
	{
		if (!bTransitionsTriggeredFromSubtreeCompletion)
		{
			CurrentlyProcessedBroadcastedDelegates.Empty();

			// All events have had the change to be reacted to, clear the event queue (if this instance owns it).
			if (InstanceData.IsOwningEventQueue() && EventQueue)
			{
				EventQueue->ClearEventsForCurrentTransitionProcessingPhase();
			}
		}
	};

	//
	// Process transition requests
	//
	for (const FStateTreeTransitionRequest& Request : InstanceData.GetTransitionRequests())
	{
		// Find frame associated with the request.
		const FStateTreeExecutionFrame* CurrentFrame = Exec.FindActiveFrame(Request.SourceFrameID);
		if (CurrentFrame && CurrentFrame->ActiveStates.Contains(Request.SourceStateID))
		{
			const UE::StateTree::ExecutionContext::FStateHandleContext TargetState(CurrentFrame->StateTree, Request.TargetState);
			const FTransitionArguments TransitionArgs = FTransitionArguments{.Priority = Request.Priority, .Fallback = Request.Fallback};
			if (RequestTransitionInternal(*CurrentFrame, Request.SourceStateID, TargetState, TransitionArgs))
			{
				check(RequestedTransition.IsValid());
				RequestedTransition->Source = FStateTreeTransitionSource(CurrentFrame->StateTree, EStateTreeTransitionSourceType::ExternalRequest, Request.TargetState, Request.Priority);
			}
		}
	}

	//@todo should only clear once when the transition is successful.
	//to prevent 2 async requests and the first requests fails for X reason.
	//they will be identified by a Frame/StateID so it's fine if they stay in the array.
	InstanceData.ResetTransitionRequests();

	//
	// Collect expired delayed transitions
	//
	TArray<FStateTreeTransitionDelayedState, TInlineAllocator<8>> ExpiredTransitionsDelayed;
	for (TArray<FStateTreeTransitionDelayedState>::TIterator It = Exec.DelayedTransitions.CreateIterator(); It; ++It)
	{
		if (It->TimeLeft <= 0.0f)
		{
			ExpiredTransitionsDelayed.Emplace(MoveTemp(*It));
			It.RemoveCurrentSwap();
		}
	}

	//
	// Collect tick, event, and task based transitions.
	//
	struct FTransitionHandler
	{
		FTransitionHandler() = default;

		FTransitionHandler(const int32 InFrameIndex, const FStateTreeStateHandle InStateHandle, const UE::StateTree::FActiveStateID InStateID, const EStateTreeTransitionPriority InPriority)
			: StateHandle(InStateHandle)
			, StateID(InStateID)
			, TaskIndex(FStateTreeIndex16::Invalid)
			, FrameIndex(InFrameIndex)
			, Priority(InPriority)
		{
		}

		FTransitionHandler(const int32 InFrameIndex, const FStateTreeStateHandle InStateHandle, const UE::StateTree::FActiveStateID InStateID, const FStateTreeIndex16 InTaskIndex, const EStateTreeTransitionPriority InPriority)
			: StateHandle(InStateHandle)
			, StateID(InStateID)
			, TaskIndex(InTaskIndex)
			, FrameIndex(InFrameIndex)
			, Priority(InPriority)
		{
		}

		FStateTreeStateHandle StateHandle;
		UE::StateTree::FActiveStateID StateID;
		FStateTreeIndex16 TaskIndex = FStateTreeIndex16::Invalid;
		int32 FrameIndex = 0;
		EStateTreeTransitionPriority Priority = EStateTreeTransitionPriority::Normal;

		bool operator<(const FTransitionHandler& Other) const
		{
			// Highest priority first.
			return Priority > Other.Priority;
		}
	};

	TArray<FTransitionHandler, TInlineAllocator<16>> TransitionHandlers;

	if (Exec.ActiveFrames.Num() > 0)
	{
		// Re-cache bHasEvents, RequestTransition above can create new events.
		const bool bHasEvents = EventQueue && EventQueue->HasEvents();
		const bool bHasBroadcastedDelegates = !CurrentlyProcessedBroadcastedDelegates.IsEmpty();

		// Transition() can TriggerTransitions() in a loop when a sub-frame completes.
		// We do not want to evaluate the transition from that sub-frame.
		const int32 EndFrameIndex = Args.CompletedSubtreeParentFrameIndex.Get(Exec.ActiveFrames.Num() - 1);
		for (int32 FrameIndex = EndFrameIndex; FrameIndex >= 0; FrameIndex--)
		{
			FStateTreeExecutionFrame& CurrentFrame = Exec.ActiveFrames[FrameIndex];
			const UStateTree* CurrentStateTree = CurrentFrame.StateTree;
			const int32 CurrentActiveNodeIndex = CurrentFrame.ActiveNodeIndex.AsInt32();

			for (int32 StateIndex = CurrentFrame.ActiveStates.Num() - 1; StateIndex >= 0; --StateIndex)
			{
				const FStateTreeStateHandle StateHandle = CurrentFrame.ActiveStates[StateIndex];
				const UE::StateTree::FActiveStateID StateID = CurrentFrame.ActiveStates.StateIDs[StateIndex];
				const FCompactStateTreeState& State = CurrentStateTree->States[StateHandle.Index];

				checkf(State.bEnabled, TEXT("Only enabled states are in ActiveStates."));

				// Transition tasks.
				if (State.bHasTransitionTasks)
				{
					bool bAdded = false;
					const int32 TaskEnd = FMath::Min(CurrentActiveNodeIndex + 1, State.TasksBegin + State.TasksNum);
					for (int32 TaskIndex = TaskEnd - 1; TaskIndex >= State.TasksBegin; --TaskIndex)
					{
						const FStateTreeTaskBase& Task = CurrentStateTree->Nodes[TaskIndex].Get<const FStateTreeTaskBase>();
						if (Task.bShouldAffectTransitions && Task.bTaskEnabled)
						{
							TransitionHandlers.Emplace(FrameIndex, StateHandle, StateID, FStateTreeIndex16(TaskIndex), Task.TransitionHandlingPriority);
							bAdded = true;
						}
					}
					ensureMsgf(bAdded, TEXT("bHasTransitionTasks is set but not task were added for the State: '%s' inside theStateTree %s"), *State.Name.ToString(), *CurrentStateTree->GetPathName());
				}

				// Has expired transition delayed.
				const bool bHasActiveTransitionDelayed = ExpiredTransitionsDelayed.ContainsByPredicate([StateID](const FStateTreeTransitionDelayedState& Other)
					{
						return Other.StateID == StateID;
					});

				// Regular transitions on state
				//or A transition task can trigger an event. We need to add the state if that is a possibility
				//or Expired transition delayed
				if (State.ShouldTickTransitions(bHasEvents, bHasBroadcastedDelegates) || State.bHasTransitionTasks || bHasActiveTransitionDelayed)
				{
					TransitionHandlers.Emplace(FrameIndex, StateHandle, StateID, EStateTreeTransitionPriority::Normal);
				}
			}

			if (CurrentFrame.bIsGlobalFrame)
			{
				// Global transition tasks.
				if (CurrentFrame.StateTree->bHasGlobalTransitionTasks)
				{
					bool bAdded = false;
					const int32 TaskEnd = FMath::Min(CurrentActiveNodeIndex + 1, CurrentStateTree->GlobalTasksBegin + CurrentStateTree->GlobalTasksNum);
					for (int32 TaskIndex = TaskEnd - 1; TaskIndex >= CurrentFrame.StateTree->GlobalTasksBegin; --TaskIndex)
					{
						const FStateTreeTaskBase& Task = CurrentStateTree->Nodes[TaskIndex].Get<const FStateTreeTaskBase>();
						if (Task.bShouldAffectTransitions && Task.bTaskEnabled)
						{
							TransitionHandlers.Emplace(FrameIndex, FStateTreeStateHandle(), UE::StateTree::FActiveStateID::Invalid, FStateTreeIndex16(TaskIndex), Task.TransitionHandlingPriority);
							bAdded = true;
						}
					}
					ensureMsgf(bAdded, TEXT("bHasGlobalTransitionTasks is set but not task were added for the StateTree `%s`"), *CurrentStateTree->GetPathName());
				}
			}
		}

		// Sort by priority and adding order.
		TransitionHandlers.StableSort();
	}

	//
	// Process task and state transitions in priority order. 
	//
	for (const FTransitionHandler& Handler : TransitionHandlers)
	{
		const int32 FrameIndex = Handler.FrameIndex;
		FStateTreeExecutionFrame* CurrentParentFrame = FrameIndex > 0 ? &Exec.ActiveFrames[FrameIndex - 1] : nullptr;
		FStateTreeExecutionFrame& CurrentFrame = Exec.ActiveFrames[FrameIndex];
		const UE::StateTree::FActiveFrameID CurrentFrameID = CurrentFrame.FrameID;
		const UStateTree* CurrentStateTree = CurrentFrame.StateTree;

		FCurrentlyProcessedFrameScope FrameScope(*this, CurrentParentFrame, CurrentFrame);
		FCurrentlyProcessedStateScope StateScope(*this, Handler.StateHandle);
		UE_STATETREE_DEBUG_SCOPED_STATE(this, Handler.StateHandle);

		if (Handler.TaskIndex.IsValid())
		{
			const FStateTreeTaskBase& Task = CurrentStateTree->Nodes[Handler.TaskIndex.Get()].Get<const FStateTreeTaskBase>();

			// Ignore disabled task
			if (Task.bTaskEnabled == false)
			{
				STATETREE_LOG(VeryVerbose, TEXT("%*sSkipped 'TriggerTransitions' for disabled Task: '%s'"), UE::StateTree::Debug::IndentSize, TEXT(""), *Task.Name.ToString());
				continue;
			}

			const FStateTreeDataView TaskInstanceView = GetDataView(CurrentParentFrame, CurrentFrame, Task.InstanceDataHandle);
			FNodeInstanceDataScope DataScope(*this, &Task, Handler.TaskIndex.Get(), Task.InstanceDataHandle, TaskInstanceView);

			// Copy bound properties.
			if (Task.BindingsBatch.IsValid())
			{
				CopyBatchOnActiveInstances(CurrentParentFrame, CurrentFrame, TaskInstanceView, Task.BindingsBatch);
			}

			STATETREE_LOG(VeryVerbose, TEXT("%*sTriggerTransitions: '%s'"), UE::StateTree::Debug::IndentSize, TEXT(""), *Task.Name.ToString());
			UE_STATETREE_DEBUG_TASK_EVENT(this, Handler.TaskIndex.Get(), TaskInstanceView, EStateTreeTraceEventType::OnEvaluating, EStateTreeRunStatus::Running);
			check(TaskInstanceView.IsValid());

			Task.TriggerTransitions(*this);
		}
		else if (Handler.StateHandle.IsValid())
		{
			check(Handler.StateID.IsValid());
			const FCompactStateTreeState& State = CurrentStateTree->States[Handler.StateHandle.Index];

			// Transitions
			for (uint8 TransitionCounter = 0; TransitionCounter < State.TransitionsNum; ++TransitionCounter)
			{
				// All transition conditions must pass
				const int16 TransitionIndex = State.TransitionsBegin + TransitionCounter;
				const FCompactStateTransition& Transition = CurrentStateTree->Transitions[TransitionIndex];

				// Skip disabled transitions
				if (Transition.bTransitionEnabled == false)
				{
					continue;
				}

				// No need to test the transition if same or higher priority transition has already been processed.
				if (RequestedTransition.IsValid() && Transition.Priority <= RequestedTransition->Transition.Priority)
				{
					continue;
				}

				// Skip completion transitions
				if (EnumHasAnyFlags(Transition.Trigger, EStateTreeTransitionTrigger::OnStateCompleted))
				{
					continue;
				}

				// If a delayed transition has passed the delay, try trigger it.
				if (Transition.HasDelay())
				{
					bool bTriggeredDelayedTransition = false;
					for (const FStateTreeTransitionDelayedState& DelayedTransition : ExpiredTransitionsDelayed)
					{
						if (DelayedTransition.StateID == Handler.StateID && DelayedTransition.TransitionIndex == FStateTreeIndex16(TransitionIndex))
						{
							STATETREE_LOG(Verbose, TEXT("Passed delayed transition from '%s' (%s) -> '%s'"),
								*GetSafeStateName(CurrentFrame, CurrentFrame.ActiveStates.Last()), *State.Name.ToString(), *GetSafeStateName(CurrentFrame, Transition.State));

							// Trigger Delayed Transition when the delay has passed.
							const UE::StateTree::ExecutionContext::FStateHandleContext TargetState(CurrentFrame.StateTree, Transition.State);
							FTransitionArguments TransitionArgs;
							TransitionArgs.Priority = Transition.Priority;
							TransitionArgs.TransitionEvent = DelayedTransition.CapturedEvent;
							TransitionArgs.Fallback = Transition.Fallback;
							TransitionArgs.SelectionRules = Transition.GetStateSelectionRulesForTransition(TransitionArgs.SelectionRules);
							if (RequestTransitionInternal(CurrentFrame, Handler.StateID, TargetState, TransitionArgs))
							{
								// If the transition was successfully requested with a specific event, consume and remove the event, it's been used.
								if (DelayedTransition.CapturedEvent.IsValid() && Transition.bConsumeEventOnSelect)
								{
									ConsumeEvent(DelayedTransition.CapturedEvent);
								}

								RequestedTransition->Source = FStateTreeTransitionSource(CurrentFrame.StateTree, FStateTreeIndex16(TransitionIndex), Transition.State, Transition.Priority);
								bTriggeredDelayedTransition = true;
								break;
							}
						}
					}

					if (bTriggeredDelayedTransition)
					{
						continue;
					}
				}

				UE::StateTree::ExecutionContext::Private::FSharedEventInlineArray TransitionEvents;
				UE::StateTree::ExecutionContext::Private::GetTriggerTransitionEvent(Transition, CurrentlyProcessedBroadcastedDelegates, FStateTreeSharedEvent(), GetEventsToProcessView(), TransitionEvents);

				for (const FStateTreeSharedEvent& TransitionEvent : TransitionEvents)
				{
					bool bPassed = false;
					{
						FCurrentlyProcessedTransitionEventScope TransitionEventScope(*this, TransitionEvent.IsValid() ? TransitionEvent.Get() : nullptr);
						UE_STATETREE_DEBUG_TRANSITION_EVENT(this, FStateTreeTransitionSource(CurrentFrame.StateTree, FStateTreeIndex16(TransitionIndex), Transition.State, Transition.Priority), EStateTreeTraceEventType::OnEvaluating);
						bPassed = TestAllConditionsOnActiveInstances(CurrentParentFrame, CurrentFrame, Handler.StateHandle, Transition.ConditionEvaluationScopeMemoryRequirement, Transition.ConditionsBegin, Transition.ConditionsNum, EStateTreeUpdatePhase::TransitionConditions);
					}

					if (bPassed)
					{
						// If the transitions is delayed, set up the delay. 
						if (Transition.HasDelay())
						{
							uint32 TransitionEventHash = 0u;
							if (TransitionEvent.IsValid())
							{
								TransitionEventHash = GetTypeHash(TransitionEvent.Get());
							}

							const bool bIsDelayedTransitionExisting = Exec.DelayedTransitions.ContainsByPredicate([CurrentStateID = Handler.StateID, TransitionIndex, TransitionEventHash](const FStateTreeTransitionDelayedState& DelayedState)
								{
									return DelayedState.StateID == CurrentStateID
										&& DelayedState.TransitionIndex.Get() == TransitionIndex
										&& DelayedState.CapturedEventHash == TransitionEventHash;
								});

							if (!bIsDelayedTransitionExisting)
							{
								// Initialize new delayed transition.
								const float DelayDuration = Transition.Delay.GetRandomDuration(Exec.RandomStream);
								if (DelayDuration > 0.0f)
								{
									FStateTreeTransitionDelayedState& DelayedState = Exec.DelayedTransitions.AddDefaulted_GetRef();
									DelayedState.StateID = Handler.StateID;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
									DelayedState.StateTree = CurrentFrame.StateTree;
									DelayedState.StateHandle = Handler.StateHandle;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
									DelayedState.TransitionIndex = FStateTreeIndex16(TransitionIndex);
									DelayedState.TimeLeft = DelayDuration;
									if (TransitionEvent.IsValid())
									{
										DelayedState.CapturedEvent = TransitionEvent;
										DelayedState.CapturedEventHash = TransitionEventHash;
									}

									BeginDelayedTransition(DelayedState);
									STATETREE_LOG(Verbose, TEXT("Delayed transition triggered from '%s' (%s) -> '%s' %.1fs"),
										*GetSafeStateName(CurrentFrame, CurrentFrame.ActiveStates.Last()), *State.Name.ToString(), *GetSafeStateName(CurrentFrame, Transition.State), DelayedState.TimeLeft);

									// Delay state added, skip requesting the transition.
									continue;
								}
								// Fallthrough to request transition if duration was zero.
							}
							else
							{
								// We get here if the transitions re-triggers during the delay, on which case we'll just ignore it.
								continue;
							}
						}

						UE_STATETREE_DEBUG_TRANSITION_EVENT(this, FStateTreeTransitionSource(CurrentFrame.StateTree, FStateTreeIndex16(TransitionIndex), Transition.State, Transition.Priority), EStateTreeTraceEventType::OnRequesting);
						const UE::StateTree::ExecutionContext::FStateHandleContext TargetState(CurrentFrame.StateTree, Transition.State);
						FTransitionArguments TransitionArgs;
						TransitionArgs.Priority = Transition.Priority;
						TransitionArgs.TransitionEvent = TransitionEvent;
						TransitionArgs.Fallback = Transition.Fallback;
						TransitionArgs.SelectionRules = Transition.GetStateSelectionRulesForTransition(TransitionArgs.SelectionRules);
						if (RequestTransitionInternal(CurrentFrame, Handler.StateID, TargetState, TransitionArgs))
						{
							// If the transition was successfully requested with a specific event, consume and remove the event, it's been used.
							if (TransitionEvent.IsValid() && Transition.bConsumeEventOnSelect)
							{
								ConsumeEvent(TransitionEvent);
							}

							RequestedTransition->Source = FStateTreeTransitionSource(CurrentFrame.StateTree, FStateTreeIndex16(TransitionIndex), Transition.State, Transition.Priority);
							break;
						}
					}
				}
			}
		}
	}

	//
	// Check state completion transitions.
	//
	bool bProcessSubTreeCompletion = true;

	if (!RequestedTransition.IsValid()
		&& (Exec.LastTickStatus != EStateTreeRunStatus::Running || Exec.bHasPendingCompletedState))
	{
		// Find the pending completed frame/state. Don't cache the result because this function is reentrant.
		//Stop at the first completion.
		int32 FrameIndexToStart = -1;
		int32 StateIndexToStart = -1;
		EStateTreeRunStatus CurrentStatus = EStateTreeRunStatus::Unset;
		for (int32 FrameIndex = 0; FrameIndex < Exec.ActiveFrames.Num(); ++FrameIndex)
		{
			using namespace UE::StateTree;
			using namespace UE::StateTree::ExecutionContext;
			using namespace UE::StateTree::ExecutionContext::Private;

			const FStateTreeExecutionFrame& CurrentFrame = Exec.ActiveFrames[FrameIndex];
			const UStateTree* CurrentStateTree = CurrentFrame.StateTree;
			const ETaskCompletionStatus FrameTasksStatus = CurrentFrame.ActiveTasksStatus.GetStatus(CurrentStateTree).GetCompletionStatus();
			if (CurrentFrame.bIsGlobalFrame && FrameTasksStatus != ETaskCompletionStatus::Running)
			{
				if (FrameIndex == 0)
				{
					// If first frame, then complete the tree execution.
					Exec.RequestedStop = GetPriorityRunStatus(Exec.RequestedStop, CastToRunStatus(FrameTasksStatus));
					break;
				}
				else if (bGlobalTasksCompleteOwningFrame)
				{
					const int32 ParentFrameIndex = FrameIndex - 1;
					FStateTreeExecutionFrame& ParentFrame = Exec.ActiveFrames[ParentFrameIndex];
					const FStateTreeStateHandle ParentLinkedState = ParentFrame.ActiveStates.Last();
					if (ensure(ParentLinkedState.IsValid()))
					{
						// Set the parent linked state as last completed state, and update tick status to the status from the transition.
						STATETREE_LOG(Verbose, TEXT("Completed subtree '%s' from global: %s"),
							*GetSafeStateName(ParentFrame, ParentLinkedState),
							*UEnum::GetDisplayValueAsText(CastToRunStatus(FrameTasksStatus)).ToString()
						);

						const FCompactStateTreeState& State = ParentFrame.StateTree->States[ParentLinkedState.Index];
						ParentFrame.ActiveTasksStatus.GetStatus(State).SetCompletionStatus(FrameTasksStatus);
						Exec.bHasPendingCompletedState = true;

						CurrentStatus = CastToRunStatus(FrameTasksStatus);
						FrameIndexToStart = ParentFrameIndex;
						StateIndexToStart = ParentFrame.ActiveStates.Num() - 1;
						break;
					}
				}
			}

			for (int32 StateIndex = 0; StateIndex < CurrentFrame.ActiveStates.Num(); ++StateIndex)
			{
				const FStateTreeStateHandle CurrentHandle = CurrentFrame.ActiveStates[StateIndex];
				const FCompactStateTreeState& State = CurrentStateTree->States[CurrentHandle.Index];
				const ETaskCompletionStatus StateTasksStatus = CurrentFrame.ActiveTasksStatus.GetStatus(State).GetCompletionStatus();
				if (StateTasksStatus != ETaskCompletionStatus::Running)
				{
					CurrentStatus = CastToRunStatus(StateTasksStatus);
					FrameIndexToStart = FrameIndex;
					StateIndexToStart = StateIndex;
					break;
				}
			}

			if (CurrentStatus != EStateTreeRunStatus::Unset)
			{
				break;
			}
		}

		if (CurrentStatus != EStateTreeRunStatus::Unset)
		{
			const bool bIsCurrentStatusSucceeded = CurrentStatus == EStateTreeRunStatus::Succeeded;
			const bool bIsCurrentStatusFailed = CurrentStatus == EStateTreeRunStatus::Failed;
			const bool bIsCurrentStatusStopped = CurrentStatus == EStateTreeRunStatus::Stopped;
			checkf(bIsCurrentStatusSucceeded || bIsCurrentStatusFailed || bIsCurrentStatusStopped, TEXT("Running is not accepted in the CurrentStatus loop."));

			const EStateTreeTransitionTrigger CompletionTrigger = bIsCurrentStatusSucceeded ? EStateTreeTransitionTrigger::OnStateSucceeded : EStateTreeTransitionTrigger::OnStateFailed;

			// Start from the last completed state and move up to the first state.
			for (int32 FrameIndex = FrameIndexToStart; FrameIndex >= 0; --FrameIndex)
			{
				FStateTreeExecutionFrame* CurrentParentFrame = FrameIndex > 0 ? &Exec.ActiveFrames[FrameIndex - 1] : nullptr;
				FStateTreeExecutionFrame& CurrentFrame = Exec.ActiveFrames[FrameIndex];
				const UStateTree* CurrentStateTree = CurrentFrame.StateTree;

				FCurrentlyProcessedFrameScope FrameScope(*this, CurrentParentFrame, CurrentFrame);

				const int32 CurrentStateIndexToStart = FrameIndex == FrameIndexToStart ? StateIndexToStart : CurrentFrame.ActiveStates.Num() - 1;

				// Check completion transitions
				for (int32 StateIndex = CurrentStateIndexToStart; StateIndex >= 0; --StateIndex)
				{
					const FStateTreeStateHandle CurrentStateHandle = CurrentFrame.ActiveStates[StateIndex];
					const UE::StateTree::FActiveStateID CurrentStateID = CurrentFrame.ActiveStates.StateIDs[StateIndex];
					const FCompactStateTreeState& CurrentState = CurrentStateTree->States[CurrentStateHandle.Index];

					if (CurrentState.ShouldTickCompletionTransitions(bIsCurrentStatusSucceeded, bIsCurrentStatusFailed))
					{
						FCurrentlyProcessedStateScope StateScope(*this, CurrentStateHandle);
						UE_STATETREE_DEBUG_SCOPED_STATE_PHASE(this, CurrentStateHandle, EStateTreeUpdatePhase::TriggerTransitions);

						for (uint8 TransitionCounter = 0; TransitionCounter < CurrentState.TransitionsNum; ++TransitionCounter)
						{
							// All transition conditions must pass
							const int16 TransitionIndex = CurrentState.TransitionsBegin + TransitionCounter;
							const FCompactStateTransition& Transition = CurrentStateTree->Transitions[TransitionIndex];

							// Skip disabled transitions
							if (!Transition.bTransitionEnabled)
							{
								continue;
							}

							const bool bTransitionAccepted = !bIsCurrentStatusStopped
								? EnumHasAnyFlags(Transition.Trigger, CompletionTrigger)
								: Transition.Trigger == EStateTreeTransitionTrigger::OnStateCompleted;
							if (bTransitionAccepted)
							{
								bool bPassed = false;
								{
									UE_STATETREE_DEBUG_TRANSITION_EVENT(this, FStateTreeTransitionSource(CurrentFrame.StateTree, FStateTreeIndex16(TransitionIndex), Transition.State, Transition.Priority), EStateTreeTraceEventType::OnEvaluating);
									bPassed = TestAllConditionsOnActiveInstances(CurrentParentFrame, CurrentFrame, CurrentStateHandle, Transition.ConditionEvaluationScopeMemoryRequirement, Transition.ConditionsBegin, Transition.ConditionsNum, EStateTreeUpdatePhase::TransitionConditions);
								}

								if (bPassed)
								{
									// No delay allowed on completion conditions.
									// No priority on completion transitions, use the priority to signal that state is selected.
									UE_STATETREE_DEBUG_TRANSITION_EVENT(this, FStateTreeTransitionSource(CurrentFrame.StateTree, FStateTreeIndex16(TransitionIndex), Transition.State, Transition.Priority), EStateTreeTraceEventType::OnRequesting);
									const UE::StateTree::ExecutionContext::FStateHandleContext TargetState(CurrentFrame.StateTree, Transition.State);
									FTransitionArguments TransitionArgs;
									TransitionArgs.Priority = EStateTreeTransitionPriority::Normal;
									TransitionArgs.Fallback = Transition.Fallback;
									TransitionArgs.SelectionRules = Transition.GetStateSelectionRulesForTransition(TransitionArgs.SelectionRules);
									if (RequestTransitionInternal(CurrentFrame, CurrentStateID, TargetState, TransitionArgs))
									{
										RequestedTransition->Source = FStateTreeTransitionSource(CurrentFrame.StateTree, FStateTreeIndex16(TransitionIndex), Transition.State, Transition.Priority);
										break;
									}
								}
							}
						}

						if (RequestedTransition.IsValid())
						{
							break;
						}
					}
				}

				// if a valid completion transition has already been found, the remaining transitions in parent frames won't have a higher priority than the found one
				// so skip the remainder. this also prevented false positive warnings and ensures from STDebugger
				if (RequestedTransition.IsValid())
				{
					break;
				}
			}

			// Handle the case where no transition was found.
			if (!RequestedTransition.IsValid())
			{
				STATETREE_LOG(Verbose, TEXT("Could not trigger completion transition, jump back to root state."));
				UE_STATETREE_DEBUG_LOG_EVENT(this, Log, TEXT("Could not trigger completion transition, jump back to root state."));

				check(!Exec.ActiveFrames.IsEmpty());
				FStateTreeExecutionFrame& RootFrame = Exec.ActiveFrames[0];
				check(RootFrame.ActiveStates.Num() != 0);
				FCurrentlyProcessedFrameScope RootFrameScope(*this, nullptr, RootFrame);
				FCurrentlyProcessedStateScope RootStateScope(*this, FStateTreeStateHandle::Root);

				UE::StateTree::ExecutionContext::FStateHandleContext TargetState(RootFrame.StateTree, FStateTreeStateHandle::Root);
				UE_STATETREE_DEBUG_TRANSITION_EVENT(this, FStateTreeTransitionSource(RootFrame.StateTree, EStateTreeTransitionSourceType::Internal, FStateTreeStateHandle::Root, EStateTreeTransitionPriority::Normal), EStateTreeTraceEventType::OnRequesting);

				if (RequestTransitionInternal(RootFrame, RootFrame.ActiveStates.StateIDs[0], TargetState, FTransitionArguments()))
				{
					RequestedTransition->Source = FStateTreeTransitionSource(RootFrame.StateTree, EStateTreeTransitionSourceType::Internal, FStateTreeStateHandle::Root, EStateTreeTransitionPriority::Normal);
				}
				else
				{
					STATETREE_LOG(Warning, TEXT("Failed to select root state. Stopping the tree with failure."));
					UE_STATETREE_DEBUG_LOG_EVENT(this, Error, TEXT("Failed to select root state. Stopping the tree with failure."));

					Exec.RequestedStop = UE::StateTree::ExecutionContext::GetPriorityRunStatus(Exec.RequestedStop, EStateTreeRunStatus::Failed);

					// In this case we don't want to complete subtrees, we want to force the whole tree to stop.
					bProcessSubTreeCompletion = false;
				}
			}
		}
	}

	// Check if the transition was succeed/failed, if we're on a sub-tree, complete the subtree instead of transition.
	if (RequestedTransition.IsValid() && RequestedTransition->Transition.TargetState.IsCompletionState() && bProcessSubTreeCompletion)
	{
		// Check that the transition source frame is a sub-tree, the first frame (0 index) is not a subtree. 
		const int32 SourceFrameIndex = Exec.IndexOfActiveFrame(RequestedTransition->Transition.SourceFrameID);
		if (SourceFrameIndex > 0)
		{
			const FStateTreeExecutionFrame& SourceFrame = Exec.ActiveFrames[SourceFrameIndex];
			const int32 ParentFrameIndex = SourceFrameIndex - 1;
			FStateTreeExecutionFrame& ParentFrame = Exec.ActiveFrames[ParentFrameIndex];
			const FStateTreeStateHandle ParentLinkedState = ParentFrame.ActiveStates.Last();

			if (ParentLinkedState.IsValid())
			{
				const EStateTreeRunStatus RunStatus = RequestedTransition->Transition.TargetState.ToCompletionStatus();

#if ENABLE_VISUAL_LOG
				const int32 NextTransitionSourceIndex = SourceFrame.ActiveStates.IndexOfReverse(RequestedTransition->Transition.SourceStateID);
				const FStateTreeStateHandle NextTransitionSourceState = NextTransitionSourceIndex != INDEX_NONE
					? SourceFrame.ActiveStates[NextTransitionSourceIndex]
					: FStateTreeStateHandle::Invalid;
				STATETREE_LOG(Verbose, TEXT("Completed subtree '%s' from state '%s': %s"),
					*GetSafeStateName(ParentFrame, ParentLinkedState),
					*GetSafeStateName(SourceFrame, NextTransitionSourceState),
					*UEnum::GetDisplayValueAsText(RunStatus).ToString()
				);
#endif

				// Set the parent linked state as last completed state, and update tick status to the status from the transition.
				const UE::StateTree::ETaskCompletionStatus TaskStatus = UE::StateTree::ExecutionContext::CastToTaskStatus(RunStatus);
				const FCompactStateTreeState& State = ParentFrame.StateTree->States[ParentLinkedState.Index];
				ParentFrame.ActiveTasksStatus.GetStatus(State).SetCompletionStatus(TaskStatus);
				Exec.bHasPendingCompletedState = true;
				Exec.LastTickStatus = RunStatus;

				Storage.ResetTemporaryInstances();

				// now starting from the linked parent state.
				bTransitionsTriggeredFromSubtreeCompletion = true;
				return TriggerTransitions({ .CompletedSubtreeParentFrameIndex = ParentFrameIndex });
			}
		}
	}

	// Request can be no-op, used for blocking other transitions.
	if (RequestedTransition.IsValid() && !RequestedTransition->Transition.TargetState.IsValid())
	{
		RequestedTransition.Reset();
	}

	return RequestedTransition.IsValid();
}

// Deprecated
TOptional<FStateTreeTransitionResult> FStateTreeExecutionContext::MakeTransitionResult(const FRecordedStateTreeTransitionResult& RecordedTransition) const
{
	FStateTreeTransitionResult Result;

#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	for (int32 RecordedFrameIndex = 0; RecordedFrameIndex < RecordedTransition.NextActiveFrames.Num(); RecordedFrameIndex++)
	{
		const FRecordedStateTreeExecutionFrame& RecordedExecutionFrame = RecordedTransition.NextActiveFrames[RecordedFrameIndex];

		if (RecordedExecutionFrame.StateTree == nullptr)
		{
			return {};
		}

		if (RecordedExecutionFrame.StateTree->GetStateFromHandle(RecordedExecutionFrame.RootState) == nullptr)
		{
			return {};
		}

		const FCompactStateTreeFrame* CompactFrame = RecordedExecutionFrame.StateTree->GetFrameFromHandle(RecordedExecutionFrame.RootState);
		if (CompactFrame == nullptr)
		{
			return {};
		}

		FStateTreeExecutionFrame& ExecutionFrame = Result.NextActiveFrames.AddDefaulted_GetRef();
		ExecutionFrame.StateTree = RecordedExecutionFrame.StateTree;
		ExecutionFrame.RootState = RecordedExecutionFrame.RootState;
		ExecutionFrame.ActiveStates = RecordedExecutionFrame.ActiveStates;
		ExecutionFrame.ActiveTasksStatus = FStateTreeTasksCompletionStatus(*CompactFrame);
		ExecutionFrame.bIsGlobalFrame = RecordedExecutionFrame.bIsGlobalFrame;
		ExecutionFrame.ExternalDataBaseIndex = const_cast<FStateTreeExecutionContext*>(this)->CollectExternalData(RecordedExecutionFrame.StateTree);

		FStateTreeFrameStateSelectionEvents& StateTreeFrameStateSelectionEvents = Result.NextActiveFrameEvents.AddDefaulted_GetRef();
		for (int32 EventIdx = 0; EventIdx < RecordedExecutionFrame.EventIndices.Num(); EventIdx++)
		{
			if (RecordedTransition.NextActiveFrameEvents.IsValidIndex(EventIdx))
			{
				const FStateTreeEvent& RecordedStateTreeEvent = RecordedTransition.NextActiveFrameEvents[EventIdx];
				StateTreeFrameStateSelectionEvents.Events[EventIdx] = FStateTreeSharedEvent(RecordedStateTreeEvent);
			}
		}
	}


	if (Result.NextActiveFrames.Num() != Result.NextActiveFrameEvents.Num())
	{
		return {};
	}

	if (RecordedTransition.SourceStateTree == nullptr)
	{
		return {};
	}

	if (RecordedTransition.SourceStateTree->GetFrameFromHandle(RecordedTransition.SourceRootState) == nullptr)
	{
		return {};
	}

	// Try to find the same frame and the same state in the currently active frames.
	// Recorded transitions can be saved and replayed out of context.
	const FStateTreeExecutionState& Exec = GetExecState();
	const FStateTreeExecutionFrame* ExecFrame = Exec.ActiveFrames.FindByPredicate([StateTree = RecordedTransition.SourceStateTree, RootState = RecordedTransition.SourceRootState](const FStateTreeExecutionFrame& Frame)
		{
			return Frame.HasRoot(StateTree, RootState);
		});
	if (ExecFrame)
	{
		Result.SourceFrameID = ExecFrame->FrameID;
		const int32 SourceStateIndex = ExecFrame->ActiveStates.IndexOfReverse(RecordedTransition.SourceState);
		if (SourceStateIndex != INDEX_NONE)
		{
			Result.SourceStateID = ExecFrame->ActiveStates.StateIDs[SourceStateIndex];
		}
	}
	Result.TargetState = RecordedTransition.TargetState;
	Result.Priority = RecordedTransition.Priority;
	Result.SourceState = RecordedTransition.SourceState;
	Result.SourceStateTree = RecordedTransition.SourceStateTree;
	Result.SourceRootState = RecordedTransition.SourceRootState;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif //WITH_EDITOR

	return Result;
}

// Deprecated
FRecordedStateTreeTransitionResult FStateTreeExecutionContext::MakeRecordedTransitionResult(const FStateTreeTransitionResult& Transition) const
{
	FRecordedStateTreeTransitionResult Result;

#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	for (int32 FrameIndex = 0; FrameIndex < Transition.NextActiveFrames.Num(); FrameIndex++)
	{
		const FStateTreeExecutionFrame& ExecutionFrame = Transition.NextActiveFrames[FrameIndex];
		const FStateTreeFrameStateSelectionEvents& StateSelectionEvents = Transition.NextActiveFrameEvents[FrameIndex];

		FRecordedStateTreeExecutionFrame& RecordedFrame = Result.NextActiveFrames.AddDefaulted_GetRef();
		RecordedFrame.StateTree = ExecutionFrame.StateTree;
		RecordedFrame.RootState = ExecutionFrame.RootState;
		RecordedFrame.ActiveStates = ExecutionFrame.ActiveStates;
		RecordedFrame.bIsGlobalFrame = ExecutionFrame.bIsGlobalFrame;

		for (int32 StateIndex = 0; StateIndex < ExecutionFrame.ActiveStates.Num(); StateIndex++)
		{
			const FStateTreeEvent* Event = StateSelectionEvents.Events[StateIndex].Get();
			if (Event)
			{
				const int32 EventIndex = Result.NextActiveFrameEvents.Add(*Event);
				RecordedFrame.EventIndices[StateIndex] = static_cast<uint8>(EventIndex);
			}
		}
	}

	const FStateTreeExecutionState& Exec = GetExecState();
	if (const FStateTreeExecutionFrame* FoundSourceFrame = Exec.FindActiveFrame(Transition.SourceFrameID))
	{
		Result.SourceStateTree = FoundSourceFrame->StateTree;
		Result.SourceRootState = FoundSourceFrame->RootState;
		const int32 ActiveStateIndex = FoundSourceFrame->ActiveStates.IndexOfReverse(Transition.SourceStateID);
		if (ActiveStateIndex != INDEX_NONE)
		{
			Result.SourceState = FoundSourceFrame->ActiveStates[ActiveStateIndex];
		}
	}
	else
	{
		Result.SourceStateTree = Transition.SourceStateTree;
		Result.SourceRootState = Transition.SourceRootState;
		Result.SourceState = Transition.SourceState;
	}
	Result.TargetState = Transition.TargetState;
	Result.Priority = Transition.Priority;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif //WITH_EDITOR

	return Result;
}

FRecordedStateTreeTransitionResult FStateTreeExecutionContext::MakeRecordedTransitionResult(const TSharedRef<FSelectStateResult>& SelectStateResult, const FStateTreeTransitionResult& TransitionResult) const
{
	using namespace UE::StateTree;
	using namespace UE::StateTree::ExecutionContext;
	using namespace UE::StateTree::ExecutionContext::Private;

	const FStateTreeExecutionState& Exec = GetExecState();

	FRecordedStateTreeTransitionResult Recorded;
	Recorded.States.Reserve(SelectStateResult->SelectedStates.Num());
	for (const UE::StateTree::FActiveState& SelectedState : SelectStateResult->SelectedStates)
	{
		const FStateTreeExecutionFrame* ProcessFrame = UE::StateTree::ExecutionContext::FindExecutionFrame(SelectedState.GetFrameID(), MakeConstArrayView(Exec.ActiveFrames), MakeConstArrayView(SelectStateResult->TemporaryFrames));
		if (ProcessFrame == nullptr)
		{
			return FRecordedStateTreeTransitionResult();
		}
		FRecordedActiveState& RecordedState = Recorded.States.AddDefaulted_GetRef();
		RecordedState.StateTree = ProcessFrame->StateTree;
		RecordedState.State = SelectedState.GetStateHandle();
	}

	Recorded.Events.Reserve(SelectStateResult->SelectionEvents.Num());
	for (const FSelectionEventWithID& SelectedEvent : SelectStateResult->SelectionEvents)
	{
		if (SelectedEvent.Event.IsValid())
		{
			const int32 RecordedEventIndex = Recorded.Events.Add(*SelectedEvent.Event.Get());
			if (const int32 SeletedStateIndex = FActiveStatePath::IndexOf(SelectStateResult->SelectedStates, SelectedEvent.State); SeletedStateIndex != INDEX_NONE)
			{
				Recorded.States[SeletedStateIndex].EventIndex = RecordedEventIndex;
			}
		}
	}

	Recorded.Priority = TransitionResult.Priority;

	return Recorded;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
// Deprecated
bool FStateTreeExecutionContext::SelectState(const FStateTreeExecutionFrame& CurrentFrame,
	const FStateTreeStateHandle NextState,
	FStateSelectionResult& OutSelectionResult,
	const FStateTreeSharedEvent* TransitionEvent,
	const EStateTreeSelectionFallback Fallback)
{
	using namespace UE::StateTree;
	using namespace UE::StateTree::ExecutionContext;
	using namespace UE::StateTree::ExecutionContext::Private;

	FActiveStateInlineArray CurrentActiveStatePath;
	GetActiveStatePath(GetExecState().ActiveFrames, CurrentActiveStatePath);

	FSelectStateArguments SelectStateArgs;
	SelectStateArgs.ActiveStates = MakeConstArrayView(CurrentActiveStatePath);
	SelectStateArgs.SourceState = FActiveState();
	SelectStateArgs.TargetState = FStateHandleContext(CurrentFrame.StateTree, NextState);
	SelectStateArgs.TransitionEvent = TransitionEvent != nullptr ? *TransitionEvent : FStateTreeSharedEvent();
	SelectStateArgs.Fallback = Fallback;
	SelectStateArgs.SelectionRules = CurrentFrame.StateTree->StateSelectionRules;

	TSharedRef<FSelectStateResult> SelectStateResult = MakeShared<FSelectStateResult>();
	TGuardValue<TSharedPtr<FSelectStateResult>> ExecutionFrameHolderScope(CurrentlyProcessedTemporaryStorage, SelectStateResult);
	return SelectState(SelectStateArgs, SelectStateResult);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool FStateTreeExecutionContext::SelectState(const FSelectStateArguments& Args, const TSharedRef<FSelectStateResult>& OutSelectionResult)
{
	// 1. Find the target root state.
	// 2. If the the target frame is active, then copy all previous states from the previous frames.
	//   If any state inside the frame is completed (before the target), then the transition fails.
	//   See EStateTreeStateSelectionRules::CompletedStateBeforeTargetFailTransition.
	// 3. In the target frame, start adding states that match the source/target.
	//   If the state is completed (before the target), then the transition fails.
	//   See EStateTreeStateSelectionRules::CompletedStateBeforeTargetFailTransition.
	// 4. New/Sustained states need to be reevaluated (see SelectStateInternal).
	// 5. Else, handle fallback.
	// 
	// Source is from where the transition request occurs.
	//  The source frame is valid but the source state can be invalid. It will be the top root state if needed.
	//  It doesn't need to be in the same frame as the target.
	// Target is where we wish to go. The selection can stop or select another state (depending on the state's type).
	// Selected is the selection result.
	//
	// ExitState: If the state >= Target and is in the selected, then the transition is "sustained" and the instance data is untouched.
	// ExitState: If the state not in the selected (removed state), then the transition is "changed" and the state is removed.
	// EnterState: If the state >= Target and it is in the actives, then the transition is "sustained" and the instance data is untouched.
	// EnterState: If the state >= Target and it is not in the actives (new state), then the transition is "changed" and the state is added.
	// 
	// Rules might impact the results.
	// In the examples,  "New State if completed" is only valid if EStateTreeStateSelectionRules::CompletedTransitionStatesCreateNewStates
	// Examples: Active | Source | Target | Selected
	//
	//           ABCD   | ABCD   | ABCDE  | ABCDEF, ABCD'EF
	//  New D if completed, always new EF.
	//  ExitState: Sustained -. Changed -, D. EnterState: Sustained -. Changed EF, D'EF.
	//
	//           ABCD   | AB     | ABI    | ABIJ, AB'IJ
	//  New B if completed, always new IJ.
	//  ExitState: Sustained -. Changed CD, BCD. EnterState: Sustained -. Changed IJ, B'IJ.
	// 
	//           ABCD   | AB     | ABC    | ABCD, ABCD', ABC'D', AB'C'D'
	//  New D, CD, BCD if completed.
	//  ExitState: Sustained CD, C, -, -. Changed -, D, CD, BCD. EnterState: Sustained CD, C, -, -. Changed -, D', C'D', B'C'D'.
	//
	//           ABCD   | ABC     | AB    | ABCD, ABCD', ABC'D', AB'C'D'
	//  New D, CD, BCD if completed.
	//  ExitState: Sustained BCD, BC, B, -. Changed -, D, CD, BCD. EnterState: Sustained BCD, BC, B, -. Changed -, D', C'D', B'C'D'.
	// 
	//           ABCD   | AB     | AX     | AXY
	//  Source is not in target. New XY.
	//  ExitState: Sustained -. Changed BCD. EnterState: Sustained -. Changed XY
	//
	//           ABCD   | AB     | AB     | ABCD, ABCD', ABC'D', AB'C'D'
	//  Source is target. New D, CD, BCD if completed.
	//  ExitState: Sustained BCD, BC, B, -. Changed -, D, CD, BCD. EnterState: Sustained BCD, BC, B, -. Changed -, D', C'D', B'C'D'.

	using namespace UE::StateTree;
	using namespace UE::StateTree::ExecutionContext;
	using namespace UE::StateTree::ExecutionContext::Private;

	FStateTreeExecutionState& Exec = GetExecState();

	// Not a valid target
	if (!ensure(Args.TargetState.StateHandle.IsValid() && !Args.TargetState.StateHandle.IsCompletionState()))
	{
		return false;
	}
	if (!ensure(Args.TargetState.StateTree && Args.TargetState.StateTree->GetStateFromHandle(Args.TargetState.StateHandle)))
	{
		return false;
	}
	// Not a valid source. Note Source can be a GlobalTask (not in active state)
	if (!ensure(Args.SourceState.GetFrameID().IsValid()))
	{
		return false;
	}

	TArray<FStateHandleContext, TInlineAllocator<FStateTreeActiveStates::MaxStates>> PathToTargetState;
	if (!GetStatesListToState(Args.TargetState, PathToTargetState))
	{
		STATETREE_LOG(Error, TEXT("%hs: Reached max execution depth when trying to select state %s from '%s'.  '%s' using StateTree '%s'."),
			__FUNCTION__,
			*GetSafeStateName(Args.TargetState.StateTree, Args.TargetState.StateHandle),
			*GetStateStatusString(Exec),
			*GetNameSafe(&Owner),
			*GetFullNameSafe(&RootStateTree)
		);
		return false;
	}
	check(!PathToTargetState.IsEmpty());

	const FExecutionFrameHandle TargetFrameHandle = FExecutionFrameHandle(Args.TargetState.StateTree, PathToTargetState[0].StateHandle);
	const FCompactStateTreeFrame* TargetFrame = TargetFrameHandle.GetStateTree()->GetFrameFromHandle(TargetFrameHandle.GetRootState());
	checkf(TargetFrame, TEXT("The frame was not compiled."));

	// Build the source path from the active path. Includes all the states from the previous frames.
	TArrayView<const FActiveState> SourceStates;
	{
		if (Args.SourceState.GetStateID().IsValid())
		{
			const int32 FoundIndex = FActiveStatePath::IndexOf(Args.ActiveStates, Args.SourceState);
			if (FoundIndex == INDEX_NONE)
			{
				STATETREE_LOG(Error, TEXT("%hs: The source do not exist in the active path when trying to select state %s from '%s'. '%s' using StateTree '%s'."),
					__FUNCTION__,
					*GetSafeStateName(Args.TargetState.StateTree, Args.TargetState.StateHandle),
					*GetStateStatusString(Exec),
					*GetNameSafe(&Owner),
					*GetFullNameSafe(&RootStateTree)
				);
				return false;
			}

			SourceStates = Args.ActiveStates.Left(FoundIndex + 1);
		}
		else
		{
			// Pick the source frame first state.
			//This is usually when the request is out of the scope or from a global task. It should start from the root of the frame.
			int32 ActiveStateIndex = 0;
			for (; ActiveStateIndex < Args.ActiveStates.Num(); ++ActiveStateIndex)
			{
				const FActiveFrameID CurrentActiveFrameID = Args.ActiveStates[ActiveStateIndex].GetFrameID();
				if (CurrentActiveFrameID == Args.SourceState.GetFrameID())
				{
					break;
				}
			}
			if (ActiveStateIndex < Args.ActiveStates.Num())
			{
				SourceStates = Args.ActiveStates.Left(ActiveStateIndex + 1);
			}
			else
			{
				// SourceFrame exists, so the state must be in the active states.
				if (!ensure(Args.ActiveStates.Num() == 0))
				{
					return false;
				}
			}
		}
	}
	checkf(SourceStates.Num() == 0 || FActiveStatePath::StartsWith(Args.ActiveStates, SourceStates), TEXT("Source is part of the active path."));
	const FStateTreeExecutionFrame* SourceExecFrame = UE::StateTree::ExecutionContext::FindExecutionFrame(Args.SourceState.GetFrameID(), MakeConstArrayView(Exec.ActiveFrames), MakeConstArrayView(OutSelectionResult->TemporaryFrames));
	if (!ensure(SourceExecFrame))
	{
		return false;
	}

	const int32 SelectedStatesSlack = Args.ActiveStates.Num() + 10; // Number to help with buffer reallocation
	OutSelectionResult->SelectedStates.Empty(SelectedStatesSlack);

	// Find the Target inside the ActiveStates.
	//Look in the SourceStates because you can select in a different frame up the tree but not down the tree.
	FActiveFrameID TargetFrameID;
	{
		if (SourceExecFrame->HasRoot(TargetFrameHandle))
		{
			TargetFrameID = SourceExecFrame->FrameID;

			// Copy active to the new selected until we reach the frame.
			FActiveFrameID CurrentFrameID;
			for (const FActiveState& ActiveState : Args.ActiveStates)
			{
				if (CurrentFrameID != ActiveState.GetFrameID())
				{
					CurrentFrameID = ActiveState.GetFrameID();
					if (ActiveState.GetFrameID() == Args.SourceState.GetFrameID())
					{
						break;
					}
					OutSelectionResult->SelectedFrames.Add(CurrentFrameID);
				}
				OutSelectionResult->SelectedStates.Add(ActiveState);
			}
		}
		else if (SourceStates.Num() >= 0)
		{
			// Can jump to a state from a previous frame.
			//Find the common frame or the first frame with the same tree asset.
			int32 FoundActiveStateIndex = SourceStates.Num() - 1;
			{
				bool bFoundRootState = false;
				bool bFoundStateTree = false;
				FActiveFrameID CurrentFrameID;
				for (; FoundActiveStateIndex >= 0; --FoundActiveStateIndex)
				{
					const FActiveFrameID CurrentActiveFrameID = SourceStates[FoundActiveStateIndex].GetFrameID();
					if (CurrentActiveFrameID != CurrentFrameID)
					{
						const FStateTreeExecutionFrame* ProcessFrame = UE::StateTree::ExecutionContext::FindExecutionFrame(CurrentActiveFrameID, MakeConstArrayView(Exec.ActiveFrames), MakeConstArrayView(OutSelectionResult->TemporaryFrames));
						if (!ensure(ProcessFrame))
						{
							return false;
						}
						if (ProcessFrame->StateTree == TargetFrameHandle.GetStateTree())
						{
							bFoundStateTree = true;
							if (ProcessFrame->RootState == TargetFrameHandle.GetRootState())
							{
								bFoundRootState = true;
							}
							else if (bFoundRootState)
							{
								TargetFrameID = CurrentActiveFrameID;
								break;
							}
						}
						else if (bFoundStateTree)
						{
							break;
						}
					}
				}

				if (FoundActiveStateIndex < 0 && !bFoundStateTree)
				{
					STATETREE_LOG(Error, TEXT("%hs: Encountered unrecognized state %s during state selection from '%s'.  '%s' using StateTree '%s'."),
						__FUNCTION__,
						*GetSafeStateName(Args.TargetState.StateTree, Args.TargetState.StateHandle),
						*GetStateStatusString(Exec),
						*GetNameSafe(&Owner),
						*GetFullNameSafe(&RootStateTree)
					);
					return false;
				}

				// Copy active states to the new selected until we reach the frame.
				for (int32 ActiveStateIndex = 0; ActiveStateIndex < FoundActiveStateIndex + 1; ++ActiveStateIndex)
				{
					const FActiveState& ActiveState = Args.ActiveStates[ActiveStateIndex];
					if (CurrentFrameID != ActiveState.GetFrameID())
					{
						CurrentFrameID = ActiveState.GetFrameID();
						OutSelectionResult->SelectedFrames.Add(CurrentFrameID);
					}
					OutSelectionResult->SelectedStates.Add(ActiveState);
				}
			}
		}
		else
		{
			// SourceExecFrame do not matches and there are no source states.
			STATETREE_LOG(Error, TEXT("%hs: Encountered out of range state %s during state selection from '%s'.  '%s' using StateTree '%s'."),
				__FUNCTION__,
				*GetSafeStateName(Args.TargetState.StateTree, Args.TargetState.StateHandle),
				*GetStateStatusString(Exec),
				*GetNameSafe(&Owner),
				*GetFullNameSafe(&RootStateTree)
			);
			return false;
		}
	}

	FSelectStateInternalArguments InternalArgs;
	InternalArgs.MissingActiveStates = Args.ActiveStates.Mid(OutSelectionResult->SelectedStates.Num());
	InternalArgs.MissingSourceStates = SourceStates.Mid(OutSelectionResult->SelectedStates.Num());
	InternalArgs.MissingStatesToReachTarget = PathToTargetState;
	InternalArgs.MissingSourceFrameID = TargetFrameID;

	// Add the state and check the prerequisites.
	if (SelectStateInternal(Args, InternalArgs, OutSelectionResult))
	{
		return true;
	}

	// Failed to Select Target State, handle fallback here
	if (Args.Fallback == EStateTreeSelectionFallback::NextSelectableSibling
		&& PathToTargetState.Num() >= 2 // we are not selecting the root
		&& TargetFrameID.IsValid()) // we are not selecting a new frame on purpose
	{
		const FStateHandleContext Parent = PathToTargetState.Last(1);
		FStateHandleContext& Target = PathToTargetState.Last();
		if (ensure(Parent.StateTree && Parent.StateHandle.IsValid() && Parent.StateTree == Target.StateTree && Target.StateHandle.IsValid()))
		{
			// Get the next sibling
			const FCompactStateTreeState& ParentState = Parent.StateTree->States[Parent.StateHandle.Index];
			for (uint16 ChildStateIndex = Target.StateTree->States[Target.StateHandle.Index].GetNextSibling(); ChildStateIndex < ParentState.ChildrenEnd; ChildStateIndex = Target.StateTree->States[Target.StateHandle.Index].GetNextSibling())
			{
				const FStateTreeStateHandle NextChildStateHandle = FStateTreeStateHandle(ChildStateIndex);

				FSelectStateArguments NewArgs = Args;
				NewArgs.TargetState.StateHandle = NextChildStateHandle;
				Target = FStateHandleContext(Parent.StateTree, NextChildStateHandle);
				if (SelectStateInternal(NewArgs, InternalArgs, OutSelectionResult))
				{
					return true;
				}
			}
		}
	}

	return false;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
// Deprecated
bool FStateTreeExecutionContext::SelectStateInternal(
	const FStateTreeExecutionFrame* CurrentParentFrame,
	FStateTreeExecutionFrame& CurrentFrame,
	const FStateTreeExecutionFrame* CurrentFrameInActiveFrames,
	TConstArrayView<FStateTreeStateHandle> PathToNextState,
	FStateSelectionResult& OutSelectionResult,
	const FStateTreeSharedEvent* TransitionEvent)
{
	return false;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool FStateTreeExecutionContext::SelectStateInternal(
	const FSelectStateArguments& Args,
	const FSelectStateInternalArguments& InternalArgs,
	const TSharedRef<FSelectStateResult>& OutSelectionResult)
{
	CSV_SCOPED_TIMING_STAT(StateTree, SelectState);

	using namespace UE::StateTree;
	using namespace UE::StateTree::ExecutionContext;
	using namespace UE::StateTree::ExecutionContext::Private;

	const FStateTreeExecutionState& Exec = GetExecState();

	if (InternalArgs.MissingStatesToReachTarget.IsEmpty())
	{
		return true;
	}

	const FStateHandleContext& NextStateContext = InternalArgs.MissingStatesToReachTarget[0];
	const UStateTree* NextStateTree = NextStateContext.StateTree;
	const FStateTreeStateHandle NextStateHandle = NextStateContext.StateHandle;
	if (!ensure(NextStateTree != nullptr && NextStateHandle.IsValid()))
	{
		return false;
	}

	const FCompactStateTreeState& NextState = NextStateTree->States[NextStateHandle.Index];
	if (!NextState.bEnabled)
	{
		STATETREE_LOG(VeryVerbose, TEXT("%hs: Ignoring disabled state '%s'.  '%s' using StateTree '%s'."),
			__FUNCTION__,
			*NextState.Name.ToString(),
			*GetNameSafe(&Owner),
			*GetFullNameSafe(NextStateTree)
		);
		return false;
	}
	if (NextState.SelectionBehavior == EStateTreeStateSelectionBehavior::None)
	{
		STATETREE_LOG(VeryVerbose, TEXT("%hs: Selection Behavior is none for state '%s'.  '%s' using StateTree '%s'."),
			__FUNCTION__,
			*NextState.Name.ToString(),
			*GetNameSafe(&Owner),
			*GetFullNameSafe(NextStateTree)
		);
		return false;
	}

	struct FCleanUpOnExit
	{
		FCleanUpOnExit(const TSharedRef<FSelectStateResult>& InSelectionResult)
			: SelectionResult(InSelectionResult)
		{ }
		~FCleanUpOnExit()
		{
			if (bStateAdded && bSucceededToSelectState == false)
			{
				SelectionResult->SelectedStates.Pop();
			}
			if (bFrameAdded && bSucceededToSelectState == false)
			{
				SelectionResult->SelectedFrames.Pop();
			}
		}

		const TSharedRef<FSelectStateResult>& SelectionResult;
		bool bSucceededToSelectState = false;
		bool bFrameAdded = false;
		bool bStateAdded = false;
	};
	FCleanUpOnExit OnExitScope = FCleanUpOnExit(OutSelectionResult);

	// Does it need a new root tree. It can create a new frame or reused an existing one.
	bool bNewFrameCreated = false;
	{
		if (InternalArgs.MissingSourceStates.Num() == 0 && !InternalArgs.MissingSourceFrameID.IsValid())
		{
			// Must be in a valid selection (state are already been selected).
			if (OutSelectionResult->SelectedStates.Num() == 0)
			{
				STATETREE_LOG(Verbose, TEXT("%hs: No root state to select '%s'.  '%s' using StateTree '%s'."),
					__FUNCTION__,
					*NextState.Name.ToString(),
					*GetNameSafe(&Owner),
					*GetFullNameSafe(NextStateTree)
				);
				return false;
			}

			const FActiveState PreviousSelectedState = OutSelectionResult->SelectedStates.Last();
			FStateTreeExecutionFrame* PreviousFramePtr = FindMutableExecutionFrame(PreviousSelectedState.GetFrameID(), MakeArrayView(GetExecState().ActiveFrames), MakeArrayView(OutSelectionResult->TemporaryFrames));
			check(PreviousFramePtr);
			check(PreviousFramePtr->StateTree == NextStateTree);

			if (PreviousSelectedState.GetStateHandle() != NextState.Parent)
			{
				const FExecutionFrameHandle FrameHandle = FExecutionFrameHandle(NextStateContext.StateTree, NextStateContext.StateHandle);
				FStateTreeExecutionFrame& NewFrame = OutSelectionResult->MakeAndAddTemporaryFrameWithNewRoot(FActiveFrameID(Storage.GenerateUniqueId()), FrameHandle, *PreviousFramePtr);
				NewFrame.ExecutionRuntimeIndexBase = FStateTreeIndex16(Storage.AddExecutionRuntimeData(GetOwner(), FrameHandle));
				OutSelectionResult->SelectedFrames.Add(NewFrame.FrameID);
				OnExitScope.bFrameAdded = true;
				bNewFrameCreated = true;
			}
		}
		else
		{
			// We are building the path toward the desired target.
			const FActiveFrameID MissingFrameID = InternalArgs.MissingSourceStates.Num() != 0 ? InternalArgs.MissingSourceStates[0].GetFrameID() : InternalArgs.MissingSourceFrameID;
			FStateTreeExecutionFrame* MissingFramePtr = FindMutableExecutionFrame(MissingFrameID, MakeArrayView(GetExecState().ActiveFrames), MakeArrayView(OutSelectionResult->TemporaryFrames));
			check(MissingFramePtr);
			if (!ensure(MissingFramePtr->StateTree == NextStateTree))
			{
				return false;
			}

			FActiveFrameID NextFrameID = MissingFrameID;
			if (InternalArgs.MissingSourceStates.Num() != 0)
			{
				if (!NextState.Parent.IsValid() && MissingFramePtr->RootState != NextStateHandle)
				{
					const FExecutionFrameHandle FrameHandle = FExecutionFrameHandle(NextStateContext.StateTree, NextStateContext.StateHandle);
					FStateTreeExecutionFrame& NewFrame = OutSelectionResult->MakeAndAddTemporaryFrameWithNewRoot(FActiveFrameID(Storage.GenerateUniqueId()), FrameHandle, *MissingFramePtr);
					NewFrame.ExecutionRuntimeIndexBase = FStateTreeIndex16(Storage.AddExecutionRuntimeData(GetOwner(), FrameHandle));
					NextFrameID = NewFrame.FrameID;
					bNewFrameCreated = true;
				}
			}
			else if (MissingFramePtr->RootState != NextStateHandle)
			{
				const FExecutionFrameHandle FrameHandle = FExecutionFrameHandle(NextStateContext.StateTree, NextStateContext.StateHandle);
				FStateTreeExecutionFrame& NewFrame = OutSelectionResult->MakeAndAddTemporaryFrameWithNewRoot(FActiveFrameID(Storage.GenerateUniqueId()), FrameHandle, *MissingFramePtr);
				NewFrame.ExecutionRuntimeIndexBase = FStateTreeIndex16(Storage.AddExecutionRuntimeData(GetOwner(), FrameHandle));
				NextFrameID = NewFrame.FrameID;
				bNewFrameCreated = true;
			}

			// Add new frames.
			if (OutSelectionResult->SelectedFrames.Num() > 0)
			{
				if (OutSelectionResult->SelectedFrames.Last() != NextFrameID)
				{
					OutSelectionResult->SelectedFrames.Add(NextFrameID);
					OnExitScope.bFrameAdded = true;
				}
			}
			else
			{
				OutSelectionResult->SelectedFrames.Add(NextFrameID);
				OnExitScope.bFrameAdded = true;
			}

			TOptional<bool> bSelected = SelectStateFromSourceInternal(Args, InternalArgs, OutSelectionResult, *MissingFramePtr, NextState, NextStateHandle, bNewFrameCreated);
			if (bSelected.IsSet())
			{
				OnExitScope.bSucceededToSelectState = bSelected.GetValue();
				return OnExitScope.bSucceededToSelectState;
			}
		}
	}

	// Use the ID to get the ExecutionFrame because it is saved in an array (and this is a recursive function), the array can grow and the pointer won't be valid.
	check(OutSelectionResult->SelectedFrames.Num() != 0);
	const UE::StateTree::FActiveFrameID NextFrameID = OutSelectionResult->SelectedFrames.Last();

	const FStateTreeExecutionFrame* NextFrame = nullptr;
	const FStateTreeExecutionFrame* NextParentFrame = nullptr;
	auto CacheNextFrame = [&NextFrame, &NextParentFrame, &NextFrameID, &Exec, &OutSelectionResult]()
		{
			NextFrame = FindExecutionFrame(NextFrameID, MakeConstArrayView(Exec.ActiveFrames), MakeConstArrayView(OutSelectionResult->TemporaryFrames));
			check(NextFrame);

	if (OutSelectionResult->SelectedFrames.Num() > 1)
	{
		NextParentFrame = FindExecutionFrame(OutSelectionResult->SelectedFrames.Last(1), MakeConstArrayView(Exec.ActiveFrames), MakeConstArrayView(OutSelectionResult->TemporaryFrames));
		check(NextParentFrame);
	}
		};
	CacheNextFrame();

	// Save the current result to use SelectionEvents in GetDataView
	TGuardValue<FSelectStateResult*> GuardCurrentlyProcessedStateSelectionResult(CurrentlyProcessedStateSelectionResult, &OutSelectionResult.Get());

	UE_STATETREE_DEBUG_SCOPED_STATE_PHASE(this, NextStateHandle, EStateTreeUpdatePhase::StateSelection);

	const bool bIsTargetState = InternalArgs.MissingStatesToReachTarget.Num() <= 1;
	const bool bShouldPrerequisitesBeChecked = Args.Behavior != ESelectStateBehavior::Forced
		&& (bIsTargetState || NextState.bCheckPrerequisitesWhenActivatingChildDirectly);

	bool bCreateNewState = true;
	const bool bIsNextTargetStateInActiveStates = !bNewFrameCreated
		&& InternalArgs.MissingActiveStates.Num() > 0
		&& InternalArgs.MissingActiveStates[0].GetStateHandle() == NextStateHandle
		&& InternalArgs.MissingActiveStates[0].GetFrameID() == NextFrame->FrameID;
	if (EnumHasAnyFlags(Args.SelectionRules, EStateTreeStateSelectionRules::ReselectedStateCreatesNewStates | EStateTreeStateSelectionRules::CompletedTransitionStatesCreateNewStates))
	{
		bCreateNewState = false;
		if (EnumHasAllFlags(Args.SelectionRules, EStateTreeStateSelectionRules::CompletedTransitionStatesCreateNewStates))
		{
			// Request a new state if it doesn't match the previous states or if the state is completed.
			bCreateNewState = !bIsNextTargetStateInActiveStates || NextFrame->ActiveTasksStatus.GetStatus(NextState).IsCompleted();
		}
		if (EnumHasAnyFlags(Args.SelectionRules, EStateTreeStateSelectionRules::ReselectedStateCreatesNewStates))
		{
			bCreateNewState = bCreateNewState || bIsTargetState || !bIsNextTargetStateInActiveStates;
		}
	}
	else
	{
		bCreateNewState = !bIsNextTargetStateInActiveStates;
	}

	// Look up linked state overrides
	const UStateTree* NextLinkedStateAsset = NextState.LinkedAsset;
	const FInstancedPropertyBag* NextLinkedStateParameterOverride = nullptr;
	if (NextState.Type == EStateTreeStateType::LinkedAsset)
	{
		// If the next state is sustained, then we should get the last valid asset.
		if (!bCreateNewState)
		{
			bCreateNewState = true;
			if (InternalArgs.MissingActiveStates.Num() > 1)
			{
				if (const FStateTreeExecutionFrame* NextLinkedFrame = UE::StateTree::ExecutionContext::FindExecutionFrame(InternalArgs.MissingActiveStates[1].GetFrameID(), MakeConstArrayView(Exec.ActiveFrames), MakeConstArrayView(OutSelectionResult->TemporaryFrames)))
				{
					check(NextLinkedFrame->StateTree);
					NextLinkedStateAsset = NextLinkedFrame->StateTree;
					ensure(NextLinkedStateAsset != NextFrame->StateTree);

					// The StateParameters data also need to match.
					FCurrentlyProcessedFrameScope FrameScope(*this, NextParentFrame, *NextFrame);
					FCurrentlyProcessedStateScope NextStateScope(*this, NextStateHandle);
					FStateTreeDataView NextStateParametersView = GetDataViewOrTemporary(NextParentFrame, *NextFrame, NextState.ParameterDataHandle);
					const FConstStructView DefaultStateParamsInstanceData = NextLinkedStateAsset->GetDefaultParameters().GetValue();
					bCreateNewState = DefaultStateParamsInstanceData.GetScriptStruct() != NextStateParametersView.GetStruct();
				}
			}
		}

		// LinkedAsset that has bindings cannot be overridden by another asset. The bindings will write invalid data.
		if (bCreateNewState && NextState.bCanOverrideLinkedAssetAtRuntime)
		{
			if (const FStateTreeReference* Override = GetLinkedStateTreeOverrideForTag(NextState.Tag))
			{
				NextLinkedStateAsset = Override->GetStateTree();
				NextLinkedStateParameterOverride = &Override->GetParameters();

				STATETREE_LOG(VeryVerbose, TEXT("%hs: In state '%s', overriding linked asset '%s' with '%s'. '%s' using StateTree '%s'."),
					__FUNCTION__,
					*GetSafeStateName(*NextFrame, NextStateHandle),
					*GetFullNameSafe(NextState.LinkedAsset),
					*GetFullNameSafe(NextLinkedStateAsset),
					*GetNameSafe(&Owner),
					*GetFullNameSafe(NextFrame->StateTree)
				);
			}
		}
	}

	// Update state parameters.
	if (NextState.ParameterDataHandle.IsValid())
	{
		FCurrentlyProcessedFrameScope FrameScope(*this, NextParentFrame, *NextFrame);
		FCurrentlyProcessedStateScope NextStateScope(*this, NextStateHandle);

		// Instantiate temporary state parameters if not done yet.
		FStateTreeDataView NextStateParametersView = GetDataViewOrTemporary(NextParentFrame, *NextFrame, NextState.ParameterDataHandle);
		if (!NextStateParametersView.IsValid())
		{
			// Allocate temporary instance for parameters if the state has params.
			// The subtree state selection below assumes that this creates always a valid temporary, we'll create the temp data even if parameters are empty.
			// @todo: Empty params is valid and common case, we should not require to create empty parameters data (this needs to be handle in compiler and UpdateInstanceData too).
			if (NextLinkedStateParameterOverride)
			{
				// Create from an override.
				FStateTreeDataView TempStateParametersView = AddTemporaryInstance(*NextFrame, FStateTreeIndex16::Invalid, NextState.ParameterDataHandle, FConstStructView(TBaseStructure<FCompactStateTreeParameters>::Get()));
				check(TempStateParametersView.IsValid());
				FCompactStateTreeParameters& StateParams = TempStateParametersView.GetMutable<FCompactStateTreeParameters>();
				StateParams = FCompactStateTreeParameters(*NextLinkedStateParameterOverride);
				NextStateParametersView = FStateTreeDataView(StateParams.GetMutableValue());
			}
			else
			{
				// Create from template in the asset.
				const FConstStructView DefaultStateParamsInstanceData = NextFrame->StateTree->DefaultInstanceData.GetStruct(NextState.ParameterTemplateIndex.Get());
				FStateTreeDataView TempStateParametersView = AddTemporaryInstance(*NextFrame, FStateTreeIndex16::Invalid, NextState.ParameterDataHandle, DefaultStateParamsInstanceData);
				check(TempStateParametersView.IsValid());
				FCompactStateTreeParameters& StateParams = TempStateParametersView.GetMutable<FCompactStateTreeParameters>();
				NextStateParametersView = FStateTreeDataView(StateParams.GetMutableValue());
			}
		}

		// Copy parameters if needed
		if (NextStateParametersView.IsValid()
			&& NextState.ParameterDataHandle.IsValid()
			&& NextState.ParameterBindingsBatch.IsValid())
		{
			// Note: the parameters are for the current (linked) state, stored in current frame.
			// The copy can fail, if the overridden parameters do not match, this is by design.
			CopyBatchWithValidation(NextParentFrame, *NextFrame, NextStateParametersView, NextState.ParameterBindingsBatch);
		}
	}

	// Check if the events are accepted.
	TArray<FStateTreeSharedEvent, TInlineAllocator<FStateTreeEventQueue::MaxActiveEvents>> StateSelectionEvents;
	if (NextState.EventDataIndex.IsValid())
	{
		if (ensure(NextState.RequiredEventToEnter.IsValid()))
		{
			// Use the same event as performed transition unless it didn't lead to this state as only state selected by the transition should get it's event.
			const bool bCanUseTransitionEvent = Args.TransitionEvent.IsValid()
				&& Args.TargetState == NextStateContext
				&& bIsTargetState;
			if (bCanUseTransitionEvent && bTargetStateRequiresTheSameEventForStateSelectionAsTheRequestedTransition)
			{
				if (NextState.RequiredEventToEnter.DoesEventMatchDesc(*Args.TransitionEvent))
				{
					StateSelectionEvents.Emplace(Args.TransitionEvent);
				}
			}
			else if (bCanUseTransitionEvent && NextState.RequiredEventToEnter.DoesEventMatchDesc(*Args.TransitionEvent))
			{
				StateSelectionEvents.Emplace(Args.TransitionEvent);
			}
			else
			{
				TArrayView<FStateTreeSharedEvent> EventsQueue = GetMutableEventsToProcessView();
				for (FStateTreeSharedEvent& Event : EventsQueue)
				{
					check(Event.IsValid());
					if (NextState.RequiredEventToEnter.DoesEventMatchDesc(*Event))
					{
						StateSelectionEvents.Emplace(Event);
					}
				}

				// Couldn't find matching state's event, but it's marked as not required. Adding an empty event which allows us to continue the state selection.
				if (!bShouldPrerequisitesBeChecked && StateSelectionEvents.IsEmpty())
				{
					StateSelectionEvents.Emplace();
				}
			}
		}

		if (StateSelectionEvents.IsEmpty())
		{
			return false;
		}
	}
	else
	{
		StateSelectionEvents.Add(FStateTreeSharedEvent());
	}

	// Add the state to the selected list. It will be removed if the selection fails.
	if (bCreateNewState)
	{
		OutSelectionResult->SelectedStates.Emplace(NextFrame->FrameID, FActiveStateID(Storage.GenerateUniqueId()), NextStateHandle);
		OnExitScope.bStateAdded = true;

		int32 StateInFrameCounter = 0;
		for (int32 Index = OutSelectionResult->SelectedStates.Num() - 1; Index >= 0; --Index)
		{
			if (OutSelectionResult->SelectedStates[Index].GetFrameID() != NextFrame->FrameID)
			{
				break;
			}
			++StateInFrameCounter;
		}

		if (StateInFrameCounter > FStateTreeActiveStates::MaxStates)
		{
			STATETREE_LOG(Error, TEXT("%hs: Reached max execution depth when trying to select state %s from '%s'.  '%s' using StateTree '%s'."),
				__FUNCTION__,
				*GetSafeStateName(*NextFrame, NextStateHandle),
				*GetFullNameSafe(NextFrame->StateTree),
				*GetNameSafe(&Owner),
				*GetFullNameSafe(&RootStateTree)
			);
			return false;
		}
	}
	else
	{
		check(InternalArgs.MissingActiveStates.Num() > 0); // bIsNextTargetStateInActiveStates is false
		OutSelectionResult->SelectedStates.Add(InternalArgs.MissingActiveStates[0]);
		OnExitScope.bStateAdded = true;
	}

	// Set the target, since this is recursive (can be called for linked state), it only set once the original target is found.
	if (bIsTargetState && !OutSelectionResult->TargetState.IsValid())
	{
		OutSelectionResult->TargetState = OutSelectionResult->SelectedStates.Last();
	}

	for (int32 EventIndex = 0; EventIndex < StateSelectionEvents.Num(); ++EventIndex)
	{
		const FStateTreeSharedEvent& StateSelectionEvent = StateSelectionEvents[EventIndex];

		// A SelectStateInternal_X might have changed the TemporaryFrames, re-cache the values
		if (EventIndex != 0)
		{
			CacheNextFrame();
		}

		// Add the event for GetDataView and for consuming it later.
		const bool bContainsSelectionEvent = OutSelectionResult->SelectionEvents.ContainsByPredicate(
			[&LatestState = OutSelectionResult->SelectedStates.Last()](const FSelectionEventWithID& Other)
			{
				return Other.State == LatestState;
			});
		ensureMsgf(!bContainsSelectionEvent, TEXT("The event should be remove at the end of the scope."));

		bool bRemoveSelectionEvents = false;
		if (StateSelectionEvent.IsValid())
		{
			
			OutSelectionResult->SelectionEvents.Add(FSelectionEventWithID{.State = OutSelectionResult->SelectedStates.Last(), .Event = StateSelectionEvent});
			bRemoveSelectionEvents = true;
		}

		auto RemoveStateSelectionEvent = [bRemoveSelectionEvents](const TSharedRef<FSelectStateResult>& SelectionResult)
			{
				if (bRemoveSelectionEvents)
				{
					check(SelectionResult->SelectionEvents.Num() > 0);
					ensureMsgf(SelectionResult->SelectionEvents.Last().State == SelectionResult->SelectedStates.Last(), TEXT("We should remove from the same element it was added."));
					SelectionResult->SelectionEvents.Pop();
				}
			};

		if (bShouldPrerequisitesBeChecked)
		{
			// Check that the state can be entered
			const bool bEnterConditionsPassed = TestAllConditionsWithValidation(NextParentFrame, *NextFrame, NextStateHandle, NextState.EnterConditionEvaluationScopeMemoryRequirement, NextState.EnterConditionsBegin, NextState.EnterConditionsNum, EStateTreeUpdatePhase::EnterConditions);

			if (!bEnterConditionsPassed)
			{
				RemoveStateSelectionEvent(OutSelectionResult);
				continue;
			}
		}

		const FSelectStateInternalArguments NewInternalArgs = FSelectStateInternalArguments{
			.MissingActiveStates = bCreateNewState ? TArrayView<const FActiveState>() : InternalArgs.MissingActiveStates.Mid(1),
			.MissingSourceFrameID = FActiveFrameID(),
			.MissingSourceStates = TArrayView<const FActiveState>(),
			.MissingStatesToReachTarget = bIsTargetState ? TArrayView<const FStateHandleContext>() : InternalArgs.MissingStatesToReachTarget.Mid(1)
		};
		if (NextState.Type == EStateTreeStateType::Linked)
		{
			// MissingStatesToReachTarget can include a linked state and the frame needs to be constructed (if needed).
			OnExitScope.bSucceededToSelectState = SelectStateInternal_Linked(Args, NewInternalArgs, OutSelectionResult, NextFrame->StateTree, NextState, bCreateNewState);
		}
		else if (NextState.Type == EStateTreeStateType::LinkedAsset)
		{
			// MissingStatesToReachTarget can include a linked asset state and the frame needs to be constructed (if needed).
			OnExitScope.bSucceededToSelectState = SelectStateInternal_LinkedAsset(Args, NewInternalArgs, OutSelectionResult, NextFrame->StateTree, NextState, NextLinkedStateAsset, bCreateNewState);
		}
		else if (!bIsTargetState)
		{
			check(NewInternalArgs.MissingStatesToReachTarget.Num() > 0);
			// Next child state is already known. Passing TransitionEvent further, states selected directly by transition can use it.
			OnExitScope.bSucceededToSelectState = SelectStateInternal(Args, NewInternalArgs, OutSelectionResult);
		}
		else if (Args.Behavior != ESelectStateBehavior::Forced)
		{
			switch (NextState.SelectionBehavior)
			{
			case EStateTreeStateSelectionBehavior::TryEnterState:
				UE_STATETREE_DEBUG_STATE_EVENT(this, NextStateHandle, EStateTreeTraceEventType::OnStateSelected);
				OnExitScope.bSucceededToSelectState = true;
				break;
			case EStateTreeStateSelectionBehavior::TrySelectChildrenInOrder:
				OnExitScope.bSucceededToSelectState = SelectStateInternal_TrySelectChildrenInOrder(Args, NewInternalArgs, OutSelectionResult, NextFrame->StateTree, NextState, NextStateHandle);
				break;
			case EStateTreeStateSelectionBehavior::TrySelectChildrenAtRandom:
				OnExitScope.bSucceededToSelectState = SelectStateInternal_TrySelectChildrenAtRandom(Args, NewInternalArgs, OutSelectionResult, NextFrame->StateTree, NextState, NextStateHandle);
				break;
			case EStateTreeStateSelectionBehavior::TrySelectChildrenWithHighestUtility:
				OnExitScope.bSucceededToSelectState = SelectStateInternal_TrySelectChildrenWithHighestUtility(Args, NewInternalArgs, OutSelectionResult, NextFrame->StateTree, NextState, NextStateHandle);
				break;
			case EStateTreeStateSelectionBehavior::TrySelectChildrenAtRandomWeightedByUtility:
				OnExitScope.bSucceededToSelectState = SelectStateInternal_TrySelectChildrenAtRandomWeightedByUtility(Args, NewInternalArgs, OutSelectionResult, NextFrame->StateTree, NextState, NextStateHandle);
				break;
			case EStateTreeStateSelectionBehavior::TryFollowTransitions:
				OnExitScope.bSucceededToSelectState = SelectStateInternal_TryFollowTransitions(Args, NewInternalArgs, OutSelectionResult, NextFrame->StateTree, NextState, NextStateHandle);
				break;
			}
		}
		else
		{
			if (NewInternalArgs.MissingStatesToReachTarget.Num() == 0)
			{
				OnExitScope.bSucceededToSelectState = true;
			}
			else
			{
				// Continue the force selection. Next child state is already known.
				OnExitScope.bSucceededToSelectState = SelectStateInternal(Args, NewInternalArgs, OutSelectionResult);
			}
		}


		if (OnExitScope.bSucceededToSelectState)
		{
			break;
		}
		else
		{
			RemoveStateSelectionEvent(OutSelectionResult);
		}
	}

	return OnExitScope.bSucceededToSelectState;
}

TOptional<bool> FStateTreeExecutionContext::SelectStateFromSourceInternal(
	const FSelectStateArguments& Args,
	const FSelectStateInternalArguments& InternalArgs,
	const TSharedRef<FSelectStateResult>& OutSelectionResult,
	const FStateTreeExecutionFrame& NextFrame,
	const FCompactStateTreeState& NextState,
	const FStateTreeStateHandle NextStateHandle,
	const bool bNewFrameCreated)
{
	using namespace UE::StateTree;
	using namespace UE::StateTree::ExecutionContext;
	using namespace UE::StateTree::ExecutionContext::Private;

	if (InternalArgs.MissingSourceStates.Num() > 0
		&& InternalArgs.MissingStatesToReachTarget.Num() > 1
		&& !bNewFrameCreated)
	{
		check(InternalArgs.MissingActiveStates.Num() > 0);
		check(InternalArgs.MissingSourceStates[0] == InternalArgs.MissingActiveStates[0]);
		const FActiveState& MissingActiveState = InternalArgs.MissingSourceStates[0];
		bool bContinueWithStateSelection = NextStateHandle != MissingActiveState.GetStateHandle();
		bool bCompletedState = false;

		if (!bContinueWithStateSelection)
		{
			if (InternalArgs.MissingSourceStates.Num() == 1)
			{
				// MissingActiveState is the transition source.
				if (EnumHasAllFlags(Args.SelectionRules, EStateTreeStateSelectionRules::CompletedTransitionStatesCreateNewStates))
				{
					bContinueWithStateSelection = NextFrame.ActiveTasksStatus.GetStatus(NextState).IsCompleted();
				}
			}
			else
			{
				if (EnumHasAllFlags(Args.SelectionRules, EStateTreeStateSelectionRules::CompletedTransitionStatesCreateNewStates))
				{
					bCompletedState = NextFrame.ActiveTasksStatus.GetStatus(NextState).IsCompleted();
				}
			}
		}

		if (!bContinueWithStateSelection)
		{
			OutSelectionResult->SelectedStates.Add(InternalArgs.MissingSourceStates[0]);

			bool bSelectStateInternalSucceeded = false;
			ON_SCOPE_EXIT{
				if (bSelectStateInternalSucceeded == false)
				{
					OutSelectionResult->SelectedStates.Pop();
				}
			};

			FSelectStateInternalArguments NewInternalArgs = FSelectStateInternalArguments{
				.MissingActiveStates = InternalArgs.MissingActiveStates.Mid(1),
				.MissingSourceFrameID = FActiveFrameID(),
				.MissingSourceStates = InternalArgs.MissingSourceStates.Mid(1),
				.MissingStatesToReachTarget = InternalArgs.MissingStatesToReachTarget.Mid(1)
			};
			bSelectStateInternalSucceeded = SelectStateInternal(Args, NewInternalArgs, OutSelectionResult);

			if (bSelectStateInternalSucceeded && bCompletedState)
			{
				check(EnumHasAllFlags(Args.SelectionRules, EStateTreeStateSelectionRules::CompletedStateBeforeTransitionSourceFailsTransition));
				// Before the source, there's a completed state.
				//The completed state should fail the transition(because of EStateTreeStateSelectionRules::CompletedStateBeforeTargetFailsTransition).
				//The transition can contain a state with "follow transition" that would remove this state from the transition.
				if (OutSelectionResult->SelectedStates.Contains(MissingActiveState))
				{
					STATETREE_LOG(VeryVerbose, TEXT("%hs: Selection fails because state '%s' is completed.  '%s' using StateTree '%s'."),
						__FUNCTION__,
						*NextState.Name.ToString(),
						*GetNameSafe(&Owner),
						*GetFullNameSafe(NextFrame.StateTree)
					);

					UE_STATETREE_DEBUG_LOG_EVENT(this, Log, TEXT("Selection fails because parent state '%s' is completed."), *NextState.Name.ToString());

					bSelectStateInternalSucceeded = false;
				}
			}
			return bSelectStateInternalSucceeded;
		}
	}
	return {};
}

namespace UE::StateTree::ExecutionContext::Private
{
	bool PreventRecursionCheck(const FExecutionFrameHandle& LinkStateFrameHandle,
		TConstArrayView<const FActiveFrameID> SelectedFrames,
		TConstArrayView<const FStateTreeExecutionFrame> ActiveFrames,
		TConstArrayView<const FStateTreeExecutionFrame> TemporaryFrames)
	{
		// Check and prevent recursion.
		return SelectedFrames.ContainsByPredicate(
			[&LinkStateFrameHandle, &ActiveFrames, &TemporaryFrames](const FActiveFrameID& FrameID)
			{
				const FStateTreeExecutionFrame* Frame = UE::StateTree::ExecutionContext::FindExecutionFrame(FrameID, ActiveFrames, TemporaryFrames);
				return Frame && Frame->HasRoot(LinkStateFrameHandle);
			});
	}

	FStateTreeExecutionFrame* SelectedFrameLinkedFrame(FStateTreeExecutionState& Exec,
		const bool bCreateNewState,
		TConstArrayView<const UE::StateTree::FActiveState> MatchingActiveStates,
		const FExecutionFrameHandle& LinkStateFrameHandle,
		const EStateTreeStateSelectionRules StateSelectionRules
	)
	{
		check(LinkStateFrameHandle.IsValid());

		FStateTreeExecutionFrame* Result = nullptr;
		if (!bCreateNewState)
		{
			// Get the next frame ID
			if (ensure(MatchingActiveStates.Num() >= 1))
			{
				const FActiveFrameID ExistingFrameID = MatchingActiveStates[0].GetFrameID();
				Result = Exec.FindActiveFrame(ExistingFrameID);
				check(Result);
				if (Result->HasRoot(LinkStateFrameHandle))
				{
					if (EnumHasAllFlags(StateSelectionRules, EStateTreeStateSelectionRules::CompletedTransitionStatesCreateNewStates)
						&& Result->ActiveTasksStatus.GetStatus(Result->StateTree).IsCompleted()
						)
					{
						return nullptr;
					}
				}
			}
		}
		return Result;
	}
}

bool FStateTreeExecutionContext::SelectStateInternal_Linked(
	const FSelectStateArguments& Args,
	const FSelectStateInternalArguments& InternalArgs,
	const TSharedRef<FSelectStateResult>& OutSelectionResult,
	TNotNull<const UStateTree*> NextStateTree,
	const FCompactStateTreeState& TargetState,
	bool bCreateNewState)
{
	using namespace UE::StateTree;
	using namespace UE::StateTree::ExecutionContext;
	using namespace UE::StateTree::ExecutionContext::Private;

	FStateTreeExecutionState& Exec = GetExecState();

	if (!TargetState.LinkedState.IsValid())
	{
		STATETREE_LOG(Warning, TEXT("%hs: Trying to enter invalid linked subtree from '%s'. '%s' using StateTree '%s'."),
			__FUNCTION__,
			*GetStateStatusString(Exec),
			*Owner.GetName(),
			*NextStateTree->GetFullName()
		);
		return false;
	}

	const FExecutionFrameHandle LinkStateFrameHandle = FExecutionFrameHandle(NextStateTree, TargetState.LinkedState);

	const bool bHasMissingState = InternalArgs.MissingStatesToReachTarget.Num() > 0 && Args.Behavior == ESelectStateBehavior::Forced;
	if (bHasMissingState)
	{
		// In a force transition, the root could be different from what is expected.
		//Ex: a previous transition go to root, then another transition go to another top level state (new root)
		if (InternalArgs.MissingStatesToReachTarget[0].StateTree != LinkStateFrameHandle.GetStateTree()
			|| InternalArgs.MissingStatesToReachTarget[0].StateHandle != LinkStateFrameHandle.GetRootState())
		{
			STATETREE_LOG(Error, TEXT("%hs: The missing state is not from the same state tree. '%s' using StateTree '%s'."),
				__FUNCTION__,
				*Owner.GetName(),
				*NextStateTree->GetFullName()
			);
			return false;
		}
	}

	if (PreventRecursionCheck(LinkStateFrameHandle, OutSelectionResult->SelectedFrames, MakeConstArrayView(Exec.ActiveFrames), MakeConstArrayView(OutSelectionResult->TemporaryFrames)))
	{
		STATETREE_LOG(Error, TEXT("%hs: Trying to recursively enter subtree '%s' from '%s'. '%s' using StateTree '%s'."),
			__FUNCTION__,
			*GetSafeStateName(LinkStateFrameHandle.GetStateTree(), LinkStateFrameHandle.GetRootState()),
			*GetStateStatusString(Exec),
			*Owner.GetName(),
			*NextStateTree->GetFullName()
		);
		return false;
	}

	const FCompactStateTreeFrame* LinkStateTreeFrame = FindStateTreeFrame(LinkStateFrameHandle);
	if (LinkStateTreeFrame == nullptr)
	{
		STATETREE_LOG(Error, TEXT("%hs: The frame '%s' from '%s' does not exist. '%s' using StateTree '%s'."),
			__FUNCTION__,
			*GetSafeStateName(LinkStateFrameHandle.GetStateTree(), LinkStateFrameHandle.GetRootState()),
			*GetStateStatusString(Exec),
			*Owner.GetName(),
			*NextStateTree->GetFullName()
		);
		return false;
	}

	// Do we have an existing frame.
	FStateTreeExecutionFrame* SelectedFrame = SelectedFrameLinkedFrame(Exec, bCreateNewState, InternalArgs.MissingActiveStates, LinkStateFrameHandle, Args.SelectionRules);
	if (SelectedFrame && !SelectedFrame->HasRoot(LinkStateFrameHandle))
	{
		if (Args.Behavior == ESelectStateBehavior::Forced)
		{
			SelectedFrame = nullptr;
		}
		else
		{
			STATETREE_LOG(Error, TEXT("%hs: The frame '%s' from '%s' does not have the same root as the active frame. '%s' using StateTree '%s'."),
				__FUNCTION__,
				*GetSafeStateName(LinkStateFrameHandle.GetStateTree(), LinkStateFrameHandle.GetRootState()),
				*GetStateStatusString(Exec),
				*Owner.GetName(),
				*NextStateTree->GetFullName()
			);
			return false;
		}
	}

	const bool bIsNewFrame = SelectedFrame == nullptr;
	if (bIsNewFrame)
	{
		constexpr bool bIsGlobalFrame = false;
		FStateTreeExecutionFrame& NewFrame = OutSelectionResult->MakeAndAddTemporaryFrame(FActiveFrameID(Storage.GenerateUniqueId()), LinkStateFrameHandle, bIsGlobalFrame);

		// Note. Adding to TemporaryFrame can invalidate ParentFrame.
		const FStateTreeExecutionFrame* NextParentFrame = UE::StateTree::ExecutionContext::FindExecutionFrame(OutSelectionResult->SelectedFrames.Last(), MakeConstArrayView(Exec.ActiveFrames), MakeConstArrayView(OutSelectionResult->TemporaryFrames));
		check(NextParentFrame);

		NewFrame.ExternalDataBaseIndex = NextParentFrame->ExternalDataBaseIndex;

		NewFrame.GlobalInstanceDataFrameID = NextParentFrame->GlobalInstanceDataFrameID;
		NewFrame.GlobalInstanceIndexBase = NextParentFrame->GlobalInstanceIndexBase;

		NewFrame.GlobalParameterDataHandle = NextParentFrame->GlobalParameterDataHandle;
		NewFrame.GlobalParameterDataFrameID = NextParentFrame->GlobalParameterDataFrameID;
		NewFrame.GlobalParameterIndexBase = NextParentFrame->GlobalParameterIndexBase;

		// Points to the linked state's state parameter
		NewFrame.StateParameterDataHandle = TargetState.ParameterDataHandle;
		NewFrame.StateParameterDataFrameID = NextParentFrame->FrameID;
		NewFrame.StateParameterIndexBase = NextParentFrame->ActiveInstanceIndexBase;

		NewFrame.ExecutionRuntimeIndexBase = FStateTreeIndex16(Storage.AddExecutionRuntimeData(GetOwner(), LinkStateFrameHandle));

		SelectedFrame = &NewFrame;
	}

	OutSelectionResult->SelectedFrames.Add(SelectedFrame->FrameID);

	// Select the root state of the new frame.
	const FStateHandleContext RootState = FStateHandleContext(LinkStateFrameHandle.GetStateTree(), LinkStateFrameHandle.GetRootState());
	const FSelectStateInternalArguments NewInternalArgs = FSelectStateInternalArguments{
		.MissingActiveStates = bIsNewFrame ? TArrayView<const FActiveState>() : InternalArgs.MissingActiveStates,
		.MissingSourceFrameID = SelectedFrame->FrameID,
		.MissingSourceStates = TArrayView<const FActiveState>(),
		.MissingStatesToReachTarget = bHasMissingState ? InternalArgs.MissingStatesToReachTarget : MakeConstArrayView(&RootState, 1)
	};
	const bool bSucceededToSelectState = SelectStateInternal(Args, NewInternalArgs, OutSelectionResult);

	if (!bSucceededToSelectState)
	{
		if (bIsNewFrame)
		{
			CleanFrame(Exec, NewInternalArgs.MissingSourceFrameID);
		}
		OutSelectionResult->SelectedFrames.Pop();
	}

	return bSucceededToSelectState;
}

bool FStateTreeExecutionContext::SelectStateInternal_LinkedAsset(
	const FSelectStateArguments& Args,
	const FSelectStateInternalArguments& InternalArgs,
	const TSharedRef<FSelectStateResult>& OutSelectionResult,
	TNotNull<const UStateTree*> NextStateTree,
	const FCompactStateTreeState& NextState,
	const UStateTree* NextLinkedStateAsset,
	bool bCreateNewState)
{
	using namespace UE::StateTree;
	using namespace UE::StateTree::ExecutionContext;
	using namespace UE::StateTree::ExecutionContext::Private;

	FStateTreeExecutionState& Exec = GetExecState();

	if (NextLinkedStateAsset == nullptr)
	{
		return false;
	}

	if (NextLinkedStateAsset->States.Num() == 0
		|| !NextLinkedStateAsset->IsReadyToRun())
	{
		STATETREE_LOG(Error, TEXT("%hs: The linked State Tree is invalid. '%s' using StateTree '%s'."),
			__FUNCTION__,
			*Owner.GetName(),
			*NextStateTree->GetFullName()
		);
		return false;
	}

	FStateTreeStateHandle NextLinkedStateRoot = FStateTreeStateHandle::Root;
	const bool bHasMissingState = InternalArgs.MissingStatesToReachTarget.Num() > 0 && Args.Behavior == ESelectStateBehavior::Forced;
	if (bHasMissingState)
	{
		// In a force transition, the root could be different from what is expected.
		//Ex: a previous transition go to root, then another transition go to another top level state (new root)
		if (InternalArgs.MissingStatesToReachTarget[0].StateTree != NextLinkedStateAsset)
		{
			STATETREE_LOG(Error, TEXT("%hs: The missing state is not from the same state tree. '%s' using StateTree '%s'."),
				__FUNCTION__,
				*Owner.GetName(),
				*NextStateTree->GetFullName()
			);
			return false;
		}
		NextLinkedStateRoot = InternalArgs.MissingStatesToReachTarget[0].StateHandle;
	}

	const FExecutionFrameHandle LinkStateFrameHandle = FExecutionFrameHandle(NextLinkedStateAsset, NextLinkedStateRoot);

	// The linked state tree should have compatible context requirements.
	if (!NextLinkedStateAsset->HasCompatibleContextData(RootStateTree)
		|| NextLinkedStateAsset->GetSchema()->GetClass() != NextStateTree->GetSchema()->GetClass())
	{
		STATETREE_LOG(Error, TEXT("%hs: The linked State Tree '%s' does not have compatible schema, trying to select state %s from '%s'. '%s' using StateTree '%s'."),
			__FUNCTION__,
			*GetFullNameSafe(NextLinkedStateAsset),
			*GetSafeStateName(LinkStateFrameHandle.GetStateTree(), LinkStateFrameHandle.GetRootState()),
			*GetStateStatusString(Exec),
			*Owner.GetName(),
			*NextStateTree->GetFullName()
		);
		return false;
	}

	if (PreventRecursionCheck(LinkStateFrameHandle, OutSelectionResult->SelectedFrames, MakeConstArrayView(Exec.ActiveFrames), MakeConstArrayView(OutSelectionResult->TemporaryFrames)))
	{
		STATETREE_LOG(Error, TEXT("%hs: Trying to recursively enter subtree '%s' from '%s'. '%s' using StateTree '%s'."),
			__FUNCTION__,
			*GetSafeStateName(LinkStateFrameHandle.GetStateTree(), LinkStateFrameHandle.GetRootState()),
			*GetStateStatusString(Exec),
			*Owner.GetName(),
			*NextStateTree->GetFullName()
		);
		return false;
	}

	const FCompactStateTreeFrame* LinkStateTreeFrame = FindStateTreeFrame(LinkStateFrameHandle);
	if (LinkStateTreeFrame == nullptr)
	{
		STATETREE_LOG(Error, TEXT("%hs: The frame '%s' from '%s' does not exist. '%s' using StateTree '%s'."),
			__FUNCTION__,
			*GetSafeStateName(LinkStateFrameHandle.GetStateTree(), LinkStateFrameHandle.GetRootState()),
			*GetStateStatusString(Exec),
			*Owner.GetName(),
			*NextStateTree->GetFullName()
		);
		return false;
	}

	// Do we have an existing frame.
	// Do not use the transition override selection rules. A transition outside the frame should not impact the current frame.
	const EStateTreeStateSelectionRules StateSelectionRules = LinkStateFrameHandle.GetStateTree()->GetStateSelectionRules();
	FStateTreeExecutionFrame* SelectedFrame = SelectedFrameLinkedFrame(Exec, bCreateNewState, InternalArgs.MissingActiveStates, LinkStateFrameHandle, StateSelectionRules);
	if (SelectedFrame && !SelectedFrame->HasRoot(LinkStateFrameHandle))
	{
		if (Args.Behavior == ESelectStateBehavior::Forced)
		{
			SelectedFrame = nullptr;
		}
		else
		{
			STATETREE_LOG(Error, TEXT("%hs: The frame '%s' from '%s' does not have the same root as the active frame. '%s' using StateTree '%s'."),
				__FUNCTION__,
				*GetSafeStateName(LinkStateFrameHandle.GetStateTree(), LinkStateFrameHandle.GetRootState()),
				*GetStateStatusString(Exec),
				*Owner.GetName(),
				*NextStateTree->GetFullName()
			);
			return false;
		}
	}

	const bool bIsNewFrame = SelectedFrame == nullptr;
	if (bIsNewFrame)
	{
		// Collect external data if needed
		const FStateTreeIndex16 ExternalDataBaseIndex = CollectExternalData(LinkStateFrameHandle.GetStateTree());
		if (!ExternalDataBaseIndex.IsValid())
		{
			STATETREE_LOG(VeryVerbose, TEXT("%hs: Cannot select state '%s' because failed to collect external data for nested tree '%s' from '%s'. '%s' using StateTree '%s'."),
				__FUNCTION__,
				*GetSafeStateName(LinkStateFrameHandle.GetStateTree(), LinkStateFrameHandle.GetRootState()),
				*GetFullNameSafe(NextLinkedStateAsset),
				*GetStateStatusString(Exec),
				*Owner.GetName(),
				*NextStateTree->GetFullName()
			);
			return false;
		}

		constexpr bool bIsGlobalFrame = true;
		FStateTreeExecutionFrame& NewFrame = OutSelectionResult->MakeAndAddTemporaryFrame(FActiveFrameID(Storage.GenerateUniqueId()), LinkStateFrameHandle, bIsGlobalFrame);

		// Adding to TemporaryFrame can invalidate ParentFrame.
		const FStateTreeExecutionFrame* NextParentFrame = UE::StateTree::ExecutionContext::FindExecutionFrame(OutSelectionResult->SelectedFrames.Last(), MakeConstArrayView(Exec.ActiveFrames), MakeConstArrayView(OutSelectionResult->TemporaryFrames));
		check(NextParentFrame);

		NewFrame.ExternalDataBaseIndex = ExternalDataBaseIndex;
		NewFrame.ExecutionRuntimeIndexBase = FStateTreeIndex16(Storage.AddExecutionRuntimeData(GetOwner(), LinkStateFrameHandle));

		// Pass the linked state's parameters as global parameters to the linked asset.
		NewFrame.GlobalParameterDataHandle = NextState.ParameterDataHandle;
		NewFrame.GlobalParameterDataFrameID = NextParentFrame->FrameID;
		NewFrame.GlobalParameterIndexBase = NextParentFrame->ActiveInstanceIndexBase;

		// Global Frame
		NewFrame.GlobalInstanceDataFrameID = NewFrame.FrameID;

		NewFrame.StateParameterDataHandle = NewFrame.StateTree->GetStateFromHandle(NewFrame.RootState)->ParameterDataHandle;
		NewFrame.StateParameterDataFrameID = NewFrame.FrameID;

		SelectedFrame = &NewFrame;
	}

	OutSelectionResult->SelectedFrames.Add(SelectedFrame->FrameID);

	if (bIsNewFrame)
	{
		// Adding to TemporaryFrame can invalidate ParentFrame.
		const FStateTreeExecutionFrame* NextParentFrame = UE::StateTree::ExecutionContext::FindExecutionFrame(OutSelectionResult->SelectedFrames.Last(1), MakeConstArrayView(Exec.ActiveFrames), MakeConstArrayView(OutSelectionResult->TemporaryFrames));
		check(NextParentFrame);

		FStateTreeExecutionFrame* NextFrame = FindMutableExecutionFrame(SelectedFrame->FrameID, MakeArrayView(Exec.ActiveFrames), MakeArrayView(OutSelectionResult->TemporaryFrames));
		check(NextFrame && SelectedFrame == NextFrame);

		// Start global tasks and evaluators temporarily, so that their data is available already during select.
		const EStateTreeRunStatus StartResult = StartTemporaryEvaluatorsAndGlobalTasks(NextParentFrame, *NextFrame);
		if (StartResult != EStateTreeRunStatus::Running)
		{
			STATETREE_LOG(VeryVerbose, TEXT("%hs: Cannot select state '%s' because cannot start nested tree's '%s' global tasks and evaluators. '%s' using StateTree '%s'."),
				__FUNCTION__,
				*GetSafeStateName(LinkStateFrameHandle.GetStateTree(), LinkStateFrameHandle.GetRootState()),
				*GetFullNameSafe(NextLinkedStateAsset),
				*Owner.GetName(),
				*NextStateTree->GetFullName()
			);

			StopTemporaryEvaluatorsAndGlobalTasks(NextParentFrame, *NextFrame, StartResult);
			CleanFrame(Exec, SelectedFrame->FrameID);

			OutSelectionResult->SelectedFrames.Pop();
			return false;
		}
	}

	// Select the root state of the new frame.
	const FStateHandleContext RootState = FStateHandleContext(LinkStateFrameHandle.GetStateTree(), LinkStateFrameHandle.GetRootState());
	FSelectStateArguments NewArgs = Args;
	NewArgs.SelectionRules = StateSelectionRules;
	const FSelectStateInternalArguments NewInternalArgs = FSelectStateInternalArguments{
		.MissingActiveStates = bIsNewFrame ? TArrayView<const FActiveState>() : InternalArgs.MissingActiveStates,
		.MissingSourceFrameID = SelectedFrame->FrameID,
		.MissingSourceStates = TArrayView<const FActiveState>(),
		.MissingStatesToReachTarget = bHasMissingState ? InternalArgs.MissingStatesToReachTarget : MakeConstArrayView(&RootState, 1)
	};
	const bool bSucceededToSelectState = SelectStateInternal(NewArgs, NewInternalArgs, OutSelectionResult);

	if (!bSucceededToSelectState)
	{
		if (bIsNewFrame)
		{
			// Adding to TemporaryFrame can invalidate ParentFrame
			const FStateTreeExecutionFrame* NextParentFrame = UE::StateTree::ExecutionContext::FindExecutionFrame(OutSelectionResult->SelectedFrames.Last(1), MakeArrayView(Exec.ActiveFrames), MakeArrayView(OutSelectionResult->TemporaryFrames));
			FStateTreeExecutionFrame* NextFrame = FindMutableExecutionFrame(OutSelectionResult->SelectedFrames.Last(), MakeArrayView(Exec.ActiveFrames), MakeArrayView(OutSelectionResult->TemporaryFrames));

			constexpr EStateTreeRunStatus CompletionStatus = EStateTreeRunStatus::Stopped;
			StopTemporaryEvaluatorsAndGlobalTasks(NextParentFrame, *NextFrame, CompletionStatus);
			CleanFrame(Exec, SelectedFrame->FrameID);
		}
		OutSelectionResult->SelectedFrames.Pop();
	}

	return bSucceededToSelectState;
}

bool FStateTreeExecutionContext::SelectStateInternal_TrySelectChildrenInOrder(
	const FSelectStateArguments& Args,
	const FSelectStateInternalArguments& InternalArgs,
	const TSharedRef<FSelectStateResult>& OutSelectionResult,
	TNotNull<const UStateTree*> NextStateTree,
	const FCompactStateTreeState& NextState,
	const FStateTreeStateHandle NextStateHandle)
{
	using namespace UE::StateTree::ExecutionContext;

	if (!NextState.HasChildren())
	{
		// Select this state (For backwards compatibility)
		UE_STATETREE_DEBUG_STATE_EVENT(this, NextStateHandle, EStateTreeTraceEventType::OnStateSelected);
		return true;
	}

	UE_STATETREE_DEBUG_SCOPED_STATE_PHASE(this, NextStateHandle, EStateTreeUpdatePhase::TrySelectBehavior);

	// If the state has children, proceed to select children.
	bool bSucceededToSelectState = false;
	for (uint16 ChildStateIndex = NextState.ChildrenBegin; ChildStateIndex < NextState.ChildrenEnd; ChildStateIndex = NextStateTree->States[ChildStateIndex].GetNextSibling())
	{
		FStateHandleContext ChildState = FStateHandleContext(NextStateTree, FStateTreeStateHandle(ChildStateIndex));
		FSelectStateInternalArguments NewInternalArgs = InternalArgs;
		NewInternalArgs.MissingStatesToReachTarget = MakeArrayView(&ChildState, 1);
		if (SelectStateInternal(Args, NewInternalArgs, OutSelectionResult))
		{
			// Selection succeeded
			bSucceededToSelectState = true;
			break;
		}
	}

	return bSucceededToSelectState;
}

bool FStateTreeExecutionContext::SelectStateInternal_TrySelectChildrenAtRandom(
	const FSelectStateArguments& Args,
	const FSelectStateInternalArguments& InternalArgs,
	const TSharedRef<FSelectStateResult>& OutSelectionResult,
	TNotNull<const UStateTree*> NextStateTree,
	const FCompactStateTreeState& NextState,
	const FStateTreeStateHandle NextStateHandle)
{
	using namespace UE::StateTree::ExecutionContext;

	if (!NextState.HasChildren())
	{
		// Select this state (For backwards compatibility)
		UE_STATETREE_DEBUG_STATE_EVENT(this, NextStateHandle, EStateTreeTraceEventType::OnStateSelected);
		return true;
	}

	UE_STATETREE_DEBUG_SCOPED_STATE_PHASE(this, NextStateHandle, EStateTreeUpdatePhase::TrySelectBehavior);

	TArray<uint16, TInlineAllocator<16, FNonconcurrentLinearArrayAllocator>> NextLevelChildStates;
	NextLevelChildStates.Reserve(NextState.ChildrenEnd - NextState.ChildrenBegin);
	for (uint16 ChildStateIndex = NextState.ChildrenBegin; ChildStateIndex < NextState.ChildrenEnd; ChildStateIndex = NextStateTree->States[ChildStateIndex].GetNextSibling())
	{
		NextLevelChildStates.Push(ChildStateIndex);
	}

	const FStateTreeExecutionState& Exec = GetExecState();
	while (!NextLevelChildStates.IsEmpty())
	{
		const int32 ChildStateIndex = Exec.RandomStream.RandRange(0, NextLevelChildStates.Num() - 1);
		FStateHandleContext ChildState = FStateHandleContext(NextStateTree, FStateTreeStateHandle(NextLevelChildStates[ChildStateIndex]));
		FSelectStateInternalArguments NewInternalArgs = InternalArgs;
		NewInternalArgs.MissingStatesToReachTarget = MakeArrayView(&ChildState, 1);
		if (SelectStateInternal(Args, NewInternalArgs, OutSelectionResult))
		{
			// Selection succeeded
			return true;
		}

		constexpr EAllowShrinking AllowShrinking = EAllowShrinking::No;
		NextLevelChildStates.RemoveAtSwap(ChildStateIndex, AllowShrinking);
	}

	return false;
}

bool FStateTreeExecutionContext::SelectStateInternal_TrySelectChildrenWithHighestUtility(
	const FSelectStateArguments& Args,
	const FSelectStateInternalArguments& InternalArgs,
	const TSharedRef<FSelectStateResult>& OutSelectionResult,
	TNotNull<const UStateTree*> NextStateTree,
	const FCompactStateTreeState& NextState,
	const FStateTreeStateHandle NextStateHandle)
{
	using namespace UE::StateTree::ExecutionContext;

	if (!NextState.HasChildren())
	{
		// Select this state (For backwards compatibility)
		UE_STATETREE_DEBUG_STATE_EVENT(this, NextStateHandle, EStateTreeTraceEventType::OnStateSelected);
		return true;
	}

	UE_STATETREE_DEBUG_SCOPED_STATE_PHASE(this, NextStateHandle, EStateTreeUpdatePhase::TrySelectBehavior);

	TArray<TPair<uint16, float>, TInlineAllocator<16, FNonconcurrentLinearArrayAllocator>> NextLevelChildStates;
	NextLevelChildStates.Reserve(NextState.ChildrenEnd - NextState.ChildrenBegin);
	{
		using namespace UE::StateTree::ExecutionContext::Private;
		const FStateTreeExecutionFrame* NextParentFrame = OutSelectionResult->SelectedFrames.Num() > 1
			? UE::StateTree::ExecutionContext::FindExecutionFrame(OutSelectionResult->SelectedFrames.Last(1), MakeConstArrayView(GetExecState().ActiveFrames), MakeConstArrayView(OutSelectionResult->TemporaryFrames))
			: nullptr;
		const FStateTreeExecutionFrame* NextFrame = UE::StateTree::ExecutionContext::FindExecutionFrame(OutSelectionResult->SelectedFrames.Last(), MakeConstArrayView(GetExecState().ActiveFrames), MakeConstArrayView(OutSelectionResult->TemporaryFrames));
		check(NextFrame);

		for (uint16 ChildState = NextState.ChildrenBegin; ChildState < NextState.ChildrenEnd; ChildState = NextStateTree->States[ChildState].GetNextSibling())
		{
			const FCompactStateTreeState& CurrentState = NextStateTree->States[ChildState];
			const float Score = EvaluateUtilityWithValidation(NextParentFrame, *NextFrame, FStateTreeStateHandle(ChildState), CurrentState.ConsiderationEvaluationScopeMemoryRequirement, CurrentState.UtilityConsiderationsBegin, CurrentState.UtilityConsiderationsNum, CurrentState.Weight);
			NextLevelChildStates.Emplace(ChildState, Score);
		}
	}

	while (!NextLevelChildStates.IsEmpty())
	{
		//Find one with highest score in the remaining candidates
		float HighestScore = std::numeric_limits<float>::lowest();
		int32 ArrayIndexWithHighestScore = INDEX_NONE;
		for (int32 Index = 0; Index < NextLevelChildStates.Num(); ++Index)
		{
			const float Score = NextLevelChildStates[Index].Get<1>();
			if (Score > HighestScore)
			{
				HighestScore = Score;
				ArrayIndexWithHighestScore = Index;
			}
		}

		if (!NextLevelChildStates.IsValidIndex(ArrayIndexWithHighestScore))
		{
			return false;
		}

		FStateHandleContext ChildState = FStateHandleContext(NextStateTree, FStateTreeStateHandle(NextLevelChildStates[ArrayIndexWithHighestScore].Get<0>()));
		FSelectStateInternalArguments NewInternalArgs = InternalArgs;
		NewInternalArgs.MissingStatesToReachTarget = MakeArrayView(&ChildState, 1);
		if (SelectStateInternal(Args, NewInternalArgs, OutSelectionResult))
		{
			return true;
		}

		// Disqualify the state we failed to enter
		constexpr EAllowShrinking AllowShrinking = EAllowShrinking::No;
		NextLevelChildStates.RemoveAtSwap(ArrayIndexWithHighestScore, AllowShrinking);
	}

	return false;
}

bool FStateTreeExecutionContext::SelectStateInternal_TrySelectChildrenAtRandomWeightedByUtility(
	const FSelectStateArguments& Args,
	const FSelectStateInternalArguments& InternalArgs,
	const TSharedRef<FSelectStateResult>& OutSelectionResult,
	TNotNull<const UStateTree*> NextStateTree,
	const FCompactStateTreeState& NextState,
	const FStateTreeStateHandle NextStateHandle)
{
	using namespace UE::StateTree::ExecutionContext;

	if (!NextState.HasChildren())
	{
		// Select this state (For backwards compatibility)
		UE_STATETREE_DEBUG_STATE_EVENT(this, NextStateHandle, EStateTreeTraceEventType::OnStateSelected);
		return true;
	}

	UE_STATETREE_DEBUG_SCOPED_STATE_PHASE(this, NextStateHandle, EStateTreeUpdatePhase::TrySelectBehavior);

	TArray<TTuple<uint16, float>, TInlineAllocator<16, FNonconcurrentLinearArrayAllocator>> NextLevelChildStates;
	NextLevelChildStates.Reserve(NextState.ChildrenEnd - NextState.ChildrenBegin);

	float TotalScore = 0.0f;
	{
		using namespace UE::StateTree::ExecutionContext::Private;
		const FStateTreeExecutionFrame* NextParentFrame = OutSelectionResult->SelectedFrames.Num() > 1
			? UE::StateTree::ExecutionContext::FindExecutionFrame(OutSelectionResult->SelectedFrames.Last(1), MakeConstArrayView(GetExecState().ActiveFrames), MakeConstArrayView(OutSelectionResult->TemporaryFrames))
			: nullptr;
		const FStateTreeExecutionFrame* NextFrame = UE::StateTree::ExecutionContext::FindExecutionFrame(OutSelectionResult->SelectedFrames.Last(), MakeConstArrayView(GetExecState().ActiveFrames), MakeConstArrayView(OutSelectionResult->TemporaryFrames));
		check(NextFrame);

		for (uint16 ChildState = NextState.ChildrenBegin; ChildState < NextState.ChildrenEnd; ChildState = NextStateTree->States[ChildState].GetNextSibling())
		{
			const FCompactStateTreeState& CurrentState = NextStateTree->States[ChildState];
			const float Score = EvaluateUtilityWithValidation(NextParentFrame, *NextFrame, FStateTreeStateHandle(ChildState), CurrentState.ConsiderationEvaluationScopeMemoryRequirement, CurrentState.UtilityConsiderationsBegin, CurrentState.UtilityConsiderationsNum, CurrentState.Weight);
			if (Score > 0.0f)
			{
				NextLevelChildStates.Emplace(ChildState, Score);
				TotalScore += Score;
			}
		}
	}

	const FStateTreeExecutionState& Exec = GetExecState();
	while (!NextLevelChildStates.IsEmpty())
	{
		const float RandomScore = Exec.RandomStream.FRand() * TotalScore;
		float AccumulatedScore = 0.0f;
		for (int32 Index = 0; Index < NextLevelChildStates.Num(); ++Index)
		{
			const TTuple<uint16, float>& StateScorePair = NextLevelChildStates[Index];
			const uint16 StateIndex = StateScorePair.Key;
			const float StateScore = StateScorePair.Value;
			AccumulatedScore += StateScore;

			if (RandomScore <= AccumulatedScore || (Index == (NextLevelChildStates.Num() - 1)))
			{
				FStateHandleContext ChildState = FStateHandleContext(NextStateTree, FStateTreeStateHandle(StateIndex));
				FSelectStateInternalArguments NewInternalArgs = InternalArgs;
				NewInternalArgs.MissingStatesToReachTarget = MakeArrayView(&ChildState, 1);
				if (SelectStateInternal(Args, NewInternalArgs, OutSelectionResult))
				{
					return true;
				}

				// Disqualify the state we failed to enter
				constexpr EAllowShrinking AllowShrinking = EAllowShrinking::No;
				NextLevelChildStates.RemoveAtSwap(Index, AllowShrinking);
				TotalScore -= StateScore;
				break;
			}
		}
	}

	return false;
}

bool FStateTreeExecutionContext::SelectStateInternal_TryFollowTransitions(
	const FSelectStateArguments& Args,
	const FSelectStateInternalArguments& InternalArgs,
	const TSharedRef<FSelectStateResult>& OutSelectionResult,
	TNotNull<const UStateTree*> NextStateTree,
	const FCompactStateTreeState& NextState,
	const FStateTreeStateHandle NextStateHandle)
{
	using namespace UE::StateTree;
	using namespace UE::StateTree::ExecutionContext;
	using namespace UE::StateTree::ExecutionContext::Private;

	UE_STATETREE_DEBUG_SCOPED_STATE_PHASE(this, NextStateHandle, EStateTreeUpdatePhase::TrySelectBehavior);

	bool bSucceededToSelectState = false;
	EStateTreeTransitionPriority CurrentPriority = EStateTreeTransitionPriority::None;

	for (uint8 Index = 0; Index < NextState.TransitionsNum; ++Index)
	{
		const int32 TransitionIndex = NextState.TransitionsBegin + Index;
		const FCompactStateTransition& Transition = NextStateTree->Transitions[TransitionIndex];

		// Skip disabled transitions
		if (Transition.bTransitionEnabled == false)
		{
			continue;
		}

		// No need to test the transition if same or higher priority transition has already been processed.
		if (Transition.Priority <= CurrentPriority)
		{
			continue;
		}

		// Skip completion transitions
		if (EnumHasAnyFlags(Transition.Trigger, EStateTreeTransitionTrigger::OnStateCompleted))
		{
			continue;
		}

		// Skip invalid state or completion state
		if (!Transition.State.IsValid() || Transition.State.IsCompletionState())
		{
			continue;
		}

		// Cannot follow transitions with delay.
		if (Transition.HasDelay())
		{
			continue;
		}

		// Try to prevent (infinite) loops in the selection.
		const bool bSelectionContainsState = OutSelectionResult->SelectedStates.ContainsByPredicate(
			[State = Transition.State](const FActiveState& Other)
			{
				return State == Other.GetStateHandle();
			});
		if (bSelectionContainsState)
		{
			STATETREE_LOG(Warning, TEXT("%hs: Loop detected when trying to select state %s from '%s'. Prior states: %s.  '%s' using StateTree '%s'.")
				, __FUNCTION__
				, *GetSafeStateName(NextStateTree, Transition.State)
				, *GetStateStatusString(GetExecState())
				, *GetStatePathAsString(&RootStateTree, OutSelectionResult->SelectedStates)
				, *Owner.GetName()
				, *NextStateTree->GetFullName());
			continue;
		}

		FSharedEventInlineArray TransitionEvents;
		GetTriggerTransitionEvent(Transition, CurrentlyProcessedBroadcastedDelegates, Args.TransitionEvent, GetEventsToProcessView(), TransitionEvents);

		for (const FStateTreeSharedEvent& SelectedStateTransitionEvent : TransitionEvents)
		{
			bool bTransitionConditionsPassed = false;
			{
				using namespace UE::StateTree::ExecutionContext::Private;
				const FStateTreeExecutionFrame* NextParentFrame = OutSelectionResult->SelectedFrames.Num() > 1
					? UE::StateTree::ExecutionContext::FindExecutionFrame(OutSelectionResult->SelectedFrames.Last(1), MakeConstArrayView(GetExecState().ActiveFrames), MakeConstArrayView(OutSelectionResult->TemporaryFrames))
					: nullptr;
				const FStateTreeExecutionFrame* NextFrame = UE::StateTree::ExecutionContext::FindExecutionFrame(OutSelectionResult->SelectedFrames.Last(), MakeConstArrayView(GetExecState().ActiveFrames), MakeConstArrayView(OutSelectionResult->TemporaryFrames));
				check(NextFrame);

				FCurrentlyProcessedTransitionEventScope TransitionEventScope(*this, SelectedStateTransitionEvent.IsValid() ? SelectedStateTransitionEvent.Get() : nullptr);
				UE_STATETREE_DEBUG_TRANSITION_EVENT(this, FStateTreeTransitionSource(NextStateTree, FStateTreeIndex16(TransitionIndex), Transition.State, Transition.Priority), EStateTreeTraceEventType::OnEvaluating);
				bTransitionConditionsPassed = TestAllConditionsWithValidation(NextParentFrame, *NextFrame, NextStateHandle, Transition.ConditionEvaluationScopeMemoryRequirement, Transition.ConditionsBegin, Transition.ConditionsNum, EStateTreeUpdatePhase::TransitionConditions);
			}

			if (bTransitionConditionsPassed)
			{
				// Using SelectState() instead of SelectStateInternal to treat the transitions the same way as regular transitions,
				// e.g. it may jump to a completely different branch.

				FSelectStateResult CopySelectStateResult = OutSelectionResult.Get();

				FSelectStateArguments SelectStateArgs;
				SelectStateArgs.ActiveStates = MakeConstArrayView(CopySelectStateResult.SelectedStates);
				if (CopySelectStateResult.SelectedStates.Num() > 0)
				{
					SelectStateArgs.SourceState = CopySelectStateResult.SelectedStates.Last();
				}
				SelectStateArgs.TargetState = FStateHandleContext(NextStateTree, Transition.State);
				SelectStateArgs.TransitionEvent = SelectedStateTransitionEvent;
				SelectStateArgs.Fallback = Transition.Fallback;
				SelectStateArgs.SelectionRules = NextStateTree->StateSelectionRules;
				SelectStateArgs.SelectionRules = Transition.GetStateSelectionRulesForTransition(SelectStateArgs.SelectionRules);

				OutSelectionResult->SelectedStates.Reset();
				OutSelectionResult->SelectedFrames.Reset();
				if (SelectState(SelectStateArgs, OutSelectionResult))
				{
					CurrentPriority = Transition.Priority;
					bSucceededToSelectState = true;

					//@todo sort the transition by priority at compile time. This will solve having to loop back once we found a valid transition.
					// Consume the transition event
					if (SelectedStateTransitionEvent.IsValid() && Transition.bConsumeEventOnSelect)
					{
						ConsumeEvent(SelectedStateTransitionEvent);
					}

					// Cannot return because higher priority transitions may override the selection. 
					break;
				}
				else
				{
					FSelectStateResult& SelectionResult = OutSelectionResult.Get();
					SelectionResult = MoveTemp(CopySelectStateResult);
				}
			}
		}
	}

	return bSucceededToSelectState;
}

FString FStateTreeExecutionContext::GetSafeStateName(const FStateTreeExecutionFrame& CurrentFrame, const FStateTreeStateHandle State) const
{
	return GetSafeStateName(CurrentFrame.StateTree, State);
}

FString FStateTreeExecutionContext::GetSafeStateName(const UStateTree* StateTree, const FStateTreeStateHandle State) const
{
	if (State == FStateTreeStateHandle::Invalid)
	{
		return TEXT("(State Invalid)");
	}
	else if (State == FStateTreeStateHandle::Succeeded)
	{
		return TEXT("(State Succeeded)");
	}
	else if (State == FStateTreeStateHandle::Failed)
	{
		return TEXT("(State Failed)");
	}
	else if (StateTree && StateTree->States.IsValidIndex(State.Index))
	{
		return *StateTree->States[State.Index].Name.ToString();
	}
	return TEXT("(Unknown)");
}

FString FStateTreeExecutionContext::DebugGetStatePath(TConstArrayView<FStateTreeExecutionFrame> ActiveFrames, const FStateTreeExecutionFrame* CurrentFrame, const int32 ActiveStateIndex) const
{
	FString StatePath;
	const UStateTree* LastStateTree = &RootStateTree;

	for (const FStateTreeExecutionFrame& Frame : ActiveFrames)
	{
		if (!ensure(Frame.StateTree))
		{
			return StatePath;
		}

		// If requested up the active state, clamp count.
		int32 Num = Frame.ActiveStates.Num();
		if (CurrentFrame == &Frame && Frame.ActiveStates.IsValidIndex(ActiveStateIndex))
		{
			Num = ActiveStateIndex + 1;
		}

		if (Frame.StateTree != LastStateTree)
		{
			StatePath.Appendf(TEXT("[%s]"), *GetNameSafe(Frame.StateTree));
			LastStateTree = Frame.StateTree;
		}

		for (int32 i = 0; i < Num; i++)
		{
			const FCompactStateTreeState& State = Frame.StateTree->States[Frame.ActiveStates[i].Index];
			StatePath.Appendf(TEXT("%s%s"), i == 0 ? TEXT("") : TEXT("."), *State.Name.ToString());
		}
	}

	return StatePath;
}

FString FStateTreeExecutionContext::GetStateStatusString(const FStateTreeExecutionState& ExecState) const
{
	if (ExecState.TreeRunStatus != EStateTreeRunStatus::Running)
	{
		return TEXT("--:") + UEnum::GetDisplayValueAsText(ExecState.LastTickStatus).ToString();
	}

	if (ExecState.ActiveFrames.Num())
	{
		const FStateTreeExecutionFrame& LastFrame = ExecState.ActiveFrames.Last();
		if (LastFrame.ActiveStates.Num() > 0)
		{
			return GetSafeStateName(LastFrame, LastFrame.ActiveStates.Last()) + TEXT(":") + UEnum::GetDisplayValueAsText(ExecState.LastTickStatus).ToString();
		}
	}
	return FString();
}

// Deprecated
FString FStateTreeExecutionContext::GetInstanceDescription() const
{
	return GetInstanceDescriptionInternal();
}


#undef STATETREE_LOG
#undef STATETREE_CLOG
