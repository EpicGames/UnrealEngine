// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeExecutionContextTypes.h"

namespace UE::StateTree::ExecutionContext
{
	const FStateTreeExecutionFrame* FindExecutionFrame(FActiveFrameID InFrameID,
		TConstArrayView<FStateTreeExecutionFrame> InActiveFrames, TConstArrayView<FStateTreeExecutionFrame> InTemporaryFrames)
	{
		const FStateTreeExecutionFrame* Frame = InActiveFrames.FindByPredicate(
			[InFrameID](const FStateTreeExecutionFrame& Frame)
			{
				return Frame.FrameID == InFrameID;
			});
		if (Frame)
		{
			return Frame;
		}
	
		return InTemporaryFrames.FindByPredicate(
			[InFrameID](const FStateTreeExecutionFrame& Frame)
			{
				return Frame.FrameID == InFrameID;
			});
	
	}
}
