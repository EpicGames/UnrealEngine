// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeExecutionExtension.h"
#include "StateTreeExecutionTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeExecutionExtension)


FStateTreeExecutionExtension::FNextTickArguments::FNextTickArguments()
	: Reason(UE::StateTree::ETickReason::None)
{
}

FStateTreeExecutionExtension::FNextTickArguments::FNextTickArguments(UE::StateTree::ETickReason InReason)
	: Reason(InReason)
{ }

