// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeExecutionTypes.h"
#include "StateTree.h"
#include "StateTreeDelegate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeExecutionTypes)

const FStateTreeExternalDataHandle FStateTreeExternalDataHandle::Invalid = FStateTreeExternalDataHandle();

#if WITH_STATETREE_TRACE
const FStateTreeInstanceDebugId FStateTreeInstanceDebugId::Invalid = FStateTreeInstanceDebugId();
#endif // WITH_STATETREE_TRACE

//----------------------------------------------------------------------//
// FStateTreeTransitionSource
//----------------------------------------------------------------------//
FStateTreeTransitionSource::FStateTreeTransitionSource(
	const UStateTree* StateTree,
	const EStateTreeTransitionSourceType SourceType,
	const FStateTreeIndex16 TransitionIndex,
	const FStateTreeStateHandle TargetState,
	const EStateTreeTransitionPriority Priority
	)
	: Asset(StateTree)
	, SourceType(SourceType)
	, TransitionIndex(TransitionIndex)
	, TargetState(TargetState)
	, Priority(Priority)
{
}

//----------------------------------------------------------------------//
// FStateTreeTransitionResult
//----------------------------------------------------------------------//
//Deprecated
FStateTreeTransitionResult::FStateTreeTransitionResult(const FRecordedStateTreeTransitionResult& RecordedTransition)
{
}

//----------------------------------------------------------------------//
// FRecordedStateTreeTransitionResult
//----------------------------------------------------------------------//
//Deprecated
FRecordedStateTreeTransitionResult::FRecordedStateTreeTransitionResult(const FStateTreeTransitionResult& Transition)
{
}


//----------------------------------------------------------------------//
// FStateTreeExecutionState
//----------------------------------------------------------------------//
UE::StateTree::FActiveStatePath FStateTreeExecutionState::GetActiveStatePath() const
{
	int32 NewNum = 0;
	for (const FStateTreeExecutionFrame& Frame : ActiveFrames)
	{
		NewNum += Frame.ActiveStates.Num();
	}

	if (NewNum == 0 || ActiveFrames[0].StateTree == nullptr)
	{
		return UE::StateTree::FActiveStatePath();
	}

	TArray<UE::StateTree::FActiveState> Elements;
	Elements.Reserve(NewNum);

	for (const FStateTreeExecutionFrame& Frame : ActiveFrames)
	{
		for (int32 StateIndex = 0; StateIndex < Frame.ActiveStates.Num(); ++StateIndex)
		{
			Elements.Emplace(Frame.FrameID, Frame.ActiveStates.StateIDs[StateIndex], Frame.ActiveStates.States[StateIndex]);
		}
	}

	return UE::StateTree::FActiveStatePath(ActiveFrames[0].StateTree, MoveTemp(Elements));
}

const FStateTreeExecutionFrame* FStateTreeExecutionState::FindActiveFrame(UE::StateTree::FActiveFrameID FrameID) const
{
	return ActiveFrames.FindByPredicate([FrameID](const FStateTreeExecutionFrame& Other)
		{
			return Other.FrameID == FrameID;
		});
}

FStateTreeExecutionFrame* FStateTreeExecutionState::FindActiveFrame(UE::StateTree::FActiveFrameID FrameID)
{
	return ActiveFrames.FindByPredicate([FrameID](const FStateTreeExecutionFrame& Other)
		{
			return Other.FrameID == FrameID;
		});
}

int32 FStateTreeExecutionState::IndexOfActiveFrame(UE::StateTree::FActiveFrameID FrameID) const
{
	return ActiveFrames.IndexOfByPredicate([FrameID](const FStateTreeExecutionFrame& Other)
		{
			return Other.FrameID == FrameID;
		});
}

UE::StateTree::FScheduledTickHandle FStateTreeExecutionState::AddScheduledTickRequest(FStateTreeScheduledTick ScheduledTick)
{
	UE::StateTree::FScheduledTickHandle Result = UE::StateTree::FScheduledTickHandle::GenerateNewHandle();
	ScheduledTickRequests.Add(FScheduledTickRequest{.Handle = Result, .ScheduledTick = ScheduledTick});
	CacheScheduledTickRequest();
	return Result;
}

bool FStateTreeExecutionState::UpdateScheduledTickRequest(UE::StateTree::FScheduledTickHandle Handle, FStateTreeScheduledTick ScheduledTick)
{
	FScheduledTickRequest* Found = ScheduledTickRequests.FindByPredicate([Handle](const FScheduledTickRequest& Other) { return Other.Handle == Handle; });
	if (Found && Found->ScheduledTick != ScheduledTick)
	{
		Found->ScheduledTick = ScheduledTick;
		CacheScheduledTickRequest();
		return true;
	}
	return false;
}

bool FStateTreeExecutionState::RemoveScheduledTickRequest(UE::StateTree::FScheduledTickHandle Handle)
{
	const int32 IndexOf = ScheduledTickRequests.IndexOfByPredicate([Handle](const FScheduledTickRequest& Other) { return Other.Handle == Handle; });
	if (IndexOf != INDEX_NONE)
	{
		ScheduledTickRequests.RemoveAtSwap(IndexOf);
		CacheScheduledTickRequest();
	}
	return IndexOf != INDEX_NONE;
}

void FStateTreeExecutionState::CacheScheduledTickRequest()
{
	auto GetBestRequest = [](TConstArrayView<FScheduledTickRequest> Requests)
		{
			const int32 ScheduledTickRequestsNum = Requests.Num();
			if (ScheduledTickRequestsNum == 0)
			{
				return FStateTreeScheduledTick();
			}
			if (ScheduledTickRequestsNum == 1)
			{
				return Requests[0].ScheduledTick;
			}

			for (const FScheduledTickRequest& Request : Requests)
			{
				if (Request.ScheduledTick.ShouldTickEveryFrames())
				{
					return Request.ScheduledTick;
				}
			}

			for (const FScheduledTickRequest& Request : Requests)
			{
				if (Request.ScheduledTick.ShouldTickOnceNextFrame())
				{
					return Request.ScheduledTick;
				}
			}

			TOptional<float> CustomTickRate;
			for (const FScheduledTickRequest& Request : Requests)
			{
				const float CachedTickRate = Request.ScheduledTick.GetTickRate();
				CustomTickRate = CustomTickRate.IsSet() ? FMath::Min(CustomTickRate.GetValue(), CachedTickRate) : CachedTickRate;
			}
			return FStateTreeScheduledTick::MakeCustomTickRate(CustomTickRate.GetValue(), UE::StateTree::ETickReason::ScheduledTickRequest);
		};

	CachedScheduledTickRequest = GetBestRequest(ScheduledTickRequests);
}

//----------------------------------------------------------------------//
// FStateTreeExecutionFrame
//----------------------------------------------------------------------//
//Deprecated
PRAGMA_DISABLE_DEPRECATION_WARNINGS
FStateTreeExecutionFrame::FStateTreeExecutionFrame(const FRecordedStateTreeExecutionFrame& RecordedExecutionFrame)
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

//----------------------------------------------------------------------//
// FRecordedStateTreeExecutionFrame
//----------------------------------------------------------------------//
//Deprecated
PRAGMA_DISABLE_DEPRECATION_WARNINGS
FRecordedStateTreeExecutionFrame::FRecordedStateTreeExecutionFrame(const FStateTreeExecutionFrame& ExecutionFrame)
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

//Deprecated
//----------------------------------------------------------------------//
// FFinishedTask
//----------------------------------------------------------------------//
UE::StateTree::FFinishedTask::FFinishedTask(FActiveFrameID InFrameID, FActiveStateID InStateID, FStateTreeIndex16 InTaskIndex, EStateTreeRunStatus InRunStatus, EReasonType InReason, bool bInTickProcessed)
	: FrameID(InFrameID)
	, StateID(InStateID)
	, TaskIndex(InTaskIndex)
	, RunStatus(InRunStatus)
	, Reason(InReason)
	, bTickProcessed(bInTickProcessed)
{
}

//----------------------------------------------------------------------//
// FStateTreeScheduledTick
//----------------------------------------------------------------------//
FStateTreeScheduledTick FStateTreeScheduledTick::MakeSleep()
{
	return FStateTreeScheduledTick(UE_FLOAT_NON_FRACTIONAL, UE::StateTree::ETickReason::None);
}

FStateTreeScheduledTick FStateTreeScheduledTick::MakeEveryFrames(UE::StateTree::ETickReason Reason)
{
	return FStateTreeScheduledTick(0.0f, Reason);
}

FStateTreeScheduledTick FStateTreeScheduledTick::MakeNextFrame(UE::StateTree::ETickReason Reason)
{
	return FStateTreeScheduledTick(UE_KINDA_SMALL_NUMBER, Reason);
}

FStateTreeScheduledTick FStateTreeScheduledTick::MakeCustomTickRate(float DeltaTime, UE::StateTree::ETickReason Reason)
{
	ensureMsgf(DeltaTime >= 0.0f, TEXT("Use a value greater than zero."));
	if (DeltaTime > 0.0f)
	{
		return FStateTreeScheduledTick(DeltaTime, Reason);
	}
	return MakeEveryFrames(Reason);
}

bool FStateTreeScheduledTick::ShouldSleep() const
{
	return NextDeltaTime >= UE_FLOAT_NON_FRACTIONAL;
}

bool FStateTreeScheduledTick::ShouldTickEveryFrames() const
{
	return NextDeltaTime == 0.0f;
}

bool FStateTreeScheduledTick::ShouldTickOnceNextFrame() const
{
	return NextDeltaTime == UE_KINDA_SMALL_NUMBER;
}

bool FStateTreeScheduledTick::HasCustomTickRate() const
{
	return NextDeltaTime > 0.0f;
}

float FStateTreeScheduledTick::GetTickRate() const
{
	return NextDeltaTime;
}

//----------------------------------------------------------------------//
// FScheduledTickHandle
//----------------------------------------------------------------------//
UE::StateTree::FScheduledTickHandle UE::StateTree::FScheduledTickHandle::GenerateNewHandle()
{
	static std::atomic<uint32> Value = 0;

	uint32 Result = 0;
	UE_AUTORTFM_OPEN
	{
		Result = ++Value;

		// Check that we wrap round to 0, because we reserve 0 for invalid.
		if (Result == 0)
		{
			Result = ++Value;
		}
	};

	return FScheduledTickHandle(Result);
}


//----------------------------------------------------------------------//
// FStateTreeDelegateActiveListeners
//----------------------------------------------------------------------//
FStateTreeDelegateActiveListeners::FActiveListener::FActiveListener(const FStateTreeDelegateListener& InListener, FSimpleDelegate InDelegate, UE::StateTree::FActiveFrameID InFrameID, UE::StateTree::FActiveStateID InStateID, FStateTreeIndex16 InOwningNodeIndex)
	: Listener(InListener)
	, Delegate(MoveTemp(InDelegate))
	, FrameID(InFrameID)
	, StateID(InStateID)
	, OwningNodeIndex(InOwningNodeIndex)
{}

FStateTreeDelegateActiveListeners::~FStateTreeDelegateActiveListeners()
{
	check(BroadcastingLockCount == 0);
}

void FStateTreeDelegateActiveListeners::Add(const FStateTreeDelegateListener& Listener, FSimpleDelegate Delegate, UE::StateTree::FActiveFrameID InFrameID, UE::StateTree::FActiveStateID InStateID, FStateTreeIndex16 OwningNodeIndex)
{
	check(Listener.IsValid());
	Remove(Listener);
	Listeners.Emplace(Listener, MoveTemp(Delegate), InFrameID, InStateID, OwningNodeIndex);
}

void FStateTreeDelegateActiveListeners::Remove(const FStateTreeDelegateListener& Listener)
{
	check(Listener.IsValid());

	const int32 Index = Listeners.IndexOfByPredicate([Listener](const FActiveListener& ActiveListener)
		{
			return ActiveListener.Listener == Listener;
		});

	if (Index == INDEX_NONE)
	{
		return;
	}

	if (BroadcastingLockCount > 0)
	{
		Listeners[Index] = FActiveListener();
		bContainsUnboundListeners = true;
	}
	else
	{
		Listeners.RemoveAtSwap(Index);
	}
}

void FStateTreeDelegateActiveListeners::RemoveAll(UE::StateTree::FActiveFrameID FrameID)
{
	check(BroadcastingLockCount == 0);
	Listeners.RemoveAllSwap([FrameID](const FActiveListener& Listener)
		{
			return Listener.FrameID == FrameID;
		});
}

void FStateTreeDelegateActiveListeners::RemoveAll(UE::StateTree::FActiveStateID StateID)
{
	check(BroadcastingLockCount == 0);
	Listeners.RemoveAllSwap([StateID](const FActiveListener& Listener)
		{
			return Listener.StateID == StateID;
		});
}

void FStateTreeDelegateActiveListeners::BroadcastDelegate(const FStateTreeDelegateDispatcher& Dispatcher, const FStateTreeExecutionState& Exec)
{
	check(Dispatcher.IsValid());

	++BroadcastingLockCount;

	const int32 NumListeners = Listeners.Num();
	for (int32 Index = 0; Index < NumListeners; ++Index)
	{
		FActiveListener& ActiveListener = Listeners[Index];
		if (ActiveListener.Listener.GetDispatcher() == Dispatcher)
		{
			if (const FStateTreeExecutionFrame* ExectionFrame = Exec.FindActiveFrame(ActiveListener.FrameID))
			{
				// Is the node active and is the state active.
				if (ActiveListener.OwningNodeIndex.Get() <= ExectionFrame->ActiveNodeIndex.Get())
				{
					if (!ActiveListener.StateID.IsValid())
					{
						// It's a global task, no need to check for the state.
						ActiveListener.Delegate.ExecuteIfBound();
					}
					else
					{
						const int32 FoundStateIndex = ExectionFrame->ActiveStates.IndexOfReverse(ActiveListener.StateID);
						if (FoundStateIndex != INDEX_NONE)
						{
							ActiveListener.Delegate.ExecuteIfBound();
						}
					}
				}
			}
		}
	}

	--BroadcastingLockCount;

	if (BroadcastingLockCount == 0)
	{
		RemoveUnbounds();
	}
}

void FStateTreeDelegateActiveListeners::RemoveUnbounds()
{
	check(BroadcastingLockCount == 0);
	if (!bContainsUnboundListeners)
	{
		return;
	}

	Listeners.RemoveAllSwap([](const FActiveListener& Listener) { return !Listener.IsValid(); });
	bContainsUnboundListeners = false;
}
