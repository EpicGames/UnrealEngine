// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeFunctionLibrary.h"

#include "Blueprint/BlueprintExceptionInfo.h"
#include "StateTree.h"
#include "StateTreeReference.h"
#include "StructUtils/PropertyBag.h"
#include "UObject/Script.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeFunctionLibrary)

#define LOCTEXT_NAMESPACE "StateTreeFunctionLibrary"

void UStateTreeFunctionLibrary::SetStateTree(FStateTreeReference& Reference, UStateTree* NewStateTree)
{
	Reference.SetStateTree(NewStateTree);
}

FStateTreeReference UStateTreeFunctionLibrary::MakeStateTreeReference(UStateTree* NewStateTree)
{
	FStateTreeReference Result;
	Result.SetStateTree(NewStateTree);
	return Result;
}

void UStateTreeFunctionLibrary::K2_SetParametersProperty(FStateTreeReference&, FGuid, const int32&)
{
	checkNoEntry();
}

void UStateTreeFunctionLibrary::K2_GetParametersProperty(const FStateTreeReference&, FGuid, int32&)
{
	checkNoEntry();
}

DEFINE_FUNCTION(UStateTreeFunctionLibrary::execK2_SetParametersProperty)
{
	// Read wildcard Value input.
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;

	P_GET_STRUCT_REF(FStateTreeReference, StateTreeReference);
	P_GET_STRUCT(FGuid, PropertyID);

	Stack.StepCompiledIn<FProperty>(nullptr);
	const FProperty* SourceProperty = Stack.MostRecentProperty;
	const uint8* SourcePtr = Stack.MostRecentPropertyAddress;
	P_FINISH;

	if (SourceProperty == nullptr|| SourcePtr == nullptr)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			LOCTEXT("SetParametersProperty_InvalidValueWarning", "Failed to resolve the Value for SetParametersProperty")
		);

		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	else
	{
		P_NATIVE_BEGIN;
		FInstancedPropertyBag& InstancedPropertyBag = StateTreeReference.GetMutableParameters();
		FStructView PropertyBagView = InstancedPropertyBag.GetMutableValue();
		const UPropertyBag* PropertyBag = InstancedPropertyBag.GetPropertyBagStruct();
		if (PropertyBagView.IsValid() && PropertyBag)
		{
			if (const FPropertyBagPropertyDesc* PropertyBagDesc = PropertyBag->FindPropertyDescByID(PropertyID))
			{
				if (const FProperty* TargetProperty = PropertyBag->FindPropertyByName(PropertyBagDesc->Name))
				{
					if (SourceProperty->SameType(TargetProperty))
					{
						void* TargetPtr = TargetProperty->ContainerPtrToValuePtr<void>(PropertyBagView.GetMemory());
						TargetProperty->CopyCompleteValue(TargetPtr, SourcePtr);
						StateTreeReference.SetPropertyOverridden(PropertyID, true);
					}
				}
			}
		}
		P_NATIVE_END;
	}
}

DEFINE_FUNCTION(UStateTreeFunctionLibrary::execK2_GetParametersProperty)
{
	P_GET_STRUCT_REF(FStateTreeReference, StateTreeReference);
	P_GET_STRUCT(FGuid, PropertyID);

	// Read wildcard Value input.
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FProperty>(nullptr);

	const FProperty* TargetProperty = Stack.MostRecentProperty;
	void* TargetPtr = Stack.MostRecentPropertyAddress;

	P_FINISH;

	if (TargetProperty == nullptr || TargetPtr == nullptr)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			LOCTEXT("GetParametersProperty_InvalidValueWarning", "Failed to resolve the Value for GetParametersProperty")
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	else
	{
		P_NATIVE_BEGIN;
		const FInstancedPropertyBag& InstancedPropertyBag = StateTreeReference.GetParameters();
		const FConstStructView PropertyBagView = InstancedPropertyBag.GetValue();
		const UPropertyBag* PropertyBag = InstancedPropertyBag.GetPropertyBagStruct();
		if (PropertyBagView.IsValid() && PropertyBag)
		{
			if (const FPropertyBagPropertyDesc* PropertyBagDesc = PropertyBag->FindPropertyDescByID(PropertyID))
			{
				if (const FProperty* SourceProperty = PropertyBag->FindPropertyByName(PropertyBagDesc->Name))
				{
					if (SourceProperty->SameType(TargetProperty))
					{
						const void* SourcePtr = SourceProperty->ContainerPtrToValuePtr<void>(PropertyBagView.GetMemory());
						TargetProperty->CopyCompleteValue(TargetPtr, SourcePtr);
					}
				}
			}
		}
		P_NATIVE_END;
	}
}

#undef LOCTEXT_NAMESPACE
