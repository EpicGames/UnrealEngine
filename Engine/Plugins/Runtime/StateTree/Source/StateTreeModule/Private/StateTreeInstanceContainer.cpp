// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeInstanceContainer.h"
#include "StateTreeInstanceDataHelpers.h"

#include "Experimental/ConcurrentLinearAllocator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeInstanceContainer)

namespace UE::StateTree::InstanceData
{

	void FInstanceContainer::Init(TNotNull<UObject*> InOwner, const FInstanceContainer& InStructs, FAddArgs Args)
	{
		InstanceStructs.Reset();
		InstanceStructs = InStructs.InstanceStructs;
		Private::PostAppendToInstanceStructContainer(InstanceStructs, InOwner, Args.bDuplicateWrappedObject, 0);
	}

	void FInstanceContainer::Init(TNotNull<UObject*> InOwner, TConstArrayView<FInstancedStruct> InStructs, FAddArgs Args)
	{
		InstanceStructs.Reset();
		Private::AppendToInstanceStructContainer(InstanceStructs, InOwner, InStructs, Args.bDuplicateWrappedObject);
	}

	void FInstanceContainer::Init(TNotNull<UObject*> InOwner, TConstArrayView<FConstStructView> InStructs, FAddArgs Args)
	{
		InstanceStructs.Reset();
		Private::AppendToInstanceStructContainer(InstanceStructs, InOwner, InStructs, Args.bDuplicateWrappedObject);
	}

	int32 FInstanceContainer::Append(TNotNull<UObject*> InOwner, const FInstanceContainer& InStructs, FAddArgs Args)
	{
		TArray<FConstStructView, FNonconcurrentLinearArrayAllocator> ToAppend;
		ToAppend.Reserve(InStructs.InstanceStructs.Num());
		for (const FConstStructView& Struct : InStructs.InstanceStructs)
		{
			ToAppend.Add(Struct);
		}
		return Append(InOwner, MakeConstArrayView(ToAppend), Args);
	}

	int32 FInstanceContainer::Append(TNotNull<UObject*> InOwner, TConstArrayView<FInstancedStruct> InStructs, FAddArgs Args)
	{
		return Private::AppendToInstanceStructContainer(InstanceStructs, InOwner, InStructs, Args.bDuplicateWrappedObject);
	}

	int32 FInstanceContainer::Append(TNotNull<UObject*> InOwner, TConstArrayView<FConstStructView> InStructs, FAddArgs Args)
	{
		return Private::AppendToInstanceStructContainer(InstanceStructs, InOwner, InStructs, Args.bDuplicateWrappedObject);
	}

	bool FInstanceContainer::AreAllInstancesValid() const
	{
		return Private::AreAllInstancesValid(InstanceStructs);
	}

	int32 FInstanceContainer::GetAllocatedMemory() const
	{
		return Private::GetAllocatedMemory(InstanceStructs);
	}

} // namespace UE::StateTree::InstanceData
