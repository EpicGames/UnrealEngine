// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeInstanceData.h"
#include "StateTreeInstanceDataHelpers.h"
#include "StateTreeExecutionTypes.h"
#include "Debugger/StateTreeRuntimeValidationInstanceData.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "VisualLogger/VisualLogger.h"
#include "StateTree.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "Serialization/CustomVersion.h"
#include "Serialization/PropertyLocalizationDataGathering.h"
#include "StateTreeDelegate.h"
#include "AutoRTFM.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeInstanceData)

const FGuid FStateTreeInstanceStorageCustomVersion::GUID(0x60C4F0DE, 0x8B264C34, 0xAA937201, 0x5DFF09CC);
FCustomVersionRegistration GRegisterStateTreeInstanceStorageCustomVersion(FStateTreeInstanceStorageCustomVersion::GUID, FStateTreeInstanceStorageCustomVersion::LatestVersion, TEXT("StateTreeInstanceStorage"));

namespace UE::StateTree::InstanceData::Private
{
	bool AreAllInstancesValid(const FInstancedStructContainer& InstanceStructs)
	{
		for (FConstStructView Instance : InstanceStructs)
		{
			if (!Instance.IsValid())
			{
				return false;
			}
			if (const FStateTreeInstanceObjectWrapper* Wrapper = Instance.GetPtr<const FStateTreeInstanceObjectWrapper>())
			{
				if (!Wrapper->InstanceObject)
				{
					return false;
				}
			}
		}
		return true;
	}

	int32 GetAllocatedMemory(const FInstancedStructContainer& InstanceStructs)
	{
		int32 Size = InstanceStructs.GetAllocatedMemory();
		for (FConstStructView Instance : InstanceStructs)
		{
			if (const FStateTreeInstanceObjectWrapper* Wrapper = Instance.GetPtr<const FStateTreeInstanceObjectWrapper>())
			{
				if (Wrapper->InstanceObject)
				{
					Size += Wrapper->InstanceObject->GetClass()->GetStructureSize();
				}
			}
		}
		return Size;
	}

	TNotNull<UObject*> CopyNodeInstance(TNotNull<UObject*> Instance, TNotNull<UObject*> InOwner, bool bDuplicate)
	{
		const UClass* InstanceClass = Instance->GetClass();
		if (InstanceClass->HasAnyClassFlags(CLASS_NewerVersionExists))
		{
			const UClass* AuthoritativeClass = InstanceClass->GetAuthoritativeClass();
			UObject* NewInstance = NewObject<UObject>(InOwner, AuthoritativeClass);

			// Try to copy the values over using serialization
			// FObjectAndNameAsStringProxyArchive is used to store and restore names and objects as memory writer does not support UObject references at all.
			TArray<uint8> Data;
			FMemoryWriter Writer(Data);
			FObjectAndNameAsStringProxyArchive WriterProxy(Writer, /*bInLoadIfFindFails*/true);
			Instance->Serialize(WriterProxy);

			FMemoryReader Reader(Data);
			FObjectAndNameAsStringProxyArchive ReaderProxy(Reader, /*bInLoadIfFindFails*/true);
			NewInstance->Serialize(ReaderProxy);

			const UStateTree* OuterStateTree = Instance->GetTypedOuter<UStateTree>();
			UE_LOGF(LogStateTree, Display, "FStateTreeInstanceData: Duplicating '%ls' with old class '%ls' as '%ls', potential data loss. Please resave State Tree asset %ls.",
				*GetFullNameSafe(Instance), *GetNameSafe(InstanceClass), *GetNameSafe(AuthoritativeClass), *GetFullNameSafe(OuterStateTree));

			return NewInstance;
		}

		if (bDuplicate)
		{
			return ::DuplicateObject(&(*Instance), &(*InOwner));
		}

		return Instance;
	}

	void PostAppendToInstanceStructContainer(FInstancedStructContainer& InstanceStructs, TNotNull<UObject*> InOwner, bool bDuplicateWrappedObject, int32 StartIndex)
	{
		for (int32 Index = StartIndex; Index < InstanceStructs.Num(); ++Index)
		{
			if (FStateTreeInstanceObjectWrapper* Wrapper = InstanceStructs[Index].GetPtr<FStateTreeInstanceObjectWrapper>())
			{
				if (Wrapper->InstanceObject)
				{
					const bool bDuplicate = bDuplicateWrappedObject || InOwner != Wrapper->InstanceObject->GetOuter();
					Wrapper->InstanceObject = CopyNodeInstance(Wrapper->InstanceObject, InOwner, bDuplicate);
				}
			}
		}
	}
}

namespace UE::StateTree
{

#if WITH_EDITORONLY_DATA
	void GatherForLocalization(const FString& PathToParent, const UScriptStruct* Struct, const void* StructData, const void* DefaultStructData, FPropertyLocalizationDataGatherer& PropertyLocalizationDataGatherer, const EPropertyLocalizationGathererTextFlags GatherTextFlags)
	{
		const FStateTreeInstanceData* ThisInstance = static_cast<const FStateTreeInstanceData*>(StructData);
		const FStateTreeInstanceData* DefaultInstance = static_cast<const FStateTreeInstanceData*>(DefaultStructData);

		PropertyLocalizationDataGatherer.GatherLocalizationDataFromStruct(PathToParent, Struct, StructData, DefaultStructData, GatherTextFlags);

		const uint8* DefaultInstanceMemory = nullptr;
		if (DefaultInstance)
		{
			DefaultInstanceMemory = reinterpret_cast<const uint8*>(&DefaultInstance->GetStorage());
		}
		
		const UScriptStruct* StructTypePtr = FStateTreeInstanceStorage::StaticStruct();
		PropertyLocalizationDataGatherer.GatherLocalizationDataFromStructWithCallbacks(PathToParent + TEXT(".InstanceStorage"), StructTypePtr, &ThisInstance->GetStorage(), DefaultInstanceMemory, GatherTextFlags);
	}

	void RegisterInstanceDataForLocalization()
	{
		{ static const FAutoRegisterLocalizationDataGatheringCallback AutomaticRegistrationOfLocalizationGatherer(TBaseStructure<FStateTreeInstanceData>::Get(), &GatherForLocalization); }
	}
#endif // WITH_EDITORONLY_DATA

} // UE::StateTree


//----------------------------------------------------------------//
// FStateTreeInstanceStorage
//----------------------------------------------------------------//

#if WITH_STATETREE_DEBUG
FStateTreeInstanceStorage::FStateTreeInstanceStorage()
	: RuntimeValidationData(MakePimpl<UE::StateTree::Debug::FRuntimeValidationInstanceData>())
{
}
#else
FStateTreeInstanceStorage::FStateTreeInstanceStorage() = default;
#endif

namespace UE::StateTree::InstanceData::Private
{
	bool IsActiveInstanceHandleSourceValid(const FStateTreeInstanceStorage& Storage, const FStateTreeExecutionFrame& CurrentFrame, const FStateTreeDataHandle& Handle)
	{
		return CurrentFrame.ActiveInstanceIndexBase.IsValid()
			&& CurrentFrame.ActiveStates.Contains(Handle.GetState())
			&& Storage.IsValidIndex(CurrentFrame.ActiveInstanceIndexBase.Get() + Handle.GetIndex());
	}

	bool IsHandleSourceValid(
		const FStateTreeInstanceStorage& InstanceStorage,
		const FStateTreeExecutionFrame& CurrentFrame,
		const FStateTreeDataHandle& Handle)
	{
		// Checks that the instance data is valid for specific handle types.
		// 
		// The CurrentFrame may not be yet properly initialized, for that reason we need to check
		// that the path to the handle makes sense (it's part of the active states) as well as that
		// we actually have instance data for the handle (index is valid).
		// 
		// The (base) indices can be invalid if the frame/state is not entered yet.
		// For active instance data we need to check that the frame is initialized for a specific state,
		// as well as that the instance data is initialized.

		switch (Handle.GetSource())
		{
		case EStateTreeDataSourceType::None:
			return true;

		case EStateTreeDataSourceType::GlobalInstanceData:
		case EStateTreeDataSourceType::GlobalInstanceDataObject:
			return CurrentFrame.GlobalInstanceIndexBase.IsValid()
				&& InstanceStorage.IsValidIndex(CurrentFrame.GlobalInstanceIndexBase.Get() + Handle.GetIndex());

		case EStateTreeDataSourceType::ActiveInstanceData:
		case EStateTreeDataSourceType::ActiveInstanceDataObject:
		case EStateTreeDataSourceType::StateParameterData:
			return IsActiveInstanceHandleSourceValid(InstanceStorage, CurrentFrame, Handle);
				
		case EStateTreeDataSourceType::SharedInstanceData:
		case EStateTreeDataSourceType::SharedInstanceDataObject:
			return true;

		case EStateTreeDataSourceType::ExecutionRuntimeData:
		case EStateTreeDataSourceType::ExecutionRuntimeDataObject:
		case EStateTreeDataSourceType::ExecutionRuntimeDataAny:
		{
			const int32 ActiveIndex = CurrentFrame.ExecutionRuntimeIndexBase.Get() + Handle.GetIndex();
			return Handle.HasValidIndex()
				&& CurrentFrame.ExecutionRuntimeIndexBase.IsValid() 
				&& InstanceStorage.GetExecutionRuntimeData().IsValidIndex(ActiveIndex);
		}

		case EStateTreeDataSourceType::GlobalParameterData:
		{
			// Defined as State Parameter in one of its parent frame
			if (CurrentFrame.GlobalParameterDataHandle.HasValidIndex())
			{
				check(CurrentFrame.GlobalParameterDataHandle.GetSource() == EStateTreeDataSourceType::StateParameterData);

				if (const FStateTreeExecutionFrame* ActiveGlobalParameterFrame = InstanceStorage.GetExecutionState().FindActiveFrame(CurrentFrame.GlobalParameterDataFrameID))
				{
					check(ActiveGlobalParameterFrame->ActiveInstanceIndexBase == CurrentFrame.GlobalParameterIndexBase);

					return IsActiveInstanceHandleSourceValid(InstanceStorage, *ActiveGlobalParameterFrame, CurrentFrame.GlobalParameterDataHandle);
				}

				// Parent Frame is temporary, the state parameter can't be active.
				return false;
			}

			// Root Global Parameter should always be valid.
			check(CurrentFrame.GlobalParameterDataHandle.GetSource() == EStateTreeDataSourceType::GlobalParameterData);
			return true;
		}

		case EStateTreeDataSourceType::SubtreeParameterData:
		{
			if (const FStateTreeExecutionFrame* ActiveParameterFrame = InstanceStorage.GetExecutionState().FindActiveFrame(CurrentFrame.StateParameterDataFrameID))
			{
				check(ActiveParameterFrame->ActiveInstanceIndexBase == CurrentFrame.StateParameterIndexBase);

				return IsActiveInstanceHandleSourceValid(InstanceStorage, *ActiveParameterFrame, CurrentFrame.StateParameterDataHandle);
			}

			// Frame hosting the parameter is not active
			return false;
		}

		default:
			checkf(false, TEXT("Unhandled case or unsupported type for InstanceDataStorage %s"), *UEnum::GetValueAsString(Handle.GetSource()));
		}

		return false;
	}

	FStateTreeDataView GetTemporaryDataView(
		FStateTreeInstanceStorage& InstanceStorage,
		const FStateTreeExecutionFrame& CurrentFrame,
		const FStateTreeDataHandle& Handle)
	{
		switch (Handle.GetSource())
		{
		case EStateTreeDataSourceType::GlobalInstanceData:
			return InstanceStorage.GetMutableTemporaryStruct(CurrentFrame.GlobalInstanceDataFrameID, Handle);
		case EStateTreeDataSourceType::ActiveInstanceData:
			return InstanceStorage.GetMutableTemporaryStruct(CurrentFrame.FrameID, Handle);
			
		case EStateTreeDataSourceType::GlobalInstanceDataObject:
			return InstanceStorage.GetMutableTemporaryObject(CurrentFrame.GlobalInstanceDataFrameID, Handle);
		case EStateTreeDataSourceType::ActiveInstanceDataObject:
			return InstanceStorage.GetMutableTemporaryObject(CurrentFrame.FrameID, Handle);

		case EStateTreeDataSourceType::GlobalParameterData:
		{
			// Defined as State Parameter in one of its parent frame
			if (CurrentFrame.GlobalParameterDataHandle.HasValidIndex())
			{
				check(CurrentFrame.GlobalParameterDataHandle.GetSource() == EStateTreeDataSourceType::StateParameterData);

				if (FCompactStateTreeParameters* Params = InstanceStorage.GetMutableTemporaryStruct(CurrentFrame.GlobalParameterDataFrameID, CurrentFrame.GlobalParameterDataHandle).GetPtr<FCompactStateTreeParameters>())
				{
					return Params->GetMutableValue();
				}
			}

			// Root global parameter is never stored in temporary
			break;
		}

		case EStateTreeDataSourceType::SubtreeParameterData:
		{
			check(Handle == CurrentFrame.StateTree->GetStateFromHandle(CurrentFrame.RootState)->ParameterDataHandle);

			// A subtree could be either linked or standalone.
			check((CurrentFrame.StateParameterDataHandle.GetSource() == EStateTreeDataSourceType::SubtreeParameterData 
					&& CurrentFrame.FrameID == CurrentFrame.StateParameterDataFrameID
					&& CurrentFrame.StateParameterDataHandle == Handle)
				|| (CurrentFrame.StateParameterDataHandle.GetSource() == EStateTreeDataSourceType::StateParameterData 
					&& CurrentFrame.FrameID != CurrentFrame.StateParameterDataFrameID));

			if (FCompactStateTreeParameters* Params = InstanceStorage.GetMutableTemporaryStruct(CurrentFrame.StateParameterDataFrameID, CurrentFrame.StateParameterDataHandle).GetPtr<FCompactStateTreeParameters>())
			{
				return Params->GetMutableValue();
			}

			break;
		}

		case EStateTreeDataSourceType::StateParameterData:
		{
			if (FCompactStateTreeParameters* Params = InstanceStorage.GetMutableTemporaryStruct(CurrentFrame.FrameID, Handle).GetPtr<FCompactStateTreeParameters>())
			{
				return Params->GetMutableValue();
			}

			break;
		}

		default:
			checkf(false, TEXT("Unhandled case or unsupported type for InstanceDataStorage %s"), *UEnum::GetValueAsString(Handle.GetSource()));
		}

		return {};
	}

} // namespace

namespace UE::StateTree::InstanceData
{
	FStateTreeDataView GetDataView(
		FStateTreeInstanceStorage& InstanceStorage,
		FStateTreeInstanceStorage* SharedInstanceStorage,
		const FStateTreeExecutionFrame& CurrentFrame,
		const FStateTreeDataHandle& Handle)
	{
		switch (Handle.GetSource())
		{
		case EStateTreeDataSourceType::GlobalInstanceData:
			return InstanceStorage.GetMutableStruct(CurrentFrame.GlobalInstanceIndexBase.Get() + Handle.GetIndex());
		case EStateTreeDataSourceType::GlobalInstanceDataObject:
			return InstanceStorage.GetMutableObject(CurrentFrame.GlobalInstanceIndexBase.Get() + Handle.GetIndex());

		case EStateTreeDataSourceType::ActiveInstanceData:
			return InstanceStorage.GetMutableStruct(CurrentFrame.ActiveInstanceIndexBase.Get() + Handle.GetIndex());
		case EStateTreeDataSourceType::ActiveInstanceDataObject:
			return InstanceStorage.GetMutableObject(CurrentFrame.ActiveInstanceIndexBase.Get() + Handle.GetIndex());

		case EStateTreeDataSourceType::SharedInstanceData:
			check(SharedInstanceStorage);
			return SharedInstanceStorage->GetMutableStruct(Handle.GetIndex());
		case EStateTreeDataSourceType::SharedInstanceDataObject:
			check(SharedInstanceStorage);
			return SharedInstanceStorage->GetMutableObject(Handle.GetIndex());

		case EStateTreeDataSourceType::ExecutionRuntimeData:
		{
			const int32 ActiveIndex = CurrentFrame.ExecutionRuntimeIndexBase.Get() + Handle.GetIndex();
			return InstanceStorage.GetExecutionRuntimeData().GetMutableStruct(ActiveIndex);
		}
		case EStateTreeDataSourceType::ExecutionRuntimeDataObject:
		{
			const int32 ActiveIndex = CurrentFrame.ExecutionRuntimeIndexBase.Get() + Handle.GetIndex();
			return InstanceStorage.GetExecutionRuntimeData().GetMutableObject(ActiveIndex);
		}
		case EStateTreeDataSourceType::ExecutionRuntimeDataAny:
		{
			const int32 ActiveIndex = CurrentFrame.ExecutionRuntimeIndexBase.Get() + Handle.GetIndex();
			UE::StateTree::InstanceData::FInstanceContainer& Container = InstanceStorage.GetExecutionRuntimeData();
			if (Container.IsObject(ActiveIndex))
			{
				return Container.GetMutableObject(ActiveIndex);
			}
			else
			{
				return Container.GetMutableStruct(ActiveIndex);
			}
		}

		case EStateTreeDataSourceType::GlobalParameterData:
		{
			if (CurrentFrame.GlobalParameterDataHandle.HasValidIndex())
			{
				// Defined as State Parameter in one of its parent frame, we must have a valid cached Index base for active instances
				check(CurrentFrame.GlobalParameterIndexBase.IsValid());

				FCompactStateTreeParameters& StateParams = InstanceStorage.GetMutableStruct(CurrentFrame.GlobalParameterIndexBase.Get() + CurrentFrame.GlobalParameterDataHandle.GetIndex()).Get<FCompactStateTreeParameters>();
				return StateParams.GetMutableValue();
			}

			// Root Global Parameters
			return InstanceStorage.GetMutableGlobalParameters();
		}

		case EStateTreeDataSourceType::SubtreeParameterData:
		{
			check(CurrentFrame.StateParameterIndexBase.IsValid() && CurrentFrame.StateParameterDataHandle.HasValidIndex());

			check(Handle == CurrentFrame.StateTree->GetStateFromHandle(CurrentFrame.RootState)->ParameterDataHandle);

			// A subtree could be either linked or standalone.
			check((CurrentFrame.StateParameterDataHandle.GetSource() == EStateTreeDataSourceType::SubtreeParameterData
					&& CurrentFrame.FrameID == CurrentFrame.StateParameterDataFrameID
					&& CurrentFrame.StateParameterDataHandle == Handle)
				|| (CurrentFrame.StateParameterDataHandle.GetSource() == EStateTreeDataSourceType::StateParameterData
					&& CurrentFrame.FrameID != CurrentFrame.StateParameterDataFrameID));

			FCompactStateTreeParameters& StateParams = InstanceStorage.GetMutableStruct(CurrentFrame.StateParameterIndexBase.Get() + CurrentFrame.StateParameterDataHandle.GetIndex()).Get<FCompactStateTreeParameters>();
			return StateParams.GetMutableValue();
		}

		case EStateTreeDataSourceType::StateParameterData:
		{
			FCompactStateTreeParameters& StateParams = InstanceStorage.GetMutableStruct(CurrentFrame.ActiveInstanceIndexBase.Get() + Handle.GetIndex()).Get<FCompactStateTreeParameters>();
			return StateParams.GetMutableValue();
		}

		case EStateTreeDataSourceType::StateEvent:
		{
			// Return FStateTreeEvent from shared event.
			FStateTreeSharedEvent& SharedEvent = InstanceStorage.GetMutableStruct(CurrentFrame.ActiveInstanceIndexBase.Get() + Handle.GetIndex()).Get<FStateTreeSharedEvent>();
			if (ensure(SharedEvent.IsValid()))
			{
				// Events are read only, but we cannot express that in FStateTreeDataView.
				return FStateTreeDataView(FStructView::Make(*SharedEvent.GetMutable()));
			}
			return {};
		}

		default:
			checkf(false, TEXT("Unhandled case or unsupported type for InstanceDataStorage %s"), *UEnum::GetValueAsString(Handle.GetSource()));
		}

		return {};
	}

	FStateTreeDataView GetDataViewOrTemporary(
		FStateTreeInstanceStorage& InstanceStorage,
		FStateTreeInstanceStorage* SharedInstanceStorage,
		const FStateTreeExecutionFrame& CurrentFrame,
		const FStateTreeDataHandle& Handle)
	{
		if (Private::IsHandleSourceValid(InstanceStorage, CurrentFrame, Handle))
		{
			return GetDataView(InstanceStorage, SharedInstanceStorage, CurrentFrame, Handle);
		}

		return Private::GetTemporaryDataView(InstanceStorage, CurrentFrame, Handle);
	}

	bool DoesRequireExecutionContext(EStateTreeDataSourceType SourceType)
	{
		switch (SourceType)
		{
		case EStateTreeDataSourceType::ContextData:
		case EStateTreeDataSourceType::EvaluationScopeInstanceData:
		case EStateTreeDataSourceType::EvaluationScopeInstanceDataObject:
		case EStateTreeDataSourceType::ExternalData:
		case EStateTreeDataSourceType::TransitionEvent:
		case EStateTreeDataSourceType::StateEvent:
		case EStateTreeDataSourceType::ExternalGlobalParameterData:
			return true;
		default:
			return false;
		}
	}

	bool DoesRequireInstanceStorage(EStateTreeDataSourceType SourceType)
	{
		switch (SourceType)
		{
		case EStateTreeDataSourceType::GlobalInstanceData:
		case EStateTreeDataSourceType::GlobalInstanceDataObject:
		case EStateTreeDataSourceType::ActiveInstanceData:
		case EStateTreeDataSourceType::ActiveInstanceDataObject:
		case EStateTreeDataSourceType::SharedInstanceData:
		case EStateTreeDataSourceType::SharedInstanceDataObject:
		case EStateTreeDataSourceType::ExecutionRuntimeData:
		case EStateTreeDataSourceType::ExecutionRuntimeDataObject:
		case EStateTreeDataSourceType::ExecutionRuntimeDataAny:
		case EStateTreeDataSourceType::GlobalParameterData:
		case EStateTreeDataSourceType::SubtreeParameterData:
		case EStateTreeDataSourceType::StateParameterData:
			return true;
		default:
			return false;
		}
	}
} // namespace

FStateTreeInstanceStorage::FStateTreeInstanceStorage(const FStateTreeInstanceStorage& Other)
	: InstanceStructs(Other.InstanceStructs)
	, ExecutionState(Other.ExecutionState)
	, ExecutionRuntimeData(Other.ExecutionRuntimeData)
	, ExecutionRuntimeDataInfos(Other.ExecutionRuntimeDataInfos)
	, TemporaryInstances(Other.TemporaryInstances)
	, EventQueue(MakeShared<FStateTreeEventQueue>(*Other.EventQueue))
	, TransitionRequests(Other.TransitionRequests)
	, GlobalParameters(Other.GlobalParameters)
#if ENABLE_MT_DETECTOR
	, AccessDetector(Other.AccessDetector)
#endif
	, bIsOwningEventQueue(true)
#if WITH_STATETREE_DEBUG
	, RuntimeValidationData(MakePimpl<UE::StateTree::Debug::FRuntimeValidationInstanceData>(*Other.RuntimeValidationData))
#endif
{
}

FStateTreeInstanceStorage::FStateTreeInstanceStorage(FStateTreeInstanceStorage&& Other) noexcept
	: InstanceStructs(MoveTemp(Other.InstanceStructs))
	, ExecutionState(MoveTemp(Other.ExecutionState))
	, ExecutionRuntimeData(MoveTemp(Other.ExecutionRuntimeData))
	, ExecutionRuntimeDataInfos(MoveTemp(Other.ExecutionRuntimeDataInfos))
	, TemporaryInstances(MoveTemp(Other.TemporaryInstances))
	, EventQueue(Other.EventQueue)
	, TransitionRequests(MoveTemp(Other.TransitionRequests))
	, GlobalParameters(MoveTemp(Other.GlobalParameters))
#if ENABLE_MT_DETECTOR
	, AccessDetector(MoveTemp(Other.AccessDetector))
#endif
	, bIsOwningEventQueue(Other.bIsOwningEventQueue)
#if WITH_STATETREE_DEBUG
	, RuntimeValidationData(MoveTemp(Other.RuntimeValidationData))
#endif
{
	Other.EventQueue = MakeShared<FStateTreeEventQueue>();
#if WITH_STATETREE_DEBUG
	Other.RuntimeValidationData = MakePimpl<UE::StateTree::Debug::FRuntimeValidationInstanceData>();
#endif
}

FStateTreeInstanceStorage& FStateTreeInstanceStorage::operator=(const FStateTreeInstanceStorage& Other)
{
	InstanceStructs = Other.InstanceStructs;
	ExecutionState = Other.ExecutionState;
	ExecutionRuntimeData = Other.ExecutionRuntimeData;
	ExecutionRuntimeDataInfos = Other.ExecutionRuntimeDataInfos;
	TemporaryInstances = Other.TemporaryInstances;
	EventQueue = MakeShared<FStateTreeEventQueue>(*Other.EventQueue);
	TransitionRequests = Other.TransitionRequests;
	GlobalParameters = Other.GlobalParameters;

#if ENABLE_MT_DETECTOR
	AccessDetector = Other.AccessDetector;
#endif

	bIsOwningEventQueue = true;

#if WITH_STATETREE_DEBUG
	RuntimeValidationData = MakePimpl<UE::StateTree::Debug::FRuntimeValidationInstanceData>(*Other.RuntimeValidationData);
#endif

	return *this;
}

FStateTreeInstanceStorage& FStateTreeInstanceStorage::operator=(FStateTreeInstanceStorage&& Other) noexcept
{
	InstanceStructs = MoveTemp(Other.InstanceStructs);
	ExecutionState = MoveTemp(Other.ExecutionState);
	ExecutionRuntimeData = MoveTemp(Other.ExecutionRuntimeData);
	ExecutionRuntimeDataInfos = MoveTemp(Other.ExecutionRuntimeDataInfos);
	TemporaryInstances = MoveTemp(Other.TemporaryInstances);
	EventQueue = Other.EventQueue;
	Other.EventQueue = MakeShared<FStateTreeEventQueue>();
	TransitionRequests = MoveTemp(Other.TransitionRequests);
	GlobalParameters = MoveTemp(Other.GlobalParameters);

#if ENABLE_MT_DETECTOR
	AccessDetector = MoveTemp(Other.AccessDetector);
#endif

	bIsOwningEventQueue = Other.bIsOwningEventQueue;
	Other.bIsOwningEventQueue = true;

#if WITH_STATETREE_DEBUG
	RuntimeValidationData = MoveTemp(Other.RuntimeValidationData);
	Other.RuntimeValidationData = MakePimpl<UE::StateTree::Debug::FRuntimeValidationInstanceData>();
#endif

	return *this;
}

void FStateTreeInstanceStorage::SetSharedEventQueue(const TSharedRef<FStateTreeEventQueue>& InSharedEventQueue)
{
	EventQueue = InSharedEventQueue;
	bIsOwningEventQueue = false;
}

void FStateTreeInstanceStorage::AddTransitionRequest(const UObject* Owner, const FStateTreeTransitionRequest& Request)
{
	constexpr int32 MaxPendingTransitionRequests = 32;
	
	if (TransitionRequests.Num() >= MaxPendingTransitionRequests)
	{
		UE_VLOG_UELOG(Owner, LogStateTree, Error, TEXT("%s: Too many transition requests sent to '%s' (%d pending). Dropping request."), ANSI_TO_TCHAR(__FUNCTION__), *GetNameSafe(Owner), TransitionRequests.Num());
		return;
	}

	TransitionRequests.Add(Request);
}

void FStateTreeInstanceStorage::MarkDelegateAsBroadcasted(const FStateTreeDelegateDispatcher& Dispatcher)
{
	// The array is reset once the transitions are processed.
	BroadcastedDelegates.AddUnique(Dispatcher);
}

bool FStateTreeInstanceStorage::IsDelegateBroadcasted(const FStateTreeDelegateDispatcher& Dispatcher) const
{
	return BroadcastedDelegates.Contains(Dispatcher);
}

TArray<FStateTreeDelegateDispatcher> FStateTreeInstanceStorage::StealBroadcastedDelegates()
{
	return MoveTemp(BroadcastedDelegates);
}

bool FStateTreeInstanceStorage::HasBroadcastedDelegates() const
{
	return !BroadcastedDelegates.IsEmpty();
}

void FStateTreeInstanceStorage::ResetTransitionRequests()
{
	TransitionRequests.Reset();
}

bool FStateTreeInstanceStorage::AreAllInstancesValid() const
{
	return UE::StateTree::InstanceData::Private::AreAllInstancesValid(InstanceStructs)
		&& ExecutionRuntimeData.AreAllInstancesValid();
}

int32 FStateTreeInstanceStorage::AddExecutionRuntimeData(TNotNull<UObject*> Owner, const UE::StateTree::FExecutionFrameHandle FrameHandle)
{
	using namespace UE::StateTree;
	using namespace UE::StateTree::InstanceData;

	check(FrameHandle.IsValid());

	const FObjectKey StateTreeKey = FrameHandle.GetStateTree();
	const FExecutionRuntimeInfo* Info = ExecutionRuntimeDataInfos.FindByPredicate([StateTreeKey](const FExecutionRuntimeInfo& Other)
		{
			return Other.StateTree == StateTreeKey;
		});
	if (Info != nullptr)
	{
		return Info->StartIndex;
	}

	const int32 StartIndex = ExecutionRuntimeData.Append(Owner, FrameHandle.GetStateTree()->GetDefaultExecutionRuntimeData(), FInstanceContainer::FAddArgs());

	FExecutionRuntimeInfo& NewInfo = ExecutionRuntimeDataInfos.AddDefaulted_GetRef();
	NewInfo.StateTree = StateTreeKey;
	NewInfo.StartIndex = StartIndex;

	return StartIndex;
}

FStructView FStateTreeInstanceStorage::AddTemporaryInstance(UObject& InOwner, const FStateTreeExecutionFrame& Frame, const FStateTreeIndex16 OwnerNodeIndex, const FStateTreeDataHandle DataHandle, FConstStructView NewInstanceData)
{
	FStateTreeTemporaryInstanceData* TempInstance = TemporaryInstances.FindByPredicate([&Frame, &OwnerNodeIndex, &DataHandle](const FStateTreeTemporaryInstanceData& TempInstance)
	{
		return TempInstance.FrameID == Frame.FrameID
				&& TempInstance.OwnerNodeIndex == OwnerNodeIndex
				&& TempInstance.DataHandle == DataHandle;
	});
	
	if (TempInstance)
	{
		if (TempInstance->Instance.GetScriptStruct() != NewInstanceData.GetScriptStruct())
		{
			TempInstance->Instance = NewInstanceData;
		}
	}
	else
	{
		TempInstance = &TemporaryInstances.AddDefaulted_GetRef();
		check(TempInstance);
		TempInstance->FrameID = Frame.FrameID;
		TempInstance->OwnerNodeIndex = OwnerNodeIndex;
		TempInstance->DataHandle = DataHandle;
		TempInstance->Instance = NewInstanceData;
	}

	if (FStateTreeInstanceObjectWrapper* Wrapper = TempInstance->Instance.GetMutablePtr<FStateTreeInstanceObjectWrapper>())
	{
		if (Wrapper->InstanceObject)
		{
			constexpr bool bDuplicate = true;
			Wrapper->InstanceObject = UE::StateTree::InstanceData::Private::CopyNodeInstance(Wrapper->InstanceObject, &InOwner, bDuplicate);
		}
	}

	return TempInstance->Instance;
}

void FStateTreeInstanceStorage::RemoveTemporaryInstance(const UE::StateTree::FActiveFrameID FrameID, const FStateTreeIndex16 OwnerNodeIndex, const FStateTreeDataHandle DataHandle)
{
	const int32 Index = TemporaryInstances.IndexOfByPredicate([&FrameID, &OwnerNodeIndex, &DataHandle](const FStateTreeTemporaryInstanceData& TempInstance)
				{
					return TempInstance.FrameID == FrameID
						&& TempInstance.OwnerNodeIndex == OwnerNodeIndex
						&& TempInstance.DataHandle == DataHandle;
		});

	if (ensure(Index != INDEX_NONE))
	{
		TemporaryInstances.RemoveAtSwap(Index);
	}
}

FStructView FStateTreeInstanceStorage::GetMutableTemporaryStruct(UE::StateTree::FActiveFrameID FrameID, const FStateTreeDataHandle DataHandle)
{
	FStateTreeTemporaryInstanceData* ExistingInstance = TemporaryInstances.FindByPredicate([FrameID, &DataHandle](const FStateTreeTemporaryInstanceData& TempInstance)
		{
			return TempInstance.FrameID == FrameID
				&& TempInstance.DataHandle == DataHandle;
		});
	return ExistingInstance ? FStructView(ExistingInstance->Instance) : FStructView();
}

UObject* FStateTreeInstanceStorage::GetMutableTemporaryObject(UE::StateTree::FActiveFrameID FrameID, const FStateTreeDataHandle DataHandle)
{
	FStateTreeTemporaryInstanceData* ExistingInstance = TemporaryInstances.FindByPredicate([FrameID, &DataHandle](const FStateTreeTemporaryInstanceData& TempInstance)
		{
			return TempInstance.FrameID == FrameID
				&& TempInstance.DataHandle == DataHandle;
		});
	if (ExistingInstance)
	{
		const FStateTreeInstanceObjectWrapper& Wrapper = ExistingInstance->Instance.Get<FStateTreeInstanceObjectWrapper>();
		return Wrapper.InstanceObject;
	}
	return nullptr;
}

void FStateTreeInstanceStorage::ResetTemporaryInstances()
{
	TemporaryInstances.Reset();
}

// Deprecated
void FStateTreeInstanceStorage::SetGlobalParameters(const FInstancedPropertyBag& Parameters)
{
	SetGlobalParameters(Parameters.GetValue());
}

void FStateTreeInstanceStorage::SetGlobalParameters(const FConstStructView Parameters)
{
	GlobalParameters = Parameters;
}

uint32 FStateTreeInstanceStorage::GenerateUniqueId()
{
	uint32 NewId = ++UniqueIdGenerator;
	if (NewId == 0)
	{
#if WITH_STATETREE_TRACE && DO_ENSURE
		ensureAlwaysMsgf(false, TEXT("The unique id overflow. Id:%d Serial:%d"), ExecutionState.InstanceDebugId.Id, ExecutionState.InstanceDebugId.SerialNumber);
#elif WITH_STATETREE_TRACE
	UE_LOGF(LogStateTree, Error, "The unique id overflow. Id:%d Serial:%d", ExecutionState.InstanceDebugId.Id, ExecutionState.InstanceDebugId.SerialNumber);
#else
	UE_LOGF(LogStateTree, Error, "The unique id overflow.");
#endif
		NewId = ++UniqueIdGenerator;
	}
	return NewId;
}

void FStateTreeInstanceStorage::AddStructReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddPropertyReferencesWithStructARO(TBaseStructure<FStateTreeInstanceStorage>::Get(), this);
	Collector.AddPropertyReferencesWithStructARO(TBaseStructure<FStateTreeEventQueue>::Get(), &EventQueue.Get());
}

void FStateTreeInstanceStorage::Reset()
{
	InstanceStructs.Reset();
	ExecutionState.Reset();
	ExecutionRuntimeData.Reset();
	ExecutionRuntimeDataInfos.Reset();
	TemporaryInstances.Reset();
	if (bIsOwningEventQueue)
	{
		EventQueue->Reset();
	}
	TransitionRequests.Reset();
	GlobalParameters.Reset();

#if WITH_STATETREE_DEBUG
	RuntimeValidationData = MakePimpl<UE::StateTree::Debug::FRuntimeValidationInstanceData>();
#endif
}

// #jira SOL-8070: Ideally, we should use the transactionally-safe access detector instead of relying on OPEN and ONABORT blocks here.
void FStateTreeInstanceStorage::AcquireReadAccess()
{
	UE_AUTORTFM_OPEN
	{
		UE_MT_ACQUIRE_READ_ACCESS(AccessDetector);
	};
	UE_AUTORTFM_ONABORT(this)
	{
		UE_MT_RELEASE_READ_ACCESS(AccessDetector);
	};
}

void FStateTreeInstanceStorage::ReleaseReadAccess()
{
	UE_AUTORTFM_OPEN
	{
		UE_MT_RELEASE_READ_ACCESS(AccessDetector);
	};
	UE_AUTORTFM_ONABORT(this)
	{
		UE_MT_ACQUIRE_READ_ACCESS(AccessDetector);
	};
}

void FStateTreeInstanceStorage::AcquireWriteAccess()
{
	UE_AUTORTFM_OPEN
	{
		UE_MT_ACQUIRE_WRITE_ACCESS(AccessDetector);
	};
	UE_AUTORTFM_ONABORT(this)
	{
		UE_MT_RELEASE_WRITE_ACCESS(AccessDetector);
	};
}

void FStateTreeInstanceStorage::ReleaseWriteAccess()
{
	UE_AUTORTFM_OPEN
	{
		UE_MT_RELEASE_WRITE_ACCESS(AccessDetector);
	};
	UE_AUTORTFM_ONABORT(this)
	{
		UE_MT_ACQUIRE_WRITE_ACCESS(AccessDetector);
	};
}

UE::StateTree::Debug::FRuntimeValidation FStateTreeInstanceStorage::GetRuntimeValidation() const
{
#if WITH_STATETREE_DEBUG
	return UE::StateTree::Debug::FRuntimeValidation(RuntimeValidationData.Get());
#else
	return UE::StateTree::Debug::FRuntimeValidation();
#endif
}

//----------------------------------------------------------------//
// FStateTreeInstanceData
//----------------------------------------------------------------//

FStateTreeInstanceData::FAddArgs FStateTreeInstanceData::FAddArgs::Default;

FStateTreeInstanceData::FStateTreeInstanceData() = default;

FStateTreeInstanceData::FStateTreeInstanceData(const FStateTreeInstanceData& Other)
{
	InstanceStorage = MakeShared<FStateTreeInstanceStorage>(*Other.InstanceStorage);
}

FStateTreeInstanceData::FStateTreeInstanceData(FStateTreeInstanceData&& Other) noexcept
{
	InstanceStorage = Other.InstanceStorage;
	Other.InstanceStorage = MakeShared<FStateTreeInstanceStorage>();
}

FStateTreeInstanceData& FStateTreeInstanceData::operator=(const FStateTreeInstanceData& Other)
{
	InstanceStorage = MakeShared<FStateTreeInstanceStorage>(*Other.InstanceStorage);
	return *this;
}

FStateTreeInstanceData& FStateTreeInstanceData::operator=(FStateTreeInstanceData&& Other) noexcept
{
	InstanceStorage = Other.InstanceStorage;
	Other.InstanceStorage = MakeShared<FStateTreeInstanceStorage>();
	return *this;
}

FStateTreeInstanceData::~FStateTreeInstanceData()
{
	Reset();
}

const FStateTreeInstanceStorage& FStateTreeInstanceData::GetStorage() const
{
	return *InstanceStorage;
}

TWeakPtr<FStateTreeInstanceStorage> FStateTreeInstanceData::GetWeakMutableStorage()
{
	return InstanceStorage;
}

TWeakPtr<const FStateTreeInstanceStorage> FStateTreeInstanceData::GetWeakStorage() const
{
	return InstanceStorage;
}

FStateTreeInstanceStorage& FStateTreeInstanceData::GetMutableStorage()
{
	return *InstanceStorage;
}

FStateTreeEventQueue& FStateTreeInstanceData::GetMutableEventQueue()
{
	return GetMutableStorage().GetMutableEventQueue();
}

const TSharedRef<FStateTreeEventQueue>& FStateTreeInstanceData::GetSharedMutableEventQueue()
{
	return GetMutableStorage().GetSharedMutableEventQueue();
}

const FStateTreeEventQueue& FStateTreeInstanceData::GetEventQueue() const
{
	return GetStorage().GetEventQueue();
}

bool FStateTreeInstanceData::IsOwningEventQueue() const
{
	return GetStorage().IsOwningEventQueue();
}

void FStateTreeInstanceData::SetSharedEventQueue(const TSharedRef<FStateTreeEventQueue>& InSharedEventQueue)
{
	return GetMutableStorage().SetSharedEventQueue(InSharedEventQueue);
}

void FStateTreeInstanceData::AddTransitionRequest(const UObject* Owner, const FStateTreeTransitionRequest& Request)
{
	GetMutableStorage().AddTransitionRequest(Owner, Request);
}

TConstArrayView<FStateTreeTransitionRequest> FStateTreeInstanceData::GetTransitionRequests() const
{
	return GetStorage().GetTransitionRequests();
}

void FStateTreeInstanceData::ResetTransitionRequests()
{
	GetMutableStorage().ResetTransitionRequests();
}

bool FStateTreeInstanceData::AreAllInstancesValid() const
{
	return GetStorage().AreAllInstancesValid();
}

int32 FStateTreeInstanceData::GetEstimatedMemoryUsage() const
{
	const FStateTreeInstanceStorage& Storage = GetStorage();
	int32 Size = sizeof(FStateTreeInstanceData);

	Size += UE::StateTree::InstanceData::Private::GetAllocatedMemory(Storage.InstanceStructs);
	Size += Storage.ExecutionRuntimeData.GetAllocatedMemory();

	return Size;
}

bool FStateTreeInstanceData::Identical(const FStateTreeInstanceData* Other, uint32 PortFlags) const
{
	if (Other == nullptr)
	{
		return false;
	}

	const FStateTreeInstanceStorage& Storage = GetStorage();
	const FStateTreeInstanceStorage& OtherStorage = Other->GetStorage();

	// Not identical if global parameters don't match.
	if (!Storage.GlobalParameters.Identical(&OtherStorage.GlobalParameters, PortFlags))
	{
		return false;
	}

	// Not identical if structs are different.
	if (Storage.InstanceStructs.Identical(&OtherStorage.InstanceStructs, PortFlags) == false)
	{
		return false;
	}
	
	// Check that the instance object contents are identical.
	// Copied from object property.
	auto AreObjectsIdentical = [](UObject* A, UObject* B, uint32 PortFlags) -> bool
	{
		if ((PortFlags & PPF_DuplicateForPIE) != 0)
		{
			return false;
		}

		if (A == B)
		{
			return true;
		}

		// Resolve the object handles and run the deep comparison logic 
		if ((PortFlags & (PPF_DeepCompareInstances | PPF_DeepComparison)) != 0)
		{
			return FObjectPropertyBase::StaticIdentical(A, B, PortFlags);
		}

		return true;
	};

	bool bResult = true;

	for (int32 Index = 0; Index < Storage.InstanceStructs.Num(); Index++)
	{
		const FStateTreeInstanceObjectWrapper* Wrapper = Storage.InstanceStructs[Index].GetPtr<const FStateTreeInstanceObjectWrapper>();
		const FStateTreeInstanceObjectWrapper* OtherWrapper = OtherStorage.InstanceStructs[Index].GetPtr<const FStateTreeInstanceObjectWrapper>();

		if (Wrapper)
		{
			if (!OtherWrapper)
			{
				bResult = false;
				break;
			}
			if (Wrapper->InstanceObject && OtherWrapper->InstanceObject)
			{
				if (!AreObjectsIdentical(Wrapper->InstanceObject, OtherWrapper->InstanceObject, PortFlags))
				{
					bResult = false;
					break;
				}
			}
		}
	}
	
	return bResult;
}

void FStateTreeInstanceData::AddStructReferencedObjects(FReferenceCollector& Collector)
{
	GetMutableStorage().AddStructReferencedObjects(Collector);
}

bool FStateTreeInstanceData::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FStateTreeInstanceStorageCustomVersion::GUID);

	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FStateTreeInstanceStorageCustomVersion::GUID) < FStateTreeInstanceStorageCustomVersion::AddedCustomSerialization)
		{
#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			StaticStruct()->SerializeTaggedProperties(Ar, (uint8*)this, StaticStruct(), nullptr);

			if (InstanceStorage_DEPRECATED.IsValid())
			{
				InstanceStorage = MakeShared<FStateTreeInstanceStorage>(MoveTemp(InstanceStorage_DEPRECATED.GetMutable()));
				InstanceStorage_DEPRECATED.Reset();
				return true;
			}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA

			InstanceStorage = MakeShared<FStateTreeInstanceStorage>();
			return true;
		}

		InstanceStorage = MakeShared<FStateTreeInstanceStorage>();
	}

	FStateTreeInstanceStorage::StaticStruct()->SerializeItem(Ar, &InstanceStorage.Get(), nullptr);

	return true;
}

void FStateTreeInstanceData::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	UScriptStruct* ScriptStruct = FStateTreeInstanceStorage::StaticStruct();
	OutDeps.Add(ScriptStruct);

	if (UScriptStruct::ICppStructOps* CppStructOps = ScriptStruct->GetCppStructOps())
	{
		CppStructOps->GetPreloadDependencies(&GetMutableStorage(), OutDeps);
	}

	for (TPropertyValueIterator<FStructProperty> It(ScriptStruct, &GetMutableStorage()); It; ++It)
	{
		const UScriptStruct* StructType = It.Key()->Struct;
		if (UScriptStruct::ICppStructOps* CppStructOps = StructType->GetCppStructOps())
		{
			void* StructDataPtr = const_cast<void*>(It.Value());
			CppStructOps->GetPreloadDependencies(StructDataPtr, OutDeps);
		}
	}
}

void FStateTreeInstanceData::CopyFrom(UObject& InOwner, const FStateTreeInstanceData& InOther)
{
	if (&InOther == this)
	{
		return;
	}

	FStateTreeInstanceStorage& Storage = GetMutableStorage();
	const FStateTreeInstanceStorage& OtherStorage = InOther.GetStorage();

	// Copy structs
	Storage.InstanceStructs = OtherStorage.InstanceStructs;

	// Copy instance objects.
	for (FStructView Instance : Storage.InstanceStructs)
	{
		if (FStateTreeInstanceObjectWrapper* Wrapper = Instance.GetPtr<FStateTreeInstanceObjectWrapper>())
		{
			if (Wrapper->InstanceObject)
			{
				constexpr bool bDuplicate = true;
				Wrapper->InstanceObject = UE::StateTree::InstanceData::Private::CopyNodeInstance(Wrapper->InstanceObject, &InOwner, bDuplicate);
			}
		}
	}
}

void FStateTreeInstanceData::Init(UObject& InOwner, TConstArrayView<FInstancedStruct> InStructs, FAddArgs Args)
{
	Reset();
	Append(InOwner, InStructs, Args);
}

void FStateTreeInstanceData::Init(UObject& InOwner, TConstArrayView<FConstStructView> InStructs, FAddArgs Args)
{
	Reset();
	Append(InOwner, InStructs, Args);
}

void FStateTreeInstanceData::Append(UObject& InOwner, TConstArrayView<FInstancedStruct> InStructs, FAddArgs Args)
{
	UE::StateTree::InstanceData::Private::AppendToInstanceStructContainer(GetMutableStorage().InstanceStructs, &InOwner, InStructs, Args.bDuplicateWrappedObject);
}

void FStateTreeInstanceData::Append(UObject& InOwner, TConstArrayView<FConstStructView> InStructs, FAddArgs Args)
{
	UE::StateTree::InstanceData::Private::AppendToInstanceStructContainer(GetMutableStorage().InstanceStructs, &InOwner, InStructs, Args.bDuplicateWrappedObject);
}

void FStateTreeInstanceData::Append(UObject& InOwner, TConstArrayView<FConstStructView> InStructs, TConstArrayView<FInstancedStruct*> InInstancesToMove, FAddArgs Args)
{
	check(InStructs.Num() == InInstancesToMove.Num());

	FStateTreeInstanceStorage& Storage = GetMutableStorage();

	const int32 StartIndex = Storage.InstanceStructs.Num();
	Storage.InstanceStructs.Append(InStructs);

	for (int32 Index = StartIndex; Index < Storage.InstanceStructs.Num(); Index++)
	{
		FStructView Struct = Storage.InstanceStructs[Index];
		FInstancedStruct* Source = InInstancesToMove[Index - StartIndex];

		// The source is used to move temporary instance data into instance data. Not all entries may have it.
		// The instance struct can be empty, in which case the temporary instance is ignored.
		// If the source is specified, move it to the instance data.
		// We assume that if the source is object wrapper, it is already the instance we want.
		if (Struct.IsValid())
		{
			if (Source && Source->IsValid())
			{
				check(Struct.GetScriptStruct() == Source->GetScriptStruct());
				
				FMemory::Memswap(Struct.GetMemory(), Source->GetMutableMemory(), Struct.GetScriptStruct()->GetStructureSize());
				Source->Reset();
			}
			else if (FStateTreeInstanceObjectWrapper* Wrapper = Struct.GetPtr<FStateTreeInstanceObjectWrapper>())
			{
				if (Wrapper->InstanceObject)
				{
					const bool bDuplicate = Args.bDuplicateWrappedObject || &InOwner != Wrapper->InstanceObject->GetOuter();
					Wrapper->InstanceObject = UE::StateTree::InstanceData::Private::CopyNodeInstance(Wrapper->InstanceObject, &InOwner, bDuplicate);
				}
			}
		}
	}
}

void FStateTreeInstanceData::Append(UObject& InOwner, TConstArrayView<FConstStructView> InStructs, TConstArrayView<TOptional<FInstancedStruct*>> InInstancesToMove, FAddArgs Args)
{
	check(InStructs.Num() == InInstancesToMove.Num());

	FStateTreeInstanceStorage& Storage = GetMutableStorage();

	const int32 StartIndex = Storage.InstanceStructs.Num();
	//The Append will not duplicate the UObject.
	Storage.InstanceStructs.Append(InStructs);

	for (int32 Index = StartIndex; Index < Storage.InstanceStructs.Num(); Index++)
	{
		FStructView Struct = Storage.InstanceStructs[Index];
		const TOptional<FInstancedStruct*>& OptionalSource = InInstancesToMove[Index - StartIndex];

		// The source is used to move temporary instance data into instance data. Not all entries may have it.
		// The instance struct can be empty, in which case the temporary instance is ignored.
		// If the source is specified, move it to the instance data.
		// We assume that if the source is object wrapper, it is already the instance we want.
		if (Struct.IsValid() && OptionalSource.IsSet())
		{
			FInstancedStruct* Source = OptionalSource.GetValue();
			if (Source && Source->IsValid())
			{
				check(Struct.GetScriptStruct() == Source->GetScriptStruct());

				FMemory::Memswap(Struct.GetMemory(), Source->GetMutableMemory(), Struct.GetScriptStruct()->GetStructureSize());
				Source->Reset();
			}
			else if (FStateTreeInstanceObjectWrapper* Wrapper = Struct.GetPtr<FStateTreeInstanceObjectWrapper>())
			{
				// The source is explicitly set to nullptr. Duplicate the UObject.
				if (Wrapper->InstanceObject)
				{
					const bool bDuplicate = Args.bDuplicateWrappedObject || &InOwner != Wrapper->InstanceObject->GetOuter();
					Wrapper->InstanceObject = UE::StateTree::InstanceData::Private::CopyNodeInstance(Wrapper->InstanceObject, &InOwner, bDuplicate);
				}
			}
		}
	}
}

void FStateTreeInstanceData::ShrinkTo(const int32 NumStructs)
{
	FStateTreeInstanceStorage& Storage = GetMutableStorage();
	check(NumStructs <= Storage.InstanceStructs.Num());  
	Storage.InstanceStructs.SetNum(NumStructs);
}

void FStateTreeInstanceData::Reset()
{
	GetMutableStorage().Reset();

}
