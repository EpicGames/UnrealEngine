// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeInstanceData.h"

namespace UE::StateTree::InstanceData::Private
{
	/** @return whether the handle is valid when it's an instance data type. */
	[[nodiscard]] bool IsActiveInstanceHandleSourceValid(
		const FStateTreeInstanceStorage& Storage,
		const FStateTreeExecutionFrame& CurrentFrame,
		const FStateTreeDataHandle& Handle);

	/** @return whether the handle is valid relative to the given frame. */
	[[nodiscard]] bool IsHandleSourceValid(
		const FStateTreeInstanceStorage& InstanceStorage,
		const FStateTreeExecutionFrame& CurrentFrame,
		const FStateTreeDataHandle& Handle);

	/** @return data view of the specified handle in temporary instance. */
	[[nodiscard]] FStateTreeDataView GetTemporaryDataView(
		FStateTreeInstanceStorage& InstanceStorage,
		const FStateTreeExecutionFrame& CurrentFrame,
		const FStateTreeDataHandle& Handle);

	/** @return true if all instances are valid. */
	[[nodiscard]] bool AreAllInstancesValid(const FInstancedStructContainer& InstanceStructs);

	/** Returns number of bytes allocated for the array  */
	[[nodiscard]] int32 GetAllocatedMemory(const FInstancedStructContainer& InstanceStructs);

	/** Duplicates object, and tries to covert old BP classes (REINST_*) to their newer version. */
	[[nodiscard]] TNotNull<UObject*> CopyNodeInstance(TNotNull<UObject*> Instance, TNotNull<UObject*> InOwner, bool bDuplicate);

	/** None generic code for AppendToInstanceStructContainer. Called after AppendToInstanceStructContainer to move code to cpp. */
	void PostAppendToInstanceStructContainer(FInstancedStructContainer& InstanceStructs, TNotNull<UObject*> InOwner, bool bDuplicateWrappedObject, int32 StartIndex);

	/** Appends new items to the instance, and moves existing data into the allocated instances. */
	template<typename TOtherType>
	int32 AppendToInstanceStructContainer(FInstancedStructContainer& InstanceStructs, TNotNull<UObject*> InOwner, TConstArrayView<TOtherType> InStructs, bool bDuplicateWrappedObject)
	{
		const int32 StartIndex = InstanceStructs.Num();
		InstanceStructs.Append(InStructs);
		PostAppendToInstanceStructContainer(InstanceStructs, InOwner, bDuplicateWrappedObject, StartIndex);
		return StartIndex;
	}
} // namespace
