// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeLinker.h"
#include "StateTree.h"
#include "StateTreeSchema.h"
#include "Templates/Casts.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeLinker)

FStateTreeLinker::FStateTreeLinker(TNotNull<const UStateTree*> InStateTree)
	: StateTree(InStateTree)
	, Schema(InStateTree->GetSchema())
{
}

void FStateTreeLinker::LinkExternalData(FStateTreeExternalDataHandle& Handle, const UStruct* Struct, const EStateTreeExternalDataRequirement Requirement)
{
	if (Schema != nullptr && !Schema->IsExternalItemAllowed(*Struct))
	{
		UE_LOGF(LogStateTree, Error,
			"External data of type '%ls' used by current node is not allowed by schema '%ls' (i.e. rejected by IsExternalItemAllowed)",
			*Struct->GetName(),
			*Schema->GetClass()->GetName());

		Handle = FStateTreeExternalDataHandle();
		Status = EStateTreeLinkerStatus::Failed;
		return;
	}

	const FStateTreeExternalDataDesc Desc(Struct, Requirement);
	int32 Index = ExternalDataDescs.Find(Desc);

	if (Index == INDEX_NONE)
	{
		Index = ExternalDataDescs.Add(Desc);
		ExternalDataDescs[Index].Handle.DataHandle = FStateTreeDataHandle(EStateTreeDataSourceType::ExternalData, Index);
	}

	Handle.DataHandle = ExternalDataDescs[Index].Handle.DataHandle;
}