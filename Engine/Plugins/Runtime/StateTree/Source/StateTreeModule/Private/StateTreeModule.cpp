// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeModule.h"
#include "StateTreeModuleImpl.h"

#include "StateTreeTypes.h"
#include "CrashReporter/StateTreeCrashReporterHandler.h"

#if WITH_STATETREE_TRACE
#include "Debugger/StateTreeTrace.h"
#include "Debugger/StateTreeTraceTypes.h"
#include "HAL/IConsoleManager.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "StateTreeDelegates.h"
#include "StateTreeSettings.h"
#endif // WITH_STATETREE_TRACE

#if WITH_STATETREE_TRACE_DEBUGGER
#include "Debugger/StateTreeDebuggerTypes.h"
#include "Debugger/StateTreeTraceModule.h"
#include "Features/IModularFeatures.h"
#include "Trace/StoreClient.h"
#include "TraceServices/AnalysisService.h"
#include "TraceServices/ITraceServicesModule.h"
#endif // WITH_STATETREE_TRACE_DEBUGGER

#if WITH_EDITORONLY_DATA
#include "StateTreeInstanceData.h"
#endif // WITH_EDITORONLY_DATA

#if WITH_STATETREE_DEBUG
#include "UObject/UObjectGlobals.h"
#endif

#define LOCTEXT_NAMESPACE "StateTree"

#if WITH_STATETREE_DEBUG
FTSSimpleMulticastDelegate FStateTreeModule::OnPreRuntimeValidationInstanceData;
FTSSimpleMulticastDelegate FStateTreeModule::OnPostRuntimeValidationInstanceData;
#endif

namespace UE::StateTree::Debug::Private
{
#if WITH_STATETREE_DEBUG
	bool bRuntimeValidationInstanceDataGC = false;
	static FAutoConsoleVariableRef CVarRuntimeValidationInstanceDataGC(
		TEXT("StateTree.RuntimeValidation.InstanceDataGC"),
		bRuntimeValidationInstanceDataGC,
		TEXT("Test after each GC if nodes were properly GCed.")
	);
#endif //WITH_STATETREE_DEBUG

#if WITH_STATETREE_TRACE_DEBUGGER
	class FStateTreeDebugInfoProvider : public IStateTreeDebugInfoProvider
	{
		virtual FText GetNodeDescription(TNotNull<const UStateTree*> StateTree, FStateTreeIndex16 NodeIndex) const override
		{
			const FStateTreeNodeBase* Node = StateTree->GetNode(NodeIndex.Get()).GetPtr<FStateTreeNodeBase>();
			return FText::FromString(FString::Printf(TEXT("%s"), Node ? *Node->Name.ToString() : *("Invalid Node Index: " + LexToString(NodeIndex.Get()))));
		}
	};
#endif // WITH_STATETREE_TRACE_DEBUGGER

} // namespace UE::StateTree::Debug::Private

#if WITH_STATETREE_TRACE_DEBUGGER
UE::Trace::FStoreClient* FStateTreeModule::GetStoreClient()
{
	if (!StoreClient.IsValid())
	{
		StoreClient = UE::Trace::FStoreClient::Create(TEXT("localhost"));
	}
	return StoreClient.Get();
}

TNotNull<const IStateTreeDebugInfoProvider*> FStateTreeModule::GetDebugInfoProvider()
{
	if (!DebugInfoProvider)
	{
		DebugInfoProvider = MakeShared<UE::StateTree::Debug::Private::FStateTreeDebugInfoProvider>();
	}

	return DebugInfoProvider.Get();
}

void FStateTreeModule::SetDebugInfoProvider(TSharedPtr<IStateTreeDebugInfoProvider> InDebugInfoProvider)
{
	DebugInfoProvider = InDebugInfoProvider;
}

#endif // WITH_STATETREE_TRACE_DEBUGGER

FStateTreeModule::FStateTreeModule()
#if WITH_STATETREE_TRACE
	: StartDebuggerTracesCommand(FAutoConsoleCommand(
		TEXT("statetree.startdebuggertraces"),
		TEXT("Turns on StateTree debugger traces if not already active."),
		FConsoleCommandDelegate::CreateLambda([]
			{
				int32 TraceId = 0;
				IStateTreeModule::Get().StartTraces(TraceId);
			})))
	, StopDebuggerTracesCommand(FAutoConsoleCommand(
		TEXT("statetree.stopdebuggertraces"),
		TEXT("Turns off StateTree debugger traces if active."),
		FConsoleCommandDelegate::CreateLambda([]
			{
				IStateTreeModule::Get().StopTraces();
			})))
#endif // WITH_STATETREE_TRACE
{
}

void FStateTreeModule::StartupModule()
{
#if UE_WITH_STATETREE_CRASHREPORTER
	UE::StateTree::FCrashReporterHandler::Register();
#endif

#if WITH_STATETREE_TRACE_DEBUGGER
	ITraceServicesModule& TraceServicesModule = FModuleManager::LoadModuleChecked<ITraceServicesModule>("TraceServices");
	TraceAnalysisService = TraceServicesModule.GetAnalysisService();
	TraceModuleService = TraceServicesModule.GetModuleService();

	IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &StateTreeTraceModule);
#endif // WITH_STATETREE_TRACE_DEBUGGER

#if WITH_STATETREE_TRACE
	UE::StateTreeTrace::RegisterGlobalDelegates();

#if !WITH_EDITOR
	// We don't automatically start traces for Editor targets since we rely on the debugger
	// to start recording either on user action or on PIE session start.
	if (UStateTreeSettings::Get().bAutoStartDebuggerTracesOnNonEditorTargets)
	{
		int32 TraceId = INDEX_NONE;
		StartTraces(TraceId);
	}
#endif // !WITH_EDITOR

#endif // WITH_STATETREE_TRACE

#if WITH_EDITORONLY_DATA
	UE::StateTree::RegisterInstanceDataForLocalization();
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	UE::PropertyBinding::PropertyBindingIndex16ConversionFuncList.Add([](const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, TNotNull<FPropertyBindingIndex16*> Index)
		{
			static FName FStateTreeIndex16TypeName = FStateTreeIndex16::StaticStruct()->GetFName();
			const FName StructFName = Tag.GetType().GetParameter(0).GetName();
			if (FStateTreeIndex16TypeName == StructFName)
			{
				FStateTreeIndex16 StateTreeIndex16;
				FStateTreeIndex16::StaticStruct()->SerializeItem(Slot, &StateTreeIndex16, /*Defaults*/nullptr);
				*Index = StateTreeIndex16;
				return true;
			}
			return false;
		});
#endif //WITH_EDITOR

#if WITH_STATETREE_DEBUG
	PreGCHandle = FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddRaw(this, &FStateTreeModule::HandlePreGC);
	PostGCHandle = FCoreUObjectDelegates::GetPostPurgeGarbageDelegate().AddRaw(this, &FStateTreeModule::HandlePostGC);
#endif
}

void FStateTreeModule::ShutdownModule()
{
#if WITH_STATETREE_DEBUG
	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().Remove(PreGCHandle);
	FCoreUObjectDelegates::GetPostPurgeGarbageDelegate().Remove(PostGCHandle);
#endif

#if WITH_STATETREE_TRACE
	StopTraces();

	UE::StateTreeTrace::UnregisterGlobalDelegates();
#endif // WITH_STATETREE_TRACE

#if WITH_STATETREE_TRACE_DEBUGGER
	if (StoreClient.IsValid())
	{
		StoreClient.Reset();
	}

	IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &StateTreeTraceModule);

	DebugInfoProvider = nullptr;
#endif // WITH_STATETREE_TRACE_DEBUGGER

#if UE_WITH_STATETREE_CRASHREPORTER
	UE::StateTree::FCrashReporterHandler::Unregister();
#endif


}

bool FStateTreeModule::StartTraces(int32& OutTraceId)
{
	OutTraceId = INDEX_NONE;

#if WITH_STATETREE_TRACE
	if (IsRunningCommandlet() || IsTracing())
	{
		return false;
	}

	FGuid SessionGuid, TraceGuid;
	const bool bAlreadyConnected = FTraceAuxiliary::IsConnected(SessionGuid, TraceGuid);

#if WITH_STATETREE_TRACE_DEBUGGER
	if (const UE::Trace::FStoreClient* Client = GetStoreClient())
	{
		const UE::Trace::FStoreClient::FSessionInfo* SessionInfo = Client->GetSessionInfoByGuid(TraceGuid);
		// Note that 0 is returned instead of INDEX_NONE to match default invalid value for GetTraceId
		OutTraceId = SessionInfo != nullptr ? SessionInfo->GetTraceId() : 0;
	}
#endif // WITH_STATETREE_TRACE_DEBUGGER

	// If trace is already connected let's keep track of enabled channels to restore them when we stop recording
	if (bAlreadyConnected)
	{
		UE::Trace::EnumerateChannels([](const ANSICHAR* Name, const bool bIsEnabled, void* Channels)
		{
			TArray<FString>* EnabledChannels = static_cast<TArray<FString>*>(Channels);
			if (bIsEnabled)
			{
				EnabledChannels->Emplace(ANSI_TO_TCHAR(Name));
			}
		}, &ChannelsToRestore);
	}
	else
	{
		// Disable all channels and then enable only those we need to minimize trace file size.
		UE::Trace::EnumerateChannels([](const ANSICHAR* ChannelName, const bool bEnabled, void*)
			{
				if (bEnabled)
				{
					FString ChannelNameFString(ChannelName);
					UE::Trace::ToggleChannel(ChannelNameFString.GetCharArray().GetData(), false);
				}
			}
		, nullptr);
	}

	bool bAreTracesStarted = false;
	if (bAlreadyConnected == false)
	{
		FTraceAuxiliary::FOptions Options;
		Options.bExcludeTail = true;
		bAreTracesStarted = FTraceAuxiliary::Start(FTraceAuxiliary::EConnectionType::Network, TEXT("localhost"), TEXT(""), &Options, LogStateTree);
	}

	if (UE::StateTree::Delegates::OnTracingStateChanged.IsBound())
	{
		UE_LOGF(LogStateTree, Log, "StateTree traces enabled");
		UE::StateTree::Delegates::OnTracingStateChanged.Broadcast(EStateTreeTraceStatus::TracesStarted);
	}

	// Enable channels after starting the trace and notifying listeners so events can't
	// be traced (on other threads) before this point.
	// This might not be required most of the time in this case (i.e., legacy StateTreeDebugger)
	// which is happening from the game thread but this is to be consistent with the RewindDebugger extension
	// that could enable channels from a worker thread.
	UE::Trace::ToggleChannel(TEXT("StateTreeDebugChannel"), true);
	UE::Trace::ToggleChannel(TEXT("FrameChannel"), true);

	return bAreTracesStarted;
#else
	return false;
#endif // WITH_STATETREE_TRACE
}

bool FStateTreeModule::IsTracing() const
{
#if UE_TRACE_ENABLED
	// We are not relying on a dedicated flag since tracing can be started from many sources (e.g., RewindDebugger)
	if (!UE::Trace::IsTracing())
	{
		return false;
	}

	const UE::Trace::FChannel* Channel = UE::Trace::FindChannel(TEXT("StateTreeDebugChannel"));
	return Channel != nullptr && Channel->IsEnabled();
#else
	return false;
#endif // UE_TRACE_ENABLED
}

void FStateTreeModule::StopTraces()
{
#if WITH_STATETREE_TRACE
	if (IsTracing() == false)
	{
		return;
	}

	if (UE::StateTree::Delegates::OnTracingStateChanged.IsBound())
	{
		UE_LOGF(LogStateTree, Log, "Stopping StateTree traces...");
		UE::StateTree::Delegates::OnTracingStateChanged.Broadcast(EStateTreeTraceStatus::StoppingTrace);
	}

	UE::Trace::ToggleChannel(TEXT("StateTreeDebugChannel"), false);
	UE::Trace::ToggleChannel(TEXT("FrameChannel"), false);

	// When we have channels to restore it also indicates that the trace were active
	// so we only toggle the channels back (i.e. not calling FTraceAuxiliary::Stop)
	if (ChannelsToRestore.Num() > 0)
	{
		for (const FString& ChannelName : ChannelsToRestore)
		{
			UE::Trace::ToggleChannel(ChannelName.GetCharArray().GetData(), true);
		}
		ChannelsToRestore.Reset();
	}
	else
	{
		FTraceAuxiliary::Stop();
	}

	if (UE::StateTree::Delegates::OnTracingStateChanged.IsBound())
	{
		UE_LOGF(LogStateTree, Log, "StateTree traces stopped");
		UE::StateTree::Delegates::OnTracingStateChanged.Broadcast(EStateTreeTraceStatus::TracesStopped);
	}
#endif // WITH_STATETREE_TRACE
}

#if WITH_STATETREE_DEBUG
void FStateTreeModule::HandlePreGC()
{
	if (UE::StateTree::Debug::Private::bRuntimeValidationInstanceDataGC)
	{
		OnPreRuntimeValidationInstanceData.Broadcast();
	}
}

void FStateTreeModule::HandlePostGC()
{
	if (UE::StateTree::Debug::Private::bRuntimeValidationInstanceDataGC)
	{
		OnPostRuntimeValidationInstanceData.Broadcast();
	}
}
#endif

IMPLEMENT_MODULE(FStateTreeModule, StateTreeModule)

#undef LOCTEXT_NAMESPACE
