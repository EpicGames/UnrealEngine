// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeModule.h"
#include "Debugger/StateTreeTraceModule.h"
#include "HAL/IConsoleManager.h"
#include "StateTreeTypes.h"
#include "UObject/UObjectGlobals.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

namespace TraceServices { class IAnalysisService; }
namespace TraceServices { class IModuleService; }
namespace UE::Trace { class FStoreClient; }
class UUserDefinedStruct;

class FStateTreeModule : public IStateTreeModule
{
public:
	FStateTreeModule();

	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

	virtual bool StartTraces(int32& OutTraceId) override;
	virtual bool IsTracing() const override;
	virtual void StopTraces() override;

#if WITH_STATETREE_DEBUG
	static FTSSimpleMulticastDelegate OnPreRuntimeValidationInstanceData;
	static FTSSimpleMulticastDelegate OnPostRuntimeValidationInstanceData;
#endif

private:

#if WITH_STATETREE_DEBUG
	void HandlePreGC();
	void HandlePostGC();
#endif

#if WITH_STATETREE_TRACE_DEBUGGER
	/**
	 * Gets the store client.
	 */
	virtual UE::Trace::FStoreClient* GetStoreClient() override;

	TSharedPtr<TraceServices::IAnalysisService> TraceAnalysisService;
	TSharedPtr<TraceServices::IModuleService> TraceModuleService;

	/** The client used to connect to the trace store. */
	TUniquePtr<UE::Trace::FStoreClient> StoreClient;

	FStateTreeTraceModule StateTreeTraceModule;

	virtual TNotNull<const IStateTreeDebugInfoProvider*> GetDebugInfoProvider() override;
	virtual void SetDebugInfoProvider(TSharedPtr<IStateTreeDebugInfoProvider> InDebugInfoProvider) override;

	TSharedPtr<IStateTreeDebugInfoProvider> DebugInfoProvider;

#endif // WITH_STATETREE_TRACE_DEBUGGER

#if WITH_STATETREE_TRACE
	TArray<const FString> ChannelsToRestore;

	FAutoConsoleCommand StartDebuggerTracesCommand;
	FAutoConsoleCommand StopDebuggerTracesCommand;
#endif // WITH_STATETREE_TRACE

#if WITH_STATETREE_DEBUG
	FDelegateHandle PreGCHandle;
	FDelegateHandle PostGCHandle;
#endif
};
