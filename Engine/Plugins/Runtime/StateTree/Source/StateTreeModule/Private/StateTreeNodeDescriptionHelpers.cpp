// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeNodeDescriptionHelpers.h"
#include "Conditions/StateTreeConditionHelpers.h"
#include "GameplayTagContainer.h"
#include "StateTreeNodeBase.h"

#if WITH_EDITOR || WITH_STATETREE_TRACE
#define LOCTEXT_NAMESPACE "StateTree"

namespace UE::StateTree::DescHelpers
{

FText GetOperatorText(const EComparisonOperator Operator, EStateTreeNodeFormatting Formatting)
{
	if (Formatting == EStateTreeNodeFormatting::RichText)
	{
		switch (Operator)
		{
		case EComparisonOperator::Equal:
			return FText::FromString(TEXT("<s>==</>"));
		case EComparisonOperator::NotEqual:
			return FText::FromString(TEXT("<s>!=</>"));
		case EComparisonOperator::Less:
			return FText::FromString(TEXT("<s>&lt;</>"));
		case EComparisonOperator::LessOrEqual:
			return FText::FromString(TEXT("<s>&lt;=</>"));
		case EComparisonOperator::Greater:
			return FText::FromString(TEXT("<s>&gt;</>"));
		case EComparisonOperator::GreaterOrEqual:
			return FText::FromString(TEXT("<s>&gt;=</>"));
		default:
			break;
		}
	}
	else
	{
		switch (Operator)
		{
		case EComparisonOperator::Equal:
			return FText::FromString(TEXT("=="));
		case EComparisonOperator::NotEqual:
			return FText::FromString(TEXT("!="));
		case EComparisonOperator::Less:
			return FText::FromString(TEXT("<"));
		case EComparisonOperator::LessOrEqual:
			return FText::FromString(TEXT("<="));
		case EComparisonOperator::Greater:
			return FText::FromString(TEXT(">"));
		case EComparisonOperator::GreaterOrEqual:
			return FText::FromString(TEXT(">="));
		default:
			break;
		}
	}

	return FText::FromString(TEXT("??"));
}

// Deprecated
FText GetOperatorText(const EGenericAICheck Operator, EStateTreeNodeFormatting Formatting)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return GetOperatorText(UE::StateTree::Conditions::GenericAICheckToComparisonOperator(Operator), Formatting);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FText GetInvertText(bool bInvert, EStateTreeNodeFormatting Formatting)
{
	FText InvertText = FText::GetEmpty();
	if (bInvert)
	{
		if (Formatting == EStateTreeNodeFormatting::RichText)
		{
			InvertText = LOCTEXT("InvertRich", "<s>Not</>  ");
		}
		else
		{
			InvertText = LOCTEXT("Invert", "Not  ");
		}
	}
	return InvertText;
}

FText GetBoolText(bool bValue, EStateTreeNodeFormatting Formatting)
{
	return bValue ? LOCTEXT("True", "True") : LOCTEXT("False", "False");
}

FText GetIntervalText(const FFloatInterval& Interval, EStateTreeNodeFormatting Formatting)
{
	return GetIntervalText(Interval.Min, Interval.Max, Formatting);
}

FText GetIntervalText(float Min, float Max, EStateTreeNodeFormatting Formatting)
{
	FNumberFormattingOptions Options;
	Options.MinimumFractionalDigits = 1;
	Options.MaximumFractionalDigits = 2;

	FText MinValueText = FText::AsNumber(Min, &Options);
	FText MaxValueText = FText::AsNumber(Max, &Options);

	return GetIntervalText(MinValueText, MaxValueText, Formatting);
}

FText GetIntervalText(const FText& MinValueText, const FText& MaxValueText, EStateTreeNodeFormatting Formatting)
{
	FText IntervalText;
	if (Formatting == EStateTreeNodeFormatting::RichText)
	{
		IntervalText = FText::FormatNamed(LOCTEXT("IntervalRich", "[{Min}<s>,</> {Max}]"),
			TEXT("Min"), MinValueText,
			TEXT("Max"), MaxValueText);
	}
	else //EStateTreeNodeFormatting::Text
	{
		IntervalText = FText::FormatNamed(LOCTEXT("Interval", "[{Min}, {Max}]"),
			TEXT("Min"), MinValueText,
			TEXT("Max"), MaxValueText);
	}

	return IntervalText;
}

FText GetGameplayTagContainerAsText(const FGameplayTagContainer& TagContainer, const int ApproxMaxLength)
{
	if (TagContainer.IsEmpty())
	{
		return LOCTEXT("Empty", "Empty");
	}
		
	FString Combined;
	for (const FGameplayTag& Tag : TagContainer)
	{
		FString TagString = Tag.ToString();

		if (Combined.Len() > 0)
		{
			Combined += TEXT(", ");
		}
			
		if ((Combined.Len() + TagString.Len()) > ApproxMaxLength)
		{
			// Overflow
			if (Combined.Len() == 0)
			{
				Combined += TagString.Left(ApproxMaxLength);
			}
			Combined += TEXT("...");
			break;
		}

		Combined += TagString;
	}

	return FText::FromString(Combined);
}

FText GetGameplayTagQueryAsText(const FGameplayTagQuery& TagQuery, const int ApproxMaxLength)
{
	// Limit the presented query description.
	FText QueryValue = LOCTEXT("Empty", "Empty");
	constexpr int32 MaxDescLen = 120; 
	FString QueryDesc = TagQuery.GetDescription();
	if (!QueryDesc.IsEmpty())
	{
		if (QueryDesc.Len() > MaxDescLen)
		{
			QueryDesc = QueryDesc.Left(MaxDescLen);
			QueryDesc += TEXT("...");
		}
		QueryValue = FText::FromString(QueryDesc);
	}
	return QueryValue;
}

FText GetExactMatchText(bool bExactMatch, EStateTreeNodeFormatting Formatting)
{
	if (Formatting == EStateTreeNodeFormatting::RichText)
	{
		return bExactMatch ? LOCTEXT("ExactlyRich", "<s>exactly</> ") : FText::GetEmpty();
	}
	return bExactMatch ? LOCTEXT("Exactly", "exactly ") : FText::GetEmpty();
}

FText GetText(const FVector& Value, EStateTreeNodeFormatting Formatting)
{
	return Value.ToCompactText();
}

FText GetText(float Value, EStateTreeNodeFormatting Formatting)
{
	return FText::AsNumber(Value);
}

FText GetText(int32 Value, EStateTreeNodeFormatting Formatting)
{
	return FText::AsNumber(Value);
}

FText GetText(const UObject* Value, EStateTreeNodeFormatting Formatting)
{
	return FText::FromName(GetFNameSafe(Value));
}

FText GetMathOperationText(const FText& OperationText, const FText& LeftValue, const FText& RightValue, EStateTreeNodeFormatting Formatting)
{
	const FText Format = (Formatting == EStateTreeNodeFormatting::RichText)
		? LOCTEXT("MathFuncRich", "({Left} <s>{Operation}</> {Right})")
		: LOCTEXT("MathFunc", "({Left} {Operation} {Right})");

	return FText::FormatNamed(Format,
		TEXT("Left"), LeftValue,
		TEXT("Operation"), OperationText,
		TEXT("Right"), RightValue);
}

FText GetSingleParamFunctionText(const FText& FunctionText, const FText& ParamText, EStateTreeNodeFormatting Formatting)
{
	const FText Format = (Formatting == EStateTreeNodeFormatting::RichText)
		? LOCTEXT("SingleParamFuncRich", "<s>{Function}</>({Input})")
		: LOCTEXT("SingleParamFunc", "{Function}({Input})");

	return FText::FormatNamed(Format,
		TEXT("Function"), FunctionText,
		TEXT("Input"), ParamText);
}
} // UE::StateTree::Helpers

#undef LOCTEXT_NAMESPACE
#endif // WITH_EDITOR || WITH_STATETREE_TRACE