// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeNodeRef.h"
#include "StateTree.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeInstanceData.h"
#include "StateTreeTaskBase.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
#if WITH_STATETREE_DEBUG
FStateTreeStrongTaskRef::FStateTreeStrongTaskRef(TStrongObjectPtr<const UStateTree> InStateTree, const FStateTreeTaskBase* InTask, FStateTreeIndex16 InNodeIndex, FGuid InNodeId)
	: StateTree(InStateTree)
	, Task(InTask)
	, NodeIndex(InNodeIndex)
	, NodeId(InNodeId)
{
}
#else
FStateTreeStrongTaskRef::FStateTreeStrongTaskRef(TStrongObjectPtr<const UStateTree> InStateTree, const FStateTreeTaskBase* InTask, FStateTreeIndex16 InNodeIndex)
	: StateTree(InStateTree)
	, Task(InTask)
	, NodeIndex(InNodeIndex)
	{
	}
#endif

const UStateTree* FStateTreeStrongTaskRef::GetStateTree() const
{
	return IsValid() ? StateTree.Get() : nullptr;
}
	
const FStateTreeTaskBase* FStateTreeStrongTaskRef::GetTask() const
{
	return IsValid() ? Task : nullptr;
}

bool FStateTreeStrongTaskRef::IsValid() const
{
	bool bIsValid = Task != nullptr && StateTree.Get() != nullptr;
#if WITH_STATETREE_DEBUG
	if (bIsValid)
	{
		bIsValid = NodeId == StateTree->GetNodeIdFromIndex(NodeIndex);
		ensureMsgf(bIsValid, TEXT("The node id changed from the last use. Did the StateTree asset recompiled?"));
	}
#endif
	return bIsValid;
}

FStateTreeWeakTaskRef::FStateTreeWeakTaskRef(TNotNull<const UStateTree*> InStateTree, FStateTreeIndex16 InTaskIndex)
	: StateTree(InStateTree)
	, NodeIndex(InTaskIndex)
{
#if WITH_STATETREE_DEBUG
	NodeId = InStateTree->GetNodeIdFromIndex(NodeIndex);
#endif
}

FStateTreeStrongTaskRef FStateTreeWeakTaskRef::Pin() const
{
	TStrongObjectPtr<const UStateTree> StateTreePinned = StateTree.Pin();
	const FStateTreeTaskBase* Task = nullptr;
	if (StateTreePinned && StateTreePinned->GetNodes().IsValidIndex(NodeIndex.AsInt32()))
	{
#if WITH_STATETREE_DEBUG
		ensureMsgf(NodeId == StateTreePinned->GetNodeIdFromIndex(NodeIndex), TEXT("The node id changed from the last use. Did the StateTree asset recompiled?"));
#endif
		Task = StateTreePinned->GetNodes()[NodeIndex.AsInt32()].GetPtr<const FStateTreeTaskBase>();
	}

#if WITH_STATETREE_DEBUG
	return Task ? FStateTreeStrongTaskRef(StateTreePinned, Task, NodeIndex, NodeId) : FStateTreeStrongTaskRef();
#else
	return Task ? FStateTreeStrongTaskRef(StateTreePinned, Task, NodeIndex) : FStateTreeStrongTaskRef();
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS