// Copyright Epic Games, Inc. All Rights Reserved.
#include "StateTreePropertyBindings.h"
#include "UObject/EnumProperty.h"
#include "Misc/EnumerateRange.h"
#include "PropertyPathHelpers.h"
#include "StructUtils/PropertyBag.h"
#include "StateTreePropertyRef.h"

#if WITH_EDITOR
#include "UObject/CoreRedirects.h"
#include "UObject/Package.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "StructUtils/UserDefinedStruct.h"
#include "UObject/Field.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreePropertyBindings)

namespace UE::StateTree
{
	bool AcceptTaskInstanceData(EStateTreeBindableStructSource Target)
	{
		// Condition and Utility are constructed before the task instance data is constructed.
		return Target != EStateTreeBindableStructSource::StateParameter
			&& Target != EStateTreeBindableStructSource::Condition
			&& Target != EStateTreeBindableStructSource::Consideration;
	}

	FString GetDescAndPathAsString(const FStateTreeBindableStructDesc& Desc, const FPropertyBindingPath& Path)
	{
		return PropertyBinding::GetDescriptorAndPathAsString(Desc, Path);
	}

#if WITH_EDITOR
	EStateTreePropertyUsage GetUsageFromMetaData(const FProperty* Property)
	{
		static const FName CategoryName(TEXT("Category"));

		if (Property == nullptr)
		{
			return EStateTreePropertyUsage::Invalid;
		}
		
		const FString Category = Property->GetMetaData(CategoryName);

		if (Category == TEXT("Input"))
		{
			return EStateTreePropertyUsage::Input;
		}
		if (Category == TEXT("Inputs"))
		{
			return EStateTreePropertyUsage::Input;
		}
		if (Category == TEXT("Output"))
		{
			return EStateTreePropertyUsage::Output;
		}
		if (Category == TEXT("Outputs"))
		{
			return EStateTreePropertyUsage::Output;
		}
		if (Category == TEXT("Context"))
		{
			return EStateTreePropertyUsage::Context;
		}

		return EStateTreePropertyUsage::Parameter;
	}

	const FProperty* GetStructSingleOutputProperty(const UStruct& InStruct)
	{
		const FProperty* FuncOutputProperty = nullptr;
		for (TFieldIterator<FProperty> PropIt(&InStruct, EFieldIteratorFlags::IncludeSuper); PropIt; ++PropIt)
		{
			if (GetUsageFromMetaData(*PropIt) == EStateTreePropertyUsage::Output)
			{
				if (FuncOutputProperty)
				{
					return nullptr;
				}

				FuncOutputProperty = *PropIt;
			}
		}

		return FuncOutputProperty;
	}
#endif

} // UE::StateTree

PRAGMA_DISABLE_DEPRECATION_WARNINGS
#if WITH_EDITORONLY_DATA
namespace UE::StateTree::Deprecation
{
	FPropertyBindingPath ConvertEditorPath(const FStateTreeEditorPropertyPath& InEditorPath)
	{
		FPropertyBindingPath Path;
		Path.SetStructID(InEditorPath.StructID);

		for (const FString& Segment : InEditorPath.Path)
		{
			const TCHAR* PropertyNamePtr = nullptr;
			int32 PropertyNameLength = 0;
			int32 ArrayIndex = INDEX_NONE;
			PropertyPathHelpers::FindFieldNameAndArrayIndex(Segment.Len(), *Segment, PropertyNameLength, &PropertyNamePtr, ArrayIndex);
			FString PropertyNameString(PropertyNameLength, PropertyNamePtr);
			const FName PropertyName(*PropertyNameString, FNAME_Find);
			Path.AddPathSegment(PropertyName, ArrayIndex);
		}
		return Path;
	}
} // UE::StateTree::Deprecation

//----------------------------------------------------------------//
//  FStateTreePropertyPathBinding
//----------------------------------------------------------------//
void FStateTreePropertyPathBinding::PostSerialize(const FArchive& Ar)
{
	if (SourcePath_DEPRECATED.IsValid())
	{
		SourcePropertyPath = UE::StateTree::Deprecation::ConvertEditorPath(SourcePath_DEPRECATED);
		SourcePath_DEPRECATED.StructID = FGuid();
		SourcePath_DEPRECATED.Path.Reset();
	}

	if (TargetPath_DEPRECATED.IsValid())
	{
		TargetPropertyPath = UE::StateTree::Deprecation::ConvertEditorPath(TargetPath_DEPRECATED);
		TargetPath_DEPRECATED.StructID = FGuid();
		TargetPath_DEPRECATED.Path.Reset();
	}
}
#endif // WITH_EDITORONLY_DATA
PRAGMA_ENABLE_DEPRECATION_WARNINGS

//----------------------------------------------------------------//
//  FStateTreeBindableStructDesc
//----------------------------------------------------------------//

FString FStateTreeBindableStructDesc::ToString() const
{
	TStringBuilder<256> Result;

	Result += *UEnum::GetDisplayValueAsText(DataSource).ToString();
	Result += TEXT(" '");
#if WITH_EDITORONLY_DATA
	Result += StatePath;
	Result += TEXT("/");
#endif
	Result += Name.ToString();
	Result += TEXT("'");

	return FString(Result);
}

//----------------------------------------------------------------//
//  FStateTreePropertyBindings
//----------------------------------------------------------------//

FStateTreePropertyBindings::FStateTreePropertyBindings()
{
	// StateTree supports Property references
	PropertyReferenceStructType = FStateTreeStructRef::StaticStruct();

	// Set copy function
	PropertyReferenceCopyFunc = [](const FStructProperty& SourceStructProperty, uint8* SourceAddress, uint8* TargetAddress)
	{
		FStateTreeStructRef* Target = reinterpret_cast<FStateTreeStructRef*>(TargetAddress);
		Target->Set(FStructView(SourceStructProperty.Struct, SourceAddress));
	};

	// Set reset function
	PropertyReferenceResetFunc = [](uint8* TargetAddress)
	{
		reinterpret_cast<FStateTreeStructRef*>(TargetAddress)->Set(FStructView());
	};
}

void FStateTreePropertyBindings::OnReset()
{
	SourceStructs.Reset();
	PropertyPathBindings.Reset();
	PropertyAccesses.Reset();
	PropertyReferencePaths.Reset();
}

int32 FStateTreePropertyBindings::GetNumBindableStructDescriptors() const
{
	return SourceStructs.Num();
}

const FPropertyBindingBindableStructDescriptor* FStateTreePropertyBindings::GetBindableStructDescriptorFromHandle(const FConstStructView InSourceHandleView) const
{
	check(InSourceHandleView.GetScriptStruct() == FStateTreeDataHandle::StaticStruct());
	return GetBindableStructDescriptorFromHandle(InSourceHandleView.Get<const FStateTreeDataHandle>());
}

const FPropertyBindingBindableStructDescriptor* FStateTreePropertyBindings::GetBindableStructDescriptorFromHandle(FStateTreeDataHandle InSourceHandle) const
{
	return SourceStructs.FindByPredicate([SourceDataHandle = InSourceHandle](const FStateTreeBindableStructDesc& Desc)
	{
		return Desc.DataHandle == SourceDataHandle;
	});
}

void FStateTreePropertyBindings::VisitSourceStructDescriptorInternal(
	TFunctionRef<EVisitResult(const FPropertyBindingBindableStructDescriptor& Descriptor)> InFunction) const
{
	for (const FStateTreeBindableStructDesc& SourceStruct : SourceStructs)
	{
		if (InFunction(SourceStruct) == EVisitResult::Break)
		{
			break;
		}
	}
}

bool FStateTreePropertyBindings::ResolveBindingCopyInfo(const FPropertyBindingBinding& InResolvedBinding,
	const FPropertyBindingPathIndirection& InBindingSourceLeafIndirection, const FPropertyBindingPathIndirection& InBindingTargetLeafIndirection,
	FPropertyBindingCopyInfo& OutCopyInfo)
{
	OutCopyInfo.bCopyFromTargetToSource = static_cast<const FStateTreePropertyPathBinding&>(InResolvedBinding).IsOutputBinding();

	return Super::ResolveBindingCopyInfo(InResolvedBinding, InBindingSourceLeafIndirection, InBindingTargetLeafIndirection, OutCopyInfo);
}

bool FStateTreePropertyBindings::OnResolvingPaths()
{
	// Base class handled common bindings, here we only need to handle Property references
	bool bResult = true;

	PropertyAccesses.Reset();
	PropertyAccesses.Reserve(PropertyReferencePaths.Num());

	for (const FStateTreePropertyRefPath& ReferencePath : PropertyReferencePaths)
	{
		FStateTreePropertyAccess& PropertyAccess = PropertyAccesses.AddDefaulted_GetRef();
		
		PropertyAccess.SourceDataHandle = ReferencePath.GetSourceDataHandle();
		const FPropertyBindingBindableStructDescriptor* SourceDesc = GetBindableStructDescriptorFromHandle(PropertyAccess.SourceDataHandle);
		PropertyAccess.SourceStructType = SourceDesc->Struct;

		FPropertyBindingPathIndirection SourceLeafIndirection;
		if (!Super::ResolvePath(SourceDesc->Struct, ReferencePath.GetSourcePath(), PropertyAccess.SourceIndirection, SourceLeafIndirection))
		{
			bResult = false;
		}

		PropertyAccess.SourceLeafProperty = SourceLeafIndirection.GetProperty();
	}

	return bResult;
}

int32 FStateTreePropertyBindings::GetNumBindings() const
{
	return PropertyPathBindings.Num();
}

void FStateTreePropertyBindings::ForEachBinding(TFunctionRef<void(const FPropertyBindingBinding& Binding)> InFunction) const
{
	for (const FStateTreePropertyPathBinding& Binding : PropertyPathBindings)
	{
		InFunction(Binding);
	}
}

void FStateTreePropertyBindings::ForEachBinding(const FPropertyBindingIndex16 InBegin, const FPropertyBindingIndex16 InEnd
	, const TFunctionRef<void(const FPropertyBindingBinding& Binding, const int32 BindingIndex)> InFunction) const
{
	checkf(InBegin.IsValid() && InEnd.IsValid(), TEXT("%hs expects valid indices."), __FUNCTION__);

	for (int32 BindingIndex = InBegin.Get(); BindingIndex < InEnd.Get(); ++BindingIndex)
	{
		InFunction(PropertyPathBindings[BindingIndex], BindingIndex);
	}
}

void FStateTreePropertyBindings::ForEachMutableBinding(TFunctionRef<void(FPropertyBindingBinding& Binding)> InFunction)
{
	for (FStateTreePropertyPathBinding& Binding : PropertyPathBindings)
	{
		InFunction(Binding);
	}
}

void FStateTreePropertyBindings::VisitBindings(TFunctionRef<EVisitResult(const FPropertyBindingBinding& Binding)> InFunction) const
{
	for (const FStateTreePropertyPathBinding& Binding : PropertyPathBindings)
	{
		if (InFunction(Binding) == EVisitResult::Break)
		{
			break;
		}
	}
}

void FStateTreePropertyBindings::VisitBindings(const FPropertyBindingIndex16 InBegin, const FPropertyBindingIndex16 InEnd,
	TFunctionRef<EVisitResult(const FPropertyBindingBinding& Binding, const int32 BindingIndex)> InFunction) const
{
	checkf(InBegin.IsValid() && InEnd.IsValid(), TEXT("%hs expects valid indices."), __FUNCTION__);

	for (int32 BindingIndex = InBegin.Get(); BindingIndex < InEnd.Get(); ++BindingIndex)
	{
		if (InFunction(PropertyPathBindings[BindingIndex], BindingIndex) == EVisitResult::Break)
		{
			break;
		}
	}
}

void FStateTreePropertyBindings::VisitMutableBindings(TFunctionRef<EVisitResult(FPropertyBindingBinding& Binding)> InFunction)
{
	for (FStateTreePropertyPathBinding& Binding : PropertyPathBindings)
	{
		if (InFunction(Binding) == EVisitResult::Break)
		{
			break;
		}
	}
}

#if WITH_EDITOR
FPropertyBindingBinding* FStateTreePropertyBindings::AddBindingInternal(const FPropertyBindingPath& InSourcePath
	, const FPropertyBindingPath& InTargetPath)
{
	checkf(false, TEXT("Not expected to get called for StateTree runtime bindings."
					" Editor operations for bindings are handled by FStateTreeEditorPropertyBindings"));
	return nullptr;
}

void FStateTreePropertyBindings::RemoveBindingsInternal(TFunctionRef<bool(FPropertyBindingBinding&)> InPredicate)
{
	checkf(false, TEXT("Not expected to get called for StateTree runtime bindings."
					" Editor operations for bindings are handled by FStateTreeEditorPropertyBindings"));
}

bool FStateTreePropertyBindings::HasBindingInternal(TFunctionRef<bool(const FPropertyBindingBinding&)> InPredicate) const
{
	checkf(false, TEXT("Not expected to get called for StateTree runtime bindings."
					" Editor operations for bindings are handled by FStateTreeEditorPropertyBindings"));
	return false;
}

const FPropertyBindingBinding* FStateTreePropertyBindings::FindBindingInternal(TFunctionRef<bool(const FPropertyBindingBinding&)> InPredicate) const
{
	checkf(false, TEXT("Not expected to get called for StateTree runtime bindings."
					" Editor operations for bindings are handled by FStateTreeEditorPropertyBindings"));
	return nullptr;
}
#endif // WITH_EDITOR

const FStateTreePropertyAccess* FStateTreePropertyBindings::GetPropertyAccess(const FStateTreePropertyRef& InPropertyReference) const
{
	if (!InPropertyReference.GetRefAccessIndex().IsValid())
	{
		return nullptr;
	}

	if (!ensure(PropertyAccesses.IsValidIndex(InPropertyReference.GetRefAccessIndex().Get())))
	{
		return nullptr;
	}

	return &PropertyAccesses[InPropertyReference.GetRefAccessIndex().Get()];
}
