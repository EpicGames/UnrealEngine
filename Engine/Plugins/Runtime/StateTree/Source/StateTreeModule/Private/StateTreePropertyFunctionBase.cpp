// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreePropertyFunctionBase.h"
#include "StateTreePropertyBindings.h"

#if WITH_EDITOR
#include "EdGraphSchema_K2.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreePropertyFunctionBase)

#if WITH_EDITOR
FColor FStateTreePropertyFunctionBase::GetIconColor() const
{
	if (const FProperty* OutputProperty = UE::StateTree::GetStructSingleOutputProperty(*GetInstanceDataType()))
	{
		const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
		check(Schema);

		FEdGraphPinType PinType;
		if (Schema->ConvertPropertyToPinType(OutputProperty, PinType))
		{
			return Schema->GetPinTypeColor(PinType).ToFColor(true);
		}
	}

	return Super::GetIconColor();
}
#endif