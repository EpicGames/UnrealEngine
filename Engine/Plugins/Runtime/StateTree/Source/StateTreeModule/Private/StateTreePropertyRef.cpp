// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreePropertyRef.h"

namespace UE::StateTree::PropertyRefHelpers::Private
{
	TOptional<FResolvePropertyReferenceIndirectionsResult> ResolvePropertyReferenceIndirections(
		const FStateTreePropertyRef& InPropertyRef,
		FStateTreeInstanceStorage& InstanceDataStorage,
		const ExecutionContext::ITemporaryStorage* InTemporaryStorage,
		const FStateTreeExecutionFrame& InExecutionFrame)
	{
		const FStateTreePropertyBindings& PropertyBindings = InExecutionFrame.StateTree->GetPropertyBindings();
		if (const FStateTreePropertyAccess* PropertyAccess = PropertyBindings.GetPropertyAccess(InPropertyRef))
		{
			// Passing empty ContextAndExternalDataViews, as PropertyRef is not allowed to point to context or external data.
			const FStateTreeDataView SourceView = InstanceData::GetDataViewOrTemporary(InstanceDataStorage, nullptr, InExecutionFrame, PropertyAccess->SourceDataHandle);

			if (IsPropertyRef(*PropertyAccess->SourceLeafProperty))
			{
				// The only possibility when PropertyRef references another PropertyRef is when source one is a global or subtree parameter, i.e lives in parent execution frame.
				// If that's the case, referenced PropertyRef is obtained and we recursively take the address where it points to.
				FActiveFrameID ParentFrameID;
				switch (PropertyAccess->SourceDataHandle.GetSource())
				{
				case EStateTreeDataSourceType::GlobalParameterData:
					ParentFrameID = InExecutionFrame.GlobalParameterDataFrameID;
					break;
				case EStateTreeDataSourceType::SubtreeParameterData:
					ParentFrameID = InExecutionFrame.StateParameterDataFrameID;
					break;
				default:
					checkf(false, TEXT("Property Reference chaining should happen in Linked Subtree or Linked Asset!"));
					return {};
				}

				const TConstArrayView<FStateTreeExecutionFrame> ActiveFrames = InstanceDataStorage.GetExecutionState().ActiveFrames;
				const TConstArrayView<FStateTreeExecutionFrame> TemporaryFrames = InTemporaryStorage ? InTemporaryStorage->GetTemporaryFrames() : TConstArrayView<FStateTreeExecutionFrame>();
				const FStateTreeExecutionFrame* ParentFrame = ExecutionContext::FindExecutionFrame(ParentFrameID, ActiveFrames, TemporaryFrames);

				if (ParentFrame == nullptr)
				{
					return {};
				}

				const FStateTreePropertyRef* ReferencedPropertyRef = PropertyBindings.GetMutablePropertyPtr<FStateTreePropertyRef>(SourceView, *PropertyAccess);
				if (ReferencedPropertyRef == nullptr)
				{
					return {};
				}

				return ResolvePropertyReferenceIndirections(*ReferencedPropertyRef, InstanceDataStorage, InTemporaryStorage, *ParentFrame);
			}

			return TOptional<FResolvePropertyReferenceIndirectionsResult>(InPlace, SourceView, PropertyAccess, &PropertyBindings);
		}

		return {};
	}

}
