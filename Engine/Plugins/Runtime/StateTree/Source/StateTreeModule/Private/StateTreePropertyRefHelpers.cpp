// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreePropertyRefHelpers.h"
#include "StateTreePropertyRef.h"
#include "UObject/TextProperty.h"
#include "UObject/EnumProperty.h"
#include "UObject/Class.h"
#include "StateTreePropertyBindings.h"

#if WITH_EDITOR
#include "EdGraphSchema_K2.h"
#include "IPropertyAccessEditor.h"
#include "EdGraph/EdGraphPin.h"
#include <limits>
#endif

namespace UE::StateTree::PropertyRefHelpers
{
#if WITH_EDITOR
	static const FLazyName BoolName = "bool";
	static const FLazyName ByteName = "byte";
	static const FLazyName Int32Name = "int32";
	static const FLazyName Int64Name = "int64";
	static const FLazyName FloatName = "float";
	static const FLazyName DoubleName = "double";
	static const FLazyName NameName = "FName";
	static const FLazyName StringName = "FString";
	static const FLazyName TextName = "FText";
	const FName IsRefToArrayName = "IsRefToArray";
	const FName CanRefToArrayName = "CanRefToArray";
	const FName RefTypeName = "RefType";
	static const FLazyName IsOptionalName = "Optional";

	bool ArePropertyRefsCompatible(const FProperty& TargetRefProperty, const FProperty& SourceRefProperty, const void* TargetRefAddress, const void* SourceRefAddress)
	{
		check(IsPropertyRef(SourceRefProperty) && IsPropertyRef(TargetRefProperty));
		check(TargetRefAddress);

		FEdGraphPinType SourceRefPin = GetPropertyRefInternalTypeAsPin(SourceRefProperty, SourceRefAddress);
		FEdGraphPinType TargetRefPin = GetPropertyRefInternalTypeAsPin(TargetRefProperty, TargetRefAddress);

		return SourceRefPin.PinCategory == TargetRefPin.PinCategory && SourceRefPin.ContainerType == TargetRefPin.ContainerType 
			&& SourceRefPin.PinSubCategoryObject == TargetRefPin.PinSubCategoryObject;
	}

	bool IsNativePropertyRefCompatibleWithProperty(const FProperty& RefProperty, const FProperty& SourceProperty)
	{
		check(IsPropertyRef(RefProperty));

		const FProperty* TestProperty = &SourceProperty;
		const bool bCanTargetRefArray = RefProperty.HasMetaData(CanRefToArrayName);
		const bool bIsTargetRefArray = RefProperty.HasMetaData(IsRefToArrayName);

		if (bIsTargetRefArray || bCanTargetRefArray)
		{
			if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(TestProperty))
			{
				TestProperty = ArrayProperty->Inner;
			}
			else if(!bCanTargetRefArray)
			{
				return false;
			}
		}

		FString TargetTypeNameFullStr = RefProperty.GetMetaData(RefTypeName);
		if (TargetTypeNameFullStr.IsEmpty())
		{
			return false;
		}

		TargetTypeNameFullStr.RemoveSpacesInline();

		TArray<FString> TargetTypes;
		TargetTypeNameFullStr.ParseIntoArray(TargetTypes, TEXT(","), true);

		const FStructProperty* SourceStructProperty = CastField<FStructProperty>(TestProperty);
		// Check inside loop are only allowed to return true to avoid shortcircuiting the loop.
		for (const FString& TargetTypeNameStr : TargetTypes)
		{
			const FName TargetTypeName = FName(*TargetTypeNameStr);
			// Compare properties metadata directly if SourceProperty is PropertyRef as well
			if (SourceStructProperty && SourceStructProperty->Struct == FStateTreePropertyRef::StaticStruct())

			{
				const FName SourceTypeName(SourceStructProperty->GetMetaData(RefTypeName));
				const bool bIsSourceRefArray = SourceStructProperty->GetBoolMetaData(IsRefToArrayName);
				if (SourceTypeName == TargetTypeName && bIsSourceRefArray == bIsTargetRefArray)
				{
					return true;
				}

			}

			if (TargetTypeName == BoolName)
			{
				if (TestProperty->IsA<FBoolProperty>())
				{
					return true;
				}
			}
			else if (TargetTypeName == ByteName)
			{
				if (TestProperty->IsA<FByteProperty>())
				{
					return true;
				}
			}
			else if (TargetTypeName == Int32Name)
			{
				if (TestProperty->IsA<FIntProperty>())
				{
					return true;
				}
			}
			else if (TargetTypeName == Int64Name)
			{
				if (TestProperty->IsA<FInt64Property>())
				{
					return true;
				}
			}
			else if (TargetTypeName == FloatName)
			{
				if (TestProperty->IsA<FFloatProperty>())
				{
					return true;
				}
			}
			else if (TargetTypeName == DoubleName)
			{
				if (TestProperty->IsA<FDoubleProperty>())
				{
					return true;
				}
			}
			else if (TargetTypeName == NameName)
			{
				if (TestProperty->IsA<FNameProperty>())
				{
					return true;
				}
			}

			else if (TargetTypeName == StringName)

			{
				if (TestProperty->IsA<FStrProperty>())
				{
					return true;
				}
			}
			else if (TargetTypeName == TextName)
			{
				if (TestProperty->IsA<FTextProperty>())
				{
					return true;
				}
			}
			else
			{
				UField* TargetRefField = UClass::TryFindTypeSlow<UField>(TargetTypeNameStr);
				if (!TargetRefField)
				{
					TargetRefField = LoadObject<UField>(nullptr, *TargetTypeNameStr);
				}

				if (SourceStructProperty)
				{
					// Only referencing struct of the same exact type should be allowed. Otherwise one could e.g assign FVector to FMyVector:FVector property through reference missing some properties.
					if (SourceStructProperty->Struct == TargetRefField)
					{
						return true;
					}
				}

				if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(TestProperty))
				{
					// Only referencing object of the same exact class should be allowed. Otherwise one could e.g assign UObject to AActor property through reference to UObject.
					if(ObjectProperty->PropertyClass == TargetRefField)
					{
						return true;

					}
				}
				else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(TestProperty))
				{
					if (EnumProperty->GetEnum() == TargetRefField)
					{
						return true;
					}
				}
				else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(TestProperty))
				{
					if (ByteProperty->Enum == TargetRefField)
					{
						return true;
					}
				}
			}
		}

		return false;
	}

	bool IsPropertyRefCompatibleWithProperty(const FProperty& RefProperty, const FProperty& SourceProperty, const void* PropertyRefAddress, const void* SourceAddress)
	{
		check(PropertyRefAddress);
		check(IsPropertyRef(RefProperty));

		if (IsPropertyRef(SourceProperty))
		{
			return ArePropertyRefsCompatible(RefProperty, SourceProperty, PropertyRefAddress, SourceAddress);
		}

		if (const FStructProperty* StructProperty = CastField<FStructProperty>(&RefProperty))
		{
			if (StructProperty->Struct == FStateTreePropertyRef::StaticStruct())
			{
				return IsNativePropertyRefCompatibleWithProperty(RefProperty, SourceProperty);
			}
			else if (StructProperty->Struct == FStateTreeBlueprintPropertyRef::StaticStruct())
			{
				return IsBlueprintPropertyRefCompatibleWithProperty(SourceProperty, PropertyRefAddress);
			}
		}

		checkNoEntry();
		return false;
	}

	bool IsPropertyAccessibleForPropertyRef(const FProperty& SourceProperty, FStateTreeBindableStructDesc SourceStruct, bool bIsOutput)
	{
		switch (SourceStruct.DataSource)
		{
		case EStateTreeBindableStructSource::Parameter:
		case EStateTreeBindableStructSource::StateParameter:
		case EStateTreeBindableStructSource::TransitionEvent:
		case EStateTreeBindableStructSource::StateEvent:
			return true;

		case EStateTreeBindableStructSource::Context:
		case EStateTreeBindableStructSource::Condition:
		case EStateTreeBindableStructSource::Consideration:
		case EStateTreeBindableStructSource::PropertyFunction:
			return false;

		case EStateTreeBindableStructSource::GlobalTask:
		case EStateTreeBindableStructSource::Evaluator:
		case EStateTreeBindableStructSource::Task:
			return bIsOutput || IsPropertyRef(SourceProperty);

		default:
			checkNoEntry();
		}

		return false;
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bool IsPropertyAccessibleForPropertyRef(TConstArrayView<FStateTreePropertyPathIndirection> SourcePropertyPathIndirections, FStateTreeBindableStructDesc SourceStruct)
	{
		return false;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	bool IsPropertyAccessibleForPropertyRef(TConstArrayView<FPropertyBindingPathIndirection> SourcePropertyPathIndirections, FStateTreeBindableStructDesc SourceStruct)
	{
		bool bIsOutput = false;
		for (const FPropertyBindingPathIndirection& Indirection : SourcePropertyPathIndirections)
		{
			if (UE::StateTree::GetUsageFromMetaData(Indirection.GetProperty()) == EStateTreePropertyUsage::Output)
			{
				bIsOutput = true;
				break;
			}
		}

		return IsPropertyAccessibleForPropertyRef(*SourcePropertyPathIndirections.Last().GetProperty(), SourceStruct, bIsOutput);
	}

	bool IsPropertyAccessibleForPropertyRef(const FProperty& SourceProperty, TConstArrayView<FBindingChainElement> BindingChain, FStateTreeBindableStructDesc SourceStruct)
	{
		bool bIsOutput = UE::StateTree::GetUsageFromMetaData(&SourceProperty) == EStateTreePropertyUsage::Output;
		for (const FBindingChainElement& ChainElement : BindingChain)
		{
			if (const FProperty* Property = ChainElement.Field.Get<FProperty>())
			{
				if (UE::StateTree::GetUsageFromMetaData(Property) == EStateTreePropertyUsage::Output)
				{
					bIsOutput = true;
					break;
				}
			}
		}

		return IsPropertyAccessibleForPropertyRef(SourceProperty, SourceStruct, bIsOutput);
	}

	FEdGraphPinType GetBlueprintPropertyRefInternalTypeAsPin(const FStateTreeBlueprintPropertyRef& PropertyRef)
	{
		FEdGraphPinType PinType;
		PinType.PinSubCategory = NAME_None;

		if (PropertyRef.IsRefToArray())
		{
			PinType.ContainerType = EPinContainerType::Array;
		}

		switch (PropertyRef.GetRefType())
		{
		case EStateTreePropertyRefType::None:
			break;

		case EStateTreePropertyRefType::Bool:
			PinType.PinCategory = UEdGraphSchema_K2::PC_Boolean;
			break;

		case EStateTreePropertyRefType::Byte:
			PinType.PinCategory = UEdGraphSchema_K2::PC_Byte;
			break;

		case EStateTreePropertyRefType::Int32:
			PinType.PinCategory = UEdGraphSchema_K2::PC_Int;
			break;

		case EStateTreePropertyRefType::Int64:
			PinType.PinCategory = UEdGraphSchema_K2::PC_Int64;
			break;

		case EStateTreePropertyRefType::Float:
			PinType.PinCategory = UEdGraphSchema_K2::PC_Real;
			PinType.PinSubCategory = UEdGraphSchema_K2::PC_Float;
			break;

		case EStateTreePropertyRefType::Double:
			PinType.PinCategory = UEdGraphSchema_K2::PC_Real;
			PinType.PinSubCategory = UEdGraphSchema_K2::PC_Double;
			break;

		case EStateTreePropertyRefType::Name:
			PinType.PinCategory = UEdGraphSchema_K2::PC_Name;
			break;

		case EStateTreePropertyRefType::String:
			PinType.PinCategory = UEdGraphSchema_K2::PC_String;
			break;

		case EStateTreePropertyRefType::Text:
			PinType.PinCategory = UEdGraphSchema_K2::PC_Text;
			break;

		case EStateTreePropertyRefType::Enum:
			PinType.PinCategory = UEdGraphSchema_K2::PC_Enum;
			PinType.PinSubCategoryObject = PropertyRef.GetTypeObject();
			if (UEnum* Enum = Cast<UEnum>(PinType.PinSubCategoryObject))
			{
				if (Enum->GetMaxEnumValue() <= (int64)std::numeric_limits<uint8>::max())
				{
					// Use byte for BP. It will use the correct picker and enum k2 node.
					PinType.PinCategory = UEdGraphSchema_K2::PC_Byte;
				}
			}
			else
			{
				UE_LOGF(LogStateTree, Warning, "The property ref of type enum has an invalid enum. %ls", *GetFullNameSafe(PinType.PinSubCategoryObject.Get()));
			}
			break;

		case EStateTreePropertyRefType::Struct:
			PinType.PinCategory = UEdGraphSchema_K2::PC_Struct;
			PinType.PinSubCategoryObject = PropertyRef.GetTypeObject();
			break;

		case EStateTreePropertyRefType::Object:
			PinType.PinCategory = UEdGraphSchema_K2::PC_Object;
			PinType.PinSubCategoryObject = PropertyRef.GetTypeObject();
			break;

		default:
			ensureMsgf(false, TEXT("Unhandled type %s"), *UEnum::GetValueAsString(PropertyRef.GetRefType()));
			break;
		}
		return PinType;
	}

	FEdGraphPinType GetNativePropertyRefInternalTypeAsPin(const FProperty& RefProperty)
	{
		TArray<FEdGraphPinType, TInlineAllocator<1>> PinTypes = GetPropertyRefInternalTypesAsPins(RefProperty);
		if (PinTypes.Num() == 1)
		{
			return PinTypes[0];
		}
		return FEdGraphPinType();
	}

	FEdGraphPinType GetPropertyRefInternalTypeAsPin(const FProperty& RefProperty, const void* PropertyRefAddress)
	{
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(&RefProperty))
		{
			if (StructProperty->Struct == FStateTreePropertyRef::StaticStruct())
			{
				return GetNativePropertyRefInternalTypeAsPin(RefProperty);
			}
			else if (StructProperty->Struct == FStateTreeBlueprintPropertyRef::StaticStruct())
			{
				// The source of the chain can be an uninitialized object.
				if (PropertyRefAddress)
				{
					return GetBlueprintPropertyRefInternalTypeAsPin(*reinterpret_cast<const FStateTreeBlueprintPropertyRef*>(PropertyRefAddress));
				}
			}
		}

		checkNoEntry();
		return FEdGraphPinType();
	}

	void STATETREEMODULE_API GetBlueprintPropertyRefInternalTypeFromPin(const FEdGraphPinType& PinType, EStateTreePropertyRefType& OutRefType, bool& bOutIsArray, UObject*& OutObjectType)
	{
		OutRefType = EStateTreePropertyRefType::None;
		bOutIsArray = false;
		OutObjectType = nullptr;

		// Set container type
		switch (PinType.ContainerType)
		{
		case EPinContainerType::Array:
			bOutIsArray = true;
			break;
		case EPinContainerType::Set:
			ensureMsgf(false, TEXT("Unsuported container type [Set] "));
			break;
		case EPinContainerType::Map:
			ensureMsgf(false, TEXT("Unsuported container type [Map] "));
			break;
		default:
			break;
		}
	
		// Value type
		if (PinType.PinCategory == UEdGraphSchema_K2::PC_Boolean)
		{
			OutRefType = EStateTreePropertyRefType::Bool;
		}
		else if (PinType.PinCategory == UEdGraphSchema_K2::PC_Byte)
		{
			if (UEnum* Enum = Cast<UEnum>(PinType.PinSubCategoryObject))
			{
				OutRefType = EStateTreePropertyRefType::Enum;
				OutObjectType = PinType.PinSubCategoryObject.Get();
			}
			else
			{
				OutRefType = EStateTreePropertyRefType::Byte;
			}
		}
		else if (PinType.PinCategory == UEdGraphSchema_K2::PC_Int)
		{
			OutRefType = EStateTreePropertyRefType::Int32;
		}
		else if (PinType.PinCategory == UEdGraphSchema_K2::PC_Int64)
		{
			OutRefType = EStateTreePropertyRefType::Int64;
		}
		else if (PinType.PinCategory == UEdGraphSchema_K2::PC_Real)
		{
			if (PinType.PinSubCategory == UEdGraphSchema_K2::PC_Float)
			{
				OutRefType = EStateTreePropertyRefType::Float;
			}
			else if (PinType.PinSubCategory == UEdGraphSchema_K2::PC_Double)
			{
				OutRefType = EStateTreePropertyRefType::Double;
			}		
		}
		else if (PinType.PinCategory == UEdGraphSchema_K2::PC_Name)
		{
			OutRefType = EStateTreePropertyRefType::Name;
		}
		else if (PinType.PinCategory == UEdGraphSchema_K2::PC_String)
		{
			OutRefType = EStateTreePropertyRefType::String;
		}
		else if (PinType.PinCategory == UEdGraphSchema_K2::PC_Text)
		{
			OutRefType = EStateTreePropertyRefType::Text;
		}
		else if (PinType.PinCategory == UEdGraphSchema_K2::PC_Enum)
		{
			OutRefType = EStateTreePropertyRefType::Enum;
			OutObjectType = PinType.PinSubCategoryObject.Get();
		}
		else if (PinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
		{
			OutRefType = EStateTreePropertyRefType::Struct;
			OutObjectType = PinType.PinSubCategoryObject.Get();
		}
		else if (PinType.PinCategory == UEdGraphSchema_K2::PC_Object)
		{
			OutRefType = EStateTreePropertyRefType::Object;
			OutObjectType = PinType.PinSubCategoryObject.Get();
		}
		else if (PinType.PinCategory == UEdGraphSchema_K2::PC_SoftObject)
		{
			OutRefType = EStateTreePropertyRefType::SoftObject;
			OutObjectType = PinType.PinSubCategoryObject.Get();
		}
		else if (PinType.PinCategory == UEdGraphSchema_K2::PC_Class)
		{
			OutRefType = EStateTreePropertyRefType::Class;
			OutObjectType = PinType.PinSubCategoryObject.Get();
		}
		else if (PinType.PinCategory == UEdGraphSchema_K2::PC_SoftClass)
		{
			OutRefType = EStateTreePropertyRefType::SoftClass;
			OutObjectType = PinType.PinSubCategoryObject.Get();
		}
		else
		{
			ensureMsgf(false, TEXT("Unhandled pin category %s"), *PinType.PinCategory.ToString());
		}
	}

	bool STATETREEMODULE_API IsPropertyRefMarkedAsOptional(const FProperty& RefProperty, const void* PropertyRefAddress)
	{
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(&RefProperty))
		{
			if (StructProperty->Struct == FStateTreePropertyRef::StaticStruct())
			{
				return RefProperty.HasMetaData(IsOptionalName);
			}
			else if (StructProperty->Struct == FStateTreeBlueprintPropertyRef::StaticStruct())
			{
				check(PropertyRefAddress);
				return reinterpret_cast<const FStateTreeBlueprintPropertyRef*>(PropertyRefAddress)->IsOptional();
			}
		}

		checkNoEntry();
		return false;
	}

	TArray<FEdGraphPinType, TInlineAllocator<1>> GetPropertyRefInternalTypesAsPins(const FProperty& RefProperty)
	{
		ensure(IsPropertyRef(RefProperty));

		const EPinContainerType ContainerType = RefProperty.HasMetaData(IsRefToArrayName) ? EPinContainerType::Array : EPinContainerType::None;

		TArray<FEdGraphPinType, TInlineAllocator<1>> PinTypes;

		FString TargetTypesString = RefProperty.GetMetaData(RefTypeName);
		if (TargetTypesString.IsEmpty())
		{
			return PinTypes;
		}

		TArray<FString> TargetTypes;
		TargetTypesString.RemoveSpacesInline();
		TargetTypesString.ParseIntoArray(TargetTypes, TEXT(","), true);

		for (const FString& TargetType : TargetTypes)
		{
			const FName TargetTypeName = *TargetType;

			FEdGraphPinType& PinType = PinTypes.AddDefaulted_GetRef();
			PinType.ContainerType = ContainerType;

			if (TargetTypeName == BoolName)
			{
				PinType.PinCategory = UEdGraphSchema_K2::PC_Boolean;
			}
			else if (TargetTypeName == ByteName)
			{
				PinType.PinCategory = UEdGraphSchema_K2::PC_Byte;
			}
			else if (TargetTypeName == Int32Name)
			{
				PinType.PinCategory = UEdGraphSchema_K2::PC_Int;
			}
			else if (TargetTypeName == Int64Name)
			{
				PinType.PinCategory = UEdGraphSchema_K2::PC_Int64;
			}
			else if (TargetTypeName == FloatName)
			{
				PinType.PinCategory = UEdGraphSchema_K2::PC_Real;
				PinType.PinSubCategory = UEdGraphSchema_K2::PC_Float;
			}
			else if (TargetTypeName == DoubleName)
			{
				PinType.PinCategory = UEdGraphSchema_K2::PC_Real;
				PinType.PinSubCategory = UEdGraphSchema_K2::PC_Double;
			}
			else if (TargetTypeName == NameName)
			{
				PinType.PinCategory = UEdGraphSchema_K2::PC_Name;
			}
			else if (TargetTypeName == StringName)
			{
				PinType.PinCategory = UEdGraphSchema_K2::PC_String;
			}
			else if (TargetTypeName == TextName)
			{
				PinType.PinCategory = UEdGraphSchema_K2::PC_Text;
			}
			else
			{
				UField* TargetRefField = UClass::TryFindTypeSlow<UField>(TargetType);
				if (!TargetRefField)
				{
					TargetRefField = LoadObject<UField>(nullptr, *TargetType);
				}

				if (UScriptStruct* Struct = Cast<UScriptStruct>(TargetRefField))
				{
					PinType.PinCategory = UEdGraphSchema_K2::PC_Struct;
					PinType.PinSubCategoryObject = Struct;
				}
				else if (UClass* ObjectClass = Cast<UClass>(TargetRefField))
				{
					PinType.PinCategory = UEdGraphSchema_K2::PC_Object;
					PinType.PinSubCategoryObject = ObjectClass;
				}
				else if (UEnum* Enum = Cast<UEnum>(TargetRefField))
				{
					PinType.PinSubCategoryObject = Enum;
					if (Enum->GetCppForm() == UEnum::ECppForm::EnumClass)
					{
						PinType.PinCategory = UEdGraphSchema_K2::PC_Enum;
					}
					else
					{
						// Use byte for BP. It will use the correct picker and enum k2 node.
						PinType.PinCategory = UEdGraphSchema_K2::PC_Byte;
					}
				}
				else
				{
					checkf(false, TEXT("Typename in meta-data (%s) is invalid"), *TargetType);
				}
			}
		}
		return PinTypes;
	}
#endif

	bool IsBlueprintPropertyRefCompatibleWithProperty(const FProperty& SourceProperty, const void* PropertyRefAddress)
	{
		const FStateTreeBlueprintPropertyRef& PropertyRef = *reinterpret_cast<const FStateTreeBlueprintPropertyRef*>(PropertyRefAddress);
		const FProperty* TestProperty = &SourceProperty;
		if (PropertyRef.IsRefToArray())
		{
			if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(TestProperty))
			{
				TestProperty = ArrayProperty->Inner;
			}
			else
			{
				return false;
			}
		}

		switch (PropertyRef.GetRefType())
		{
		case EStateTreePropertyRefType::None:
			return false;

		case EStateTreePropertyRefType::Bool:
			return Validator<bool>::IsValid(*TestProperty);

		case EStateTreePropertyRefType::Byte:
			return Validator<uint8>::IsValid(*TestProperty);

		case EStateTreePropertyRefType::Int32:
			return Validator<int32>::IsValid(*TestProperty);

		case EStateTreePropertyRefType::Int64:
			return Validator<int64>::IsValid(*TestProperty);

		case EStateTreePropertyRefType::Float:
			return Validator<float>::IsValid(*TestProperty);

		case EStateTreePropertyRefType::Double:
			return Validator<double>::IsValid(*TestProperty);

		case EStateTreePropertyRefType::Name:
			return Validator<FName>::IsValid(*TestProperty);

		case EStateTreePropertyRefType::String:
			return Validator<FString>::IsValid(*TestProperty);

		case EStateTreePropertyRefType::Text:
			return Validator<FText>::IsValid(*TestProperty);

		case EStateTreePropertyRefType::Enum:
			if (const UEnum* Enum = Cast<UEnum>(PropertyRef.GetTypeObject()))
			{
				return IsPropertyCompatibleWithEnum(*TestProperty, *Enum);
			}
			return false;

		case EStateTreePropertyRefType::Struct:
			if (const UScriptStruct* Struct = Cast<UScriptStruct>(PropertyRef.GetTypeObject()))
			{
				return IsPropertyCompatibleWithStruct(*TestProperty, *Struct);
			}
			return false;

		case EStateTreePropertyRefType::Object:
			if (const UClass* Class = Cast<UClass>(PropertyRef.GetTypeObject()))
			{
				return IsPropertyCompatibleWithClass(*TestProperty, *Class);
			}
			return false;

		default:
			checkNoEntry();
		}

		return false;
	}

	bool IsPropertyRef(const FProperty& Property)
	{
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(&Property))
		{
			return StructProperty->Struct->IsChildOf(FStateTreePropertyRef::StaticStruct());
		}

		return false;
	}

	bool IsPropertyCompatibleWithEnum(const FProperty& Property, const UEnum& Enum)
	{
		if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(&Property))
		{
			return EnumProperty->GetEnum() == &Enum;
		}
		return false;
	}

	bool IsPropertyCompatibleWithClass(const FProperty& Property, const UClass& Class)
	{
		if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(&Property))
		{
			return ObjectProperty->PropertyClass == &Class;
		}
		return false;
	}

	bool IsPropertyCompatibleWithStruct(const FProperty& Property, const UScriptStruct& Struct)
	{
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(&Property))
		{
			return StructProperty->Struct == &Struct;
		}
		return false;
	}
}