// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeReference.h"
#include "StateTree.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeReference)

void FStateTreeReference::SyncParameters()
{
	if (StateTree == nullptr)
	{
		Parameters.Reset();
	}
	else
	{
		// In editor builds, sync with overrides.
		Parameters.MigrateToNewBagInstanceWithOverrides(StateTree->GetDefaultParameters(), PropertyOverrides);
		
		// Remove overrides that do not exists anymore
		if (!PropertyOverrides.IsEmpty())
		{
			if (const UPropertyBag* Bag = Parameters.GetPropertyBagStruct())
			{
				for (TArray<FGuid>::TIterator It = PropertyOverrides.CreateIterator(); It; ++It)
				{
					if (!Bag->FindPropertyDescByID(*It))
					{
						It.RemoveCurrentSwap();
					}
				}
			}
		}
	}
}

bool FStateTreeReference::RequiresParametersSync() const
{
	bool bShouldSync = false;
	
	if (StateTree)
	{
		const FInstancedPropertyBag& DefaultParameters = StateTree->GetDefaultParameters();
		const UPropertyBag* DefaultParametersBag = DefaultParameters.GetPropertyBagStruct();
		const UPropertyBag* ParametersBag = Parameters.GetPropertyBagStruct();
		
		// Mismatching property bags, needs sync.
		if (DefaultParametersBag != ParametersBag)
		{
			bShouldSync = true;
		}
		else if (ParametersBag && DefaultParametersBag)
		{
			// Check if non-overridden parameters are not identical, needs sync.
			const uint8* SourceAddress = DefaultParameters.GetValue().GetMemory();
			const uint8* TargetAddress = Parameters.GetValue().GetMemory();
			check(SourceAddress);
			check(TargetAddress);

			for (const FPropertyBagPropertyDesc& Desc : ParametersBag->GetPropertyDescs())
			{
				// Skip overridden
				if (PropertyOverrides.Contains(Desc.ID))
				{
					continue;
				}

				const uint8* SourceValueAddress = SourceAddress + Desc.CachedProperty->GetOffset_ForInternal();
				const uint8* TargetValueAddress = TargetAddress + Desc.CachedProperty->GetOffset_ForInternal();
				if (!Desc.CachedProperty->Identical(SourceValueAddress, TargetValueAddress))
				{
					// Mismatching values, should sync.
					bShouldSync = true;
					break;
				}
			}
		}
	}
	else
	{
		// Empty state tree should not have parameters
		bShouldSync = Parameters.IsValid();
	}
	
	return bShouldSync;
}

void FStateTreeReference::ConditionallySyncParameters() const
{
	if (RequiresParametersSync())
	{
		FStateTreeReference* NonConstThis = const_cast<FStateTreeReference*>(this);
		NonConstThis->SyncParameters();
		UE_LOGF(LogStateTree, Log, "Parameters for '%ls' stored in StateTreeReference were auto-fixed to be usable at runtime.", *GetNameSafe(StateTree));	
	}
}

void FStateTreeReference::SetPropertyOverridden(const FGuid PropertyID, const bool bIsOverridden)
{
	if (bIsOverridden)
	{
		PropertyOverrides.AddUnique(PropertyID);
	}
	else
	{
		PropertyOverrides.Remove(PropertyID);
		ConditionallySyncParameters();
	}
}

bool FStateTreeReference::Serialize(FStructuredArchive::FSlot Slot)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const FGuid StateTreeCustomVersion = FStateTreeCustomVersion::GUID;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	Slot.GetUnderlyingArchive().UsingCustomVersion(StateTreeCustomVersion);
	return false; // Let the default serializer handle serializing.
}

void FStateTreeReference::PostSerialize(const FArchive& Ar)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const int32 CurrentStateTreeCustomVersion = Ar.CustomVer(FStateTreeCustomVersion::GUID);
	constexpr int32 OverridableParametersVersion = FStateTreeCustomVersion::OverridableParameters;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (CurrentStateTreeCustomVersion < OverridableParametersVersion)
	{
		// In earlier versions, all parameters were overwritten.
		if (const UPropertyBag* Bag = Parameters.GetPropertyBagStruct())
		{
			for (const FPropertyBagPropertyDesc& Desc : Bag->GetPropertyDescs())
			{
				PropertyOverrides.Add(Desc.ID);
			}
		}
	}
}
