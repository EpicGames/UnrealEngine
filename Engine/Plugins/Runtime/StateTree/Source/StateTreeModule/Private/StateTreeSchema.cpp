// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeSchema.h"
#include "StateTreeTypes.h"
#include "Blueprint/StateTreeNodeBlueprintBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeSchema)

namespace UE::StateTree::Private
{
	bool bCompletedTransitionStatesCreateNewStates = true;
	FAutoConsoleVariableRef CVarCompletedTransitionStatesCreateNewStates(
		TEXT("StateTree.SelectState.CompletedTransitionStatesCreateNewStates"),
		bCompletedTransitionStatesCreateNewStates,
		TEXT("Activate the EStateTreeStateSelectionRules::CompletedTransitionStatesCreateNewStates rule.")
	);

	bool bCompletedStateBeforeTransitionSourceFailsTransition = true;
	FAutoConsoleVariableRef CVarCompletedStateBeforeTransitionSourceFailsTransition(
		TEXT("StateTree.SelectState.CompletedStateBeforeTransitionSourceFailsTransition"),
		bCompletedStateBeforeTransitionSourceFailsTransition,
		TEXT("Activate the EStateTreeStateSelectionRules::CompletedStateBeforeTransitionSourceFailsTransition rule.")
	);
}


bool UStateTreeSchema::IsChildOfBlueprintBase(const UClass* InClass)
{
	return InClass->IsChildOf<UStateTreeNodeBlueprintBase>();
}

EStateTreeParameterDataType UStateTreeSchema::GetGlobalParameterDataType() const
{
	return EStateTreeParameterDataType::GlobalParameterData;
}

EStateTreeStateSelectionRules UStateTreeSchema::GetStateSelectionRules() const
{
	EStateTreeStateSelectionRules Result = EStateTreeStateSelectionRules::None;
	if (UE::StateTree::Private::bCompletedTransitionStatesCreateNewStates)
	{
		Result |= EStateTreeStateSelectionRules::CompletedTransitionStatesCreateNewStates;
	}

	if (UE::StateTree::Private::bCompletedStateBeforeTransitionSourceFailsTransition)
	{
		Result |= EStateTreeStateSelectionRules::CompletedStateBeforeTransitionSourceFailsTransition;
	}

	return Result;
}
