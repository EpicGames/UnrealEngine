// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeStatePath.h"

#include "StateTree.h"
#include "StateTreeTypes.h"

namespace UE::StateTree
{
	const FActiveFrameID FActiveFrameID::Invalid = FActiveFrameID();
	const FActiveStateID FActiveStateID::Invalid = FActiveStateID();

	FActiveStatePath::FActiveStatePath(TNotNull<const UStateTree*> InStateTree, const TArrayView<const FActiveState> InElements)
		: StateTree(InStateTree)
		, States(InElements)
	{
#if WITH_STATETREE_DEBUG
		bool bIsValid = true;
		for (const FActiveState& Element : InElements)
		{
			if (!Element.GetStateHandle().IsValid() || Element.GetStateHandle().IsCompletionState())
			{
				bIsValid = false;
				break;
			}
		}
		checkf(bIsValid, TEXT("Contains an invalid handle."));
#endif
	}

	FActiveStatePath::FActiveStatePath(TNotNull<const UStateTree*> InStateTree, TArray<FActiveState> InElements)
		: StateTree(InStateTree)
		, States(MoveTemp(InElements))
	{
#if WITH_STATETREE_DEBUG
		bool bIsValid = true;
		for (const FActiveState& Element : InElements)
		{
			if (!Element.GetStateHandle().IsValid() || Element.GetStateHandle().IsCompletionState())
			{
				bIsValid = false;
				break;
			}
		}
		checkf(bIsValid, TEXT("Contains an invalid handle."));
#endif
	}

	bool FActiveStatePath::Matches(const TArrayView<const FActiveState>& A, const TArrayView<const FActiveState>& B)
	{
		return A.Num() == B.Num() && Intersect(A, B).Num() == A.Num();
	}

	bool FActiveStatePath::Matches(const FActiveStatePath& Other) const
	{
		return Matches(MakeConstArrayView(States), MakeConstArrayView(Other.States));
	}

	bool FActiveStatePath::Matches(const TArrayView<const FActiveState>& States, FActiveState Other)
	{
		return States.Num() > 0 && States.Last() == Other;
	}

	bool FActiveStatePath::Matches(FActiveState Other) const
	{
		return Matches(MakeConstArrayView(States), Other);
	}

	bool FActiveStatePath::Matches(const TArrayView<const FActiveState>& States, FActiveStateID Other)
	{
		return States.Num() > 0 && States.Last().GetStateID() == Other;
	}
	
	bool FActiveStatePath::Matches(FActiveStateID Other) const
	{
		return Matches(MakeConstArrayView(States), Other);
	}

	const TArrayView<const FActiveState> FActiveStatePath::Intersect(const TArrayView<const FActiveState>& A, const TArrayView<const FActiveState>& B)
	{
		// If the ID is the same, then are equal.
		if (A.Num() == B.Num()
			&& A.Num() > 0
			&& A.Last() == B.Last())
		{
			return A;
		}

		int32 MatchIndex = 0;
		const int32 MinNum = FMath::Min(A.Num(), B.Num());
		for (MatchIndex = 0; MatchIndex < MinNum; ++MatchIndex)
		{
			if (A[MatchIndex] != B[MatchIndex])
			{
				break;
			}
		}
		return MakeConstArrayView(A.GetData(), MatchIndex);
	}

	const TArrayView<const FActiveState> FActiveStatePath::Intersect(const FActiveStatePath& Other) const
	{
		return Intersect(MakeConstArrayView(States), MakeConstArrayView(Other.States));
	}

	bool FActiveStatePath::StartsWith(const TArrayView<const FActiveState>& A, const TArrayView<const FActiveState>& B)
	{
		// No need to check all the element. If the first and last match then they all matches.
		return B.Num() > 0 ? Contains(A, B.Last()) && A[0] == B[0] : false;
	}

	bool FActiveStatePath::StartsWith(const FActiveStatePath& Other) const
	{
		return StartsWith(MakeConstArrayView(States), MakeConstArrayView(Other.States));
	}

	bool FActiveStatePath::Contains(const TArrayView<const FActiveState>& States, FActiveState Other)
	{
		return States.Contains(Other);
	}

	bool FActiveStatePath::Contains(FActiveState Other) const
	{
		return Contains(MakeConstArrayView(States), Other);
	}

	bool FActiveStatePath::Contains(const TArrayView<const FActiveState>& States, FActiveStateID Other)
	{
		return Other.IsValid()
			&& States.ContainsByPredicate([Other](const FActiveState& Element)
			{
				return Element.GetStateID() == Other;
			});
	}

	bool FActiveStatePath::Contains(FActiveStateID Other) const
	{
		return Contains(MakeConstArrayView(States), Other);
	}

	int32 FActiveStatePath::IndexOf(const TArrayView<const FActiveState>& States, FActiveState Other)
	{
		return States.IndexOfByKey(Other);
	}

	int32 FActiveStatePath::IndexOf(FActiveState Other) const
	{
		return IndexOf(MakeConstArrayView(States), Other);
	}

	int32 FActiveStatePath::IndexOf(const TArrayView<const FActiveState>& States, FActiveStateID Other)
	{
		return Other.IsValid()
			? States.IndexOfByPredicate([Other](const FActiveState& Element)
				{
					return Element.GetStateID() == Other;
				})
			: INDEX_NONE;
	}

	int32 FActiveStatePath::IndexOf(FActiveStateID Other) const
	{
		return IndexOf(MakeConstArrayView(States), Other);
	}

	TValueOrError<FString, void> FActiveStatePath::Describe(TNotNull<const UStateTree*> StateTree, const TArrayView<const FActiveState>& States)
	{
		const UStateTree* CurrentTree = StateTree;
		TStringBuilder<256> Builder;
		bool bNewTree = true;
		for (const FActiveState& Element : States)
		{
			if (!Element.GetStateHandle().IsValid() || Element.GetStateHandle().IsCompletionState())
			{
				return MakeError();
			}

			{
				if (Builder.Len() > 0)
				{
					Builder << TEXT("; ");
				}
				if (bNewTree)
				{
					Builder << TEXT('{');
					CurrentTree->GetPathName(nullptr, Builder);
					Builder << TEXT('}');
					bNewTree = false;
				}

				if (!CurrentTree->GetStates().IsValidIndex(Element.GetStateHandle().Index))
				{
					return MakeError();
				}

				const FCompactStateTreeState& State = CurrentTree->GetStates()[Element.GetStateHandle().Index];
				Builder << State.Name;
				Builder << TEXT('(');
				Builder << Element.GetStateHandle().Index;
				Builder << TEXT(')');

				if (State.Type == EStateTreeStateType::LinkedAsset)
				{
					CurrentTree = State.LinkedAsset;
					if (CurrentTree == nullptr)
					{
						return MakeError();
					}
				}
			}
		}
		return MakeValue(Builder.ToString());
	}

	TValueOrError<FString, void> FActiveStatePath::Describe() const
	{
		const UStateTree* CurrentTree = StateTree.Get();
		if (CurrentTree == nullptr)
		{
			return MakeError();
		}
		return Describe(CurrentTree, MakeConstArrayView(States));
	}
}
