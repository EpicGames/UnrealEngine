// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeTaskBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeTaskBase)


#if WITH_GAMEPLAY_DEBUGGER
FString FStateTreeTaskBase::GetDebugInfo(const FStateTreeReadOnlyExecutionContext& Context) const
{
	// @todo: this needs to include more info
	TStringBuilder<256> Builder;
	Builder << TEXT('[');
	Builder << Name;
	Builder << TEXT("]\n");
	return Builder.ToString();
}
#endif // WITH_GAMEPLAY_DEBUGGER

