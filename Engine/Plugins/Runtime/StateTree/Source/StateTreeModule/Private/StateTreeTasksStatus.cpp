// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeTasksStatus.h"
#include "StateTree.h"
#include "StateTreeTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeTasksStatus)

namespace UE::StateTree::Private
{
	inline uint32* GetInlinedBufferPtr(FStateTreeTasksCompletionStatus::FMaskType*& Buffer)
	{
		return reinterpret_cast<uint32*>(&Buffer);
	}
}

FStateTreeTasksCompletionStatus::FStateTreeTasksCompletionStatus(const FCompactStateTreeFrame& Frame)
{
	BufferNum = Frame.NumberOfTasksStatusMasks;
	MallocBufferIfNeeded();
}

FStateTreeTasksCompletionStatus::~FStateTreeTasksCompletionStatus()
{
	if (!UseInlineBuffer())
	{
		FMemory::Free(Buffer);
	}
	Buffer = nullptr;
	BufferNum = 0;
}

FStateTreeTasksCompletionStatus::FStateTreeTasksCompletionStatus(const FStateTreeTasksCompletionStatus& Other)
	: BufferNum(Other.BufferNum)
{
	CopyBuffer(Other);
}

FStateTreeTasksCompletionStatus::FStateTreeTasksCompletionStatus(FStateTreeTasksCompletionStatus&& Other)
	: Buffer(Other.Buffer)
	, BufferNum(Other.BufferNum)
{
	Other.Buffer = nullptr;
	Other.BufferNum = 0;
}

FStateTreeTasksCompletionStatus& FStateTreeTasksCompletionStatus::operator=(const FStateTreeTasksCompletionStatus& Other)
{
	check(this != &Other);
	this->~FStateTreeTasksCompletionStatus();
	BufferNum = Other.BufferNum;
	CopyBuffer(Other);
	return *this;
}

FStateTreeTasksCompletionStatus& FStateTreeTasksCompletionStatus::operator=(FStateTreeTasksCompletionStatus&& Other)
{
	check(this != &Other);
	this->~FStateTreeTasksCompletionStatus();
	BufferNum = Other.BufferNum;
	Buffer = Other.Buffer;
	Other.Buffer = nullptr;
	Other.BufferNum = 0;
	return *this;
}

void FStateTreeTasksCompletionStatus::MallocBufferIfNeeded()
{
	if (!UseInlineBuffer())
	{
		const int32 NumberOfBytes = sizeof(FStateTreeTasksCompletionStatus::FMaskType) * BufferNum * 2;
		Buffer = (FMaskType*)FMemory::Malloc(sizeof(FMaskType) * NumberOfBytes);
		FMemory::Memzero(Buffer, NumberOfBytes);
	}
}

void FStateTreeTasksCompletionStatus::CopyBuffer(const FStateTreeTasksCompletionStatus& Other)
{
	if (!Other.UseInlineBuffer())
	{
		const int32 NumberOfBytes = sizeof(FStateTreeTasksCompletionStatus::FMaskType) * BufferNum * 2;
		Buffer = (FStateTreeTasksCompletionStatus::FMaskType*)FMemory::Malloc(NumberOfBytes);
		FMemory::Memcpy(Buffer, Other.Buffer, NumberOfBytes);
	}
	else
	{
		Buffer = Other.Buffer;
	}
}

template<typename TTasksCompletionStatusType>
TTasksCompletionStatusType FStateTreeTasksCompletionStatus::GetStatusInternal(FMaskType Mask, uint8 BufferIndex, uint8 BitsOffset, EStateTreeTaskCompletionType Control)
{
	const bool bIsValid = BufferNum > BufferIndex;
	if (bIsValid && UseInlineBuffer())
	{
		check(BufferIndex == 0);

		return TTasksCompletionStatusType(
			UE::StateTree::Private::GetInlinedBufferPtr(Buffer),
			UE::StateTree::Private::GetInlinedBufferPtr(Buffer) + 1,
			Mask,
			BitsOffset,
			Control
		);
	}

	if (!bIsValid)
	{
		check(false);
		// In case of invalid data (and the check continues), we prefer to not set any task completion than writing in random memory.
		// Because the mask is 0, no bit will be tested or set. The state tree will never complete.
		BufferNum = 1;
		return TTasksCompletionStatusType(
			UE::StateTree::Private::GetInlinedBufferPtr(Buffer),
			UE::StateTree::Private::GetInlinedBufferPtr(Buffer) + 1,
			0,
			0,
			EStateTreeTaskCompletionType::Any
		);
	}

	constexpr int32 NumberOfBuffers = 2;
	return TTasksCompletionStatusType(
		Buffer + (BufferIndex * NumberOfBuffers),
		Buffer + (BufferIndex * NumberOfBuffers) + 1,
		Mask,
		BitsOffset,
		Control
	);
}

UE::StateTree::FTasksCompletionStatus FStateTreeTasksCompletionStatus::GetStatus(const FCompactStateTreeState& State)
{
	return GetStatusInternal<UE::StateTree::FTasksCompletionStatus>(
		State.CompletionTasksMask,
		State.CompletionTasksMaskBufferIndex,
		State.CompletionTasksMaskBitsOffset,
		State.CompletionTasksControl
		);
}

UE::StateTree::FConstTasksCompletionStatus FStateTreeTasksCompletionStatus::GetStatus(const FCompactStateTreeState& State) const
{
	return const_cast<FStateTreeTasksCompletionStatus*>(this)->GetStatusInternal<UE::StateTree::FConstTasksCompletionStatus>(
		State.CompletionTasksMask,
		State.CompletionTasksMaskBufferIndex,
		State.CompletionTasksMaskBitsOffset,
		State.CompletionTasksControl
		);
}

UE::StateTree::FTasksCompletionStatus FStateTreeTasksCompletionStatus::GetStatus(TNotNull<const UStateTree*> StateTree)
{
	constexpr int32 BufferIndex = 0;
	constexpr int32 BitOffset = 0;
	return GetStatusInternal<UE::StateTree::FTasksCompletionStatus>(
		StateTree->CompletionGlobalTasksMask,
		BufferIndex,
		BitOffset,
		StateTree->CompletionGlobalTasksControl
		);
}

UE::StateTree::FConstTasksCompletionStatus FStateTreeTasksCompletionStatus::GetStatus(TNotNull<const UStateTree*> StateTree) const
{
	constexpr int32 BufferIndex = 0;
	constexpr int32 BitOffset = 0;
	return const_cast<FStateTreeTasksCompletionStatus*>(this)->GetStatusInternal<UE::StateTree::FConstTasksCompletionStatus>(
		StateTree->CompletionGlobalTasksMask,
		BufferIndex,
		BitOffset,
		StateTree->CompletionGlobalTasksControl)
		;
}

void FStateTreeTasksCompletionStatus::Push(const FCompactStateTreeState& State)
{
	check(BufferNum > State.CompletionTasksMaskBufferIndex);
	GetStatus(State).ResetStatus(State.TasksNum);
}

bool FStateTreeTasksCompletionStatus::Serialize(FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		int8 NewBufferNum = 0;
		Ar << NewBufferNum;
		if (NewBufferNum != BufferNum)
		{
			this->~FStateTreeTasksCompletionStatus();
			BufferNum = NewBufferNum;
			MallocBufferIfNeeded();
		}

		if (UseInlineBuffer())
		{
			Ar << *(UE::StateTree::Private::GetInlinedBufferPtr(Buffer));
			Ar << *(UE::StateTree::Private::GetInlinedBufferPtr(Buffer) + 1);
		}
		else
		{
			for (int32 Index = 0; Index < BufferNum*2; ++Index)
			{
				Ar << Buffer[Index];
			}
		}
	}
	else if (Ar.IsSaving())
	{
		Ar << BufferNum;
		if (UseInlineBuffer())
		{
			Ar << *(UE::StateTree::Private::GetInlinedBufferPtr(Buffer));
			Ar << *(UE::StateTree::Private::GetInlinedBufferPtr(Buffer) + 1);
		}
		else
		{
			for (int32 Index = 0; Index < BufferNum*2; ++Index)
			{
				Ar << Buffer[Index];
			}
		}
	}
	return true;
}

bool FStateTreeTasksCompletionStatus::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	bOutSuccess = true;
	return Serialize(Ar);
}