// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeTypes.h"
#include "StateTreeEvents.h"
#include "StateTree.h" // FStateTreeCustomVersion
#include "Math/ColorList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeTypes)

DEFINE_LOG_CATEGORY(LogStateTree);

namespace UE::StateTree::Colors
{
	FColor Darken(const FColor Col, const float Level)
	{
		const int32 Mul = (int32)FMath::Clamp(Level * 255.f, 0.f, 255.f);
		const int32 R = (int32)Col.R * Mul / 255;
		const int32 G = (int32)Col.G * Mul / 255;
		const int32 B = (int32)Col.B * Mul / 255;
		return FColor((uint8)R, (uint8)G, (uint8)B, Col.A);
	}

	const FColor Grey = FColor::FromHex(TEXT("#949494"));
	const FColor Red = FColor::FromHex(TEXT("#DE6659"));
	const FColor Orange = FColor::FromHex(TEXT("#E3983F"));
	const FColor Yellow = FColor::FromHex(TEXT("#EFD964"));
	const FColor Green = FColor::FromHex(TEXT("#8AB75E"));
	const FColor Cyan = FColor::FromHex(TEXT("#56C3BD"));
	const FColor Blue = FColor::FromHex(TEXT("#649ED3"));
	const FColor Purple = FColor::FromHex(TEXT("#B397D6"));
	const FColor Magenta = FColor::FromHex(TEXT("#CE85C7"));
	const FColor Bronze = FColorList::Bronze;

	constexpr float DarkenLevel = 0.6f;
	const FColor DarkGrey = Darken(Grey, DarkenLevel);
	const FColor DarkRed = Darken(Red, DarkenLevel);
	const FColor DarkOrange = Darken(Orange, DarkenLevel);
	const FColor DarkYellow = Darken(Yellow, DarkenLevel);
	const FColor DarkGreen = Darken(Green, DarkenLevel);
	const FColor DarkCyan = Darken(Cyan, DarkenLevel);
	const FColor DarkBlue = Darken(Blue, DarkenLevel);
	const FColor DarkPurple = Darken(Purple, DarkenLevel);
	const FColor DarkMagenta = Darken(Magenta, DarkenLevel);
	const FColor DarkBronze = Darken(Bronze, DarkenLevel);
} // UE::StateTree::Colors


const FStateTreeStateHandle FStateTreeStateHandle::Invalid = FStateTreeStateHandle();
const FStateTreeStateHandle FStateTreeStateHandle::Succeeded = FStateTreeStateHandle(SucceededIndex);
const FStateTreeStateHandle FStateTreeStateHandle::Failed = FStateTreeStateHandle(FailedIndex);
const FStateTreeStateHandle FStateTreeStateHandle::Stopped = FStateTreeStateHandle(StoppedIndex);
const FStateTreeStateHandle FStateTreeStateHandle::Root = FStateTreeStateHandle(0);

const FStateTreeDataHandle FStateTreeDataHandle::Invalid = FStateTreeDataHandle();

const FStateTreeIndex16 FStateTreeIndex16::Invalid = FStateTreeIndex16();
const FStateTreeIndex8 FStateTreeIndex8::Invalid = FStateTreeIndex8();


//////////////////////////////////////////////////////////////////////////
// FStateTreeStateHandle

EStateTreeRunStatus FStateTreeStateHandle::ToCompletionStatus() const
{
	if (Index == SucceededIndex)
	{
		return EStateTreeRunStatus::Succeeded;
	}

	if (Index == FailedIndex)
	{
		return EStateTreeRunStatus::Failed;
	}

	if (Index == StoppedIndex)
	{
		return EStateTreeRunStatus::Stopped;
	}
	return EStateTreeRunStatus::Unset;
}

FStateTreeStateHandle FStateTreeStateHandle::FromCompletionStatus(const EStateTreeRunStatus Status)
{
	if (Status == EStateTreeRunStatus::Succeeded)
	{
		return Succeeded;
	}

	if (Status == EStateTreeRunStatus::Failed)
	{
		return Failed;
	}

	if (Status == EStateTreeRunStatus::Stopped)
	{
		return Stopped;
	}
	return {};
}

FString FStateTreeStateHandle::Describe() const
{
	switch (Index)
	{
	case InvalidIndex:
		return TEXT("Invalid");
	case SucceededIndex:
		return TEXT("Succeeded");
	case FailedIndex:
		return TEXT("Failed");
	case StoppedIndex:
		return TEXT("Stopped");
	default:
		return FString::Printf(TEXT("%d"), Index);
	}
}

//////////////////////////////////////////////////////////////////////////
// FStateTreeStateLink

bool FStateTreeStateLink::Serialize(FStructuredArchive::FSlot Slot)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const FGuid StateTreeCustomVersionID = FStateTreeCustomVersion::GUID;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	Slot.GetUnderlyingArchive().UsingCustomVersion(StateTreeCustomVersionID);
	return false; // Let the default serializer handle serializing.
}

void FStateTreeStateLink::PostSerialize(const FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const int32 CurrentStateTreeCustomVersion = Ar.CustomVer(FStateTreeCustomVersion::GUID);
	constexpr int32 AddedExternalTransitionsVersion = FStateTreeCustomVersion::AddedExternalTransitions;

	if (CurrentStateTreeCustomVersion < AddedExternalTransitionsVersion)
	{
		LinkType = Type_DEPRECATED;
		if (LinkType == EStateTreeTransitionType::NotSet)
		{
			LinkType = EStateTreeTransitionType::None;
		}

	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
}

//////////////////////////////////////////////////////////////////////////
// FStateTreeDataHandle

bool FStateTreeDataHandle::IsObjectSource() const
{
	return Source == EStateTreeDataSourceType::GlobalInstanceDataObject
		|| Source == EStateTreeDataSourceType::ActiveInstanceDataObject
		|| Source == EStateTreeDataSourceType::SharedInstanceDataObject
		|| Source == EStateTreeDataSourceType::EvaluationScopeInstanceDataObject
		|| Source == EStateTreeDataSourceType::ExecutionRuntimeDataObject;
}

FStateTreeDataHandle FStateTreeDataHandle::ToObjectSource() const
{
	switch (Source)
	{
	case EStateTreeDataSourceType::GlobalInstanceData:
		return FStateTreeDataHandle(EStateTreeDataSourceType::GlobalInstanceDataObject, Index, StateHandle);
	case EStateTreeDataSourceType::ActiveInstanceData:
		return FStateTreeDataHandle(EStateTreeDataSourceType::ActiveInstanceDataObject, Index, StateHandle);
	case EStateTreeDataSourceType::SharedInstanceData:
		return FStateTreeDataHandle(EStateTreeDataSourceType::SharedInstanceDataObject, Index, StateHandle);
	case EStateTreeDataSourceType::EvaluationScopeInstanceData:
		return FStateTreeDataHandle(EStateTreeDataSourceType::EvaluationScopeInstanceDataObject, Index, StateHandle);
	case EStateTreeDataSourceType::ExecutionRuntimeData:
		return FStateTreeDataHandle(EStateTreeDataSourceType::ExecutionRuntimeDataObject, Index, StateHandle);
	case EStateTreeDataSourceType::ExecutionRuntimeDataAny:
		return FStateTreeDataHandle(EStateTreeDataSourceType::ExecutionRuntimeDataObject, Index, StateHandle);
	default:
		return *this;
	}
}

FStateTreeDataHandle FStateTreeDataHandle::ToStructSource() const
{
	switch (Source)
	{
	case EStateTreeDataSourceType::GlobalInstanceDataObject:
		return FStateTreeDataHandle(EStateTreeDataSourceType::GlobalInstanceData, Index, StateHandle);
	case EStateTreeDataSourceType::ActiveInstanceDataObject:
		return FStateTreeDataHandle(EStateTreeDataSourceType::ActiveInstanceData, Index, StateHandle);
	case EStateTreeDataSourceType::SharedInstanceDataObject:
		return FStateTreeDataHandle(EStateTreeDataSourceType::SharedInstanceData, Index, StateHandle);
	case EStateTreeDataSourceType::EvaluationScopeInstanceDataObject:
		return FStateTreeDataHandle(EStateTreeDataSourceType::EvaluationScopeInstanceData, Index, StateHandle);
	case EStateTreeDataSourceType::ExecutionRuntimeDataObject:
		return FStateTreeDataHandle(EStateTreeDataSourceType::ExecutionRuntimeData, Index, StateHandle);
	case EStateTreeDataSourceType::ExecutionRuntimeDataAny:
		return FStateTreeDataHandle(EStateTreeDataSourceType::ExecutionRuntimeData, Index, StateHandle);
	default:
		return *this;
	}
}

FString FStateTreeDataHandle::Describe() const
{
	switch (Source)
	{
	case EStateTreeDataSourceType::None:
		return TEXT("None");
	case EStateTreeDataSourceType::GlobalInstanceData:
		return FString::Printf(TEXT("Global[%d]"), Index);
	case EStateTreeDataSourceType::GlobalInstanceDataObject:
		return FString::Printf(TEXT("GlobalObj[%d]"), Index);
	case EStateTreeDataSourceType::ActiveInstanceData:
		return FString::Printf(TEXT("Active[%d]"), Index);
	case EStateTreeDataSourceType::ActiveInstanceDataObject:
		return FString::Printf(TEXT("ActiveObj[%d]"), Index);
	case EStateTreeDataSourceType::SharedInstanceData:
		return FString::Printf(TEXT("Shared[%d]"), Index);
	case EStateTreeDataSourceType::SharedInstanceDataObject:
		return FString::Printf(TEXT("SharedObj[%d]"), Index);
	case EStateTreeDataSourceType::EvaluationScopeInstanceData:
		return FString::Printf(TEXT("EvalScope[%d]"), Index);
	case EStateTreeDataSourceType::EvaluationScopeInstanceDataObject:
		return FString::Printf(TEXT("EvalScopeObj[%d]"), Index);
	case EStateTreeDataSourceType::ExecutionRuntimeData:
		return FString::Printf(TEXT("ExecRun[%d]"), Index);
	case EStateTreeDataSourceType::ExecutionRuntimeDataObject:
		return FString::Printf(TEXT("ExecRunObj[%d]"), Index);
	case EStateTreeDataSourceType::ContextData:
		return FString::Printf(TEXT("Context[%d]"), Index);
	case EStateTreeDataSourceType::GlobalParameterData:
		return TEXT("GlobalParam");
	case EStateTreeDataSourceType::ExternalGlobalParameterData:
		return TEXT("ExternalGlobalParam");
	case EStateTreeDataSourceType::SubtreeParameterData:
		return FString::Printf(TEXT("SubtreeParam[%d]"), Index);
	case EStateTreeDataSourceType::StateParameterData:
		return FString::Printf(TEXT("LinkedParam[%d]"), Index);
	default:
		return TEXT("---");
	}
}

//////////////////////////////////////////////////////////////////////////
// FCompactEventDesc

bool FCompactEventDesc::DoesEventMatchDesc(const FStateTreeEvent& Event) const
{
	if (Tag.IsValid() && (!Event.Tag.IsValid() || !Event.Tag.MatchesTag(Tag)))
	{
		return false;
	}

	const UScriptStruct* EventPayloadStruct = Event.Payload.GetScriptStruct();
	if (PayloadStruct && (EventPayloadStruct == nullptr || !EventPayloadStruct->IsChildOf(PayloadStruct)))
	{
		return false;
	}

	return true;
}