// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeDebugTextTask.h"
#include "StateTreeExecutionContext.h"
#include "GameFramework/Actor.h"
#include "DrawDebugHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeDebugTextTask)

#define LOCTEXT_NAMESPACE "StateTree"

FStateTreeDebugTextTask::FStateTreeDebugTextTask()
{
	bShouldCallTick = false;
	// We do not want to change the ReferenceActor and bEnabled if anything is bound.
	bShouldCopyBoundPropertiesOnTick = false;
	bShouldCopyBoundPropertiesOnExitState = false;

#if WITH_EDITORONLY_DATA
	bConsideredForCompletion = false;
	bCanEditConsideredForCompletion = false;
#endif
}

EStateTreeRunStatus FStateTreeDebugTextTask::EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	if (!InstanceData.bEnabled)
	{
		return EStateTreeRunStatus::Running;
	}
	
	const UWorld* World = Context.GetWorld();
	if (World == nullptr && InstanceData.ReferenceActor != nullptr)
	{
		World = InstanceData.ReferenceActor->GetWorld();
	}

	// Reference actor is not required (offset will be used as a global world location)
	// but a valid world is required.
	if (World == nullptr)
	{
		return EStateTreeRunStatus::Failed;
	}

	if (!Text.IsEmpty())
	{
		DrawDebugString(World, Offset, Text, InstanceData.ReferenceActor, TextColor, /*Duration*/-1, /*DrawShadows*/true, FontScale);
	}

	if (!InstanceData.BindableText.IsEmpty())
	{
		DrawDebugString(World, Offset, InstanceData.BindableText, InstanceData.ReferenceActor, TextColor, /*Duration*/-1, /*DrawShadows*/true, FontScale);
	}

	return EStateTreeRunStatus::Running;
}

void FStateTreeDebugTextTask::ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	if (!InstanceData.bEnabled)
	{
		return;
	}
	
	const UWorld* World = Context.GetWorld();
	if (World == nullptr && InstanceData.ReferenceActor != nullptr)
	{
		World = InstanceData.ReferenceActor->GetWorld();
	}

	// Reference actor is not required (offset was used as a global world location)
	// but a valid world is required.
	if (World == nullptr)
	{
		return;
	}

	// Drawing an empty text will remove the HUD DebugText entries associated to the target actor
	DrawDebugString(World, Offset, "",	InstanceData.ReferenceActor);
}

#if WITH_EDITOR
void FStateTreeDebugTextTask::PostLoad(FStateTreeDataView InstanceDataView)
{
	static constexpr int32 EnabledMovedToInstanceData_Version = 1;
	static constexpr int32 CurrentVersion = EnabledMovedToInstanceData_Version;

	if (DeprecatedVersion < EnabledMovedToInstanceData_Version)
	{
		if (FInstanceDataType* InstanceData = InstanceDataView.GetMutablePtr<FInstanceDataType>())
		{
			InstanceData->bEnabled = bEnabled_DEPRECATED;
			DeprecatedVersion = CurrentVersion;
		}
		else
		{
			UE_LOGF(LogStateTree, Error, "FStateTreeDebugTextTask::PostLoad: Invalid or unexpected instance data type.");
		}
	}
}

FText FStateTreeDebugTextTask::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	// Note that FStateTreeDebugTextTaskInstanceData::BindableText is not added to the formatted string
	//  since the bindings are not copied at this point so there is nothing to display when not at runtime.
	const FText Format = (Formatting == EStateTreeNodeFormatting::RichText)
		? LOCTEXT("DebugTextRich", "<b>Debug Text</> \"{Text}\"")
		: LOCTEXT("DebugText", "Debug Text \"{Text}\"");

	return FText::FormatNamed(Format,
		TEXT("Text"), FText::FromString(Text));
}
#endif

#undef LOCTEXT_NAMESPACE