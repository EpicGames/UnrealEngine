// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeTaskBase.h"
#include "StateTreeDebugTextTask.generated.h"

#define UE_API STATETREEMODULE_API

enum class EStateTreeRunStatus : uint8;
struct FStateTreeTransitionResult;

USTRUCT()
struct FStateTreeDebugTextTaskInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Parameter")
	bool bEnabled = true;

	/** Optional actor where to draw the text at. */
	UPROPERTY(EditAnywhere, Category = "Input", meta=(Optional))
	TObjectPtr<AActor> ReferenceActor = nullptr;

	UPROPERTY(EditAnywhere, Category = "Parameter")
	FString BindableText;
};

/**
 * Draws debug text on the HUD associated to the player controller.
 */
USTRUCT(meta = (DisplayName = "Debug Text Task"))
struct FStateTreeDebugTextTask : public FStateTreeTaskCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeDebugTextTaskInstanceData;
	
	UE_API FStateTreeDebugTextTask();

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	UE_API virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	UE_API virtual void ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
#if WITH_EDITOR
	UE_API virtual void PostLoad(FStateTreeDataView InstanceDataView) override;
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
	virtual FName GetIconName() const override
	{
		return FName("StateTreeEditorStyle|Node.Text");
	}
	virtual FColor GetIconColor() const override
	{
		return UE::StateTree::Colors::Grey;
	}
#endif
	
	UPROPERTY(EditAnywhere, Category = "Parameter")
	FString Text;

	UPROPERTY(EditAnywhere, Category = "Parameter")
	FColor TextColor = FColor::White;

	UPROPERTY(EditAnywhere, Category = "Parameter", meta=(ClampMin = 0, UIMin = 0))
	float FontScale = 1.0f;
	
	UPROPERTY(EditAnywhere, Category = "Parameter")
	FVector Offset = FVector::ZeroVector;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	bool bEnabled_DEPRECATED = true;

	UPROPERTY()
	int32 DeprecatedVersion = 0;
#endif
};

#undef UE_API
