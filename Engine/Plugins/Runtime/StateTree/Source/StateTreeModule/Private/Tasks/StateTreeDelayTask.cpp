// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeDelayTask.h"
#include "StateTreeExecutionContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeDelayTask)

#define LOCTEXT_NAMESPACE "StateTree"

FStateTreeDelayTask::FStateTreeDelayTask()
{
	bConsideredForScheduling = false;
	bShouldCopyBoundPropertiesOnTick = false;
	bShouldCopyBoundPropertiesOnExitState = false;
}

EStateTreeRunStatus FStateTreeDelayTask::EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	if (!InstanceData.bRunForever)
	{
		InstanceData.RemainingTime = FMath::FRandRange(
			FMath::Max(0.0f, InstanceData.Duration - InstanceData.RandomDeviation), (InstanceData.Duration + InstanceData.RandomDeviation));

		InstanceData.ScheduledTickHandle = Context.AddScheduledTickRequest(FStateTreeScheduledTick::MakeCustomTickRate(InstanceData.RemainingTime));
	}
	
	return EStateTreeRunStatus::Running;
}

EStateTreeRunStatus FStateTreeDelayTask::Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	if (!InstanceData.bRunForever)
	{
		InstanceData.RemainingTime -= DeltaTime;

		if (InstanceData.RemainingTime <= 0.f)
		{
			Context.RemoveScheduledTickRequest(InstanceData.ScheduledTickHandle);
			return EStateTreeRunStatus::Succeeded;
		}
		Context.UpdateScheduledTickRequest(InstanceData.ScheduledTickHandle, FStateTreeScheduledTick::MakeCustomTickRate(InstanceData.RemainingTime));
	}
	
	return EStateTreeRunStatus::Running;
}

void FStateTreeDelayTask::ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	Context.RemoveScheduledTickRequest(InstanceData.ScheduledTickHandle);
}

#if WITH_EDITOR
FText FStateTreeDelayTask::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	const FInstanceDataType* InstanceData = InstanceDataView.GetPtr<FInstanceDataType>();
	check(InstanceData);

	FText Value = FText::GetEmpty();

	if (const FPropertyBindingPath* RunForeverSourcePath = BindingLookup.GetPropertyBindingSource(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FStateTreeDelayTaskInstanceData, bRunForever))))
	{
		Value = FText::Format(LOCTEXT("ForeverBound", "Forever={0}"),
			BindingLookup.GetPropertyPathDisplayName(*RunForeverSourcePath, Formatting));
	}
	else if (InstanceData->bRunForever)
	{
		Value = LOCTEXT("Forever", "Forever");
	}
	else
	{
		FNumberFormattingOptions Options;
		Options.MinimumFractionalDigits = 1;
		Options.MaximumFractionalDigits = 3;

		FText DurationText = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, Duration)), Formatting);
		if (DurationText.IsEmpty())
		{
			DurationText = FText::AsNumber(InstanceData->Duration, &Options);
		}

		FText RandomDeviationText = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, RandomDeviation)), Formatting);
		if (RandomDeviationText.IsEmpty()
			&& !FMath::IsNearlyZero(InstanceData->RandomDeviation))
		{
			RandomDeviationText = FText::AsNumber(InstanceData->RandomDeviation, &Options);
		}

		if (RandomDeviationText.IsEmpty())
		{
			Value = DurationText;
		}
		else
		{
			if (Formatting == EStateTreeNodeFormatting::RichText)
			{
				Value = FText::Format(LOCTEXT("DelayValueRich", "{0} <s>\u00B1{1}</>"), // +-
					DurationText,
					RandomDeviationText);
			}
			else
			{
				Value = FText::Format(LOCTEXT("DelayValue", "{0} \u00B1{1}"), // +-
					DurationText,
					RandomDeviationText);
			}
		}
	}

	const FText Format = (Formatting == EStateTreeNodeFormatting::RichText)
		? LOCTEXT("DelayRich", "<b>Delay</> {Time}")
		: LOCTEXT("Delay", "Delay {Time}");

	return FText::FormatNamed(Format,
		TEXT("Time"), Value);
}
#endif

#undef LOCTEXT_NAMESPACE
