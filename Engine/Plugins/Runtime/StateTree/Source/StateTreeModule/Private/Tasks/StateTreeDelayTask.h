// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeTaskBase.h"
#include "StateTreeDelayTask.generated.h"

#define UE_API STATETREEMODULE_API

enum class EStateTreeRunStatus : uint8;
struct FStateTreeTransitionResult;

USTRUCT()
struct FStateTreeDelayTaskInstanceData
{
	GENERATED_BODY()
	
	/** Delay before the task ends. */
	UPROPERTY(EditAnywhere, Category = Parameter, meta = (EditCondition = "!bRunForever", ClampMin="0.0"))
	float Duration = 1.f;
	
	/** Adds random range to the Duration. */
	UPROPERTY(EditAnywhere, Category = Parameter, meta = (EditCondition = "!bRunForever", ClampMin="0.0"))
	float RandomDeviation = 0.f;
	
	/** If true the task will run forever until a transition stops it. */
	UPROPERTY(EditAnywhere, Category = Parameter)
	bool bRunForever = false;

	/** Internal countdown in seconds. */
	float RemainingTime = 0.f;

	/** The handle of the scheduled tick request. */
	UE::StateTree::FScheduledTickHandle ScheduledTickHandle;
};

/**
 * Simple task to wait indefinitely or for a given time (in seconds) before succeeding.
 */
USTRUCT(meta = (DisplayName = "Delay Task"))
struct FStateTreeDelayTask : public FStateTreeTaskCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeDelayTaskInstanceData;
	
	UE_API FStateTreeDelayTask();

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	UE_API virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	UE_API virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override;
	UE_API virtual void ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;

#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
	virtual FName GetIconName() const override
	{
		return FName("StateTreeEditorStyle|Node.Time");
	}
	virtual FColor GetIconColor() const override
	{
		return UE::StateTree::Colors::Grey;
	}
#endif
};

#undef UE_API
