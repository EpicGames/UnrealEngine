// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/StateTreeRunParallelStateTreeTask.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeExecutionTypes.h"
#include "StateTreeInstanceData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeRunParallelStateTreeTask)

#define LOCTEXT_NAMESPACE "StateTree"

void FStateTreeRunParallelStateTreeTaskInstanceData::UpdateGlobalParameters()
{
	if (RunningStateTree && RunningStateTree == StateTree.GetStateTree())
	{
		if (RunningStateTree->GetDefaultParameters().GetPropertyBagStruct() == StateTree.GetGlobalParameters().GetScriptStruct())
		{
			TreeInstanceData.GetMutableStorage().SetGlobalParameters(StateTree.GetGlobalParameters());
		}
		else
		{
			UE_LOGF(LogStateTree
				, Verbose
				, "StateTree '%ls' global parameters struct doesn't match the one of the parallel tree '%ls', global parameters won't be copied."
				, *GetPathNameSafe(StateTree.GetStateTree())
				, *RunningStateTree->GetPathName());
		}
	}
}

void FStateTreeRunParallelStateTreeExecutionExtension::ScheduleNextTick(const FContextParameters& Context, const FNextTickArguments& Args)
{
	const FStateTreeMinimalExecutionContext ExecutionContext(&Context.Owner, &Context.StateTree, Context.InstanceData);
	const FStateTreeScheduledTick ScheduledTick = ExecutionContext.GetNextScheduledTick();
	WeakExecutionContext.UpdateScheduledTickRequest(ScheduledTickHandle, ScheduledTick);
}

FStateTreeRunParallelStateTreeTask::FStateTreeRunParallelStateTreeTask()
{
	// The flags are updated FStateTreeRunParallelStateTreeTask::PostLoad
	bShouldCopyBoundPropertiesOnTick = false;
	bShouldCopyBoundPropertiesOnExitState = false;
	bShouldAffectTransitions = true;
	bConsideredForScheduling = false;
}

EStateTreeRunStatus FStateTreeRunParallelStateTreeTask::EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transitions) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	const UStateTree* CurrentlyProcessedTree = Context.GetCurrentlyProcessedFrame() != nullptr ? Context.GetCurrentlyProcessedFrame()->StateTree.Get() : nullptr;
	if (CurrentlyProcessedTree == nullptr)
	{
		UE_LOGF(LogStateTree, Warning, "Trying to start a parallel tree from an invalid context.");
		return EStateTreeRunStatus::Failed;
	}

	const FStateTreeReference& StateTreeToRun = GetStateTreeToRun(Context, InstanceData);
	if (!StateTreeToRun.IsValid())
	{
		UE_LOGF(LogStateTree, Warning, "Trying to start an invalid parallel tree from the tree '%ls'", *CurrentlyProcessedTree->GetPathName());
		return EStateTreeRunStatus::Failed;
	}

	// Find if it's a recursive call. The detection is not perfect. For example: StateTrees with a parallel task that links to each other cannot be detected.
	const bool bInParentContext = Context.GetActiveFrames().ContainsByPredicate([NewTree = StateTreeToRun.GetStateTree()](const FStateTreeExecutionFrame& Frame)
		{
			return Frame.StateTree == NewTree;
		});
	const bool bFromParentProcessedFrame = CurrentlyProcessedTree == StateTreeToRun.GetStateTree();
	if (bInParentContext || bFromParentProcessedFrame)
	{
		UE_LOGF(LogStateTree, Warning, "Trying to start a new parallel tree from the same tree '%ls'", *StateTreeToRun.GetStateTree()->GetPathName());
		return EStateTreeRunStatus::Failed;
	}

	InstanceData.RunningStateTree = StateTreeToRun.GetStateTree();
	FStateTreeExecutionContext ParallelTreeContext(Context, *InstanceData.RunningStateTree, InstanceData.TreeInstanceData);
	if (!ParallelTreeContext.IsValid())
	{
		return EStateTreeRunStatus::Failed;
	}

	FStateTreeInstanceData* OuterInstanceData = Context.GetMutableInstanceData();
	check(OuterInstanceData);

	FStateTreeRunParallelStateTreeExecutionExtension Extension;
	Extension.WeakExecutionContext = Context.MakeWeakExecutionContext();
	const EStateTreeRunStatus RunStatus = ParallelTreeContext.Start(FStateTreeExecutionContext::FStartParameters
		{
			.InitialGlobalParameters = StateTreeToRun.GetGlobalParameters(),
			.ExecutionExtension = TInstancedStruct<FStateTreeRunParallelStateTreeExecutionExtension>::Make(MoveTemp(Extension)),
			.SharedEventQueue = OuterInstanceData->GetSharedMutableEventQueue().ToSharedPtr()
		});

	if (RunStatus == EStateTreeRunStatus::Running)
	{
		check(!InstanceData.ScheduledTickHandle.IsValid());
		InstanceData.ScheduledTickHandle = Context.AddScheduledTickRequest(ParallelTreeContext.GetNextScheduledTick());
		InstanceData.TreeInstanceData.GetMutableExecutionState()->ExecutionExtension.GetMutable<FStateTreeRunParallelStateTreeExecutionExtension>().ScheduledTickHandle = InstanceData.ScheduledTickHandle;
	}

	return RunStatus;
}

EStateTreeRunStatus FStateTreeRunParallelStateTreeTask::Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	if (!InstanceData.RunningStateTree)
	{
		return EStateTreeRunStatus::Failed;
	}

	FStateTreeExecutionContext ParallelTreeContext(Context, *InstanceData.RunningStateTree, InstanceData.TreeInstanceData);
	if (!ParallelTreeContext.IsValid())
	{
		return EStateTreeRunStatus::Failed;
	}

	if (bShouldCopyParametersOnTick)
	{
		InstanceData.UpdateGlobalParameters();
	}

	const EStateTreeRunStatus RunStatus = ParallelTreeContext.TickUpdateTasks(DeltaTime);
	Context.UpdateScheduledTickRequest(InstanceData.ScheduledTickHandle, ParallelTreeContext.GetNextScheduledTick());
	return RunStatus;
}

void FStateTreeRunParallelStateTreeTask::TriggerTransitions(FStateTreeExecutionContext& Context) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	if (!InstanceData.RunningStateTree)
	{
		return;
	}

	FStateTreeExecutionContext ParallelTreeContext(Context, *InstanceData.RunningStateTree, InstanceData.TreeInstanceData);
	if (!ParallelTreeContext.IsValid())
	{
		return;
	}

	const EStateTreeRunStatus LastTreeRunStatus = InstanceData.TreeInstanceData.GetExecutionState()->TreeRunStatus;

	{
		// The parent tree owning the event queue clears events at end of transition processing phase
		// Parallel Trees process state selection, exiting/entering states inside parent tree's transition processing phase
		// Events need to live till the parent tree's next transition processing phase so events sent during task's enter state can be sustained correctly
		using namespace UE::StateTree::Event;
		FEventsPendingForNextTransitionProcessingScope PendingEventsScope(&InstanceData.TreeInstanceData.GetMutableEventQueue());
		ParallelTreeContext.TickTriggerTransitions();
	}

	const EStateTreeRunStatus NewTreeRunStatus = InstanceData.TreeInstanceData.GetExecutionState()->TreeRunStatus;
	if (LastTreeRunStatus != NewTreeRunStatus)
	{
		ensure(NewTreeRunStatus != EStateTreeRunStatus::Running);
		Context.FinishTask(*this, NewTreeRunStatus == EStateTreeRunStatus::Succeeded ? EStateTreeFinishTaskType::Succeeded : EStateTreeFinishTaskType::Failed);
	}
	Context.UpdateScheduledTickRequest(InstanceData.ScheduledTickHandle, ParallelTreeContext.GetNextScheduledTick());
}

void FStateTreeRunParallelStateTreeTask::ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	// Reset Tick Handle regardless of whether the tree is still valid
	if (InstanceData.ScheduledTickHandle.IsValid())
	{
		Context.RemoveScheduledTickRequest(InstanceData.ScheduledTickHandle);
		InstanceData.ScheduledTickHandle = UE::StateTree::FScheduledTickHandle();
		InstanceData.TreeInstanceData.GetMutableExecutionState()->ExecutionExtension.GetMutable<FStateTreeRunParallelStateTreeExecutionExtension>().ScheduledTickHandle = InstanceData.ScheduledTickHandle;
	}

	if (!InstanceData.RunningStateTree)
	{
		return;
	}

	FStateTreeExecutionContext ParallelTreeContext(Context, *InstanceData.RunningStateTree, InstanceData.TreeInstanceData);
	if (!ParallelTreeContext.IsValid())
	{
		return;
	}

	if (bShouldCopyParametersOnExitState)
	{
		InstanceData.UpdateGlobalParameters();
	}

	ParallelTreeContext.Stop();
}

const FStateTreeReference& FStateTreeRunParallelStateTreeTask::GetStateTreeToRun(FStateTreeExecutionContext& Context, FInstanceDataType& InstanceData) const
{
	if (StateTreeOverrideTag.IsValid())
	{
		if (const FStateTreeReference* Override = Context.GetLinkedStateTreeOverrideForTag(StateTreeOverrideTag))
		{
			return *Override;
		}
	}

	return InstanceData.StateTree;
}

void FStateTreeRunParallelStateTreeTask::PostLoad(FStateTreeDataView InstanceDataView)
{
	if (FInstanceDataType* DataType = InstanceDataView.GetMutablePtr<FInstanceDataType>())
	{
		DataType->StateTree.SyncParameters();
	}

	bShouldCopyBoundPropertiesOnTick = bShouldCopyParametersOnTick;
	bShouldCopyBoundPropertiesOnExitState = bShouldCopyParametersOnExitState;
}

#if WITH_EDITOR
EDataValidationResult FStateTreeRunParallelStateTreeTask::Compile(UE::StateTree::ICompileNodeContext& Context)
{
	TransitionHandlingPriority = EventHandlingPriority;
	bShouldCopyBoundPropertiesOnTick = bShouldCopyParametersOnTick;
	bShouldCopyBoundPropertiesOnExitState = bShouldCopyParametersOnExitState;

	return EDataValidationResult::Valid;
}

void FStateTreeRunParallelStateTreeTask::PostEditInstanceDataChangeChainProperty(const FPropertyChangedChainEvent& PropertyChangedEvent, FStateTreeDataView InstanceDataView)
{
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FStateTreeRunParallelStateTreeTaskInstanceData, StateTree))
	{
		InstanceDataView.GetMutable<FInstanceDataType>().StateTree.SyncParameters();
	}
}

FText FStateTreeRunParallelStateTreeTask::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	const FInstanceDataType* InstanceData = InstanceDataView.GetPtr<FInstanceDataType>();
	check(InstanceData);

	FText StateTreeValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, StateTree)), Formatting);
	if (StateTreeValue.IsEmpty())
	{
		StateTreeValue = FText::FromString(GetNameSafe(InstanceData->StateTree.GetStateTree()));
	}

	const FText Format = (Formatting == EStateTreeNodeFormatting::RichText)
		? LOCTEXT("RunParallelRich", "<b>Run Parallel</> {Asset}")
		: LOCTEXT("RunParallel", "Run Parallel {Asset}");

	return FText::FormatNamed(Format,
		TEXT("Asset"), StateTreeValue);
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
