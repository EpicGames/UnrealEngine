// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubclassOf.h"
#include "StateTreeConditionBase.h"
#include "StateTreeNodeBlueprintBase.h"
#include "StateTreeConditionBlueprintBase.generated.h"

#define UE_API STATETREEMODULE_API

struct FStateTreeExecutionContext;

/*
 * Base class for Blueprint based Conditions. 
 */
UCLASS(MinimalAPI, Abstract, Blueprintable)
class UStateTreeConditionBlueprintBase : public UStateTreeNodeBlueprintBase
{
	GENERATED_BODY()
public:
	UE_API UStateTreeConditionBlueprintBase(const FObjectInitializer& ObjectInitializer);

	UFUNCTION(BlueprintImplementableEvent)
	UE_API bool ReceiveTestCondition() const;

protected:
	UE_API virtual bool TestCondition(FStateTreeExecutionContext& Context) const;

	friend struct FStateTreeBlueprintConditionWrapper;

	uint8 bHasTestCondition : 1;
};

/**
 * Wrapper for Blueprint based Conditions.
 */
USTRUCT()
struct FStateTreeBlueprintConditionWrapper : public FStateTreeConditionBase
{
	GENERATED_BODY()

	virtual const UStruct* GetInstanceDataType() const override { return ConditionClass; };
	UE_API virtual bool TestCondition(FStateTreeExecutionContext& Context) const override;
#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
	UE_API virtual FName GetIconName() const override;
	UE_API virtual FColor GetIconColor() const override;
#endif
	
	UPROPERTY()
	TSubclassOf<UStateTreeConditionBlueprintBase> ConditionClass;
};

#undef UE_API
