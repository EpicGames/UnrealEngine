// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeConsiderationBase.h"
#include "StateTreeNodeBlueprintBase.h"
#include "Templates/SubclassOf.h"
#include "StateTreeConsiderationBlueprintBase.generated.h"

#define UE_API STATETREEMODULE_API

struct FStateTreeExecutionContext;

/*
 * Base class for Blueprint based Considerations.
 */
UCLASS(MinimalAPI, Abstract, Blueprintable)
class UStateTreeConsiderationBlueprintBase : public UStateTreeNodeBlueprintBase
{
	GENERATED_BODY()

public:
	UE_API UStateTreeConsiderationBlueprintBase(const FObjectInitializer& ObjectInitializer);

	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "GetScore"))
	UE_API float ReceiveGetScore() const;

protected:
	UE_API virtual float GetScore(FStateTreeExecutionContext& Context) const;

	friend struct FStateTreeBlueprintConsiderationWrapper;

	uint8 bHasGetScore : 1;
};

/**
 * Wrapper for Blueprint based Considerations.
 */
USTRUCT()
struct FStateTreeBlueprintConsiderationWrapper : public FStateTreeConsiderationBase
{
	GENERATED_BODY()

	//~ Begin FStateTreeNodeBase Interface
	virtual const UStruct* GetInstanceDataType() const override { return ConsiderationClass; };
#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
	UE_API virtual FName GetIconName() const override;
	UE_API virtual FColor GetIconColor() const override;
#endif //WITH_EDITOR
	//~ End FStateTreeNodeBase Interface

protected:
	//~ Begin FStateTreeConsiderationBase Interface
	UE_API virtual float GetScore(FStateTreeExecutionContext& Context) const override;
	//~ End FStateTreeConsiderationBase Interface

public:
	UPROPERTY()
	TSubclassOf<UStateTreeConsiderationBlueprintBase> ConsiderationClass;
};

#undef UE_API
