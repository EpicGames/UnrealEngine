// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubclassOf.h"
#include "StateTreeEvaluatorBase.h"
#include "StateTreeNodeBlueprintBase.h"
#include "StateTreeEvaluatorBlueprintBase.generated.h"

#define UE_API STATETREEMODULE_API

struct FStateTreeExecutionContext;

/*
 * Base class for Blueprint based evaluators. 
 */
UCLASS(MinimalAPI, Abstract, Blueprintable)
class UStateTreeEvaluatorBlueprintBase : public UStateTreeNodeBlueprintBase
{
	GENERATED_BODY()
public:
	UE_API UStateTreeEvaluatorBlueprintBase(const FObjectInitializer& ObjectInitializer);

	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "TreeStart"))
	UE_API void ReceiveTreeStart();

	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "TreeStop"))
	UE_API void ReceiveTreeStop();

	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Tick"))
	UE_API void ReceiveTick(const float DeltaTime);

protected:
	UE_API virtual void TreeStart(FStateTreeExecutionContext& Context);
	UE_API virtual void TreeStop(FStateTreeExecutionContext& Context);
	UE_API virtual void Tick(FStateTreeExecutionContext& Context, const float DeltaTime);

	uint8 bHasTreeStart : 1;
	uint8 bHasTreeStop : 1;
	uint8 bHasTick : 1;

	friend struct FStateTreeBlueprintEvaluatorWrapper;
};

/**
 * Wrapper for Blueprint based Evaluators.
 */
USTRUCT()
struct FStateTreeBlueprintEvaluatorWrapper : public FStateTreeEvaluatorBase
{
	GENERATED_BODY()

	virtual const UStruct* GetInstanceDataType() const override { return EvaluatorClass; };
	
	UE_API virtual void TreeStart(FStateTreeExecutionContext& Context) const override;
	UE_API virtual void TreeStop(FStateTreeExecutionContext& Context) const override;
	UE_API virtual void Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override;
#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
	UE_API virtual FName GetIconName() const override;
	UE_API virtual FColor GetIconColor() const override;
#endif
	
	UPROPERTY()
	TSubclassOf<UStateTreeEvaluatorBlueprintBase> EvaluatorClass;
};

#undef UE_API
