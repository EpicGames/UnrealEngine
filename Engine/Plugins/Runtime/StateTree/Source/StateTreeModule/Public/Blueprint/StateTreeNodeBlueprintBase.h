// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeExecutionTypes.h"
#include "StateTreeNodeBase.h"
#include "UObject/ObjectMacros.h"
#include "StateTreeAsyncExecutionContext.h"
#include "StateTreeNodeBlueprintBase.generated.h"

#define UE_API STATETREEMODULE_API

struct FStateTreeEvent;
struct FStateTreeEventQueue;
struct FStateTreeInstanceStorage;
struct FStateTreeLinker;
struct FStateTreeExecutionContext;
struct FStateTreeBlueprintPropertyRef;
class UStateTree;

UENUM()
enum class EStateTreeBlueprintPropertyCategory : uint8
{
	NotSet,
	Input,	
	Parameter,
	Output,
	ContextObject,
};


/** Struct use to copy external data to the Blueprint item instance, resolved during StateTree linking. */
struct FStateTreeBlueprintExternalDataHandle
{
	const FProperty* Property = nullptr;
	FStateTreeExternalDataHandle Handle;
};


UCLASS(MinimalAPI, Abstract, meta = (DisallowLevelActorReference = true))
class UStateTreeNodeBlueprintBase : public UObject
{
	GENERATED_BODY()

public:
	/** Sends event to the StateTree. */
	UFUNCTION(BlueprintCallable, Category = "StateTree", meta = (HideSelfPin = "true", DisplayName = "StateTree Send Event"))
	UE_API void SendEvent(const FStateTreeEvent& Event);

	/** Request state transition. */
	UFUNCTION(BlueprintCallable, Category = "StateTree", meta = (HideSelfPin = "true", DisplayName = "StateTree Request Transition"))
	UE_API void RequestTransition(const FStateTreeStateLink& TargetState, const EStateTreeTransitionPriority Priority = EStateTreeTransitionPriority::Normal);

	/** Returns a reference to selected property in State Tree. */
	UFUNCTION(CustomThunk)
	UE_API void GetPropertyReference(const FStateTreeBlueprintPropertyRef& PropertyRef) const;

	/** Returns true if reference to selected property in State Tree is accessible. */
	UFUNCTION()
	UE_API bool IsPropertyRefValid(const FStateTreeBlueprintPropertyRef& PropertyRef) const;

	/** @return text describing the property, either direct value or binding description. Used internally. */
	UFUNCTION(BlueprintCallable, Category = "StateTree", meta=( BlueprintInternalUseOnly="true" ))
	UE_API FText GetPropertyDescriptionByPropertyName(FName PropertyName) const;

#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const;

	FName GetIconName() const
	{
		return IconName;
	}
	
	FColor GetIconColor() const
	{
		return IconColor;
	}
#endif
	
protected:

	/** Event to implement to get node description. */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Get Description"))
	UE_API FText ReceiveGetDescription(EStateTreeNodeFormatting Formatting) const;

	UE_API virtual UWorld* GetWorld() const override;
	UE_API AActor* GetOwnerActor(const FStateTreeExecutionContext& Context) const;

	/** These methods are const as they set mutable variables and need to be called from a const method. */
	UE_API void SetCachedInstanceDataFromContext(const FStateTreeExecutionContext& Context) const;
	UE_API void ClearCachedInstanceData() const;

	FStateTreeWeakExecutionContext GetWeakExecutionContext() const
	{
		return WeakExecutionContext;
	}

private:
	DECLARE_FUNCTION(execGetPropertyReference);

	UE_API void* GetMutablePtrToProperty(const FStateTreeBlueprintPropertyRef& PropertyRef, FProperty*& OutSourceProperty) const;
	
	/** Cached execution context while the node is active for async nodes. */
	mutable FStateTreeWeakExecutionContext WeakExecutionContext;

#if WITH_EDITORONLY_DATA

	UE_DEPRECATED(5.6, "WeakInstanceStorage is deprecated. Use WeakExecutionContext")
	/** Cached instance data while the node is active for async nodes. */
	mutable TWeakPtr<FStateTreeInstanceStorage> WeakInstanceStorage;

	UE_DEPRECATED(5.6, "CachedFrameStateTree is deprecated.")
	/** Cached State Tree of owning execution frame. */
	UPROPERTY()
	mutable TObjectPtr<const UStateTree> CachedFrameStateTree = nullptr;

	UE_DEPRECATED(5.6, "CachedFrameRootState is deprecated.")
	/** Cached root state of owning execution frame. */
	mutable FStateTreeStateHandle CachedFrameRootState;

	/** Description of the node. */
	UPROPERTY(EditDefaultsOnly, Category="Description")
	FText Description;

	/**
	 * Name of the icon in format:
	 *		StyleSetName | StyleName [ | SmallStyleName | StatusOverlayStyleName]
	 *		SmallStyleName and StatusOverlayStyleName are optional.
	 *		Example: "StateTreeEditorStyle|Node.Animation"
	 */
	UPROPERTY(EditDefaultsOnly, Category="Description")
	FName IconName;

	/** Color of the icon. */
	UPROPERTY(EditDefaultsOnly, Category="Description")
	FColor IconColor = UE::StateTree::Colors::Grey;
#endif // 	WITH_EDITORONLY_DATA

#if WITH_EDITOR
	/** Cached values used during editor to make some BP nodes simpler to use. */
	static UE_API FGuid CachedNodeID;
	static UE_API const IStateTreeBindingLookup* CachedBindingLookup;
#endif	
};

#undef UE_API
