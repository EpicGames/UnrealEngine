// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubclassOf.h"
#include "StateTreeTaskBase.h"
#include "StateTreeNodeBlueprintBase.h"
#include "StateTreeNodeRef.h"
#include "StateTreeTaskBlueprintBase.generated.h"

#define UE_API STATETREEMODULE_API

struct FStateTreeExecutionContext;

DECLARE_DYNAMIC_DELEGATE(FStateTreeDynamicDelegate);

/*
 * Base class for Blueprint based Tasks.
 */
UCLASS(MinimalAPI, Abstract, Blueprintable)
class UStateTreeTaskBlueprintBase : public UStateTreeNodeBlueprintBase
{
	GENERATED_BODY()

public:
	UE_API UStateTreeTaskBlueprintBase(const FObjectInitializer& ObjectInitializer);

	/**
	 * Called when a new state is entered and task is part of active states.
	 * Use FinishTask() to set the task execution completed. State completion is controlled by completed tasks.
	 *
	 * GameplayTasks and other latent actions should be generally triggered on EnterState. When using a GameplayTasks it's required
	 * to manually cancel active tasks on ExitState if the GameplayTask's lifetime is tied to the State Tree task.
	 *
	 * @param Transition Describes the states involved in the transition
	 */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "EnterState"))
	UE_API void ReceiveLatentEnterState(const FStateTreeTransitionResult& Transition);

	/**
	 * Called when a current state is exited and task is part of active states.
	 * @param Transition Describes the states involved in the transition
	 */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "ExitState"))
	UE_API void ReceiveExitState(const FStateTreeTransitionResult& Transition);

	/**
	 * Called right after a state has been completed, but before new state has been selected. StateCompleted is called in reverse order to allow to propagate state to other Tasks that
	 * are executed earlier in the tree. Note that StateCompleted is not called if conditional transition changes the state.
	 * @param CompletionStatus Describes the running status of the completed state (Succeeded/Failed).
	 * @param CompletedActiveStates Active states at the time of completion.
	 */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "StateCompleted"))
	UE_API void ReceiveStateCompleted(const EStateTreeRunStatus CompletionStatus, const FStateTreeActiveStates CompletedActiveStates);

	/**
	 * Called during state tree tick when the task is on active state.
	 * Use FinishTask() to set the task execution completed. State completion is controlled by completed tasks.
	 *
	 * Triggering GameplayTasks and other latent action should generally be done on EnterState. Tick is called on each update (or event)
	 * and can cause huge amount of task added if the logic is not handled carefully.
	 * Tick should be generally be used for monitoring that require polling, or actions that require constant ticking.  
	 *
	 * Note: The method is called only if bShouldCallTick or bShouldCallTickOnlyOnEvents is set.
	 * @param DeltaTime Time since last StateTree tick.
	 */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Tick"))
	UE_API void ReceiveLatentTick(const float DeltaTime);

	UE_DEPRECATED(all, "Use the new EnterState event without without return value instead. Task status is now controlled via FinishTask node, instead of a return value. Default status is running.")
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "EnterState (Deprecated)", DeprecatedFunction, DeprecationMessage="Use the new EnterState event without without return value instead. Task status is now controlled via FinishTask node, instead of a return value. Default status is running."))
	UE_API EStateTreeRunStatus ReceiveEnterState(const FStateTreeTransitionResult& Transition);

	UE_DEPRECATED(all, "Use the new Tick event without without return value instead. Task status is now controlled via FinishTask node, instead of a return value. Default status is running.")
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Tick (Deprecated)", DeprecatedFunction, DeprecationMessage="Use the new Tick event without without return value instead. Task status is now controlled via FinishTask node, instead of a return value. Default status is running."))
	UE_API EStateTreeRunStatus ReceiveTick(const float DeltaTime);

protected:
	UE_API virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition);
	UE_API virtual void ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition);

	UE_API virtual void StateCompleted(FStateTreeExecutionContext& Context, const EStateTreeRunStatus CompletionStatus, const FStateTreeActiveStates& CompletedActiveStates);
	UE_API virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& Context, const float DeltaTime);

	/** Finish the task and sets it's status. */
	UFUNCTION(BlueprintCallable, Category = "StateTree", meta = (HideSelfPin = "true", DisplayName = "Finish Task"))
	UE_API void FinishTask(const bool bSucceeded = true);

	/** Broadcasts the dispatcher. It will triggers bound callback (bound with BindDelegate) and bound transitions. */
	UFUNCTION(BlueprintCallable, Category = "StateTree")
	UE_API void BroadcastDelegate(FStateTreeDelegateDispatcher Dispatcher);

	/**
	 * Registers the callback to the listener.
	 * If the listener was previously registered, then unregister it first before registering it again with the new delegate callback.
	 * The listener is bound to a dispatcher in the editor.
	 */
	UFUNCTION(BlueprintCallable, Category = "StateTree")
	UE_API void BindDelegate(const FStateTreeDelegateListener& Listener, const FStateTreeDynamicDelegate& Delegate);
	
	/** Unregisters the callback bound to the listener. */
	UFUNCTION(BlueprintCallable, Category = "StateTree")
	UE_API void UnbindDelegate(const FStateTreeDelegateListener& Listener);

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.6, "FStateTreeWeakTaskRef is deprecated.")
	/** Cached task while the node is active for async execution. */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	mutable FStateTreeWeakTaskRef WeakTaskRef;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	/** Run status when using latent EnterState and Tick */
	EStateTreeRunStatus RunStatus = EStateTreeRunStatus::Running;
	
	/**
	 * If set to true, the task will receive EnterState/ExitState even if the state was previously active.
	 * Generally this should be true for action type tasks, like playing animation,
	 * and false on state like tasks like claiming a resource that is expected to be acquired on child states. */
	UPROPERTY(EditDefaultsOnly, Category="Default")
	uint8 bShouldStateChangeOnReselect : 1;

	/**
	 * If set to true, Tick() is called. Not ticking implies no property copy. Default true.
	 * Note: this is intentionally not a property, should be only set by C++ derived classes when the tick should not be called.
	 */
	uint8 bShouldCallTick : 1;

	/** If set to true, Tick() is called. Default false. */
	UPROPERTY(EditDefaultsOnly, Category="Default")
	uint8 bShouldCallTickOnlyOnEvents : 1;

	/** If set to true, copy the values of bound properties before calling Tick(). Default true. */
	UPROPERTY(EditDefaultsOnly, Category="Default")
	uint8 bShouldCopyBoundPropertiesOnTick : 1;
	
	/** If set to true, copy the values of bound properties before calling ExitState(). Default true. */
	UPROPERTY(EditDefaultsOnly, Category="Default")
	uint8 bShouldCopyBoundPropertiesOnExitState : 1;

public:
#if WITH_EDITORONLY_DATA
	/**
	 * True if the task is considered for completion.
	 * False if the task runs in the background without affecting the state completion.
	 */
	UPROPERTY(EditDefaultsOnly, Category = "Default")
	uint8 bConsideredForCompletion : 1;

	/** True if the user can edit bConsideredForCompletion in the editor. */
	UPROPERTY(EditDefaultsOnly, Category = "Default")
	uint8 bCanEditConsideredForCompletion : 1;
#endif

protected:
	uint8 bHasExitState : 1;
	uint8 bHasStateCompleted : 1;
	uint8 bHasLatentEnterState : 1;
	uint8 bHasLatentTick : 1;
	UE_DEPRECATED(all, "Use bHasLatentEnterState instead.")
	uint8 bHasEnterState_DEPRECATED : 1;
	UE_DEPRECATED(all, "Use bHasLatentTick instead.")
	uint8 bHasTick_DEPRECATED : 1;

	mutable uint8 bIsProcessingEnterStateOrTick : 1;

	friend struct FStateTreeBlueprintTaskWrapper;
};

/**
 * Wrapper for Blueprint based Tasks.
 */
USTRUCT()
struct FStateTreeBlueprintTaskWrapper : public FStateTreeTaskBase
{
	GENERATED_BODY()

	virtual const UStruct* GetInstanceDataType() const override
	{
		return TaskClass;
	};

	UE_API virtual bool Link(FStateTreeLinker& Linker) override;
	UE_API virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	UE_API virtual void ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	UE_API virtual void StateCompleted(FStateTreeExecutionContext& Context, const EStateTreeRunStatus CompletionStatus, const FStateTreeActiveStates& CompletedActiveStates) const override;
	UE_API virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override;
#if WITH_EDITOR
	UE_API virtual EDataValidationResult Compile(UE::StateTree::ICompileNodeContext& Context) override;
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
	UE_API virtual FName GetIconName() const override;
	UE_API virtual FColor GetIconColor() const override;
#endif	

	UPROPERTY()
	TSubclassOf<UStateTreeTaskBlueprintBase> TaskClass;

	UPROPERTY()
	uint8 TaskFlags = 0;

private:
#if WITH_EDITORONLY_DATA
	// The node will use the blueprint data instead.
	using FStateTreeTaskBase::bConsideredForCompletion;
	using FStateTreeTaskBase::bCanEditConsideredForCompletion;
#endif
};

#undef UE_API
