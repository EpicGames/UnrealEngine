// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "AITypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#include "StateTreeConditionBase.h"
#include "StateTreeAnyEnum.h"
#include "StateTreeCommonConditions.generated.h"

#define UE_API STATETREEMODULE_API

enum class EGenericAICheck : uint8;
struct FStateTreeDataView;

USTRUCT()
struct FStateTreeCompareIntConditionInstanceData
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Input")
	int32 Left = 0;

	UPROPERTY(EditAnywhere, Category = "Parameter")
	int32 Right = 0;
};
UE_STATETREE_ZEROED_TRIVIALLY_COPIED_NO_DESTRUCTOR_INSTANCEDATA(FStateTreeCompareIntConditionInstanceData);

/**
 * Condition comparing two integers.
 */
USTRUCT(DisplayName = "Integer Compare")
struct FStateTreeCompareIntCondition : public FStateTreeConditionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeCompareIntConditionInstanceData;

	FStateTreeCompareIntCondition() = default;
	explicit FStateTreeCompareIntCondition(const UE::StateTree::EComparisonOperator InOperator, const EStateTreeCompare InInverts = EStateTreeCompare::Default)
		: bInvert(InInverts == EStateTreeCompare::Invert)
		, Operator(InOperator)
	{}

	UE_DEPRECATED(5.8, "Use EComparisonOperator instead.")
	UE_API explicit FStateTreeCompareIntCondition(const EGenericAICheck InOperator, const EStateTreeCompare InInverts = EStateTreeCompare::Default);

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	UE_API virtual bool TestCondition(FStateTreeExecutionContext& Context) const override;
#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
#endif
	
	UPROPERTY(EditAnywhere, Category = "Parameter")
	bool bInvert = false;

	UE_DEPRECATED_FORGAME(5.8, "The Operator type changed from EGenericAICheck to EComparisonOperator.")
	UPROPERTY(EditAnywhere, Category = "Parameter", meta = (InvalidEnumValues = "IsTrue"))
	UE::StateTree::EComparisonOperator Operator = UE::StateTree::EComparisonOperator::Equal;
};


USTRUCT()
struct FStateTreeCompareFloatConditionInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Input")
	double Left = 0.0;

	UPROPERTY(EditAnywhere, Category = "Parameter")
	double Right = 0.0;
};
UE_STATETREE_ZEROED_TRIVIALLY_COPIED_NO_DESTRUCTOR_INSTANCEDATA(FStateTreeCompareFloatConditionInstanceData);

/**
 * Condition comparing two floats.
 */
USTRUCT(DisplayName = "Float Compare")
struct FStateTreeCompareFloatCondition : public FStateTreeConditionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeCompareFloatConditionInstanceData;

	FStateTreeCompareFloatCondition() = default;
	explicit FStateTreeCompareFloatCondition(const UE::StateTree::EComparisonOperator InOperator, const EStateTreeCompare InInverts = EStateTreeCompare::Default)
		: bInvert(InInverts == EStateTreeCompare::Invert)
		, Operator(InOperator)
	{}

	UE_DEPRECATED(5.8, "Use EComparisonOperator instead.")
	UE_API explicit FStateTreeCompareFloatCondition(const EGenericAICheck InOperator, const EStateTreeCompare InInverts = EStateTreeCompare::Default);

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	UE_API virtual bool TestCondition(FStateTreeExecutionContext& Context) const override;
#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
#endif

	UPROPERTY(EditAnywhere, Category = "Parameter")
	bool bInvert = false;

	UE_DEPRECATED_FORGAME(5.8, "The Operator type changed from EGenericAICheck to EComparisonOperator.")
	UPROPERTY(EditAnywhere, Category = "Parameter", meta = (InvalidEnumValues = "IsTrue"))
	UE::StateTree::EComparisonOperator Operator = UE::StateTree::EComparisonOperator::Equal;
};


USTRUCT()
struct FStateTreeCompareBoolConditionInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Input")
	bool bLeft = false;

	UPROPERTY(EditAnywhere, Category = "Parameter")
	bool bRight = false;
};
UE_STATETREE_ZEROED_TRIVIALLY_COPIED_NO_DESTRUCTOR_INSTANCEDATA(FStateTreeCompareBoolConditionInstanceData);

/**
 * Condition comparing two booleans.
 */
USTRUCT(DisplayName = "Bool Compare")
struct FStateTreeCompareBoolCondition : public FStateTreeConditionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeCompareBoolConditionInstanceData;

	FStateTreeCompareBoolCondition() = default;
	explicit FStateTreeCompareBoolCondition(const EStateTreeCompare InInverts)
		: bInvert(InInverts == EStateTreeCompare::Invert)
	{}

	FStateTreeCompareBoolCondition(const bool bInInverts)
		: bInvert(bInInverts)
	{}

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	UE_API virtual bool TestCondition(FStateTreeExecutionContext& Context) const override;
#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
#endif

	UPROPERTY(EditAnywhere, Category = "Parameter")
	bool bInvert = false;
};


USTRUCT()
struct FStateTreeCompareEnumConditionInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Input", meta=(AllowAnyBinding))
	FStateTreeAnyEnum Left;

	UPROPERTY(EditAnywhere, Category = "Parameter")
	FStateTreeAnyEnum Right;
};
UE_STATETREE_CONSTRUCTED_TRIVIALLY_COPIED_NO_DESTRUCTOR_INSTANCEDATA(FStateTreeCompareEnumConditionInstanceData);

/**
 * Condition comparing two enums.
 */
USTRUCT(DisplayName = "Enum Compare")
struct FStateTreeCompareEnumCondition : public FStateTreeConditionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeCompareEnumConditionInstanceData;

	FStateTreeCompareEnumCondition() = default;
	explicit FStateTreeCompareEnumCondition(const EStateTreeCompare InInverts)
		: bInvert(InInverts == EStateTreeCompare::Invert)
	{}

	FStateTreeCompareEnumCondition(const bool bInInverts)
		: bInvert(bInInverts)
	{}

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	UE_API virtual bool TestCondition(FStateTreeExecutionContext& Context) const override;
#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
	UE_API virtual void OnBindingChanged(const FGuid& ID, FStateTreeDataView InstanceData, const FPropertyBindingPath& SourcePath, const FPropertyBindingPath& TargetPath, const IStateTreeBindingLookup& BindingLookup) override;
#endif

	UPROPERTY(EditAnywhere, Category = "Parameter")
	bool bInvert = false;
};


USTRUCT()
struct FStateTreeCompareDistanceConditionInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Input")
	FVector Source;

	UPROPERTY(EditAnywhere, Category = "Parameter")
	FVector Target;

	UPROPERTY(EditAnywhere, Category = "Parameter")
	double Distance = 0.0;
};
UE_STATETREE_ZEROED_TRIVIALLY_COPIED_NO_DESTRUCTOR_INSTANCEDATA(FStateTreeCompareDistanceConditionInstanceData);

/**
 * Condition comparing distance between two vectors.
 */
USTRUCT(DisplayName = "Distance Compare")
struct FStateTreeCompareDistanceCondition : public FStateTreeConditionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeCompareDistanceConditionInstanceData;

	FStateTreeCompareDistanceCondition() = default;
	explicit FStateTreeCompareDistanceCondition(const UE::StateTree::EComparisonOperator InOperator, const EStateTreeCompare InInverts = EStateTreeCompare::Default)
		: bInvert(InInverts == EStateTreeCompare::Invert)
		, Operator(InOperator)
	{}

	UE_DEPRECATED(5.8, "Use EComparisonOperator instead.")
	UE_API explicit FStateTreeCompareDistanceCondition(const EGenericAICheck InOperator, const EStateTreeCompare InInverts = EStateTreeCompare::Default);
	
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	UE_API virtual bool TestCondition(FStateTreeExecutionContext& Context) const override;
#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
#endif

	UPROPERTY(EditAnywhere, Category = "Condition")
	bool bInvert = false;

	UE_DEPRECATED_FORGAME(5.8, "The Operator type changed from EGenericAICheck to EComparisonOperator.")
	UPROPERTY(EditAnywhere, Category = "Condition", meta = (InvalidEnumValues = "IsTrue"))
	UE::StateTree::EComparisonOperator Operator = UE::StateTree::EComparisonOperator::Equal;
};

USTRUCT()
struct FStateTreeCompareNameConditionInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Input")
	FName Left = NAME_None;

	UPROPERTY(EditAnywhere, Category = "Parameter")
	FName Right = NAME_None;
};
UE_STATETREE_CONSTRUCTED_TRIVIALLY_COPIED_NO_DESTRUCTOR_INSTANCEDATA(FStateTreeCompareNameConditionInstanceData);

/**
 * Condition comparing two FNames.
 */
USTRUCT(DisplayName = "Name Compare")
struct FStateTreeCompareNameCondition : public FStateTreeConditionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeCompareNameConditionInstanceData;

	FStateTreeCompareNameCondition() = default;
	explicit FStateTreeCompareNameCondition(const EStateTreeCompare InInverts)
	: bInvert(InInverts == EStateTreeCompare::Invert)
	{}

	FStateTreeCompareNameCondition(const bool bInInverts)
	: bInvert(bInInverts)
	{}

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	UE_API virtual bool TestCondition(FStateTreeExecutionContext& Context) const override;
#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
#endif

	UPROPERTY(EditAnywhere, Category = "Parameter")
	bool bInvert = false;
};


USTRUCT()
struct FStateTreeRandomConditionInstanceData
{
	GENERATED_BODY()
	UPROPERTY(EditAnywhere, Category = "Parameter", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float Threshold = 0.5f;
};
UE_STATETREE_CONSTRUCTED_TRIVIALLY_COPIED_NO_DESTRUCTOR_INSTANCEDATA(FStateTreeRandomConditionInstanceData);

/**
 * Random condition
 */
USTRUCT(DisplayName = "Random")
struct FStateTreeRandomCondition : public FStateTreeConditionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeRandomConditionInstanceData;

	FStateTreeRandomCondition() = default;
	
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	UE_API virtual bool TestCondition(FStateTreeExecutionContext& Context) const override;
#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
#endif
};

#undef UE_API
