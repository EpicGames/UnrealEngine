// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "AITypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#include "StateTreeTypes.h"

#define UE_API STATETREEMODULE_API

enum class EGenericAICheck : uint8;

namespace UE::StateTree::Conditions
{
	template<typename T>
	bool CompareNumbers(const T Left, const T Right, const EComparisonOperator Operator)
	{
		switch (Operator)
		{
		case EComparisonOperator::Equal:
			return Left == Right;
		case EComparisonOperator::NotEqual:
			return Left != Right;
		case EComparisonOperator::Less:
			return Left < Right;
		case EComparisonOperator::LessOrEqual:
			return Left <= Right;
		case EComparisonOperator::Greater:
			return Left > Right;
		case EComparisonOperator::GreaterOrEqual:
			return Left >= Right;
		default:
			ensureMsgf(false, TEXT("Unhandled operator %d"), EnumToUnderlyingType(Operator));
			return false;
		}
	}

	UE_DEPRECATED(5.8, "Use EComparisonOperator instead.")
	extern UE_API EComparisonOperator GenericAICheckToComparisonOperator(EGenericAICheck Operator);
} // UE::StateTree::Conditions

#undef UE_API
