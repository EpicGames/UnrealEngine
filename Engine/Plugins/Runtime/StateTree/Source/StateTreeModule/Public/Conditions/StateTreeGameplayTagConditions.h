// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeConditionBase.h"
#include "StateTreeGameplayTagConditions.generated.h"

#define UE_API STATETREEMODULE_API

USTRUCT()
struct FGameplayTagSingleMatchConditionInstanceData
{
	GENERATED_BODY()

	/** First tag to compare. */
	UPROPERTY(EditAnywhere, Category = Input)
	FGameplayTag Left;

	/** Second tag to compare. */
	UPROPERTY(EditAnywhere, Category = Parameter)
	FGameplayTag Right;
};

/**
 * MatchesTag condition
 * Succeeds if the tag matches the specified tag.
 * 
 * Condition can be used with multiple configurations:
 *	Does Tag "A.1" match Tag "A" ?
 *		exact match 'false' will SUCCEED
 *		exact match 'true' will FAIL
 *	Does Tag "A" match Tag "A.1" ?
 *		always returns false
 */
USTRUCT(DisplayName="Matches Tag", Category="Gameplay Tags")
struct FGameplayTagSingleMatchCondition : public FStateTreeConditionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FGameplayTagSingleMatchConditionInstanceData;

	FGameplayTagSingleMatchCondition() = default;
	
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	UE_API virtual bool TestCondition(FStateTreeExecutionContext& Context) const override;
#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
	virtual FName GetIconName() const override
	{
		return FName("StateTreeEditorStyle|Node.Tag");
	}
	virtual FColor GetIconColor() const override
	{
		return UE::StateTree::Colors::DarkGrey;
	}
#endif // WITH_EDITOR
	/** If true, the tag has to match exactly, if false then the tag may match its parent tags */
	UPROPERTY(EditAnywhere, Category = Condition)
	bool bExactMatch = false;

	UPROPERTY(EditAnywhere, Category = Condition)
	bool bInvert = false;
};


USTRUCT()
struct FGameplayTagMatchConditionInstanceData
{
	GENERATED_BODY()

	/** Container to check for the tag. */
	UPROPERTY(EditAnywhere, Category = Parameter)
	FGameplayTagContainer TagContainer;

	/** Tag to check for in the container. */
	UPROPERTY(EditAnywhere, Category = Parameter)
	FGameplayTag Tag;
};

/**
 * HasTag condition
 * Succeeds if the tag container has the specified tag.
 * 
 * Condition can be used with multiple configurations:
 *	Does TagContainer {"A.1"} has Tag "A" ?
 *		exact match 'false' will SUCCEED
 *		exact match 'true' will FAIL
 */
USTRUCT(DisplayName="Has Tag", Category="Gameplay Tags")
struct FGameplayTagMatchCondition : public FStateTreeConditionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FGameplayTagMatchConditionInstanceData;

	FGameplayTagMatchCondition() = default;
	
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	UE_API virtual bool TestCondition(FStateTreeExecutionContext& Context) const override;
#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
	virtual FName GetIconName() const override
	{
		return FName("StateTreeEditorStyle|Node.Tag");
	}
	virtual FColor GetIconColor() const override
	{
		return UE::StateTree::Colors::DarkGrey;
	}
#endif
	/** If true, the tag has to be exactly present, if false then TagContainer will include its parent tags while matching */
	UPROPERTY(EditAnywhere, Category = Condition)
	bool bExactMatch = false;

	UPROPERTY(EditAnywhere, Category = Condition)
	bool bInvert = false;
};


USTRUCT()
struct FGameplayTagContainerMatchConditionInstanceData
{
	GENERATED_BODY()

	/** Container to check if it matches any of the tags in the other container. */
	UPROPERTY(EditAnywhere, Category = Input)
	FGameplayTagContainer TagContainer;

	/** Container to check against. */
	UPROPERTY(EditAnywhere, Category = Parameter)
	FGameplayTagContainer OtherContainer;
};

/**
 * HasAnyTags / HasAllTags condition
 * Succeeds  if the specified tag container has ANY or ALL of the tags in the other container.
 *
 * Condition can be used with multiple configurations:
 * 
 *	Has ANY Tags:
 *		exact match 'false':
 *			TagContainer {"A.1"} has any of OtherContainer {"A","B"} will SUCCEED
 * 			TagContainer {"A"} has any of OtherContainer {"A.1","B"} will FAIL
 *		
 *		exact match 'true':
* 			TagContainer {"A","B"} has any of OtherContainer {"A.1"} will FAIL
 *
 *		If TagContainer is empty/invalid it will always fail.
 *
 *	Has ALL Tags:
 *		exact match 'false':
 *			TagContainer {"A.1","B.1"} has all of OtherContainer {"A","B"} will SUCCEED
 *			TagContainer {"A","B"} has all of OtherContainer {"A.1","B.1"} will FAIL
 *
 *		exact match 'true':
 *			TagContainer {"A.1","B.1"} has all of OtherContainer {"A","B"} will FAIL
 *
 *		If TagContainer is empty/invalid it will always SUCCEED, because there were no failed checks.
 */
USTRUCT(DisplayName="Has Tags", Category="Gameplay Tags")
struct FGameplayTagContainerMatchCondition : public FStateTreeConditionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FGameplayTagContainerMatchConditionInstanceData ;

	FGameplayTagContainerMatchCondition() = default;
	
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	UE_API virtual bool TestCondition(FStateTreeExecutionContext& Context) const override;
#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
	virtual FName GetIconName() const override
	{
		return FName("StateTreeEditorStyle|Node.Tag");
	}
	virtual FColor GetIconColor() const override
	{
		return UE::StateTree::Colors::DarkGrey;
	}
#endif
	UPROPERTY(EditAnywhere, Category = Condition)
	EGameplayContainerMatchType MatchType = EGameplayContainerMatchType::Any;

	/** If true, the tag has to be exactly present, if false then TagContainer will include it's parent tags while matching */
	UPROPERTY(EditAnywhere, Category = Condition)
	bool bExactMatch = false;

	UPROPERTY(EditAnywhere, Category = Condition)
	bool bInvert = false;
};


USTRUCT()
struct FGameplayTagQueryConditionInstanceData
{
	GENERATED_BODY()

	/** Container that needs to match the query. */
	UPROPERTY(EditAnywhere, Category = Input)
	FGameplayTagContainer TagContainer;
};

/**
 * DoesContainerMatchTagQuery condition
 * Succeeds if the specified tag container matches the given Tag Query.
 */
USTRUCT(DisplayName="Does Container Match Tag Query", Category="Gameplay Tags")
struct FGameplayTagQueryCondition : public FStateTreeConditionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FGameplayTagQueryConditionInstanceData;

	FGameplayTagQueryCondition() = default;
	
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	UE_API virtual bool TestCondition(FStateTreeExecutionContext& Context) const override;
#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
	virtual FName GetIconName() const override
	{
		return FName("StateTreeEditorStyle|Node.Tag");
	}
	virtual FColor GetIconColor() const override
	{
		return UE::StateTree::Colors::DarkGrey;
	}
#endif
	/** Query to match against */
	UPROPERTY(EditAnywhere, Category = Condition)
	FGameplayTagQuery TagQuery;

	UPROPERTY(EditAnywhere, Category = Condition)
	bool bInvert = false;
};

#undef UE_API
