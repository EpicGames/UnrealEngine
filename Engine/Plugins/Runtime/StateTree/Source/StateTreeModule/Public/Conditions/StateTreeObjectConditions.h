// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeConditionBase.h"
#include "StateTreeObjectConditions.generated.h"

#define UE_API STATETREEMODULE_API

USTRUCT()
struct FStateTreeObjectIsValidConditionInstanceData
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = Input)
	TObjectPtr<UObject> Object = nullptr;
};

/**
 * Condition testing if specified object is valid.
 */
USTRUCT(DisplayName = "Object Is Valid", Category = "Object")
struct FStateTreeObjectIsValidCondition : public FStateTreeConditionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeObjectIsValidConditionInstanceData;

	FStateTreeObjectIsValidCondition() = default;
	explicit FStateTreeObjectIsValidCondition(const EStateTreeCompare InInverts)
		: bInvert(InInverts == EStateTreeCompare::Invert)
	{}

	virtual const UStruct* GetInstanceDataType() const override
	{
		return FInstanceDataType::StaticStruct();
	}
	UE_API virtual bool TestCondition(FStateTreeExecutionContext& Context) const override;

#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
#endif

	UPROPERTY(EditAnywhere, Category = Parameter)
	bool bInvert = false;
};


USTRUCT()
struct FStateTreeObjectEqualsConditionInstanceData
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Input")
	TObjectPtr<UObject> Left = nullptr;

	UPROPERTY(EditAnywhere, Category = "Parameter")
	TObjectPtr<UObject> Right = nullptr;
};

/**
 * Condition testing if two object pointers point to the same object.
 */
USTRUCT(DisplayName = "Object Equals", Category = "Object")
struct FStateTreeObjectEqualsCondition : public FStateTreeConditionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeObjectEqualsConditionInstanceData;

	FStateTreeObjectEqualsCondition() = default;
	explicit FStateTreeObjectEqualsCondition(const EStateTreeCompare InInverts)
		: bInvert(InInverts == EStateTreeCompare::Invert)
	{}

	virtual const UStruct* GetInstanceDataType() const override
	{
		return FInstanceDataType::StaticStruct();
	}
	UE_API virtual bool TestCondition(FStateTreeExecutionContext& Context) const override;

#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
#endif

	UPROPERTY(EditAnywhere, Category = "Parameter")
	bool bInvert = false;
};


USTRUCT()
struct FStateTreeObjectIsChildOfClassConditionInstanceData
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Input")
	TObjectPtr<UObject> Object = nullptr;

	UPROPERTY(EditAnywhere, Category = "Parameter")
	TObjectPtr<UClass> Class = nullptr;
};

/**
 * Condition testing if object is child of specified class.
 */
USTRUCT(DisplayName = "Object Class Is", Category = "Object")
struct FStateTreeObjectIsChildOfClassCondition : public FStateTreeConditionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeObjectIsChildOfClassConditionInstanceData;

	FStateTreeObjectIsChildOfClassCondition() = default;
	explicit FStateTreeObjectIsChildOfClassCondition(const EStateTreeCompare InInverts)
		: bInvert(InInverts == EStateTreeCompare::Invert)
	{}

	virtual const UStruct* GetInstanceDataType() const override
	{
		return FInstanceDataType::StaticStruct();
	}
	UE_API virtual bool TestCondition(FStateTreeExecutionContext& Context) const override;

#if WITH_EDITOR
		UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
#endif

	UPROPERTY(EditAnywhere, Category = "Parameter")
	bool bInvert = false;
};

#undef UE_API
