// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/CurveFloat.h"
#include "StateTreeAnyEnum.h"
#include "StateTreeConsiderationBase.h"
#include "StateTreeCommonConsiderations.generated.h"

#define UE_API STATETREEMODULE_API

USTRUCT()
struct FStateTreeConstantConsiderationInstanceData
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, meta = (ClampMin = 0, ClampMax = 1, UIMin = 0, UIMax = 1), Category = "Default")
	float Constant = 0.f;
};

/**
 * Consideration using a constant as its score.
 */
USTRUCT(DisplayName = "Constant")
struct FStateTreeConstantConsideration: public FStateTreeConsiderationCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeConstantConsiderationInstanceData;

	//~ Begin FStateTreeNodeBase Interface
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
	//~ End FStateTreeNodeBase Interface
#endif

protected:
	//~ Begin FStateTreeConsiderationBase Interface
	UE_API virtual float GetScore(FStateTreeExecutionContext& Context) const override;
	//~ End FStateTreeConsiderationBase Interface
};

USTRUCT()
struct FStateTreeConsiderationResponseCurve
{
	GENERATED_BODY()

	/**
	 * Evaluate the output value from curve
	 * @param NormalizedInput the normalized input value to the response curve
	 * @return The output value. If the curve is not set, will simply return NormalizedInput.
	 */
	float Evaluate(float NormalizedInput) const
	{
		if (const FRichCurve* Curve = CurveInfo.GetRichCurveConst())
		{
			if (!Curve->IsEmpty())
			{
				return Curve->Eval(NormalizedInput);
			}
		}

		return NormalizedInput;
	}

	/* Curve used to output the final score for the Consideration. */
	UPROPERTY(EditAnywhere, Category = Default, DisplayName = "Curve")
	FRuntimeFloatCurve CurveInfo;
};

USTRUCT()
struct FStateTreeFloatInputConsiderationInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Input")
	float Input = 0.f;

	UPROPERTY(EditAnywhere, DisplayName = "InputRange", Category = "Parameter")
	FFloatInterval Interval = FFloatInterval(0.f, 1.f);
};

/**
 * Consideration using a Float as input to the response curve.
 */
USTRUCT(DisplayName = "Float Input")
struct FStateTreeFloatInputConsideration : public FStateTreeConsiderationCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeFloatInputConsiderationInstanceData;
	
	//~ Begin FStateTreeNodeBase Interface
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
#if WITH_EDITOR
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
	//~ End FStateTreeNodeBase Interface
#endif

protected:
	//~ Begin FStateTreeConsiderationBase Interface
	UE_API virtual float GetScore(FStateTreeExecutionContext& Context) const override;
	//~ End FStateTreeConsiderationBase Interface

public:
	UPROPERTY(EditAnywhere, Category = "Default")
	FStateTreeConsiderationResponseCurve ResponseCurve;
};

USTRUCT()
struct FStateTreeEnumValueScorePair
{
	GENERATED_BODY()

	bool operator==(const FStateTreeEnumValueScorePair& RHS) const
	{
		return EnumValue == RHS.EnumValue && Score == RHS.Score;
	}

	bool operator!=(const FStateTreeEnumValueScorePair& RHS) const
	{
		return EnumValue != RHS.EnumValue || Score != RHS.Score;
	}

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = Default)
	FName EnumName;
#endif

	UPROPERTY(EditAnywhere, Category = Default)
	int64 EnumValue = 0;

	UPROPERTY(EditAnywhere, Category = Default, meta = (UIMin = 0.0, ClampMin = 0.0, UIMax = 1.0, ClampMax = 1.0))
	float Score = 0.f;
};

USTRUCT()
struct FStateTreeEnumValueScorePairs
{
	GENERATED_BODY()
	
#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = Default)
	TObjectPtr<const UEnum> Enum;
#endif //WITH_EDITORONLY_DATA

	UPROPERTY(EditAnywhere, Category = Default, DisplayName = EnumValueScorePairs)
	TArray<FStateTreeEnumValueScorePair> Data;

#if WITH_EDITORONLY_DATA
	/** Initializes the class to specific enum.*/
	void Initialize(const UEnum* NewEnum)
	{
		if (Enum != NewEnum || NewEnum == nullptr)
		{
			Data.Empty();
		}

		Enum = NewEnum;
	}
#endif //WITH_EDITORONLY_DATA
};

USTRUCT()
struct FStateTreeEnumInputConsiderationInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Input", meta=(AllowAnyBinding))
	FStateTreeAnyEnum Input;
};

USTRUCT(DisplayName = "Enum Input")
struct FStateTreeEnumInputConsideration : public FStateTreeConsiderationCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeEnumInputConsiderationInstanceData;

	//~ Begin FStateTreeNodeBase Interface
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
#if WITH_EDITOR
	UE_API virtual EDataValidationResult Compile(UE::StateTree::ICompileNodeContext& Context) override;
	UE_API virtual void OnBindingChanged(const FGuid& ID, FStateTreeDataView InstanceData, const FPropertyBindingPath& SourcePath, const FPropertyBindingPath& TargetPath, const IStateTreeBindingLookup& BindingLookup) override;

	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
	//~ End FStateTreeNodeBase Interface
#endif //WITH_EDITOR

protected:
	//~ Begin FStateTreeConsiderationBase Interface
	UE_API virtual float GetScore(FStateTreeExecutionContext& Context) const override;
	//~ End FStateTreeConsiderationBase Interface
	
	UPROPERTY(EditAnywhere, Category = "Default")
	FStateTreeEnumValueScorePairs EnumValueScorePairs;
};

#undef UE_API
