// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_STATETREE_TRACE_DEBUGGER

#include "StateTreeTraceTypes.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Containers/Timelines.h"

namespace UE::StateTreeDebugger { struct FInstanceDescriptor; }
struct FStateTreeInstanceDebugId;
class UStateTree;

class IStateTreeTraceProvider : public TraceServices::IProvider
{
public:
	typedef TraceServices::ITimeline<FStateTreeTraceEventVariantType> FEventsTimeline;

	/**
	 * Return instance descriptor associated to the given instance id.
	 * @param InstanceId Id of a specific instance to get the descriptor for.
	 * @return Shared pointer to the descriptor if the specified instance was found.
	 */
	virtual TSharedPtr<const UE::StateTreeDebugger::FInstanceDescriptor> GetInstanceDescriptor(const FStateTreeInstanceDebugId InstanceId) const = 0;

	/**
	 * Return all instances with events in the traces.
	 * @param OutInstances List of all instances with events in the traces regardless if they are active or not.
	 */
	virtual void GetInstances(TArray<const TSharedRef<const UE::StateTreeDebugger::FInstanceDescriptor>>& OutInstances) const = 0;

	UE_DEPRECATED(5.7, "Use the version with TSharedRef array instead")
	virtual void GetInstances(TArray<UE::StateTreeDebugger::FInstanceDescriptor>& OutInstances) const final
	{
	}

	/**
	 * Execute given function receiving an event timeline for a given instance or all timelines if instance not specified.  
	 * @param InstanceId Id of a specific instance to get the timeline for; could be an FStateTreeInstanceDebugId::Invalid to go through all timelines
	 * @param Callback Function called for timeline(s) matching the provided Id 
	 * @return True if the specified instance was found for a given Id or at least one timeline was found when no specific Id is provided. 
	 */
	virtual bool ReadTimelines(const FStateTreeInstanceDebugId InstanceId, TFunctionRef<void(const FStateTreeInstanceDebugId ProcessedInstanceId, const FEventsTimeline&)> Callback) const = 0;

	/**
	 * Execute given function receiving an event timeline for all timelines matching a given asset.  
	 * @param StateTree Asset that must be used by the instance to be processed
	 * @param Callback Function called for timeline(s) matching the provided asset 
	 * @return True if the specified instance was found for a given Id or at least one timeline was found when no specific Id is provided. 
	 */
	virtual bool ReadTimelines(const UStateTree& StateTree, TFunctionRef<void(const FStateTreeInstanceDebugId ProcessedInstanceId, const FEventsTimeline&)> Callback) const = 0;
};

#endif // WITH_STATETREE_TRACE_DEBUGGER
