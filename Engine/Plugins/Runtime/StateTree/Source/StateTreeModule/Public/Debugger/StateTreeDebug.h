// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeTypes.h"

class UStateTree;
struct FStateTreeExecutionContext;
struct FStateTreeMinimalExecutionContext;
struct FStateTreeSharedEvent;
struct FStateTreeTransitionSource;
enum class EStateTreeUpdatePhase : uint8;
enum class EStateTreeTraceEventType : uint8;

#if WITH_STATETREE_DEBUG

#define UE_STATETREE_DEBUG_CONDITION_ENTER_STATE(InCtx, InStateTree, InNodeIndex) ::UE::StateTree::Debug::ConditionEnterState(*(InCtx), ::UE::StateTree::Debug::FNodeReference((InStateTree), FStateTreeIndex16(InNodeIndex)))
#define UE_STATETREE_DEBUG_CONDITION_TEST_CONDITION(InCtx, InStateTree, InNodeIndex) ::UE::StateTree::Debug::ConditionTest(*(InCtx), ::UE::StateTree::Debug::FNodeReference((InStateTree), FStateTreeIndex16(InNodeIndex)))
#define UE_STATETREE_DEBUG_CONDITION_EXIT_STATE(InCtx, InStateTree, InNodeIndex) ::UE::StateTree::Debug::ConditionExitState(*(InCtx), ::UE::StateTree::Debug::FNodeReference((InStateTree), FStateTreeIndex16(InNodeIndex)))

#define UE_STATETREE_DEBUG_EVALUATOR_ENTER_TREE(InCtx, InStateTree, InNodeIndex) ::UE::StateTree::Debug::EvaluatorEnterTree(*(InCtx), ::UE::StateTree::Debug::FNodeReference((InStateTree), FStateTreeIndex16(InNodeIndex)))
#define UE_STATETREE_DEBUG_EVALUATOR_TICK(InCtx, InStateTree, InNodeIndex) ::UE::StateTree::Debug::EvaluatorTick(*(InCtx), ::UE::StateTree::Debug::FNodeReference((InStateTree), FStateTreeIndex16(InNodeIndex)))
#define UE_STATETREE_DEBUG_EVALUATOR_EXIT_TREE(InCtx, InStateTree, InNodeIndex) ::UE::StateTree::Debug::EvaluatorExitTree(*(InCtx), ::UE::StateTree::Debug::FNodeReference((InStateTree), FStateTreeIndex16(InNodeIndex)))

#define UE_STATETREE_DEBUG_TASK_ENTER_STATE(InCtx, InStateTree, InNodeIndex) ::UE::StateTree::Debug::TaskEnterState(*(InCtx), ::UE::StateTree::Debug::FNodeReference((InStateTree), FStateTreeIndex16(InNodeIndex)))
#define UE_STATETREE_DEBUG_TASK_TICK(InCtx, InStateTree, InNodeIndex) ::UE::StateTree::Debug::TaskTick(*(InCtx), ::UE::StateTree::Debug::FNodeReference((InStateTree), FStateTreeIndex16(InNodeIndex)))
#define UE_STATETREE_DEBUG_TASK_EXIT_STATE(InCtx, InStateTree, InNodeIndex) ::UE::StateTree::Debug::TaskExitState(*(InCtx), ::UE::StateTree::Debug::FNodeReference((InStateTree), FStateTreeIndex16(InNodeIndex)))

#define UE_STATETREE_DEBUG_STATE_UTILITY_EVALUATED(InCtx, InStateHandled, Score) ::UE::StateTree::Debug::StateUtilityEvaluated(*(InCtx), InStateHandled, Score)

#define UE_STATETREE_DEBUG_SEND_EVENT(InCtx, InStateTree, InTag, InPayload, InOrigin) ::UE::StateTree::Debug::EventSent(*(InCtx), InStateTree, InTag, InPayload, InOrigin)
#define UE_STATETREE_DEBUG_EVENT_CONSUMED(InExecutionContextPtr, Event) ::UE::StateTree::Debug::EventConsumed(*(InExecutionContextPtr), Event)

#define UE_STATETREE_DEBUG_ENTER_PHASE(InCtx, InPhase) \
		::UE::StateTree::Debug::EnterPhase(*(InCtx), (InPhase), FStateTreeStateHandle::Invalid); \
		TRACE_STATETREE_PHASE_EVENT((InCtx), (InPhase), EStateTreeTraceEventType::Push, FStateTreeStateHandle::Invalid)

#define UE_STATETREE_DEBUG_EXIT_PHASE(InCtx, InPhase) \
		::UE::StateTree::Debug::ExitPhase(*(InCtx), (InPhase), FStateTreeStateHandle::Invalid); \
		TRACE_STATETREE_PHASE_EVENT((InCtx), (InPhase), EStateTreeTraceEventType::Pop, FStateTreeStateHandle::Invalid)

#define UE_STATETREE_DEBUG_STATE_EVENT(InCtx, InStateHandle, InEventType) \
		::UE::StateTree::Debug::StateEvent(*(InCtx), (InStateHandle), (InEventType)); \
		TRACE_STATETREE_STATE_EVENT((InCtx), (InStateHandle), (InEventType))

#define UE_STATETREE_DEBUG_TRANSITION_EVENT(InCtx, InTransitionSource, InEventType) \
		::UE::StateTree::Debug::TransitionEvent(*(InCtx), (InTransitionSource), (InEventType)); \
		TRACE_STATETREE_TRANSITION_EVENT((InCtx), (InTransitionSource), (InEventType))

#define UE_STATETREE_DEBUG_LOG_EVENT(InCtx, InLogVerbosity, InFormat, ...) TRACE_STATETREE_LOG_EVENT((InCtx), InLogVerbosity, InFormat, ##__VA_ARGS__)
#define UE_STATETREE_DEBUG_CONDITION_EVENT(InCtx, InIndex, InDataView, InEventType) TRACE_STATETREE_CONDITION_EVENT((InCtx), FStateTreeIndex16(InIndex), (InDataView), (InEventType));
#define UE_STATETREE_DEBUG_INSTANCE_EVENT(InCtx, InEventType) TRACE_STATETREE_INSTANCE_EVENT((InCtx), (InEventType));
#define UE_STATETREE_DEBUG_INSTANCE_FRAME_EVENT(InCtx, InFrame) TRACE_STATETREE_INSTANCE_FRAME_EVENT((InCtx), (InFrame));
#define UE_STATETREE_DEBUG_ACTIVE_STATES_EVENT(InCtx, InActiveFrames) TRACE_STATETREE_ACTIVE_STATES_EVENT((InCtx), InActiveFrames);
#define UE_STATETREE_DEBUG_TASK_EVENT(InCtx, InIndex, InDataView, InEventType, InStatus) TRACE_STATETREE_TASK_EVENT((InCtx), FStateTreeIndex16(InIndex), (InDataView), (InEventType), (InStatus));
#define UE_STATETREE_DEBUG_EVALUATOR_EVENT(InCtx, InIndex, InDataView, InEventType) TRACE_STATETREE_EVALUATOR_EVENT((InCtx), FStateTreeIndex16(InIndex), (InDataView), (InEventType));

#define UE_STATETREE_ID_NAME UE_JOIN(InstanceId,__LINE__) \

/** Scope based macros captures the instance ID since it might not be accessible when exiting the scope */
#define UE_STATETREE_DEBUG_SCOPED_PHASE(InCtx, InPhase) \
		UE_STATETREE_DEBUG_ENTER_PHASE((InCtx), (InPhase)); \
		ON_SCOPE_EXIT \
		{ \
			::UE::StateTree::Debug::ExitPhase(*(InCtx), (InPhase), FStateTreeStateHandle::Invalid); \
			TRACE_STATETREE_PHASE_EVENT((InCtx), (InPhase), EStateTreeTraceEventType::Pop, FStateTreeStateHandle::Invalid); \
		}

/** Scope based macros captures the instance ID since it might not be accessible when exiting the scope */
#define UE_STATETREE_DEBUG_SCOPED_STATE(InCtx, InStateHandle) \
		UE_STATETREE_DEBUG_STATE_EVENT((InCtx), (InStateHandle), EStateTreeTraceEventType::Push); \
		ON_SCOPE_EXIT \
		{ \
			::UE::StateTree::Debug::StateEvent(*(InCtx), (InStateHandle), EStateTreeTraceEventType::Pop); \
			TRACE_STATETREE_STATE_EVENT((InCtx), (InStateHandle), EStateTreeTraceEventType::Pop); \
		}

/** Scope based macros captures the instance ID since it might not be accessible when exiting the scope */
#define UE_STATETREE_DEBUG_SCOPED_STATE_PHASE(InCtx, InStateHandle, InPhase) \
		::UE::StateTree::Debug::EnterPhase(*(InCtx), (InPhase), (InStateHandle)); \
		TRACE_STATETREE_PHASE_EVENT((InCtx), (InPhase), EStateTreeTraceEventType::Push, (InStateHandle)); \
		ON_SCOPE_EXIT \
		{ \
			::UE::StateTree::Debug::ExitPhase(*(InCtx), (InPhase), (InStateHandle)); \
			TRACE_STATETREE_PHASE_EVENT((InCtx), (InPhase), EStateTreeTraceEventType::Pop, (InStateHandle)); \
		}

namespace UE::StateTree::Debug
{
	struct FNodeReference
	{
		explicit FNodeReference(TNotNull<const UStateTree*> InStateTree, FStateTreeIndex16 InNodeIndex);
		TNotNull<const UStateTree*> StateTree;
		FStateTreeIndex16 Index;
	};

	struct FNodeDelegateArgs
	{
		FNodeReference Node;
		FGuid NodeId;
	};

	struct FEventSentDelegateArgs
	{
		TNotNull<const UStateTree*> StateTree;
		FGameplayTag Tag;
		FConstStructView Payload;
		FName Origin;
	};

	DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FEventSentDelegate, const FStateTreeMinimalExecutionContext& ExecutionContext, const FEventSentDelegateArgs& EventSentArgs);
	DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FEventConsumedDelegate, const FStateTreeExecutionContext& ExecutionContext, const FStateTreeSharedEvent& Event);
	DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FNodeDelegate, const FStateTreeExecutionContext&, FNodeDelegateArgs);
	DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FPhaseDelegate, const FStateTreeExecutionContext&, EStateTreeUpdatePhase, FStateTreeStateHandle);
	DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FStateDelegate, const FStateTreeExecutionContext&, FStateTreeStateHandle, EStateTreeTraceEventType);
	DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FTransitionDelegate, const FStateTreeExecutionContext&, const FStateTreeTransitionSource&, EStateTreeTraceEventType);
	DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FStateUtilityEvaluatedDelegate, const FStateTreeExecutionContext&, FStateTreeStateHandle, float);

	void ConditionEnterState(const FStateTreeExecutionContext& ExecutionContext, FNodeReference Node);
	void ConditionTest(const FStateTreeExecutionContext& ExecutionContext, FNodeReference Node);
	void ConditionExitState(const FStateTreeExecutionContext& ExecutionContext, FNodeReference Node);

	void EvaluatorEnterTree(const FStateTreeExecutionContext& ExecutionContext, FNodeReference Node);
	void EvaluatorTick(const FStateTreeExecutionContext& ExecutionContext, FNodeReference Node);
	void EvaluatorExitTree(const FStateTreeExecutionContext& ExecutionContext, FNodeReference Node);

	void TaskEnterState(const FStateTreeExecutionContext& ExecutionContext, FNodeReference Node);
	void TaskTick(const FStateTreeExecutionContext& ExecutionContext, FNodeReference Node);
	void TaskExitState(const FStateTreeExecutionContext& ExecutionContext, FNodeReference Node);

	void StateUtilityEvaluated(const FStateTreeExecutionContext& ExecutionContext, FStateTreeStateHandle StateHandle, float Score);

	void EventSent(const FStateTreeMinimalExecutionContext& ExecutionContext, TNotNull<const UStateTree*> StateTree, FGameplayTag Tag, FConstStructView Payload, FName Origin);
	void EventConsumed(const FStateTreeExecutionContext& ExecutionContext, const FStateTreeSharedEvent& Event);

	void EnterPhase(const FStateTreeExecutionContext& ExecutionContext, EStateTreeUpdatePhase Phase, FStateTreeStateHandle StateHandle);
	void ExitPhase(const FStateTreeExecutionContext& ExecutionContext, EStateTreeUpdatePhase Phase, FStateTreeStateHandle StateHandle);

	void StateEvent(const FStateTreeExecutionContext& ExecutionContext, FStateTreeStateHandle StateHandle, EStateTreeTraceEventType EventType);

	void TransitionEvent(const FStateTreeExecutionContext& ExecutionContext, const FStateTreeTransitionSource& TransitionSource, EStateTreeTraceEventType EventType);

	/**
	 * Debugging callback for when a condition activates.
	 * @note The callback executes inside the StateTree logic.
	 * @note The StateTree can execute on any thread.
	 */
	STATETREEMODULE_API extern FNodeDelegate OnConditionEnterState_AnyThread;

	/**
	 * Debugging callback for before a condition is tested.
	 * @note The callback executes inside the StateTree logic.
	 * @note The StateTree can execute on any thread.
	 */
	STATETREEMODULE_API extern FNodeDelegate OnTestCondition_AnyThread;

	/**
	 * Debugging callback for when a condition deactivates.
	 * @note The callback executes inside the StateTree logic.
	 * @note The StateTree can execute on any thread.
	 */
	STATETREEMODULE_API extern FNodeDelegate OnConditionExitState_AnyThread;

	/**
	 * Debugging callback for when a evaluator activates.
	 * @note The callback executes inside the StateTree logic.
	 * @note The StateTree can execute on any thread.
	 */
	STATETREEMODULE_API extern FNodeDelegate OnEvaluatorEnterTree_AnyThread;

	/**
	 * Debugging callback for before an evaluator ticks.
	 * @note The callback executes inside the StateTree logic.
	 * @note The StateTree can execute on any thread.
	 */
	STATETREEMODULE_API extern FNodeDelegate OnTickEvaluator_AnyThread;

	/**
	 * Debugging callback for when a evaluator deactivates.
	 * @note The callback executes inside the StateTree logic.
	 * @note The StateTree can execute on any thread.
	 */
	STATETREEMODULE_API extern FNodeDelegate OnEvaluatorExitTree_AnyThread;

	/**
	 * Debugging callback for when a task activates.
	 * @note The callback executes inside the StateTree logic.
	 * @note The StateTree can execute on any thread.
	 */
	STATETREEMODULE_API extern FNodeDelegate OnTaskEnterState_AnyThread;

	/**
	 * Debugging callback executed before a task ticks.
	 * @note The callback executes inside the StateTree logic.
	 * @note The StateTree can execute on any thread.
	 */
	STATETREEMODULE_API extern FNodeDelegate OnTickTask_AnyThread;

	/**
	 * Debugging callback for when a task deactivates.
	 * @note The callback executes inside the StateTree logic.
	 * @note The StateTree can execute on any thread.
	 */
	STATETREEMODULE_API extern FNodeDelegate OnTaskExitState_AnyThread;

	/**
	 * Debugging callback for when entering an update phase (global to the tree or specific to a state).
	 * @note The callback executes inside the StateTree logic.
	 * @note The StateTree can execute on any thread.
	 */
	STATETREEMODULE_API extern FPhaseDelegate OnBeginUpdatePhase_AnyThread;

	/**
	 * Debugging callback for when exiting an update phase (global to the tree or specific to a state).
	 * @note The callback executes inside the StateTree logic.
	 * @note The StateTree can execute on any thread.
	 */
	STATETREEMODULE_API extern FPhaseDelegate OnEndUpdatePhase_AnyThread;

	/**
	 * Debugging callback when an action related to a state is executing (e.g., entering, exiting, selecting, etc.).
	 * @note The callback executes inside the StateTree logic.
	 * @note The StateTree can execute on any thread.
	 */
	STATETREEMODULE_API extern FStateDelegate OnStateEvent_AnyThread;

	/**
	 * Debugging callback when an action related to a transition is executing (e.g., requesting, evaluating, etc.).
	 * @note The callback executes inside the StateTree logic.
	 * @note The StateTree can execute on any thread.
	 */
	STATETREEMODULE_API extern FTransitionDelegate OnTransitionEvent_AnyThread;

	/**
	 * Debugging callback for when an event is sent.
	 * @note The callback executes inside the StateTree logic.
	 * @note The StateTree can execute on any thread.
	 */
	STATETREEMODULE_API extern FEventSentDelegate OnEventSent_AnyThread;

	/**
	 * Debugging callback for when an event is consumed.
	 * @note The callback executes inside the StateTree logic.
	 * @note The StateTree can execute on any thread.
	 */
	STATETREEMODULE_API extern FEventConsumedDelegate OnEventConsumed_AnyThread;

	/**
	 * Debugging callback for when an utility score is calculated for a state.
	 * @note The callback executes inside the StateTree logic.
	 * @note The StateTree can execute on any thread.
	 */
	STATETREEMODULE_API extern FStateUtilityEvaluatedDelegate OnStateUtilityEvaluated_AnyThread;
} //UE::StateTree::Debug

#else

#define UE_STATETREE_DEBUG_CONDITION_ENTER_STATE(...)
#define UE_STATETREE_DEBUG_CONDITION_TEST_CONDITION(...)
#define UE_STATETREE_DEBUG_CONDITION_EXIT_STATE(...)

#define UE_STATETREE_DEBUG_EVALUATOR_ENTER_TREE(...)
#define UE_STATETREE_DEBUG_EVALUATOR_TICK(...)
#define UE_STATETREE_DEBUG_EVALUATOR_EXIT_TREE(...)

#define UE_STATETREE_DEBUG_TASK_ENTER_STATE(...)
#define UE_STATETREE_DEBUG_TASK_TICK(...)
#define UE_STATETREE_DEBUG_TASK_EXIT_STATE(...)

#define UE_STATETREE_DEBUG_STATE_UTILITY_EVALUATED(...)

#define UE_STATETREE_DEBUG_SEND_EVENT(...)
#define UE_STATETREE_DEBUG_EVENT_CONSUMED(...)

#define UE_STATETREE_DEBUG_ENTER_PHASE(...)
#define UE_STATETREE_DEBUG_EXIT_PHASE(...)
#define UE_STATETREE_DEBUG_STATE_EVENT(...)
#define UE_STATETREE_DEBUG_TRANSITION_EVENT(...)
#define UE_STATETREE_DEBUG_LOG_EVENT(...)
#define UE_STATETREE_DEBUG_CONDITION_EVENT(...)
#define UE_STATETREE_DEBUG_INSTANCE_EVENT(...)
#define UE_STATETREE_DEBUG_INSTANCE_FRAME_EVENT(...)
#define UE_STATETREE_DEBUG_ACTIVE_STATES_EVENT(...)
#define UE_STATETREE_DEBUG_TASK_EVENT(...)
#define UE_STATETREE_DEBUG_EVALUATOR_EVENT(...)
#define UE_STATETREE_DEBUG_CONDITION_EVENT(...)
#define UE_STATETREE_DEBUG_SCOPED_PHASE(...)
#define UE_STATETREE_DEBUG_SCOPED_STATE(...)
#define UE_STATETREE_DEBUG_SCOPED_STATE_PHASE(...)

#endif // WITH_STATETREE_DEBUG
