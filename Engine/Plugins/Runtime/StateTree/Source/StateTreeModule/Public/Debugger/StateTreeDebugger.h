// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_STATETREE_TRACE_DEBUGGER

#include "Delegates/IDelegateInstance.h"
#include "IStateTreeTraceProvider.h"
#include "StateTree.h"
#include "StateTreeDebuggerTypes.h"
#include "StateTreeTypes.h"
#include "Tickable.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/Diagnostics.h"
#include "TraceServices/Model/Frames.h"

#define UE_API STATETREEMODULE_API

enum class EStateTreeBreakpointType : uint8;

namespace UE::Trace
{
	class FStoreClient;
}

class IStateTreeModule;
class UStateTreeState;
class UStateTree;

namespace UE::StateTreeDebugger
{
	struct ITraceReader
	{
		virtual ~ITraceReader() = default;
		virtual FInstanceEventCollection* GetOrCreateEventCollection(FStateTreeInstanceDebugId InstanceId) = 0;
		virtual void OnTraceEventProcessed(FStateTreeInstanceDebugId InstanceId, const FStateTreeTraceEventVariantType& Event)
		{
		}
	};

	struct FTraceFilter
	{
		FTraceFilter() = delete;
		explicit FTraceFilter(const TNotNull<const UStateTree*>& Asset)
			: Asset(Asset)
		{
		}
		explicit FTraceFilter(const FStateTreeInstanceDebugId& Instance)
			: Instance(Instance)
		{
		}

		TObjectPtr<const UStateTree> Asset = nullptr;
		FStateTreeInstanceDebugId Instance;
	};

	UE_API void ReadTrace(
		const TraceServices::IAnalysisSession& Session,
		const double ScrubTime,
		TNotNull<ITraceReader*> TraceReader,
		const FTraceFilter& Filter,
		double& OutLastTraceReadTime);

	UE_API void ReadTrace(
		const TraceServices::IAnalysisSession& Session,
		const TraceServices::IFrameProvider& FrameProvider,
		const TraceServices::FFrame& Frame,
		TNotNull<ITraceReader*> TraceReader,
		const FTraceFilter& Filter,
		double& OutLastTraceReadTime);
}

DECLARE_DELEGATE_OneParam(FOnStateTreeDebuggerScrubStateChanged, const UE::StateTreeDebugger::FScrubState& ScrubState);
DECLARE_DELEGATE_TwoParams(FOnStateTreeDebuggerBreakpointHit, FStateTreeInstanceDebugId InstanceId, const FStateTreeDebuggerBreakpoint Breakpoint);
DECLARE_DELEGATE_OneParam(FOnStateTreeDebuggerActiveStatesChanges, const FStateTreeTraceActiveStates& ActiveStates);
DECLARE_DELEGATE_OneParam(FOnStateTreeDebuggerNewInstance, FStateTreeInstanceDebugId InstanceId);
DECLARE_DELEGATE(FOnStateTreeDebuggerNewSession);

UE_DEPRECATED(5.7, "Delegate type no longer used and will be removed")
DECLARE_DELEGATE(FOnStateTreeDebuggerDebuggedInstanceSet);

struct FStateTreeDebugger : FTickableGameObject, UE::StateTreeDebugger::ITraceReader
{
	struct FTraceDescriptor
	{
		FTraceDescriptor() = default;
		FTraceDescriptor(const FString& Name, const uint32 Id) : Name(Name), TraceId(Id)
		{
		}

		bool operator==(const FTraceDescriptor& Other) const
		{
			return Other.TraceId == TraceId;
		}

		bool operator!=(const FTraceDescriptor& Other) const
		{
			return !(Other == *this);
		}

		bool IsValid() const
		{
			return TraceId != INDEX_NONE;
		}

		FString Name;
		uint32 TraceId = INDEX_NONE;

		TraceServices::FSessionInfo SessionInfo;
	};

	struct FHitBreakpoint
	{
		bool IsSet() const
		{
			return Index != INDEX_NONE;
		}
		void Reset()
		{
			InstanceId = FStateTreeInstanceDebugId::Invalid;
			Time = 0;
			Index = INDEX_NONE;
		}

		/** Indicates the instance for which the breakpoint has been hit */
		FStateTreeInstanceDebugId InstanceId = FStateTreeInstanceDebugId::Invalid;

		/**
		 * Store the time at which the breakpoint was hit since we might have process more events before
		 * sending the notifications.
		 */
		double Time = 0;

		/** Indicates the index of the breakpoint that has been hit */
		int32 Index = INDEX_NONE;
	};

	UE_API FStateTreeDebugger();
	UE_API virtual ~FStateTreeDebugger() override;

	const UStateTree* GetAsset() const
	{
		return StateTreeAsset.Get();
	}

	void SetAsset(const UStateTree* Asset)
	{
		StateTreeAsset = Asset;
		InstanceRootAsset = Asset;
	}

	/** Forces a single refresh to latest state. Useful when simulation is paused. */
	UE_API void SyncToCurrentSessionDuration();

	UE_API bool CanStepBackToPreviousStateWithEvents() const;
	UE_API void StepBackToPreviousStateWithEvents();

	UE_API bool CanStepForwardToNextStateWithEvents() const;
	UE_API void StepForwardToNextStateWithEvents();

	UE_API bool CanStepBackToPreviousStateChange() const;
	UE_API void StepBackToPreviousStateChange();

	UE_API bool CanStepForwardToNextStateChange() const;
	UE_API void StepForwardToNextStateChange();

	UE_API bool IsActiveInstance(double Time, FStateTreeInstanceDebugId InstanceId) const;

	UE_API FText GetInstanceName(FStateTreeInstanceDebugId InstanceId) const;
	UE_API FText GetInstanceDescription(FStateTreeInstanceDebugId InstanceId) const;

	UE_API void SelectInstance(const FStateTreeInstanceDebugId InstanceId);

	void ClearSelection()
	{
		SelectInstance({});
	}

	FStateTreeInstanceDebugId GetSelectedInstanceId() const
	{
		return SelectedInstanceId;
	}

	UE_API TSharedPtr<const UE::StateTreeDebugger::FInstanceDescriptor> GetDescriptor(const FStateTreeInstanceDebugId InstanceId) const;
	TSharedPtr<const UE::StateTreeDebugger::FInstanceDescriptor> GetSelectedDescriptor() const
	{
		return GetDescriptor(SelectedInstanceId);
	}

	UE_DEPRECATED(5.7, "Use GetDescriptor instead")
	const UE::StateTreeDebugger::FInstanceDescriptor* GetInstanceDescriptor(const FStateTreeInstanceDebugId InstanceId) const
	{
		return GetDescriptor(InstanceId).Get();
	}
	UE_DEPRECATED(5.7, "Use GetSelectedDescriptor instead")
	const UE::StateTreeDebugger::FInstanceDescriptor* GetSelectedInstanceDescriptor() const
	{
		return GetDescriptor(SelectedInstanceId).Get();
	}

	UE_API bool HasStateBreakpoint(FStateTreeStateHandle StateHandle, EStateTreeBreakpointType BreakpointType) const;
	UE_API bool HasTaskBreakpoint(FStateTreeIndex16 Index, EStateTreeBreakpointType BreakpointType) const;
	UE_API bool HasTransitionBreakpoint(FStateTreeIndex16 Index, EStateTreeBreakpointType BreakpointType) const;
	UE_API void SetStateBreakpoint(FStateTreeStateHandle StateHandle, EStateTreeBreakpointType BreakpointType);
	UE_API void SetTransitionBreakpoint(FStateTreeIndex16 SubIndex, EStateTreeBreakpointType BreakpointType);
	UE_API void SetTaskBreakpoint(FStateTreeIndex16 NodeIndex, EStateTreeBreakpointType BreakpointType);
	UE_API void ClearBreakpoint(FStateTreeIndex16 NodeIndex, EStateTreeBreakpointType BreakpointType);
	UE_API void ClearAllBreakpoints();

	int32 NumBreakpoints() const
	{
		return Breakpoints.Num();
	}

	bool HasHitBreakpoint() const
	{
		return HitBreakpoint.IsSet();
	}

	bool CanProcessBreakpoints() const
	{
		return OnBreakpointHit.IsBound();
	}

	static UE_API FText DescribeTrace(const FTraceDescriptor& TraceDescriptor);
	static UE_API FText DescribeInstance(const UE::StateTreeDebugger::FInstanceDescriptor& StateTreeInstanceDesc);

	/**
	 * Finds and returns the event collection associated to a given instance Id.
	 * An invalid empty collection is returned if not found (IsValid needs to be called).
	 * @param InstanceId Id of the instance for which the event collection is returned. 
	 * @return Event collection associated to the provided Id or an invalid one if not found.
	 */
	UE_API const UE::StateTreeDebugger::FInstanceEventCollection& GetEventCollection(FStateTreeInstanceDebugId InstanceId) const;

	/** Clears events from all instances. */
	UE_API void ResetEventCollections();

	/** Returns the world time associated to the last processed event. */
	double GetLastProcessedRecordedWorldTime() const
	{
		return LastProcessedRecordedWorldTime;
	}

	UE_DEPRECATED(5.7, "Use GetLastProcessedRecordedWorldTime instead")
	double GetRecordingDuration() const
	{
		return GetLastProcessedRecordedWorldTime();
	}

	/** Returns the duration of the analysis session. This is not related to the world simulation time. */
	double GetAnalysisDuration() const
	{
		return AnalysisDuration;
	}

	/** Returns the time (based on the recording duration) associated to the selected frame. */
	double GetScrubTime() const
	{
		return ScrubState.GetScrubTime();
	}

	UE_API void SetScrubTime(double ScrubTime);

	UE_API void GetLiveTraces(TArray<FTraceDescriptor>& OutTraceDescriptors) const;

	/**
	 * Queue a request to auto start an analysis session on the next available live trace.
	 * @return True if connection was successfully requested or was able to use active trace, false otherwise.
	 */
	UE_API bool RequestAnalysisOfEditorSession();
	bool IsAnalyzingEditorSession() const
	{
		return AnalysisTransitionType == EAnalysisTransitionType::NoneToEditor
			|| AnalysisTransitionType == EAnalysisTransitionType::EditorToEditor
			|| AnalysisTransitionType == EAnalysisTransitionType::SelectedToEditor;
	}

	bool WasAnalyzingEditorSession() const
	{
		return AnalysisTransitionType == EAnalysisTransitionType::EditorToSelected
			|| AnalysisTransitionType == EAnalysisTransitionType::EditorToEditor;
	}

	/**
	 * Indicates whether the analysis session was started and not stopped yet.
	 */
	bool IsAnalysisSessionActive() const
	{
		return bSessionAnalysisActive;
	}

	/**
	 * Indicates that the debugger no longer processes new events from the analysis session until it gets resumed.
	 * This can be an external explicit request or after hitting a breakpoint.
	 */
	bool IsAnalysisSessionPaused() const
	{
		return bSessionAnalysisPaused;
	}

	UE_API const TraceServices::IAnalysisSession* GetAnalysisSession() const;

	/**
	 * Tries to start an analysis for a given trace descriptor.
	 * On success this method will execute the OnNewSession delegate.
	 * @param TraceDescriptor Descriptor of the trace that needs to be analyzed
	 * @return True if analysis was successfully started, false otherwise.
	 */
	UE_API bool RequestSessionAnalysis(const FTraceDescriptor& TraceDescriptor);

	void PauseSessionAnalysis()
	{
		bSessionAnalysisPaused = true;
	}

	void ResumeSessionAnalysis()
	{
		bSessionAnalysisPaused = false;
		HitBreakpoint.Reset();
	}

	UE_API void StopSessionAnalysis();

	FTraceDescriptor GetSelectedTraceDescriptor() const
	{
		return ActiveSessionTraceDescriptor;
	}

	UE_API FText GetSelectedTraceDescription() const;

	UE_DEPRECATED(5.7, "Use GetSessionInstanceDescriptors instead")
	UE_API void GetSessionInstances(TArray<UE::StateTreeDebugger::FInstanceDescriptor>& OutInstances) const;
	UE_API void GetSessionInstanceDescriptors(TArray<const TSharedRef<const UE::StateTreeDebugger::FInstanceDescriptor>>& OutInstances) const;

	FOnStateTreeDebuggerNewSession OnNewSession;
	FOnStateTreeDebuggerNewInstance OnNewInstance;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.7, "Delegate no longer used and will be removed")
	FOnStateTreeDebuggerDebuggedInstanceSet OnSelectedInstanceCleared;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FOnStateTreeDebuggerScrubStateChanged OnScrubStateChanged;
	FOnStateTreeDebuggerBreakpointHit OnBreakpointHit;
	FOnStateTreeDebuggerActiveStatesChanges OnActiveStatesChanged;

protected:
	virtual void Tick(float DeltaTime) override;

	virtual bool IsTickable() const override
	{
		return true;
	}

	virtual bool IsTickableWhenPaused() const override
	{
		return true;
	}

	virtual bool IsTickableInEditor() const override
	{
		return true;
	}

	virtual TStatId GetStatId() const override
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FStateTreeDebugger, STATGROUP_Tickables);
	}

private:

	UE::StateTreeDebugger::FInstanceEventCollection* GetMutableEventCollection(const FStateTreeInstanceDebugId InstanceId);

	enum class EAnalysisSourceType : uint8
	{
		// Analysis selected from available sessions
		SelectedSession,

		// Analysis automatically started from Editor new recording
		EditorSession
	};

	enum class EAnalysisTransitionType : uint8
	{
		Unset,
		NoneToSelected,
		NoneToEditor,
		EditorToSelected,
		EditorToEditor,
		SelectedToSelected,
		SelectedToEditor,
	};

	void UpdateAnalysisTransitionType(EAnalysisSourceType SourceType);

	void ReadTrace(double ScrubTime);
	void ReadTrace(uint64 FrameIndex);

	//~ Begin UE::StateTreeDebugger::ITraceReader interface
	virtual UE::StateTreeDebugger::FInstanceEventCollection* GetOrCreateEventCollection(FStateTreeInstanceDebugId InstanceId) override;
	virtual void OnTraceEventProcessed(FStateTreeInstanceDebugId InstanceId, const FStateTreeTraceEventVariantType& Event) override;
	//~ End UE::StateTreeDebugger::ITraceReader interface

	/**
	 * Tests event for given instance id against breakpoints.
	 * @param InstanceId Id of the statetree instance that produces the event.
	 * @param Event The event received from the instance.
	 * @return True if a breakpoint has been it, false otherwise.
	 */
	bool EvaluateBreakpoints(FStateTreeInstanceDebugId InstanceId, const FStateTreeTraceEventVariantType& Event);

	void SendNotifications();

	void SetActiveStates(const FStateTreeTraceActiveStates& NewActiveStates);

	/**
	 * Request an analysis session on the latest next available live trace.
	 * This will replace the current analysis session if any.
	 */
	void RequestAnalysisOfLatestTrace();

	/**
	 * Looks for a new live traces to start an analysis session.
	 * On failure, if 'RetryPollingDuration is > 0', will retry connecting every frame for 'RetryPollingDuration' seconds 
	 * @param RetryPollingDuration - On failure, how many seconds to retry connecting during FStateTreeDebugger::Tick
	 * @return True if the analysis was successfully started; false otherwise.
	 */
	bool TryStartNewLiveSessionAnalysis(float RetryPollingDuration);

	bool StartSessionAnalysis(const FTraceDescriptor& TraceDescriptor);

	void SetScrubStateCollection(const UE::StateTreeDebugger::FInstanceEventCollection* Collection);

	/**
	 * Recompute index of the span that contains the active states change event and update the active states.
	 * This method handles unselected instances in which case it will reset the active states and set the span index to INDEX_NONE
	 * */
	void RefreshActiveStates();

	UE::Trace::FStoreClient* GetStoreClient() const;

	void UpdateMetadata(FTraceDescriptor& TraceDescriptor) const;

	/** Module used to access the store client and analysis sessions .*/
	IStateTreeModule& StateTreeModule;

	/**
	 * The StateTree asset associated to the selected instance.
	 * This can be different from 'StateTreeAsset' if the debugger is associated to a linked asset.
	 */
	TWeakObjectPtr<const UStateTree> InstanceRootAsset;

	/** The StateTree asset associated to the last frame event processed by the debugger */
	TWeakObjectPtr<const UStateTree> CurrentFrameStateTreeAsset;

	/** The StateTree asset associated to this debugger. All instances will be using this asset. */
	TWeakObjectPtr<const UStateTree> StateTreeAsset;

	/** The trace analysis session. */
	TSharedPtr<const TraceServices::IAnalysisSession> AnalysisSession;

	/** Descriptor of the currently selected session */
	FTraceDescriptor ActiveSessionTraceDescriptor;

	/** Processed events for each instance. */
	TIndirectArray<UE::StateTreeDebugger::FInstanceEventCollection> EventCollections;

	/** Specific instance selected for more details */
	FStateTreeInstanceDebugId SelectedInstanceId;

	/** List of breakpoints set. This is per asset and not specific to an instance. */
	TArray<FStateTreeDebuggerBreakpoint> Breakpoints;

	/** List of currently active states in the selected instance */
	FStateTreeTraceActiveStates ActiveStates;

	/**
	 * When auto-connecting on next live session it is possible that a few frames are required for the tracing session to be accessible and connected to.
	 * This is to keep track of the previous last live session id so we can detect when the new one is available.
	 */
	int32 LastLiveSessionId = INDEX_NONE;

	/**
	 * When auto-connecting on next live session it is possible that a few frames are required for the tracing session to be accessible and connected to.
	 * This is to keep track of the time window where we will retry.
	 */
	float RetryLoadNextLiveSessionTimer = 0.0f;

	/**
	 * Most recent processed world time. This value is updated from received events so it might increase by large deltas.
	 */
	double LastProcessedRecordedWorldTime = 0;

	/** Duration of the analysis session. This is not related to the world simulation time. */
	double AnalysisDuration = 0;

	/** Last time in the recording that we use to fetch events and we will use for the next read. */
	double LastTraceReadTime = 0;

	/** Combined information regarding current scrub time (e.g. frame index, event collection index, etc.) */
	UE::StateTreeDebugger::FScrubState ScrubState;

	/** Information stored when a breakpoint is hit while processing events and used to send notifications. */
	FHitBreakpoint HitBreakpoint;

	/** List of new instances discovered by processing event in the analysis session. */
	TArray<FStateTreeInstanceDebugId> NewInstances;

	/** Indicates whether the analysis session is controlled by an external system, and we don't control its lifetime. */
	bool bIsExternalSessionAnalysis = false;

	/**
	 * Indicates whether the analysis session is valid and active.
	 * This flag is required since once an analysis session is stopped there is nothing in its API to retrieve its status.
	 */
	bool bSessionAnalysisActive = false;

	/**
	 * Indicates that the debugger no longer process new events from the analysis session until it gets resumed.
	 * This can be an external explicit request or after hitting a breakpoint.
	 */
	bool bSessionAnalysisPaused = false;

	/**
	 * Indicates the last transition type between two consecutive analyses to manage track cleanup properly.
	 */
	EAnalysisTransitionType AnalysisTransitionType = EAnalysisTransitionType::Unset;

	/**
	 * Delegate Handle bound to UE::StateTree::Delegates::OnTracingStateChanged
	 */
	FDelegateHandle TracingStateChangedHandle;

	/**
	 * Delegate Handle bound to UE::StateTree::Delegates::OnTraceAnalysisStateChanged
	 */
	FDelegateHandle TraceAnalysisStateChangedHandle;

	/**
	 * Delegate Handle bound to UE::StateTree::Delegates::OnTracingTimelineScrubbed
	 */
	FDelegateHandle TracingTimelineScrubbedHandle;
};

#undef UE_API

#endif // WITH_STATETREE_TRACE_DEBUGGER
