// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "StateTreeTraceTypes.h"

#if WITH_STATETREE_TRACE_DEBUGGER

#include "Math/Range.h"
#include "StateTree.h"
#include "StateTreeTypes.h"
#include "TraceServices/Model/Frames.h"

#endif // WITH_STATETREE_TRACE_DEBUGGER

#include "StateTreeDebuggerTypes.generated.h"

UENUM()
enum class EStateTreeBreakpointType : uint8
{
	Unset,
	OnEnter,
	OnExit,
	OnTransition,
};

#if WITH_STATETREE_TRACE_DEBUGGER

#define UE_API STATETREEMODULE_API

class UStateTree;
enum class EStateTreeTraceEventType : uint8;

namespace UE::StateTreeDebugger
{
/**
 * Struct indicating the index of the first event for a given trace recording frame.
 */
struct FFrameSpan
{
	FFrameSpan() = default;
	FFrameSpan(const TraceServices::FFrame& Frame, const double RecordingWorldTime, const int32 EventIdx)
		: Frame(Frame)
		, WorldTime(RecordingWorldTime)
		, EventIdx(EventIdx)
	{
	}

	double GetWorldTimeStart() const { return WorldTime; }
	double GetWorldTimeEnd() const { return WorldTime + (Frame.EndTime - Frame.StartTime); }

	/** Frame index in the analysis session */
	TraceServices::FFrame Frame;

	/** World simulation time associated to that Frame index */
	double WorldTime = 0;

	/** Index of the first event for that Frame index */
	int32 EventIdx = INDEX_NONE;
};


/**
 * Struct describing a state tree instance for a given StateTree asset
 */
struct FInstanceDescriptor : TSharedFromThis<FInstanceDescriptor>
{
	FInstanceDescriptor() = default;
	UE_API FInstanceDescriptor(const UStateTree* InStateTree, const FStateTreeInstanceDebugId InId, const FString& InName, const TRange<double>& InLifetime);

	UE_API bool IsValid() const;

	bool operator==(const FInstanceDescriptor& Other) const
	{
		return StateTree == Other.StateTree && Id == Other.Id;
	}

	bool operator!=(const FInstanceDescriptor& Other) const
	{
		return !(*this == Other);
	}

	/** Whether the associated instance is still active (i.e., not explicitly stopped by an execution context) */
	bool IsInstanceActive() const
	{
		return Lifetime.GetUpperBoundValue() == ActiveInstanceEndTime;
	}

	friend FString LexToString(const FInstanceDescriptor& InstanceDesc)
	{
		return FString::Printf(TEXT("%s | %s | %s"),
			*GetNameSafe(InstanceDesc.StateTree.Get()),
			*LexToString(InstanceDesc.Id),
			*InstanceDesc.Name);
	}

	friend uint32 GetTypeHash(const FInstanceDescriptor& Desc)
	{
		return GetTypeHash(Desc.Id);
	}

	static constexpr double ActiveInstanceEndTime = std::numeric_limits<double>::max();
	TRange<double> Lifetime = TRange<double>(0);
	TWeakObjectPtr<const UStateTree> StateTree = nullptr;
	TArray<TWeakObjectPtr<const UStateTree>> LinkedStateTrees;
	FString Name;
	FStateTreeInstanceDebugId Id = FStateTreeInstanceDebugId::Invalid;
};


/**
 * Struct holding organized events associated to a given state tree instance.
 */
struct FInstanceEventCollection
{
	FInstanceEventCollection() = default;
	explicit FInstanceEventCollection(const FStateTreeInstanceDebugId& InstanceId)
		: InstanceId(InstanceId)
	{
	}

	friend bool operator==(const FInstanceEventCollection& Lhs, const FInstanceEventCollection& RHS)
	{
		return Lhs.InstanceId == RHS.InstanceId;
	}

	friend bool operator!=(const FInstanceEventCollection& Lhs, const FInstanceEventCollection& RHS)
	{
		return !(Lhs == RHS);
	}

	bool IsValid() const
	{
		return InstanceId.IsValid();
	}

	bool IsInvalid() const
	{
		return !IsValid();
	}

	struct FActiveStatesChangePair
	{
		FActiveStatesChangePair(const int32 SpanIndex, const int32 EventIndex)
			: SpanIndex(SpanIndex)
			, EventIndex(EventIndex)
		{
		}

		int32 SpanIndex = INDEX_NONE;
		int32 EventIndex = INDEX_NONE;
	};

	struct FContiguousTraceInfo
	{
		explicit FContiguousTraceInfo(int32 LastSpanIndex)
			: LastSpanIndex(LastSpanIndex)
		{
		}

		/** Indicates the index of the last spans of the trace and from which the frame index will be used to offset new events since their frames will restart at 0. */
		int32 LastSpanIndex = INDEX_NONE;
	};

	/** Id of the instance associated to the stored events. */
	FStateTreeInstanceDebugId InstanceId;

	/** All events received for this instance. */
	TArray<FStateTreeTraceEventVariantType> Events;

	/** Spans for frames with events. Each span contains the frame information and the index of the first event for that frame. */
	TArray<FFrameSpan> FrameSpans;

	/** This list is only used to merge events when dealing with multiple traces related to the same tree instance. */
	TArray<FContiguousTraceInfo> ContiguousTracesData;

	/** Indices of span and event for frames with a change of activate states. */
	TArray<FActiveStatesChangePair> ActiveStatesChanges;

	/**
	 * Returns the event collection associated to the currently selected instance.
	 * An invalid empty collection is returned if there is no selected instance. (IsValid needs to be called).
	 * @return Event collection associated to the selected instance or an invalid one if not found.
	 */
	static UE_API const FInstanceEventCollection Invalid;
};

struct FScrubState
{
	FScrubState() = default;
	explicit FScrubState(const FInstanceEventCollection* EventCollection)
		: EventCollection(EventCollection)
	{
	}
	UE_DEPRECATED(5.7, "FScrubState will no longer support multiple collections.")
	explicit FScrubState(const TArray<FInstanceEventCollection>& EventCollections)
	{
	}

	UE_DEPRECATED(5.7, "FScrubState will no longer support multiple collections.")
	int32 GetEventCollectionIndex() const
	{
		return INDEX_NONE;
	}

	UE_DEPRECATED(5.7, "FScrubState will no longer support multiple collections.")
	void SetEventCollectionIndex(const int32 InEventCollectionIndex)
	{
	}

	/** Assigns a new collection and updates internal indices for current scrub time. */
	UE_API void SetEventCollection(const FInstanceEventCollection* InEventCollection);

	/** @return Index of the span for the currently selected frame; INDEX_NONE if there is no span for the current scrub time. */
	int32 GetFrameSpanIndex() const
	{
		return FrameSpanIndex;
	}

	/** @return Index of the list of active states for the currently selected frame; INDEX_NONE if there is no active states for the current scrub time. */
	int32 GetActiveStatesIndex() const
	{
		return ActiveStatesIndex;
	}

	/** @return Current scrub time. */
	double GetScrubTime() const
	{
		return ScrubTime;
	}

	/**
	 * Updates internal indices based on the new time.
	 * @param NewScrubTime The new scrub time to use to update all internal indices
	 * @param bForceRefresh Whether to force update of internal indices regardless if the current stored scrub time is the same as the provided one
	 * @return true if values were updated; false otherwise (i.e. no changes)
	 */
	UE_API bool SetScrubTime(double NewScrubTime, bool bForceRefresh = false);

	/**
	 * Indicates if the current scrub state points to a valid frame.
	 * @return True if the frame index is set
	 */
	bool IsInBounds() const
	{
		return ScrubTimeBoundState == EScrubTimeBoundState::InBounds;
	}

	/**
	 * Indicates if the current scrub state points to an active states entry in the event collection.
	 * @return True if the collection and active states indices are set
	 */
	bool IsPointingToValidActiveStates() const
	{
		return EventCollection != nullptr && ActiveStatesIndex != INDEX_NONE;
	}

	/** Indicates if there is a frame before with events. */
	UE_API bool HasPreviousFrame() const;

	/**
	 * Set scrubbing info using the previous frame with events.
	 * HasPreviousFrame must be used to validate that this method can be called otherwise some checks might fail.
	 * @return Adjusted scrub time
	 */
	UE_API double GotoPreviousFrame();

	/** Indicates if there is a frame after with events. */
	UE_API bool HasNextFrame() const;

	/**
	 * Set scrubbing info using the next frame with events.
	 * HasPreviousFrame must be used to validate that this method can be called otherwise some checks might fail.
	 * @return Adjusted scrub time
	 */
	UE_API double GotoNextFrame();

	/** Indicates if there is a frame before where the StateTree has a different list of active states. */
	UE_API bool HasPreviousActiveStates() const;

	/**
	 * Set scrubbing info using the previous frame where the StateTree has a different list of active states.
	 * HasPreviousActiveStates must be used to validate that this method can be called otherwise some checks might fail.
	 * @return Adjusted scrub time
	 */
	UE_API double GotoPreviousActiveStates();

	/** Indicates if there is a frame after where the StateTree has a different list of active states. */
	UE_API bool HasNextActiveStates() const;

	/**
	 * Set scrubbing info using the next frame where the StateTree has a different list of active states.
	 * HasNextActiveStates must be used to validate that this method can be called otherwise some checks might fail.
	 * @return Adjusted scrub time
	 */
	UE_API double GotoNextActiveStates();

	/**
	 * Returns the event collection associated.
	 * An invalid empty collection is returned if no collection was set (IsValid needs to be called).
	 * @return Event collection set or an invalid one if not set.
	 */
	UE_API const FInstanceEventCollection& GetEventCollection() const;

private:
	enum class EScrubTimeBoundState : uint8
	{
		Unset,
		/** There are events but current time is before the first frame. */
		BeforeLowerBound,
		/** There are events and current time is within the frames received. */
		InBounds,
		/** There are events but current time is after the last frame. */
		AfterHigherBound
	};

	UE_API void SetFrameSpanIndex(int32 NewFrameSpanIndex);
	UE_API void SetActiveStatesIndex(int32 NewActiveStatesIndex);
	UE_API void UpdateActiveStatesIndex(int32 SpanIndex);

	const FInstanceEventCollection* EventCollection = nullptr;
	double ScrubTime = 0;
	uint64 TraceFrameIndex = INDEX_NONE;
	int32 FrameSpanIndex = INDEX_NONE;
	int32 ActiveStatesIndex = INDEX_NONE;
	EScrubTimeBoundState ScrubTimeBoundState = EScrubTimeBoundState::Unset;
};

} // UE::StateTreeDebugger

struct FStateTreeDebuggerBreakpoint
{
	// Wrapper structs to be able to use TVariant with more than one type based on FStateTreeIndex16 (can't use 'using')
	struct FStateTreeTaskIndex
	{
		FStateTreeTaskIndex() = default;
		explicit FStateTreeTaskIndex(const FStateTreeIndex16& Index)
			: Index(Index)
		{
		}

		FStateTreeIndex16 Index;
	};

	struct FStateTreeTransitionIndex
	{
		FStateTreeTransitionIndex() = default;
		explicit FStateTreeTransitionIndex(const FStateTreeIndex16& Index)
			: Index(Index)
		{
		}
		FStateTreeIndex16 Index;
	};

	using FIdentifierVariantType = TVariant<FStateTreeStateHandle, FStateTreeTaskIndex, FStateTreeTransitionIndex>;

	FStateTreeDebuggerBreakpoint();
	explicit FStateTreeDebuggerBreakpoint(const FStateTreeStateHandle StateHandle, const EStateTreeBreakpointType BreakpointType);
	explicit FStateTreeDebuggerBreakpoint(const FStateTreeTaskIndex Index, const EStateTreeBreakpointType BreakpointType);
	explicit FStateTreeDebuggerBreakpoint(const FStateTreeTransitionIndex Index, const EStateTreeBreakpointType BreakpointType);

	UE_API bool IsMatchingEvent(const FStateTreeTraceEventVariantType& Event) const;

	FIdentifierVariantType ElementIdentifier;
	EStateTreeBreakpointType BreakpointType;
	EStateTreeTraceEventType EventType;

private:
	static EStateTreeTraceEventType GetMatchingEventType(EStateTreeBreakpointType BreakpointType);
};

#undef UE_API

#endif // WITH_STATETREE_TRACE_DEBUGGER
