// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeTypes.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Debugger/StateTreeDebug.h"
#endif

class UStateTree;
class UObject;

#if WITH_STATETREE_DEBUG
namespace UE::StateTree::Debug
{
	class FRuntimeValidationInstanceData;
}
#endif

namespace UE::StateTree::Debug
{

/**
 * For debugging purposes, test the actions that are made on the StateTree instance.
 */
struct FRuntimeValidation
{
public:
	FRuntimeValidation() = default;

#if WITH_STATETREE_DEBUG
	FRuntimeValidation(TNotNull<FRuntimeValidationInstanceData*> Instance);
	FRuntimeValidationInstanceData* GetInstanceData() const;
#endif

	/** Set the Owner of the data */
	void SetContext(TNotNull<const UObject*> Owner, TNotNull<const UStateTree*> StateTree, bool bInstanceDataWriteAccessAcquired) const;

private:
#if WITH_STATETREE_DEBUG
	FRuntimeValidationInstanceData* RuntimeValidationData = nullptr;
#endif // WITH_STATETREE_DEBUG
};

} // UE::StateTree::Debug
