// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_STATETREE_TRACE

#include "Trace/Trace.h"
#include "Containers/ContainersFwd.h"
#include "Containers/UnrealString.h"
#include "Containers/StringView.h"
#include "Misc/NotNull.h"

class UStateTree;
struct FStateTreeActiveStates;
struct FStateTreeDataView;
struct FStateTreeExecutionFrame;
struct FStateTreeIndex16;
struct FStateTreeInstanceDebugId;
struct FStateTreeReadOnlyExecutionContext;
struct FStateTreeStateHandle;
struct FStateTreeTransitionSource;
enum class EStateTreeStateSelectionBehavior : uint8;
enum class EStateTreeRunStatus : uint8;
enum class EStateTreeTraceEventType : uint8;
enum class EStateTreeUpdatePhase : uint8;
namespace ELogVerbosity { enum Type : uint8; }

UE_TRACE_CHANNEL_EXTERN(StateTreeDebugChannel, STATETREEMODULE_API)

#define UE_API STATETREEMODULE_API

namespace UE::StateTreeTrace
{

/** Struct allowing StateTree nodes to add (or replace) a custom string to the trace event */
struct FNodeCustomDebugData
{
	enum class EMergePolicy: uint8
	{
		Unset,
		Append,
		Override
	};

	FNodeCustomDebugData() = default;

	FNodeCustomDebugData(FStringView TraceDebuggerString, const EMergePolicy MergePolicy): TraceDebuggerString(TraceDebuggerString)
	, MergePolicy(MergePolicy)
	{
	}

	FNodeCustomDebugData(FNodeCustomDebugData&& Other)
		: TraceDebuggerString(MoveTemp(Other.TraceDebuggerString))
		, MergePolicy(Other.MergePolicy)
	{
		Other.Reset();
	}

	FNodeCustomDebugData& operator=(FNodeCustomDebugData&& Other)
	{
		if (this == &Other)
		{
			return *this;
		}
		TraceDebuggerString = MoveTemp(Other.TraceDebuggerString);
		MergePolicy = Other.MergePolicy;
		Other.Reset();
		return *this;
	}

	bool IsSet() const
	{
		return MergePolicy != EMergePolicy::Unset && !TraceDebuggerString.IsEmpty();
	}

	void Reset()
	{
		TraceDebuggerString.Reset();
		MergePolicy = EMergePolicy::Unset;
	}

	bool ShouldOverrideDataView() const
	{
		return MergePolicy == EMergePolicy::Override;
	}
	bool ShouldAppendToDataView() const
	{
		return MergePolicy == EMergePolicy::Append;
	}

	FStringView GetTraceDebuggerString() const
	{
		return TraceDebuggerString;
	}

private:
	FString TraceDebuggerString;
	EMergePolicy MergePolicy = EMergePolicy::Unset;
};

void RegisterGlobalDelegates();
void UnregisterGlobalDelegates();
void OutputAssetDebugIdEvent(const UStateTree* StateTree, FStateTreeIndex16 AssetDebugId);
void OutputPhaseScopeEvent(TNotNull<const FStateTreeReadOnlyExecutionContext*> Context, EStateTreeUpdatePhase Phase, EStateTreeTraceEventType EventType, FStateTreeStateHandle StateHandle);
void OutputInstanceLifetimeEvent(TNotNull<const FStateTreeReadOnlyExecutionContext*> Context, EStateTreeTraceEventType EventType);
void OutputInstanceAssetEvent(TNotNull<const FStateTreeReadOnlyExecutionContext*> Context, const UStateTree* StateTree);
void OutputInstanceFrameEvent(TNotNull<const FStateTreeReadOnlyExecutionContext*> Context, const FStateTreeExecutionFrame* Frame);
void OutputLogEventTrace(TNotNull<const FStateTreeReadOnlyExecutionContext*> Context, ELogVerbosity::Type Verbosity, const TCHAR* Fmt, ...);
void OutputStateEventTrace(TNotNull<const FStateTreeReadOnlyExecutionContext*> Context, FStateTreeStateHandle StateHandle, EStateTreeTraceEventType EventType);
void OutputTaskEventTrace(TNotNull<const FStateTreeReadOnlyExecutionContext*> Context, FStateTreeIndex16 TaskIdx, FStateTreeDataView DataView, EStateTreeTraceEventType EventType, EStateTreeRunStatus Status);
void OutputEvaluatorEventTrace(TNotNull<const FStateTreeReadOnlyExecutionContext*> Context, FStateTreeIndex16 EvaluatorIdx, FStateTreeDataView DataView, EStateTreeTraceEventType EventType);
void OutputConditionEventTrace(TNotNull<const FStateTreeReadOnlyExecutionContext*> Context, FStateTreeIndex16 ConditionIdx, FStateTreeDataView DataView, EStateTreeTraceEventType EventType);
void OutputTransitionEventTrace(TNotNull<const FStateTreeReadOnlyExecutionContext*> Context, FStateTreeTransitionSource TransitionSource, EStateTreeTraceEventType EventType);
void OutputActiveStatesEventTrace(TNotNull<const FStateTreeReadOnlyExecutionContext*> Context, TConstArrayView<FStateTreeExecutionFrame> ActiveFrames);

UE_DEPRECATED(5.6, "This method will no longer be exposed publicly.")
FStateTreeIndex16 FindOrAddDebugIdForAsset(const UStateTree* StateTree);
UE_DEPRECATED(5.7, "Use the overload taking a pointer to the execution context instead.")
void OutputPhaseScopeEvent(FStateTreeInstanceDebugId InstanceId, EStateTreeUpdatePhase Phase, EStateTreeTraceEventType EventType, FStateTreeStateHandle StateHandle);
UE_DEPRECATED(5.7, "Use the overload taking a pointer to the execution context instead.")
void OutputInstanceLifetimeEvent(FStateTreeInstanceDebugId InstanceId, const UStateTree* StateTree, const TCHAR* InstanceName, EStateTreeTraceEventType EventType);
UE_DEPRECATED(5.7, "Use the overload taking a pointer to the execution context instead.")
void OutputInstanceAssetEvent(FStateTreeInstanceDebugId InstanceId, const UStateTree* StateTree);
UE_DEPRECATED(5.7, "Use the overload taking a pointer to the execution context instead.")
void OutputInstanceFrameEvent(FStateTreeInstanceDebugId InstanceId, const FStateTreeExecutionFrame* Frame);
UE_DEPRECATED(5.7, "Use the overload taking a pointer to the execution context instead.")
void OutputLogEventTrace(FStateTreeInstanceDebugId InstanceId, ELogVerbosity::Type Verbosity, const TCHAR* Fmt, ...);
UE_DEPRECATED(5.7, "Use the overload taking a pointer to the execution context instead.")
void OutputStateEventTrace(FStateTreeInstanceDebugId InstanceId, FStateTreeStateHandle StateHandle, EStateTreeTraceEventType EventType);
UE_DEPRECATED(5.7, "Use the overload taking a pointer to the execution context instead.")
void OutputTaskEventTrace(FStateTreeInstanceDebugId InstanceId, FNodeCustomDebugData&& DebugData, FStateTreeIndex16 TaskIdx, FStateTreeDataView DataView, EStateTreeTraceEventType EventType, EStateTreeRunStatus Status);
UE_DEPRECATED(5.7, "Use the overload taking a pointer to the execution context instead.")
void OutputEvaluatorEventTrace(FStateTreeInstanceDebugId InstanceId, FNodeCustomDebugData&& DebugData, FStateTreeIndex16 EvaluatorIdx, FStateTreeDataView DataView, EStateTreeTraceEventType EventType);
UE_DEPRECATED(5.7, "Use the overload taking a pointer to the execution context instead.")
void OutputConditionEventTrace(FStateTreeInstanceDebugId InstanceId, FNodeCustomDebugData&& DebugData, FStateTreeIndex16 ConditionIdx, FStateTreeDataView DataView, EStateTreeTraceEventType EventType);
UE_DEPRECATED(5.7, "Use the overload taking a pointer to the execution context instead.")
void OutputTransitionEventTrace(FStateTreeInstanceDebugId InstanceId, FStateTreeTransitionSource TransitionSource, EStateTreeTraceEventType EventType);
UE_DEPRECATED(5.7, "Use the overload taking a pointer to the execution context instead.")
void OutputActiveStatesEventTrace(FStateTreeInstanceDebugId InstanceId, TConstArrayView<FStateTreeExecutionFrame> ActiveFrames);

} // UE::StateTreeTrace

#define TRACE_STATETREE_INSTANCE_EVENT(Context, EventType) \
	UE::StateTreeTrace::OutputInstanceLifetimeEvent(Context, EventType);

#define TRACE_STATETREE_INSTANCE_FRAME_EVENT(Context, Frame) \
	UE::StateTreeTrace::OutputInstanceFrameEvent(Context, Frame);

#define TRACE_STATETREE_PHASE_EVENT(Context, Phase, EventType, StateHandle) \
	UE::StateTreeTrace::OutputPhaseScopeEvent(Context, Phase, EventType, StateHandle); \

#define TRACE_STATETREE_LOG_EVENT(Context, TraceVerbosity, Format, ...) \
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(StateTreeDebugChannel)) \
	{ \
		UE::StateTreeTrace::OutputLogEventTrace(Context, ELogVerbosity::TraceVerbosity, Format, ##__VA_ARGS__); \
	}

#define TRACE_STATETREE_STATE_EVENT(Context, StateHandle, EventType) \
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(StateTreeDebugChannel)) \
	{ \
		UE::StateTreeTrace::OutputStateEventTrace(Context, StateHandle, EventType); \
	}

#define TRACE_STATETREE_TASK_EVENT(Context, TaskIdx, DataView, EventType, Status) \
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(StateTreeDebugChannel)) \
	{ \
		UE::StateTreeTrace::OutputTaskEventTrace(Context, TaskIdx, DataView, EventType, Status); \
	}

#define TRACE_STATETREE_EVALUATOR_EVENT(Context, EvaluatorIdx, DataView, EventType) \
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(StateTreeDebugChannel)) \
	{ \
		UE::StateTreeTrace::OutputEvaluatorEventTrace(Context, EvaluatorIdx, DataView, EventType); \
	}

#define TRACE_STATETREE_CONDITION_EVENT(Context, ConditionIdx, DataView, EventType) \
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(StateTreeDebugChannel)) \
	{ \
		UE::StateTreeTrace::OutputConditionEventTrace(Context, ConditionIdx, DataView, EventType); \
	}

#define TRACE_STATETREE_TRANSITION_EVENT(Context, TransitionIdx, EventType) \
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(StateTreeDebugChannel)) \
	{ \
		UE::StateTreeTrace::OutputTransitionEventTrace(Context, TransitionIdx, EventType); \
	}

#define TRACE_STATETREE_ACTIVE_STATES_EVENT(Context, ActivateFrames) \
		UE::StateTreeTrace::OutputActiveStatesEventTrace(Context, ActivateFrames);

#undef UE_API

#else //WITH_STATETREE_TRACE

#define TRACE_STATETREE_INSTANCE_EVENT(...)
#define TRACE_STATETREE_INSTANCE_FRAME_EVENT(...)
#define TRACE_STATETREE_PHASE_EVENT(...)
#define TRACE_STATETREE_LOG_EVENT(...)
#define TRACE_STATETREE_STATE_EVENT(...)
#define TRACE_STATETREE_TASK_EVENT(...)
#define TRACE_STATETREE_EVALUATOR_EVENT(...)
#define TRACE_STATETREE_CONDITION_EVENT(...)
#define TRACE_STATETREE_TRANSITION_EVENT(...)
#define TRACE_STATETREE_ACTIVE_STATES_EVENT(...)

#endif // WITH_STATETREE_TRACE
