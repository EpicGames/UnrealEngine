// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_STATETREE_TRACE_DEBUGGER

#include "Trace/Analyzer.h"

class FStateTreeTraceProvider;
namespace TraceServices { class IAnalysisSession; }

class FStateTreeTraceAnalyzer : public UE::Trace::IAnalyzer
{
public:
	FStateTreeTraceAnalyzer(TraceServices::IAnalysisSession& InSession, FStateTreeTraceProvider& InProvider);

	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override {}
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	enum : uint16
	{
		RouteId_WorldTimestamp,
		RouteId_AssetDebugId,
		RouteId_Instance,
		RouteId_InstanceFrame,
		RouteId_Phase,
		RouteId_LogMessage,
		RouteId_State,
		RouteId_Task,
		RouteId_Evaluator,
		RouteId_Transition,
		RouteId_Condition,
		RouteId_ActiveStates
	};

	TraceServices::IAnalysisSession& Session;
	FStateTreeTraceProvider& Provider;
	double WorldTime = 0;
};
#endif // WITH_STATETREE_TRACE_DEBUGGER