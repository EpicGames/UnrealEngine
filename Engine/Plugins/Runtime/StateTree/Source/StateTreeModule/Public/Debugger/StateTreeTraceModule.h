// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_STATETREE_TRACE_DEBUGGER

#include "TraceServices/ModuleService.h"

class FStateTreeTraceModule : public TraceServices::IModule
{
public:
	// TraceServices::IModule interface
	virtual void GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo) override;
	virtual void OnAnalysisBegin(TraceServices::IAnalysisSession& Session) override;
	virtual void GetLoggers(TArray<const TCHAR *>& OutLoggers) override;

private:
	static FName ModuleName;
};

#endif // WITH_STATETREE_TRACE_DEBUGGER