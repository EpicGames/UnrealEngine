// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_STATETREE_TRACE_DEBUGGER

#include "IStateTreeTraceProvider.h"
#include "Common/ProviderLock.h"
#include "Model/PointTimeline.h"
#include "StateTreeIndexTypes.h"

namespace TraceServices
{
	class IAnalysisSession;
	extern thread_local FProviderLock::FThreadLocalState GStateTreeProviderLockState;
}
class UStateTree;
struct FStateTreeInstanceDebugId;
namespace UE::StateTreeDebugger { struct FInstanceDescriptor; }

class FStateTreeTraceProvider : public IStateTreeTraceProvider, public TraceServices::IEditableProvider
{
public:
	STATETREEMODULE_API static FName ProviderName;

	explicit FStateTreeTraceProvider(TraceServices::IAnalysisSession& InSession);

	//~ IProvider (read lock)
	virtual void BeginRead() const override
	{
		Lock.BeginRead(TraceServices::GStateTreeProviderLockState);
	}
	virtual void EndRead() const override
	{
		Lock.EndRead(TraceServices::GStateTreeProviderLockState);
	}
	virtual void ReadAccessCheck() const override
	{
		Lock.ReadAccessCheck(TraceServices::GStateTreeProviderLockState);
	}

	//~ IEditableProvider (edit lock)
	virtual void BeginEdit() const override
	{
		Lock.BeginWrite(TraceServices::GStateTreeProviderLockState);
	}
	virtual void EndEdit() const override
	{
		Lock.EndWrite(TraceServices::GStateTreeProviderLockState);
	}
	virtual void EditAccessCheck() const override
	{
		Lock.WriteAccessCheck(TraceServices::GStateTreeProviderLockState);
	}
	
	void AppendEvent(FStateTreeInstanceDebugId InInstanceId, double InTime, const FStateTreeTraceEventVariantType& InEvent);
	void AppendInstanceEvent(
		const FStateTreeIndex16 AssetDebugId,
		const FStateTreeInstanceDebugId InInstanceId,
		const TCHAR* InInstanceName,
		double InTime,
		double InWorldRecordingTime,
		EStateTreeTraceEventType InEventType);

	void AppendAssetDebugId(const UStateTree* InStateTree, const FStateTreeIndex16 AssetDebugId);
	bool GetAssetFromDebugId(const FStateTreeIndex16 AssetDebugId, TWeakObjectPtr<const UStateTree>& WeakStateTree) const;
	bool GetAssetFromInstanceId(const FStateTreeInstanceDebugId InstanceId, TWeakObjectPtr<const UStateTree>& WeakStateTree) const;

protected:
	//~ IStateTreeDebuggerProvider interface
	virtual TSharedPtr<const UE::StateTreeDebugger::FInstanceDescriptor> GetInstanceDescriptor(const FStateTreeInstanceDebugId InstanceId) const override;
	virtual void GetInstances(TArray<const TSharedRef<const UE::StateTreeDebugger::FInstanceDescriptor>>& OutInstances) const override;
	virtual bool ReadTimelines(const FStateTreeInstanceDebugId InstanceId, TFunctionRef<void(const FStateTreeInstanceDebugId ProcessedInstanceId, const FEventsTimeline&)> Callback) const override;
	virtual bool ReadTimelines(const UStateTree& StateTree, TFunctionRef<void(const FStateTreeInstanceDebugId ProcessedInstanceId, const FEventsTimeline&)> Callback) const override;

private:
	TraceServices::IAnalysisSession& Session;
	mutable TraceServices::FProviderLock Lock;

	TMap<FStateTreeInstanceDebugId, uint32> InstanceIdToDebuggerEntryTimelines;
	TArray<TSharedRef<UE::StateTreeDebugger::FInstanceDescriptor>> Descriptors;
	TArray<TSharedRef<TraceServices::TPointTimeline<FStateTreeTraceEventVariantType>>> EventsTimelines;

	struct FStateTreeDebugIdPair
	{
		FStateTreeDebugIdPair(const TWeakObjectPtr<const UStateTree>& WeakStateTree, const FStateTreeIndex16 Id)
			: WeakStateTree(WeakStateTree)
			, Id(Id)
		{
		}

		TWeakObjectPtr<const UStateTree> WeakStateTree;
		FStateTreeIndex16 Id;
	};

	TArray<FStateTreeDebugIdPair> StateTreeAssets;
};
#endif // WITH_STATETREE_TRACE_DEBUGGER