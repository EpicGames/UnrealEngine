// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/TVariant.h"
#include "StateTreeIndexTypes.h"
#include "StateTreeExecutionTypes.h"
#include "StateTreeTraceTypes.generated.h"

class UStateTree;
struct FStateTreeStateHandle;
enum class EStateTreeStateSelectionBehavior : uint8;
enum class EStateTreeRunStatus : uint8;

UENUM()
enum class EStateTreeTraceStatus : uint8
{
	TracesStarted,
	StoppingTrace,
	TracesStopped,
	Cleared
};

UENUM()
enum class EStateTreeTraceAnalysisStatus : uint8
{
	Started,
	Stopped,
	Cleared
};

UENUM()
enum class EStateTreeTraceEventType : uint8
{
	Unset,
	OnEntering				UMETA(DisplayName = "Entering"),
	OnEntered				UMETA(DisplayName = "Entered"),
	OnExiting				UMETA(DisplayName = "Exiting"),
	OnExited				UMETA(DisplayName = "Exited"),
	Push					UMETA(DisplayName = "Push"),
	Pop						UMETA(DisplayName = "Pop"),
	OnStateSelected			UMETA(DisplayName = "Selected"),
	OnStateCompleted		UMETA(DisplayName = "Completed"),
	OnTicking				UMETA(DisplayName = "Tick"),
	OnTaskCompleted			UMETA(DisplayName = "Completed"),
	OnTicked				UMETA(DisplayName = "Ticked"),
	Passed					UMETA(DisplayName = "Passed"),
	Failed					UMETA(DisplayName = "Failed"),
	ForcedSuccess			UMETA(DisplayName = "Forced Success"),
	ForcedFailure			UMETA(DisplayName = "Forced Failure"),
	InternalForcedFailure	UMETA(DisplayName = "Internal Forced Failure"),
	OnRequesting			UMETA(DisplayName = "Requesting"),
	OnEvaluating			UMETA(DisplayName = "Evaluating"),
	OnTransition			UMETA(DisplayName = "Applying Transition"),
	OnTreeStarted			UMETA(DisplayName = "Tree Started"),
	OnTreeStopped			UMETA(DisplayName = "Tree Stopped")
};

#if WITH_STATETREE_TRACE_DEBUGGER

struct FStateTreeTraceBaseEvent
{
	explicit FStateTreeTraceBaseEvent(const double RecordingWorldTime, const EStateTreeTraceEventType EventType)
		: RecordingWorldTime(RecordingWorldTime)
		, EventType(EventType)
	{
	}

	static FString GetDataTypePath()
	{
		return TEXT("");
	}

	static FString GetDataAsText()
	{
		return TEXT("");
	}

	static FString GetDebugText()
	{
		return TEXT("");
	}

	double RecordingWorldTime = 0;
	EStateTreeTraceEventType EventType;
};

struct FStateTreeTracePhaseEvent : FStateTreeTraceBaseEvent
{
	explicit FStateTreeTracePhaseEvent(const double RecordingWorldTime, const EStateTreeUpdatePhase Phase, const EStateTreeTraceEventType EventType, const FStateTreeStateHandle StateHandle)
		: FStateTreeTraceBaseEvent(RecordingWorldTime, EventType)
		, Phase(Phase)
		, StateHandle(StateHandle)
	{
	}

	STATETREEMODULE_API FString ToFullString(const UStateTree& StateTree) const;
	STATETREEMODULE_API FString GetValueString(const UStateTree& StateTree) const;
	STATETREEMODULE_API FString GetTypeString(const UStateTree& StateTree) const;

	EStateTreeUpdatePhase Phase;
	FStateTreeStateHandle StateHandle;
};

struct FStateTreeTraceLogEvent : FStateTreeTraceBaseEvent
{
	explicit FStateTreeTraceLogEvent(const double RecordingWorldTime, const ELogVerbosity::Type Verbosity, const FString& Message)
		: FStateTreeTraceBaseEvent(RecordingWorldTime, EStateTreeTraceEventType::Unset)
		, Verbosity(Verbosity)
		, Message(Message)
	{
	}

	STATETREEMODULE_API FString ToFullString(const UStateTree& StateTree) const;
	STATETREEMODULE_API FString GetValueString(const UStateTree& StateTree) const;
	STATETREEMODULE_API FString GetTypeString(const UStateTree& StateTree) const;

	ELogVerbosity::Type Verbosity;
	FString Message;
};

struct FStateTreeTracePropertyEvent : FStateTreeTraceLogEvent
{
	explicit FStateTreeTracePropertyEvent(const double RecordingWorldTime, const FString& Message)
		: FStateTreeTraceLogEvent(RecordingWorldTime, ELogVerbosity::Verbose, Message)
	{
	}

	explicit FStateTreeTracePropertyEvent(const double RecordingWorldTime, const ELogVerbosity::Type Verbosity, const FString& Message)
		: FStateTreeTraceLogEvent(RecordingWorldTime, Verbosity, Message)
	{
	}

	STATETREEMODULE_API FString ToFullString(const UStateTree& StateTree) const;
	STATETREEMODULE_API FString GetValueString(const UStateTree& StateTree) const;
	STATETREEMODULE_API FString GetTypeString(const UStateTree& StateTree) const;
};

struct FStateTreeTraceTransitionEvent : FStateTreeTraceBaseEvent
{
	explicit FStateTreeTraceTransitionEvent(const double RecordingWorldTime, const FStateTreeTransitionSource TransitionSource, const EStateTreeTraceEventType EventType)
		: FStateTreeTraceBaseEvent(RecordingWorldTime, EventType)
		, TransitionSource(TransitionSource)
	{
	}

	STATETREEMODULE_API FString ToFullString(const UStateTree& StateTree) const;
	STATETREEMODULE_API FString GetValueString(const UStateTree& StateTree) const;
	STATETREEMODULE_API FString GetTypeString(const UStateTree& StateTree) const;

	FStateTreeTransitionSource TransitionSource;
};

struct FStateTreeTraceNodeEvent : FStateTreeTraceBaseEvent
{
	explicit FStateTreeTraceNodeEvent(const double RecordingWorldTime, const FStateTreeIndex16 Index, const EStateTreeTraceEventType EventType)
		: FStateTreeTraceBaseEvent(RecordingWorldTime, EventType)
		, Index(Index)
	{
	}

	STATETREEMODULE_API FString ToFullString(const UStateTree& StateTree) const;
	STATETREEMODULE_API FString GetValueString(const UStateTree& StateTree) const;
	STATETREEMODULE_API FString GetTypeString(const UStateTree& StateTree) const;
	
	FStateTreeIndex16 Index;
};

struct FStateTreeTraceStateEvent : FStateTreeTraceNodeEvent
{
	explicit FStateTreeTraceStateEvent(const double RecordingWorldTime, const FStateTreeIndex16 Index, const EStateTreeTraceEventType EventType)
		: FStateTreeTraceNodeEvent(RecordingWorldTime, Index, EventType)
	{
	}

	STATETREEMODULE_API FString ToFullString(const UStateTree& StateTree) const;
	STATETREEMODULE_API FString GetValueString(const UStateTree& StateTree) const;
	STATETREEMODULE_API FString GetTypeString(const UStateTree& StateTree) const;
	STATETREEMODULE_API FStateTreeStateHandle GetStateHandle() const;
};

struct FStateTreeTraceTaskEvent : FStateTreeTraceNodeEvent
{
	explicit FStateTreeTraceTaskEvent(const double RecordingWorldTime
									, const FStateTreeIndex16 Index
									, const EStateTreeTraceEventType EventType
									, const EStateTreeRunStatus Status
									, const FString& TypePath
									, const FString& InstanceDataAsText
									, const FString& DebugText)
		: FStateTreeTraceNodeEvent(RecordingWorldTime, Index, EventType)
		, TypePath(TypePath)
		, InstanceDataAsText(InstanceDataAsText)
		, DebugText(DebugText)
		, Status(Status)
	{
	}

	STATETREEMODULE_API FString ToFullString(const UStateTree& StateTree) const;
	STATETREEMODULE_API FString GetValueString(const UStateTree& StateTree) const;
	STATETREEMODULE_API FString GetTypeString(const UStateTree& StateTree) const;

	FString GetDataTypePath() const
	{
		return TypePath;
	}

	FString GetDataAsText() const
	{
		return InstanceDataAsText;
	}

	FString GetDebugText() const
	{
		return DebugText;
	}

	FString TypePath;
	FString InstanceDataAsText;
	FString DebugText;
	EStateTreeRunStatus Status;
};

struct FStateTreeTraceEvaluatorEvent : FStateTreeTraceNodeEvent
{
	explicit FStateTreeTraceEvaluatorEvent(const double RecordingWorldTime
										, const FStateTreeIndex16 Index
										, const EStateTreeTraceEventType EventType
										, const FString& TypePath
										, const FString& InstanceDataAsText
										, const FString& DebugText)
		: FStateTreeTraceNodeEvent(RecordingWorldTime, Index, EventType)
		, TypePath(TypePath)
		, InstanceDataAsText(InstanceDataAsText)
		, DebugText(DebugText)
	{
	}

	STATETREEMODULE_API FString ToFullString(const UStateTree& StateTree) const;
	STATETREEMODULE_API FString GetValueString(const UStateTree& StateTree) const;
	STATETREEMODULE_API FString GetTypeString(const UStateTree& StateTree) const;

	FString GetDataTypePath() const
	{
		return TypePath;
	}

	FString GetDataAsText() const
	{
		return InstanceDataAsText;
	}

	FString GetDebugText() const
	{
		return DebugText;
	}

	FString TypePath;
	FString InstanceDataAsText;
	FString DebugText;
};

struct FStateTreeTraceConditionEvent : FStateTreeTraceNodeEvent
{
	explicit FStateTreeTraceConditionEvent(const double RecordingWorldTime
										, const FStateTreeIndex16 Index
										, const EStateTreeTraceEventType EventType
										, const FString& TypePath
										, const FString& InstanceDataAsText
										, const FString& DebugText)
		: FStateTreeTraceNodeEvent(RecordingWorldTime, Index, EventType)
		, TypePath(TypePath)
		, InstanceDataAsText(InstanceDataAsText)
		, DebugText(DebugText)
	{
	}

	STATETREEMODULE_API FString ToFullString(const UStateTree& StateTree) const;
	STATETREEMODULE_API FString GetValueString(const UStateTree& StateTree) const;
	STATETREEMODULE_API FString GetTypeString(const UStateTree& StateTree) const;

	FString GetDataTypePath() const
	{
		return TypePath;
	}

	FString GetDataAsText() const
	{
		return InstanceDataAsText;
	}

	FString GetDebugText() const
	{
		return DebugText;
	}

	FString TypePath;
	FString InstanceDataAsText;
	FString DebugText;
};

struct FStateTreeTraceActiveStates
{
	struct FAssetActiveStates
	{
		bool operator==(const FAssetActiveStates& Other) const = default;

		TWeakObjectPtr<const UStateTree> WeakStateTree;
		TArray<FStateTreeStateHandle> ActiveStates;
	};

	bool operator==(const FStateTreeTraceActiveStates& Other) const = default;

	TArray<FAssetActiveStates> PerAssetStates;
};

struct FStateTreeTraceActiveStatesEvent : FStateTreeTraceBaseEvent
{
	// Intentionally implemented in source file to compile 'TArray<FStateTreeStateHandle>' using only forward declaration.
	STATETREEMODULE_API explicit FStateTreeTraceActiveStatesEvent(const double RecordingWorldTime);

	STATETREEMODULE_API FString ToFullString(const UStateTree& StateTree) const;
	STATETREEMODULE_API FString GetValueString(const UStateTree& StateTree) const;
	STATETREEMODULE_API FString GetTypeString(const UStateTree& StateTree) const;

	FStateTreeTraceActiveStates ActiveStates;
};

struct FStateTreeTraceInstanceFrameEvent : FStateTreeTraceBaseEvent
{
	explicit FStateTreeTraceInstanceFrameEvent(const double RecordingWorldTime, const EStateTreeTraceEventType EventType, const UStateTree* StateTree);

	STATETREEMODULE_API FString ToFullString(const UStateTree& StateTree) const;
	STATETREEMODULE_API FString GetValueString(const UStateTree& StateTree) const;
	STATETREEMODULE_API FString GetTypeString(const UStateTree& StateTree) const;

	TWeakObjectPtr<const UStateTree> WeakStateTree;
};

/** Type aliases for statetree trace events */
using FStateTreeTraceEventVariantType = TVariant<FStateTreeTracePhaseEvent,
												FStateTreeTraceLogEvent,
												FStateTreeTracePropertyEvent,
												FStateTreeTraceNodeEvent,
												FStateTreeTraceStateEvent,
												FStateTreeTraceTaskEvent,
												FStateTreeTraceEvaluatorEvent,
												FStateTreeTraceTransitionEvent,
												FStateTreeTraceConditionEvent,
												FStateTreeTraceActiveStatesEvent,
												FStateTreeTraceInstanceFrameEvent>;

#endif // WITH_STATETREE_TRACE_DEBUGGER
