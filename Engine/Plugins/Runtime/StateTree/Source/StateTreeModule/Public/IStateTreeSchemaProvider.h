// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "Templates/SubclassOf.h"

class UStateTreeSchema;

#include "IStateTreeSchemaProvider.generated.h"

UINTERFACE(MinimalAPI)
class UStateTreeSchemaProvider : public UInterface
{
	GENERATED_BODY()
};

/**
* Implementing this interface allows derived class to override the schema used to filter valid state trees for a FStateTreeReference.
* The state tree reference property needs to be marked with SchemaCanBeOverriden metatag.
* Ex:
*	UPROPERTY(EditAnywhere, Category = AI, meta=(Schema="/Script/GameplayStateTreeModule.StateTreeComponentSchema", SchemaCanBeOverriden))
*	FStateTreeReference StateTreeRef;
*/
class IStateTreeSchemaProvider
{
	GENERATED_BODY()

public:
	virtual TSubclassOf<UStateTreeSchema> GetSchema() const PURE_VIRTUAL(IStateTreeSchemaProvider::GetSchema(), return nullptr;)
};