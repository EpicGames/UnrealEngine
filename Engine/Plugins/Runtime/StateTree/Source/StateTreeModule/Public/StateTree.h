// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "StateTreeEvaluationScopeInstanceContainer.h"
#include "StateTreeSchema.h"
#include "StateTreePropertyBindings.h"
#include "StateTreeInstanceData.h"
#include "StateTreeTypes.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "UObject/ObjectKey.h"
#include "StateTree.generated.h"

#define UE_API STATETREEMODULE_API

class UStateTreeExtension;
class UUserDefinedStruct;

template<bool>
struct TStateTreeStrongExecutionContext;

/** Custom serialization version for StateTree Asset */
struct
UE_DEPRECATED(all, "Use a stream custom version. Data made with a custom version for feature do not merge between streams.")
FStateTreeCustomVersion
{
	enum Type
	{
		// Before any version changes were made in the plugin
		BeforeCustomVersionWasAdded = 0,
		// Separated conditions to shared instance data.
		SharedInstanceData,
		// Moved evaluators to be global.
		GlobalEvaluators,
		// Moved instance data to arrays.
		InstanceDataArrays,
		// Added index types.
		IndexTypes,
		// Added events.
		AddedEvents,
		// Testing mishap
		AddedFoo,
		// Changed transition delay
		TransitionDelay,
		// Added external transitions
		AddedExternalTransitions,
		// Changed how bindings are represented
		ChangedBindingsRepresentation,
		// Added guid to transitions
		AddedTransitionIds,
		// Added data handles
		AddedDataHandlesIds,
		// Added linked asset state
		AddedLinkedAssetState,
		// Change how external data is accessed
		ChangedExternalDataAccess,
		// Added override option for parameters
		OverridableParameters,
		// Added override option for state parameters
		OverridableStateParameters,
		// Added storing global parameters in instance storage
		StoringGlobalParametersInInstanceStorage,
		// Added binding to events
		AddedBindingToEvents,
		// Added checking parent states' prerequisites when activating child state directly.
		AddedCheckingParentsPrerequisites,
		// Added Tick Parameter Binding.
		TickParameterBindings,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	/** The GUID for this custom version number */
	UE_API const static FGuid GUID;

private:
	FStateTreeCustomVersion() {}
};


#if WITH_EDITOR

namespace UE::StateTree::Compiler
{
	/** 
	 * Helper method to correctly rename a no longer needed object. 
	 * 
	 * @param ObjectToRename - Object to be renamed, will have flags updated & export invalidated
	 * @param Name - New name of the object. Null by default which means it will only get moved to transient outer.
	 */
	extern UE_API void RenameObjectToTransientPackage(TNotNull<UObject*> ObjectToRename, FStringView Name = nullptr);
}

namespace UE::StateTree::Compiler::Private
{
	class FCompilerManagerImpl;
}

/** Struct containing information about the StateTree runtime memory usage. */
struct FStateTreeMemoryUsage
{
	FStateTreeMemoryUsage() = default;
	FStateTreeMemoryUsage(const FString InName, const FStateTreeStateHandle InHandle = FStateTreeStateHandle::Invalid)
		: Name(InName)
		, Handle(InHandle)
	{
	}
	
	void AddUsage(FConstStructView View);
	void AddUsage(const UObject* Object);

	FString Name;
	FStateTreeStateHandle Handle;
	int32 NodeCount = 0;
	int32 EstimatedMemoryUsage = 0;
	int32 ChildNodeCount = 0;
	int32 EstimatedChildMemoryUsage = 0;
};
#endif

namespace UE::StateTree
{
	
}

/**
 * StateTree asset. Contains the StateTree definition in both editor and runtime (baked) formats.
 */
UCLASS(MinimalAPI, BlueprintType)
class UStateTree : public UDataAsset
{
	GENERATED_BODY()

public:
	/** @return Default instance data. */
	const FStateTreeInstanceData& GetDefaultInstanceData() const
	{
		return DefaultInstanceData;
	}

	/** @return Shared instance data. */
	UE_API TSharedPtr<FStateTreeInstanceData> GetSharedInstanceData() const;

	/** @return Number of context data views required for StateTree execution (Tree params, context data, External data). */
	int32 GetNumContextDataViews() const
	{
		return NumContextData;
	}

	/** @return Default evaluation scope instance data. */
	const UE::StateTree::InstanceData::FInstanceContainer& GetDefaultEvaluationScopeInstanceData() const
	{
		return DefaultEvaluationScopeInstanceData;
	}

	/** @return Default execution runtime data. */
	const UE::StateTree::InstanceData::FInstanceContainer& GetDefaultExecutionRuntimeData() const
	{
		return DefaultExecutionRuntimeData;
	}

	/** @return List of external data required by the state tree */
	TConstArrayView<FStateTreeExternalDataDesc> GetExternalDataDescs() const
	{
		return ExternalDataDescs;
	}

	/** @return List of context data enforced by the schema that must be provided through the execution context. */
	TConstArrayView<FStateTreeExternalDataDesc> GetContextDataDescs() const
	{
		return ContextDataDescs;
	}

	/** @return true if the other StateTree has compatible context data. */
	UE_API bool HasCompatibleContextData(const UStateTree& Other) const;

	/** @return true if the other StateTree has compatible context data. */
	UE_API bool HasCompatibleContextData(TNotNull<const UStateTree*> Other) const;
	
	/** @return List of default parameters of the state tree. Default parameter values can be overridden at runtime by the execution context. */
	const FInstancedPropertyBag& GetDefaultParameters() const
	{
		return Parameters;
	}

	/**
	 * @return true if the tree asset can be used at runtime.
	 * @note the asset can be in the compilation queue. This will make sure it is up to date.
	 */
	[[nodiscard]] UE_API bool IsReadyToRun() const;

	/** @return schema that was used to compile the StateTree. */
	const UStateTreeSchema* GetSchema() const
	{
		return Schema;
	}

	/** @return Pointer to a frame or null if frame not found. */
	UE_API const FCompactStateTreeFrame* GetFrameFromHandle(const FStateTreeStateHandle StateHandle) const;

	/** @return Pointer to a state or null if state not found */ 
	UE_API const FCompactStateTreeState* GetStateFromHandle(const FStateTreeStateHandle StateHandle) const;

	/** @return State handle matching a given Id; invalid handle if state not found. */
	UE_API FStateTreeStateHandle GetStateHandleFromId(const FGuid Id) const;

	/*
	 * Determines how we match a state gameplay tag
	 */
	enum class EStateGameplayTagQueryMethod
	{
		// Foo matches both Foo and Foo.Bar. Foo.Bar does not match Foo.
		Includes,

		// Foo only matches Foo. Foo does not match Foo.Bar. Foo.Bar does not match Foo
		MatchesExact
	};

	/**
	 * Get state handle by gameplay tag. States with same tag are searched in breadth first order.
	 * @param Tag Gameplay tags to query
	 * @param QueryMethod how to query tags
	 * @return State handle matching a given gameplay tag
	 */
	UE_API FStateTreeStateHandle GetStateHandleFromGameplayTag(const FGameplayTag Tag, EStateGameplayTagQueryMethod QueryMethod) const;

	/** @return ID of the state matching a given state handle; invalid Id if state not found. */
	UE_API FGuid GetStateIdFromHandle(const FStateTreeStateHandle Handle) const;

	/** @return Struct view of the node matching a given node index; invalid view if state not found. */
	UE_API FConstStructView GetNode(const int32 NodeIndex) const;

	/** @return Struct views of all nodes */
	const FInstancedStructContainer& GetNodes() const
	{
		return Nodes;
	}

	/** @return index to first global evaluator in GetNodes */
	const uint16 GetGlobalEvaluatorsBegin() const
	{
		return EvaluatorsBegin;
	}

	/** @return number of global evaluators. */
	const uint16 GetGlobalEvaluatorsNum() const
	{
		return EvaluatorsNum;
	}

	/** @return index to first global tasks in GetNodes */
	const uint16 GetGlobalTasksBegin() const
	{
		return GlobalTasksBegin;
	}
	
	/** @return number of global tasks. */
	const uint16 GetGlobalTasksNum() const
	{
		return GlobalTasksNum;
	}

	/** @return Node index matching a given Id; invalid index if node not found. */
	UE_API FStateTreeIndex16 GetNodeIndexFromId(const FGuid Id) const;

	/** @return Id of the node matching a given node index; invalid Id if node not found. */
	UE_API FGuid GetNodeIdFromIndex(const FStateTreeIndex16 NodeIndex) const;

	/** @return View of all states. */
	TConstArrayView<FCompactStateTreeState> GetStates() const
	{
		return States;
	}

	/** @return Pointer to the transition at a given index; null if not found. */ 
	UE_API const FCompactStateTransition* GetTransitionFromIndex(const FStateTreeIndex16 TransitionIndex) const;
	
	/** @return Runtime transition index matching a given Id; invalid index if node not found. */
	UE_API FStateTreeIndex16 GetTransitionIndexFromId(const FGuid Id) const;

	/** @return Id of the transition matching a given runtime transition index; invalid Id if transition not found. */
	UE_API FGuid GetTransitionIdFromIndex(const FStateTreeIndex16 Index) const;

	/** @return Property bindings */
	const FStateTreePropertyBindings& GetPropertyBindings() const
	{
		return PropertyBindings;
	}

	/** @return View of all extensions. */
	TConstArrayView<UStateTreeExtension*> GetExtensions() const
	{
		return Extensions;
	}

	/** @return View of all task completion dispatchers. */
	TConstArrayView<UE::StateTree::FTaskCompletionDispatcher> GetTaskCompletionDispatchers() const
	{
		return TaskCompletionDispatchers;
	}

	/** Find the first extension of the requested type. */
	template<typename ExtensionType>
	const ExtensionType* GetExtension() const
	{
		return CastChecked<const ExtensionType>(K2_GetExtension(ExtensionType::StaticClass()), ECastCheckedType::NullAllowed);
	}

	/** Find the first extension of the requested type. */
	UFUNCTION(BlueprintCallable, Category = "StateTree|Extension", Meta = (DisplayName="Get Extension", DeterminesOutputType = "ExtensionType"))
	UE_API const UStateTreeExtension* K2_GetExtension(TSubclassOf<UStateTreeExtension> ExtensionType) const;

	/** @return True if there is any global tasks need ticking. */
	bool DoesRequestTickGlobalTasks(bool bHasEvents) const
	{
		return bCachedRequestGlobalTick || (bHasEvents && bCachedRequestGlobalTickOnlyOnEvents);
	}
	
	/** @return True if there is any global tasks that ticks. */
	bool ShouldTickGlobalTasks(bool bHasEvents) const
	{
		return bHasGlobalTickTasks || (bHasEvents && bHasGlobalTickTasksOnlyOnEvents);
	}

	/** @return true if the tree can use the scheduled tick feature. */
	bool IsScheduledTickAllowed() const
	{
		return bScheduledTickAllowed;
	}

	/** @return the rules used by the execution context for selecting states. */
	EStateTreeStateSelectionRules GetStateSelectionRules() const
	{
		return StateSelectionRules;
	}

#if WITH_EDITOR
	/** Resets the compiled data to empty. */
	UE_API void ResetCompiled();

	/**
	 * Mark the state tree dirty.
	 * It will require the StateTree to be compiled before it can be used.
	 * @param bPubliclyModified Whether to also mark the public dependency dirty. It will affect the other asset that references the StateTree asset.
	 */
	UE_API void MarkAsModified(bool bPubliclyModified);

	/** @return whether the state tree asset is dirty and needs to be compiled. */
	[[nodiscard]] UE_API bool IsEditorDataDirty() const;

	/** Compile the state tree if the editor data as changed since the last compilation. */
	UE_API void CompileIfChanged();

	/** Calculates runtime memory usage for different sections of the tree. */
	UE_API TArray<FStateTreeMemoryUsage> CalculateEstimatedMemoryUsage() const;

	UE_DEPRECATED(5.7, "Use the compiler manager.")
	/** Called when the editor is preparing to start a pie session. */
	void OnPreBeginPIE(const bool bIsSimulating)
	{
	}

	/** The expected result of GetAssetSavedVersion. If the values do not match, the UI will suggest saving the asset. */
	UE_API static const int32 LatestCustomAssetSavedVersion;

	/** Custom asset version to notify the user when the package needs to be resaved. */
	int32 GetAssetSavedVersion() const
	{
		return CustomAssetSavedVersion;
	}
#endif

#if WITH_EDITOR || WITH_STATETREE_DEBUG
	/** @return the internal content of the state tree compiled asset. */
	[[nodiscard]] UE_API FString DebugInternalLayoutAsString() const;
#endif

#if WITH_EDITORONLY_DATA
	/** Edit time data for the StateTree, instance of UStateTreeEditorData */
	UPROPERTY()
	TObjectPtr<UObject> EditorData;

	UE_DEPRECATED(5.7, "Use the compiler manager.")
	FDelegateHandle OnObjectsReinstancedHandle;
	UE_DEPRECATED(5.7, "Use the compiler manager.")
	FDelegateHandle OnUserDefinedStructReinstancedHandle;
	UE_DEPRECATED(5.7, "Use the compiler manager.")
	FDelegateHandle OnPreBeginPIEHandle;
#endif

	/** Hash of the editor data from last compile. Also used to detect mismatching events from recorded traces. */
	UPROPERTY()
	uint32 LastCompiledEditorDataHash = 0;

protected:
	
	/**
	 * Resolves references between data in the StateTree.
	 * @return true if all references to internal and external data are resolved properly, false otherwise.
	 */
	[[nodiscard]] UE_API bool Link();

	UE_API virtual void PostLoad() override;

#if WITH_EDITORONLY_DATA
	static UE_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
	static UE_API void AppendToClassSchema(FAppendToClassSchemaContext& Context);
#endif
	UE_API virtual void Serialize(FStructuredArchiveRecord Record) override;

#if WITH_STATETREE_DEBUG
	UE_API virtual void BeginDestroy() override;
	UE_API virtual void PostInitProperties() override;
#endif

#if WITH_EDITOR
	UE_API virtual void PreSave(FObjectPreSaveContext Context) override;
	UE_API virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_API void BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform) override;
protected:
	UE_API virtual void ThreadedPostLoadAssetRegistryTagsOverride(FPostLoadAssetRegistryTagsContext& Context) const override;
public:
	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif

	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	
private:
	/**
	 * Reset the data generated by Link(), this in turn will cause IsReadyToRun() to return false.
	 * Used during linking, or to invalidate the linked data when data version is old (requires recompile). 
	 */
	UE_API void ResetLinked();

	/** @return true if all the source instance data types match with the node's instance data types */
	UE_API bool ValidateInstanceData();

	/** Set StateTree's flag that can't be set when compiled. */
	UE_API void UpdateRuntimeFlags();

	UE_API bool PatchBindings();

	// Data created during compilation, source data in EditorData.
	
	/** Schema used to compile the StateTree. */
	UPROPERTY(Instanced)
	TObjectPtr<UStateTreeSchema> Schema = nullptr;

	/** Runtime frames */
	UPROPERTY()
	TArray<FCompactStateTreeFrame> Frames;

	/** Runtime states, root state at index 0 */
	UPROPERTY()
	TArray<FCompactStateTreeState> States;

	/** Runtime transitions. */
	UPROPERTY()
	TArray<FCompactStateTransition> Transitions;

	/** Evaluators, Tasks, Condition and Consideration nodes. */
	UPROPERTY()
	FInstancedStructContainer Nodes;

	/** Default node instance data (e.g. evaluators, tasks). */
	UPROPERTY()
	FStateTreeInstanceData DefaultInstanceData;

	/** Default node instance data for evaluation scope (e.g. conditions, considerations, functions) */
	UPROPERTY()
	UE::StateTree::InstanceData::FInstanceContainer DefaultEvaluationScopeInstanceData;

	/** Default node execution runtime data for execution runtime (e.g. tasks, conditions, considerations, functions) */
	UPROPERTY()
	UE::StateTree::InstanceData::FInstanceContainer DefaultExecutionRuntimeData;

	/** Shared node instance data (e.g. conditions, considerations). */
	UPROPERTY()
	FStateTreeInstanceData SharedInstanceData;

	mutable FTransactionallySafeRWLock PerThreadSharedInstanceDataLock;
	mutable TArray<TSharedPtr<FStateTreeInstanceData>> PerThreadSharedInstanceData;
	
	/** List of names external data enforced by the schema, created at compilation. */
	UPROPERTY()
	TArray<FStateTreeExternalDataDesc> ContextDataDescs;

	UPROPERTY()
	FStateTreePropertyBindings PropertyBindings;

	using FMemoryRequirement = UE::StateTree::InstanceData::FEvaluationScopeInstanceContainer::FMemoryRequirement;
	/** The amount of memory used by property binding copy info batch for property function. */
	TArray<FMemoryRequirement> PropertyFunctionEvaluationScopeMemoryRequirements;

	/** Task completion dispatchers. */
	UPROPERTY()
	TArray<UE::StateTree::FTaskCompletionDispatcher> TaskCompletionDispatchers;

	/** The asset extensions. A place to add extra information for plugins. */
	UPROPERTY()
	TArray<TObjectPtr<UStateTreeExtension>> Extensions;

	/** Mapping of state guid for the Editor and state handles, created at compilation. */
	UPROPERTY()
	TArray<FStateTreeStateIdToHandle> IDToStateMappings;

	/** Mapping of node guid for the Editor and node index, created at compilation. */
	UPROPERTY()
	TArray<FStateTreeNodeIdToIndex> IDToNodeMappings;
	
	/** Mapping of state transition identifiers and runtime compact transition index, created at compilation. */
	UPROPERTY()
	TArray<FStateTreeTransitionIdToIndex> IDToTransitionMappings;

	/**
	 * Parameters that could be used for bindings within the Tree.
	 * Default values are stored within the asset but StateTreeReference can be used to parameterized the tree.
	 * @see FStateTreeReference
	 */
	UPROPERTY()
	FInstancedPropertyBag Parameters;

	//~ Data created during linking.
	/** List of external data required by the state tree, created during linking. */
	UPROPERTY(Transient)
	TArray<FStateTreeExternalDataDesc> ExternalDataDescs;

	/** Mask used to test the global tasks completion. */
	UPROPERTY()
	uint32 CompletionGlobalTasksMask = 0;

	/** Number of context data, include parameters and all context data. */
	UPROPERTY()
	uint16 NumContextData = 0;

	/** Number of global instance data. */
	UPROPERTY()
	uint16 NumGlobalInstanceData = 0;

	/** Index of first evaluator in Nodes. */
	UPROPERTY()
	uint16 EvaluatorsBegin = 0;

	/** Number of evaluators. */
	UPROPERTY()
	uint16 EvaluatorsNum = 0;

	/** Index of first global task in Nodes. */
	UPROPERTY()
	uint16 GlobalTasksBegin = 0;

	/** Number of global tasks. */
	UPROPERTY()
	uint16 GlobalTasksNum = 0;

	/** The cached value of UStateTreeSchema::GetStateSelectionRules */
	EStateTreeStateSelectionRules StateSelectionRules = EStateTreeStateSelectionRules::Default;

	/** How the global tasks control the completion of the frame. */
	UPROPERTY()
	EStateTreeTaskCompletionType CompletionGlobalTasksControl = EStateTreeTaskCompletionType::Any;

	/** The parameter data type used by the schema. */
	UPROPERTY()
	EStateTreeParameterDataType ParameterDataType = EStateTreeParameterDataType::GlobalParameterData;

	enum class ECompileStatus : uint8
	{
		Public,			// The exported dependencies need to be compiled. 
		Internal,		// The public compiled and the internal dependencies need to be compiled.
		Link,			// Compiled and not yet link.
		Executable,		// Compiled and ready to use.
	};
	/**
	 * The current state of the asset.
	 * The compilation could have failed or is pending (in the compiler queue).
	 * This is different from EditorDataDirtyStatus.
	 * The compilation/link could have failed but the data didn't change since the last compilation/link.
	 */
	ECompileStatus CompileStatus = ECompileStatus::Link;

#if WITH_EDITORONLY_DATA
	enum class EDirtyStatus : uint8
	{
		Public,			// Publicly produced dependencies changed since the last compilation.
		Internal,		// Internal/Private nodes/properties changed since the last compilation.
						//A new task is considered a modification.
						//A property value changed on a task is considered a modification.
						//A BP task that added a property is not considered a modification. It wouldn't affect the compilation result.
		Link,			// Internal/Private dependencies that have a binding changed since the last compilation.
						//A BP task that removed a property and that property had a binding is not considered a modification.
		None			// Nothing changed since the last compilation.
	};
	/**
	 * Whether the editor data dirtied since the last compilation.
	 * After a compilation (success or failure), the dirty status returns to none.
	 */
	EDirtyStatus EditorDataDirtyStatus = EDirtyStatus::None;

	/** The asset compilation was requested. The asset is in the pending queue. */
	bool bCompilationPending = false;
#endif

	/** True if any global task is a transition task. */
	UPROPERTY()
	uint8 bHasGlobalTransitionTasks : 1 = false;

	/**
	 * True if any global task has bShouldCallTick.
	 * Not ticking implies no property copy.
	 */
	uint8 bHasGlobalTickTasks : 1 = false;

	/**
	 * True if any global task has bShouldCallTickOnlyOnEvents.
	 * No effect if bHasGlobalTickTasks is true.
	 * Not ticking implies no property copy.
	 */
	uint8 bHasGlobalTickTasksOnlyOnEvents : 1 = false;
	
	/** True if any global tasks request a tick every frame. */
	uint8 bCachedRequestGlobalTick : 1 = false;

	/**
	 * True if any global tasks request a tick every frame but only if there are events.
	 * No effect if bCachedRequestGlobalTick is true.
	 */
	uint8 bCachedRequestGlobalTickOnlyOnEvents : 1 = false;

	/** True when the scheduled tick is allowed by the schema. */
	uint8 bScheduledTickAllowed : 1 = false;

#if WITH_EDITORONLY_DATA
	/** List of Struct that are out of date and waiting to be replaced with the new instance. */
	TSet<FObjectKey> OutOfDateStructs;

	/** The custom version number the package was last saved with. */
	UPROPERTY()
	int32 CustomAssetSavedVersion = 0;
#endif

private:
#if WITH_STATETREE_DEBUG
	//~ Info for RuntimeValidation InstanceData GC
	struct FDebugInstanceData
	{
		FWeakObjectPtr Object;
		int32 InstanceDataStructIndex = INDEX_NONE;
		int32 SharedInstanceDataIndex = INDEX_NONE;
		enum class EContainer : uint8
		{
			DefaultInstance,
			SharedInstance,
		};
		EContainer Container = EContainer::DefaultInstance;
		enum class EObjectType : uint8
		{
			ObjectInstance,
			Struct,
		};
		EObjectType Type = EObjectType::ObjectInstance;
	};
	TArray<FDebugInstanceData> GCObjectDatas;
	FDelegateHandle PreGCHandle;
	FDelegateHandle PostGCHandle;

	void HandleRuntimeValidationPreGC();
	void HandleRuntimeValidationPostGC();
#endif

	friend struct FStateTreeInstance;
	friend struct FStateTreeExecutionContext;
	friend struct FStateTreeTasksCompletionStatus;
	friend struct FStateTreeMinimalExecutionContext;
	friend struct FStateTreeReadOnlyExecutionContext;
	friend struct FStateTreeWeakExecutionContext;
	friend TStateTreeStrongExecutionContext<true>;

#if WITH_EDITOR
	friend struct FStateTreeCompiler;
	friend class UE::StateTree::Compiler::Private::FCompilerManagerImpl;
#endif
};

#undef UE_API
