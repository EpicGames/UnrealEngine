// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeAnyEnum.generated.h"

/**
 * Enum that can be any type in the UI. Helper class to deal with any enum in property binding.
 */
USTRUCT()
struct FStateTreeAnyEnum
{
	GENERATED_BODY()

	bool operator==(const FStateTreeAnyEnum& RHS) const
	{
		return Value == RHS.Value && Enum == RHS.Enum;
	}

	bool operator!=(const FStateTreeAnyEnum& RHS) const
	{
		return Value != RHS.Value || Enum != RHS.Enum;
	}

	/** Initializes the class and value to specific enum. The value is set to the first value of the enum or 0 if class is null */
	void Initialize(UEnum* NewEnum)
	{
		Enum = NewEnum;
		Value = Enum == nullptr ? 0 : int32(Enum->GetValueByIndex(0));
	}

	/** The enum integer value. */
	UPROPERTY(EditAnywhere, Category = Enum)
	uint32 Value = 0;

	/** The enum class associated with this enum. */
	UPROPERTY(EditAnywhere, Category = Enum)
	TObjectPtr<UEnum> Enum = nullptr;
};
