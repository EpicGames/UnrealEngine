// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "StateTree.h"
#include "StateTreeExecutionContextTypes.h"
#include "StateTreeInstanceData.h"

#define UE_API STATETREEMODULE_API

namespace UE::StateTree
{
struct FScheduledTickHandle;
}

struct FStateTreeExecutionFrame;
struct FStateTreeTaskBase;
struct FStateTreeScheduledTick;
struct FStateTreeWeakTaskRef;
struct FStateTreeExecutionState;
struct FStateTreeInstanceStorage;
struct FStateTreeExecutionContext;
struct FStateTreeWeakExecutionContext;

enum class EStateTreeFinishTaskType : uint8;


namespace UE::StateTree::Async
{
	struct FActivePathInfo
	{
		bool IsValid() const
		{
			return Frame != nullptr;
		}

		const FStateTreeNodeBase& GetNode() const;

		FStateTreeExecutionFrame* Frame = nullptr;
		FStateTreeExecutionFrame* ParentFrame = nullptr;
		FStateTreeStateHandle StateHandle;
		FStateTreeIndex16 NodeIndex;
	};
}

/**
 * Execution context to interact with the state tree instance data asynchronously.
 * It should only be allocated on the stack.
 * You are responsible for making it thread-safe if needed.
 *		ThreadSafeAsyncCallback.AddLambda(
 *			[MyTag, WeakContext = Context.MakeWeakExecutionContext()]()
 *			{
 *				TStateTreeStrongExecutionContext<true> StrongContext = WeakContext.CreateStrongContext();
 *				if (StrongContext.SendEvent())
 *				{
 *					...
 *				}
 *			});
 */
template<bool bWithWriteAccess>
struct TStateTreeStrongExecutionContext
{
	TStateTreeStrongExecutionContext() = default;
	UE_API explicit TStateTreeStrongExecutionContext(const FStateTreeWeakExecutionContext& WeakContext);
	UE_API ~TStateTreeStrongExecutionContext();

	TStateTreeStrongExecutionContext(const TStateTreeStrongExecutionContext& Other) = delete;
	TStateTreeStrongExecutionContext& operator=(const TStateTreeStrongExecutionContext& Other) = delete;

	/** @return The owner of the context */
	TStrongObjectPtr<UObject> GetOwner() const
	{
		return Owner;
	}

	/** @return the StateTree asset in use. */
	TStrongObjectPtr<const UStateTree> GetStateTree() const
	{
		return StateTree;
	}

	/** @return the Instance Storage. */
	TSharedPtr<const FStateTreeInstanceStorage> GetStorage() const
	{
		return Storage;
	}

	/** @return the Temporary Storage. */
	TSharedPtr<const UE::StateTree::ExecutionContext::ITemporaryStorage> GetTemporaryStorage() const
	{
		return TemporaryStorage;
	}

	/**
	 * Sends event for the StateTree.
	 * @return false if the context is not valid or the event could not be sent.
	 */
	UE_API bool SendEvent(const FGameplayTag Tag, const FConstStructView Payload = FConstStructView(), const FName Origin = FName()) const requires bWithWriteAccess;

	/**
	 * Requests transition to a state.
	 * If called during transition processing (e.g. from FStateTreeTaskBase::TriggerTransitions()) the transition
	 * is attempted to be activated immediately (it can fail e.g. because of preconditions on a target state).
	 * If called outside the transition handling, the request is buffered and handled at the beginning of next transition processing.
	 * @param TargetState The state to transition to.
	 * @param Priority The priority of the transition.
	 * @param Fallback of the transition if it fails to select the target state.
	 * @return false if the context is not valid or doesn't have a valid frame anymore or the request failed.
	 */
	UE_API bool RequestTransition(FStateTreeStateHandle TargetState, EStateTreeTransitionPriority Priority = EStateTreeTransitionPriority::Normal, EStateTreeSelectionFallback Fallback = EStateTreeSelectionFallback::None) const requires bWithWriteAccess;

	/**
	 * Broadcasts the delegate.
	 * It executes bound delegates immediately and triggers bound transitions (when transitions are evaluated).
	 * @return false if the context is not valid or doesn't have a valid frame anymore or the broadcast failed.
	 */
	UE_API bool BroadcastDelegate(const FStateTreeDelegateDispatcher& Dispatcher) const requires bWithWriteAccess;

	/**
	 * Registers the delegate to the listener.
	 * If the listener was previously registered, then unregister it first before registering it again with the new delegate callback.
	 * The listener is bound to a dispatcher in the editor.
	 * @return false if the context is not valid or doesn't have a valid frame anymore or the bind failed.
	 */
	UE_API bool BindDelegate(const FStateTreeDelegateListener& Listener, FSimpleDelegate Delegate) const requires bWithWriteAccess;

	/**
	 * Unregisters the callback bound to the listener.
	 * @return false if the context is not valid or the unbind failed.
	 */
	UE_API bool UnbindDelegate(const FStateTreeDelegateListener& Listener) const requires bWithWriteAccess;

	/**
	 * Finishes a task.
	 * If called during tick processing, then the state completes immediately.
	 * If called outside of the tick processing, then the request is buffered and handled on the next tick.
	 * @return false if the context is not valid or doesn't have a valid frame anymore or finish task failed.
	 */
	UE_API bool FinishTask(EStateTreeFinishTaskType FinishType) const requires bWithWriteAccess;

	/**
	 * Updates the scheduled tick of a previous request.
	 * @return false if the context is not valid.
	 */
	UE_API bool UpdateScheduledTickRequest(UE::StateTree::FScheduledTickHandle Handle, FStateTreeScheduledTick ScheduledTick) const requires bWithWriteAccess;

	/**
	 * Get the Instance Data of recorded node. Only callable on lvalue because it would have lost the access track.
	 * @return nullptr if the frame or state containing the node is no longer active.
	 */
	template<typename T>
	std::conditional_t<bWithWriteAccess, T*, const T*> GetInstanceDataPtr() const&
	{
		FStateTreeDataView DataView = GetInstanceDataPtrInternal();
		if (DataView.IsValid() && ensure(DataView.GetStruct() == T::StaticStruct()))
		{
			return static_cast<T*>(DataView.GetMutableMemory());
		}

		return nullptr;
	}

	template<typename T>
	std::conditional_t<bWithWriteAccess, T*, const T*> GetInstanceDataPtr() const&& = delete;

	/**
	 * Get the Execution Runtime Data of recorded node. Only callable on lvalue because it would have lost the access track.
	 * @return nullptr if the frame or state containing the node is no longer active.
	 */
	template<typename T>
	std::conditional_t<bWithWriteAccess, T*, const T*> GetExecutionRuntimeDataPtr() const&
	{
		FStateTreeDataView DataView = GetExecutionRuntimeDataPtrInternal();
		if (DataView.IsValid() && ensure(DataView.GetStruct() == T::StaticStruct()))
		{
			return static_cast<T*>(DataView.GetMutableMemory());
		}

		return nullptr;
	}

	template<typename T>
	std::conditional_t<bWithWriteAccess, T*, const T*> GetExecutionRuntimeDataPtr() const&& = delete;

	/**
	 * Copies the input bindings for the recorded node.
	 * Propagates data FROM bound sources INTO the node's instance data (similar to what happens before Tick in normal execution).
	 * Binding sources requiring context or external data are not available in async context and are skipped (and returns false).
	 * Batches that include property functions cannot be evaluated without a full FStateTreeExecutionContext.
	 * @return true if all bindings are copied.
	 */
	UE_API bool CopyInputBindings() const requires bWithWriteAccess;

	/**
	 * Copies the output bindings for the recorded node.
	 * Propagates data FROM the node's instance data BACK to bound targets (similar to what happens after Tick in normal execution).
	 * Output binding batches never contain property functions, so this call is always async-safe.
	 * Binding targets requiring context or external data are skipped.
	 * @return true if all bindings are copied.
	 */
	UE_API bool CopyOutputBindings() const requires bWithWriteAccess;

	/**
	 * Get the info of active frame, state and node this Context is based on
	 * @return an invalid FActivePathInfo if the frame or state containing the node is no longer active.
	 */
	UE_API UE::StateTree::Async::FActivePathInfo GetActivePathInfo() const;

	/**
	 * Checks if the context is valid.
	 * Validity: Pinned Members are valid AND (WeakContext is created outside the ExecContext loop OR the recorded frame and state are still active)    
	 * @return false if the context is not valid.
	 */
	bool IsValid() const
	{
		// skipped checking Pinned members, because the expression will be false if those members are not valid in ctor anyway.
		return IsValidInstanceStorage() && (!FrameID.IsValid() || GetActivePathInfo().IsValid());
	}

private:
	bool IsValidInstanceStorage() const
	{
		return Owner && StateTree && Storage;
	}

	UE_API FStateTreeDataView GetInstanceDataPtrInternal() const;
	UE_API FStateTreeDataView GetExecutionRuntimeDataPtrInternal() const;
	bool CopyBindingBatchForAsync(const FStateTreeExecutionFrame* CurrentFrame, const FStateTreeDataView TargetView, const FStateTreeIndex16 BindingsBatch) const requires bWithWriteAccess;

	UE_API static void ScheduleNextTick(TNotNull<UObject*> Owner, TNotNull<const UStateTree*> RootStateTree, FStateTreeInstanceStorage& Storage, UE::StateTree::ETickReason Reason) requires bWithWriteAccess;

	TStrongObjectPtr<UObject> Owner;
	TStrongObjectPtr<const UStateTree> StateTree;
	TSharedPtr<FStateTreeInstanceStorage> Storage;
	TSharedPtr<UE::StateTree::ExecutionContext::ITemporaryStorage> TemporaryStorage;

	UE::StateTree::FActiveFrameID FrameID;
	UE::StateTree::FActiveStateID StateID;
	FStateTreeIndex16 NodeIndex;
	uint8 bAccessAcquired : 1 = false;

	friend struct FStateTreePropertyRef;
};

using FStateTreeStrongExecutionContext = TStateTreeStrongExecutionContext<true>;
using FStateTreeStrongReadOnlyExecutionContext = TStateTreeStrongExecutionContext<false>;

/**
 * Execution context that can be saved/copied and used asynchronously. 
 * You are responsible for making it thread-safe if needed.
 * The context is valid if the state (or global context) from which it was created is still
 * active. The owner, state tree, and storage also need to be valid. 
 *
 *		ThreadSafeAsyncCallback.AddLambda(
 *			[MyTag, WeakContext = Context.MakeWeakExecutionContext]()
 *			{
 *				if (WeakContext.SendEvent(MyTag))
 *				{
 *					...
 *				}
 *			});
 */
struct FStateTreeWeakExecutionContext
{
	template<bool bRequireWriteAccess>
	friend struct TStateTreeStrongExecutionContext;

public:
	FStateTreeWeakExecutionContext() = default;
	UE_API explicit FStateTreeWeakExecutionContext(const FStateTreeExecutionContext& Context);

public:
	/** @return The owner of the context */
	TStrongObjectPtr<UObject> GetOwner() const
	{
		return Owner.Pin();
	}

	/** @return the StateTree asset in use. */
	TStrongObjectPtr<const UStateTree> GetStateTree() const
	{
		return StateTree.Pin();
	}

	[[nodiscard]] FStateTreeStrongReadOnlyExecutionContext MakeStrongReadOnlyExecutionContext() const
	{
		return FStateTreeStrongReadOnlyExecutionContext(*this);
	}

	[[nodiscard]] FStateTreeStrongExecutionContext MakeStrongExecutionContext() const
	{
		return FStateTreeStrongExecutionContext(*this);
	}

	/**
	 * Sends event for the StateTree.
	 * @return false if the context is not valid or the event could not be sent.
	 */
	UE_API bool SendEvent(const FGameplayTag Tag, const FConstStructView Payload = FConstStructView(), const FName Origin = FName()) const;

	/**
	 * Requests transition to a state.
	 * If called during transition processing (e.g. from FStateTreeTaskBase::TriggerTransitions()) the transition
	 * is attempted to be activated immediately (it can fail e.g. because of preconditions on a target state).
	 * If called outside the transition handling, the request is buffered and handled at the beginning of next transition processing.
	 * @param TargetState The state to transition to.
	 * @param Priority The priority of the transition.
	 * @param Fallback of the transition if it fails to select the target state.
	 * @return false if the context is not valid or doesn't have a valid frame anymore or the request failed.
	 */
	UE_API bool RequestTransition(FStateTreeStateHandle TargetState, EStateTreeTransitionPriority Priority = EStateTreeTransitionPriority::Normal, EStateTreeSelectionFallback Fallback = EStateTreeSelectionFallback::None) const;

	/**
	 * Broadcasts the delegate.
	 * It executes bound delegates immediately and triggers bound transitions (when transitions are evaluated).
	 * @return false if the context is not valid or doesn't have a valid frame anymore or the broadcast failed.
	 */
	UE_API bool BroadcastDelegate(const FStateTreeDelegateDispatcher& Dispatcher) const;

	/**
	 * Registers the delegate to the listener.
	 * If the listener was previously registered, then unregister it first before registering it again with the new delegate callback.
	 * The listener is bound to a dispatcher in the editor.
	 * @return false if the context is not valid or doesn't have a valid frame anymore or the bind failed.
	 */
	UE_API bool BindDelegate(const FStateTreeDelegateListener& Listener, FSimpleDelegate Delegate) const;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UE_DEPRECATED(5.6, "Use the version without FStateTreeWeakTaskRef")
		UE_API bool BindDelegate(const FStateTreeWeakTaskRef& Task, const FStateTreeDelegateListener& Listener, FSimpleDelegate Delegate) const;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(5.6, "Use UnbindDelegate")
	/** Removes Delegate Listener. */
	UE_API bool RemoveDelegateListener(const FStateTreeDelegateListener& Listener) const;

	/**
	 * Unregisters the callback bound to the listener.
	 * @return false if the context is not valid or the unbind failed.
	 */
	UE_API bool UnbindDelegate(const FStateTreeDelegateListener& Listener) const;

	/**
	 * Finishes a task.
	 * If called during tick processing, then the state completes immediately.
	 * If called outside of the tick processing, then the request is buffered and handled on the next tick.
	 * @return false if the context is not valid or doesn't have a valid frame anymore or finish task failed.
	 */
	UE_API bool FinishTask(EStateTreeFinishTaskType FinishType) const;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UE_DEPRECATED(5.6, "Use the version without FStateTreeWeakTaskRef.")
		UE_API bool FinishTask(const FStateTreeWeakTaskRef& Task, EStateTreeFinishTaskType FinishType) const;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Updates the scheduled tick of a previous request.
	 * @return false if the context is not valid.
	 */
	UE_API bool UpdateScheduledTickRequest(UE::StateTree::FScheduledTickHandle Handle, FStateTreeScheduledTick ScheduledTick) const;

private:
	TWeakObjectPtr<UObject> Owner;
	TWeakObjectPtr<const UStateTree> StateTree;
	TWeakPtr<FStateTreeInstanceStorage> Storage;
	TWeakPtr<UE::StateTree::ExecutionContext::ITemporaryStorage> TemporaryStorage;

	UE::StateTree::FActiveFrameID FrameID;
	UE::StateTree::FActiveStateID StateID;
	FStateTreeIndex16 NodeIndex;
};

#undef UE_API
