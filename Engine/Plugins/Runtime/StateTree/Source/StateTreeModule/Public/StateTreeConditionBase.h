// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeNodeBase.h"
#include "StateTreeExecutionTypes.h"
#include "StateTreeConditionBase.generated.h"

struct FStateTreeExecutionContext;

enum class EStateTreeCompare : uint8
{
	Default,
	Invert,
};

/**
 * Base struct for all conditions.
 */
USTRUCT(meta = (Hidden))
struct FStateTreeConditionBase : public FStateTreeNodeBase
{
	GENERATED_BODY()
	
	/** @return True if the condition passes. */
	virtual bool TestCondition(FStateTreeExecutionContext& Context) const { return false; }

	/**
	 * Called when a new state is entered and task is part of active states.
	 * Note: The condition instance data is shared between all the uses a State Tree asset.
	 *       You should not modify the instance data in this callback.    
	 * @param Context Reference to current execution context.
	 * @param Transition Describes the states involved in the transition
	 * @return Succeed/Failed will end the state immediately and trigger to select new state, Running will carry on to tick the state.
	 */
	virtual void EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const {}

	/**
	 * Called when a current state is exited and task is part of active states.
	 * Note: The condition instance data is shared between all the uses a State Tree asset.
	 *       You should not modify the instance data in this callback.    
	 * @param Context Reference to current execution context.
	 * @param Transition Describes the states involved in the transition
	 */
	virtual void ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const {}

	/**
	 * Called right after a state has been completed, but before new state has been selected. StateCompleted is called in reverse order to allow to propagate state to other Tasks that
	 * are executed earlier in the tree. Note that StateCompleted is not called if conditional transition changes the state.
	 * Note: The condition instance data is shared between all the uses a State Tree asset.
	 *       You should not modify the instance data in this callback.    
	 * @param Context Reference to current execution context.
	 * @param CompletionStatus Describes the running status of the completed state (Succeeded/Failed).
	 * @param CompletedActiveStates Active states at the time of completion.
	 */
	virtual void StateCompleted(FStateTreeExecutionContext& Context, const EStateTreeRunStatus CompletionStatus, const FStateTreeActiveStates& CompletedActiveStates) const {}

	UPROPERTY()
	EStateTreeExpressionOperand Operand = EStateTreeExpressionOperand::And;

	UPROPERTY()
	int8 DeltaIndent = 0;

	UPROPERTY()
	EStateTreeConditionEvaluationMode EvaluationMode = EStateTreeConditionEvaluationMode::Evaluated;

	/** If set to true, EnterState, ExitState, and StateCompleted are called on the condition. */
	uint8 bHasShouldCallStateChangeEvents : 1 = false;
	
	/**
	 * If set to true, the condition will receive EnterState/ExitState even if the state was previously active.
	 * Default value is true.
	 */
	uint8 bShouldStateChangeOnReselect : 1 = true;
};

/**
 * Base class (namespace) for all common Conditions that are generally applicable.
 * This allows schemas to safely include all conditions child of this struct. 
 */
USTRUCT(meta = (Hidden))
struct FStateTreeConditionCommonBase : public FStateTreeConditionBase
{
	GENERATED_BODY()
};
