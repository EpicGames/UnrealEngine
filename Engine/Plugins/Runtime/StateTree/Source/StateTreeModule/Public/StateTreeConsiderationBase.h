// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeNodeBase.h"
#include "StateTreeConsiderationBase.generated.h"

#define UE_API STATETREEMODULE_API

struct FStateTreeExecutionContext;
enum class EStateTreeExpressionOperand : uint8;

/**
 * This feature is experimental and the API is expected to change. 
 * Base struct for all utility considerations.
 */
USTRUCT(meta = (Hidden))
struct FStateTreeConsiderationBase : public FStateTreeNodeBase
{
	GENERATED_BODY()

	UE_API FStateTreeConsiderationBase();

public:
	UE_API float GetNormalizedScore(FStateTreeExecutionContext& Context) const;

protected:
	virtual float GetScore(FStateTreeExecutionContext& Context) const { return 0.f; };

public:
	UPROPERTY()
	EStateTreeExpressionOperand Operand;

	UPROPERTY()
	int8 DeltaIndent;
};

/**
 * Base class (namespace) for all common Utility Considerations that are generally applicable.
 * This allows schemas to safely include all considerations child of this struct.
 */
USTRUCT(meta = (Hidden))
struct FStateTreeConsiderationCommonBase : public FStateTreeConsiderationBase
{
	GENERATED_BODY()
};

#undef UE_API
