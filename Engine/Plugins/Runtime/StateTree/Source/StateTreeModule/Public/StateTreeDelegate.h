// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeIndexTypes.h"
#include "StateTreeDelegate.generated.h"


/**
 * StateTree's delegates are used to send events through the state's hierarchy.
 * Similar to events but more control. As a designer, you know exactly which delegate can be broadcast in the state tree asset.
 * Delegates can be used in transitions (like events) and to trigger callbacks.
 * They are primarily used in async tasks. Instead of ticking every frame (polling), you can register a delegate and wait for the callback to request a new transition.
 * Delegates are designed to be bound in the editor (dispatcher and listener).
 * By design, you cannot share delegates between state tree assets.
 *
 * StateTree's delegates are composed of a sender (dispatcher) and a receiver (listener).
 * The listener is linked to a simple callback at execution with FStateTreeExecutionContext.BindDelegate.
 * You can only bind a listener to a dispatcher. (Cannot bind listener to listener or dispatcher to dispatcher).
 * When binding to a task, a dispatcher is automatically created. The dispatcher is broadcast when the task completes (succeeded or failed).
 *
 * See StateTreeDelegateTests.cpp for examples.
 */

 /**
  * StateTree's delegate dispatcher.
  */
USTRUCT(BlueprintType, meta = (NoBinding))
struct FStateTreeDelegateDispatcher
{
	GENERATED_BODY()

public:
	/** @return if dispatcher is valid. */
	bool IsValid() const
	{
		return ID.IsValid();
	}

	bool operator==(const FStateTreeDelegateDispatcher&) const = default;
	bool operator!=(const FStateTreeDelegateDispatcher&) const = default;

private:
	UPROPERTY()
	FGuid ID;

	friend struct FStateTreePropertyBindingCompiler;
};

/**
 * The receiver of a delegate binding.
 * Can be bound in the editor to a delegate dispatcher.
 */
USTRUCT(BlueprintType, meta = (NoPromoteToParameter))
struct FStateTreeDelegateListener
{
	GENERATED_BODY()

public:
	/** @return the bound delegate. */
	FStateTreeDelegateDispatcher GetDispatcher() const
	{
		return Dispatcher;
	}

	/** @return if the listener is valid. */
	bool IsValid() const
	{
		return Dispatcher.IsValid() && ID != 0;
	}

	bool operator==(const FStateTreeDelegateListener&) const = default;
	bool operator!=(const FStateTreeDelegateListener&) const = default;


private:
	/** ID of the dispatcher that listener is or will be bound to. */
	UPROPERTY()
	FStateTreeDelegateDispatcher Dispatcher;

	/** The generated ID of the listener. */
	UPROPERTY()
	int32 ID = 0;

	friend struct FStateTreePropertyBindingCompiler;
};

namespace UE::StateTree
{

/**
 * Task completion event condition.
 * Used to control when a task completion dispatcher is broadcast.
 */
UENUM()
enum class ETaskCompletionCondition : uint8
{
	Succeeds,    // Broadcast only when task Succeeded
	Fails,       // Broadcast only when task Failed (or Stopped for backward compatibility)
	Completes,   // Broadcast on either Succeeded or Failed (or Stopped for backward compatibility)
};

/**
 * Task completion dispatcher entry compiled into the state tree asset.
 * Maps a task node to a dispatcher that is broadcast when the task completes.
 */
USTRUCT()
struct FTaskCompletionDispatcher
{
	GENERATED_BODY()

public:
	/** The dispatcher GUID to broadcast (auto-generated at compile time) */
	UPROPERTY()
	FStateTreeDelegateDispatcher Dispatcher;

	/** Index of the task node in UStateTree::Nodes[] */
	UPROPERTY()
	FStateTreeIndex16 TaskNodeIndex;

	/** Condition for when to broadcast (Succeeds/Fails/Completes) */
	UPROPERTY()
	ETaskCompletionCondition Condition = ETaskCompletionCondition::Completes;

	bool operator==(const UE::StateTree::FTaskCompletionDispatcher&) const = default;
	bool operator!=(const UE::StateTree::FTaskCompletionDispatcher&) const = default;
};

} // namespace UE::StateTree
