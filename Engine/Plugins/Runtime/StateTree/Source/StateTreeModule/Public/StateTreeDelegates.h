// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

#define UE_API STATETREEMODULE_API

class UStateTree;
enum class EStateTreeTraceStatus : uint8;
enum class EStateTreeTraceAnalysisStatus : uint8;

namespace UE::StateTree::Delegates
{

#if WITH_EDITOR

/** Called when linkable name in a StateTree has changed. */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnIdentifierChanged, const UStateTree& /*StateTree*/);
extern UE_API FOnIdentifierChanged OnIdentifierChanged;

/**
 * Called when schema of the StateTree EditorData has changed.
 * This is used to refresh the asset editor.
 * Note that this is NOT called when updating the StateTree schema from the EditorData on successful compilation.
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnSchemaChanged, const UStateTree& /*StateTree*/);
extern UE_API FOnSchemaChanged OnSchemaChanged;

/**
 * Called when parameters of the StateTree EditorData changed.
 * This should mainly be used by the asset editor to maintain consistency in the UI for manipulations on the EditorData
 * until the tree gets compiled.
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnParametersChanged, const UStateTree& /*StateTree*/);
extern UE_API FOnParametersChanged OnParametersChanged;

/**
 * Called when parameters of a StateTree State changed.
 * This should mainly be used by the asset editor to maintain consistency in the UI for manipulations.
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnStateParametersChanged, const UStateTree& /*StateTree*/, const FGuid /*StateID*/);
extern UE_API FOnStateParametersChanged OnStateParametersChanged;

/**
 * Called when Global Tasks or Evaluators of the StateTree EditorData changed.
 * This should mainly be used by the asset editor to maintain consistency in the UI for manipulations on the EditorData.
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnGlobalDataChanged, const UStateTree& /*StateTree*/);
extern UE_API FOnGlobalDataChanged OnGlobalDataChanged;

/**
 * Called when the theme colors change.
 * This should mainly be used by the asset editor to maintain consistency in the UI for manipulations on the EditorData.
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnVisualThemeChanged, const UStateTree& /*StateTree*/);
extern UE_API FOnVisualThemeChanged OnVisualThemeChanged;

/**
 * Called when breakpoints of the StateTree EditorData changed.
 * This should mainly be used by the asset editor to update the debugger.
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnBreakpointsChanged, const UStateTree& /*StateTree*/);
extern UE_API FOnBreakpointsChanged OnBreakpointsChanged;

/** Called when the state tree compiles. */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnPostCompile, const UStateTree& /*StateTree*/);
extern UE_API FOnPostCompile OnPostCompile;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UE_DEPRECATED(5.8, "Use the UStateTreeEditingSubsystem to trigger a compilation.")
/** Request StateTree compilation. */
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnRequestCompile, UStateTree& /*StateTreeToCompile*/);
UE_DEPRECATED(5.8, "Use the UStateTreeEditingSubsystem to trigger a compilation.")
extern UE_API FOnRequestCompile OnRequestCompile;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

/** Request the editor hash of the given state tree. */
DECLARE_DELEGATE_RetVal_OneParam(uint32, FOnRequestEditorHash, const UStateTree& /*StateTreeToHash*/);
extern UE_API FOnRequestEditorHash OnRequestEditorHash;

#endif // WITH_EDITOR

#if WITH_STATETREE_TRACE

/** Called by the StateTree module when StateTree traces are enabled/disabled. */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnTracingStateChanged, EStateTreeTraceStatus);
extern UE_API FOnTracingStateChanged OnTracingStateChanged;

#endif // WITH_STATETREE_TRACE

#if WITH_STATETREE_TRACE_DEBUGGER

/** Called by the StateTree module when StateTree traces analysis is started/stopped. */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnTraceAnalysisStateChanged, EStateTreeTraceAnalysisStatus);
extern UE_API FOnTraceAnalysisStateChanged OnTraceAnalysisStateChanged;

/** Called by the StateTree module whenever tracing timeline is scrubbed on Rewind Debugger */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnTracingTimelineScrubbed, const double /* InScrubTime */);
extern UE_API FOnTracingTimelineScrubbed OnTracingTimelineScrubbed;

#endif // WITH_STATETREE_TRACE_DEBUGGER

}; // UE::StateTree::Delegates

#undef UE_API
