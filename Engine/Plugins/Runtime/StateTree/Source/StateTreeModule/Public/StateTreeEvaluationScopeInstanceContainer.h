// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeTypes.h"
#include "StateTreeInstanceContainer.h"

#define UE_API STATETREEMODULE_API

namespace UE::StateTree::InstanceData
{
	/**
	 * Container of instance data for evaluation scope data.
	 * The memory is pre-allocated and only available for a short amount of time.
	 */
	struct FEvaluationScopeInstanceContainer
	{
	public:
		/** The memory requirement for the container allocation. */
		using FMemoryRequirement = UE::StateTree::InstanceData::FEvaluationScopeMemoryRequirement;

		/** Builds the amount of memory required to store the provided structures. */
		struct FMemoryRequirementBuilder
		{
		public:
			FMemoryRequirementBuilder() = default;

			/** Add a structure to the current memory requirement. Takes into account the structure alignment. */
			UE_API void Add(TNotNull<const UScriptStruct*> Struct);

			/** @return the requirement memory for all added structures. */
			[[nodiscard]] UE_API FMemoryRequirement Build();

		private:
			FMemoryRequirement MemoryRequirement;
			int32 FirstStructAlignment = -1;
		};

	public:
		FEvaluationScopeInstanceContainer() = default;
		UE_API explicit FEvaluationScopeInstanceContainer(TNotNull<void*> InMemory, const FMemoryRequirement& InRequirement);
		UE_API ~FEvaluationScopeInstanceContainer();
		
		FEvaluationScopeInstanceContainer(const FEvaluationScopeInstanceContainer& Other) = delete;
		FEvaluationScopeInstanceContainer& operator=(const FEvaluationScopeInstanceContainer& Other) = delete;

		FEvaluationScopeInstanceContainer(FEvaluationScopeInstanceContainer&& Other)
		{
			FMemory::Memcpy(this, &Other, sizeof(FEvaluationScopeInstanceContainer));
			FMemory::Memzero(&Other, sizeof(FEvaluationScopeInstanceContainer));
		}

		FEvaluationScopeInstanceContainer& operator=(FEvaluationScopeInstanceContainer&& Other)
		{
			Reset();
			FMemory::Memcpy(this, &Other, sizeof(FEvaluationScopeInstanceContainer));
			FMemory::Memzero(&Other, sizeof(FEvaluationScopeInstanceContainer));
			return *this;
		}

	public:
		/** @return number of items in the storage. */
		[[nodiscard]] int32 Num() const
		{
			return NumberOfElements;
		}

		/** @return true if the index can be used to get data. */
		[[nodiscard]] bool IsValidDataHandle(FStateTreeDataHandle DataHandle) const
		{
			return Get(DataHandle) != nullptr;
		}

		/** @return specified item of an added structure. */
		[[nodiscard]] FStructView GetDataView(FStateTreeDataHandle DataHandle)
		{
			return Get(DataHandle)->Instance;
		}

		/** @return specified item if the structure was added. */
		[[nodiscard]] FStructView* GetDataViewPtr(FStateTreeDataHandle DataHandle)
		{
			FItem* FoundItem = Get(DataHandle);
			return FoundItem ? &FoundItem->Instance : nullptr;
		}

		/**
		 * Add a structure to the current memory requirement.
		 * Takes into account the structure alignment.
		 * UObject are created in the transient package and returned.
		 */
		UE_API void Add(FStateTreeDataHandle DataHandle, FConstStructView Default);

		/** Resets the data to empty. */
		void UE_API Reset();

		/** Returns number of bytes allocated for the array  */
		[[nodiscard]] int32 GetAllocatedMemory() const
		{
			return MemoryRequirement.Size;
		}

	private:
		struct FItem
		{
			FStructView Instance;
			FStateTreeDataHandle DataHandle;
		};

		FItem* Get(const FStateTreeDataHandle& Handle) const;

		//~ Debug Tags
		void AddTableDebugTag();
		void AddStructDebugTag(int32 Index);
		void TestDebugTags() const;

	private:
		/** Struct instances (Not transient, as we use FStateTreeInstanceData to store default values for instance data) */
		void* Memory = nullptr;
		FMemoryRequirement MemoryRequirement;
		int32 NumberOfElements = 0;
		bool bStructsHaveDestructor = false;
	};
}

#undef UE_API
