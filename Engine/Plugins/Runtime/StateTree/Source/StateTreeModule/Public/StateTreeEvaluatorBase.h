// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeNodeBase.h"
#include "StateTreeEvaluatorBase.generated.h"

#define UE_API STATETREEMODULE_API

struct FStateTreeExecutionContext;
struct FStateTreeReadOnlyExecutionContext;

/**
 * Base struct of StateTree Evaluators.
 * Evaluators calculate and expose data to be used for decision making in a StateTree.
 */
USTRUCT(meta = (Hidden))
struct FStateTreeEvaluatorBase : public FStateTreeNodeBase
{
	GENERATED_BODY()
	
	/**
	 * Called when StateTree is started.
	 * @param Context Reference to current execution context.
	 */
	virtual void TreeStart(FStateTreeExecutionContext& Context) const {}

	/**
	 * Called when StateTree is stopped.
	 * @param Context Reference to current execution context.
	 */
	virtual void TreeStop(FStateTreeExecutionContext& Context) const {}

	/**
	 * Called each frame to update the evaluator.
	 * @param Context Reference to current execution context.
	 * @param DeltaTime Time since last StateTree tick, or 0 if called during preselection.
	 */
	virtual void Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const {}

#if WITH_GAMEPLAY_DEBUGGER
	UE_API virtual FString GetDebugInfo(const FStateTreeReadOnlyExecutionContext& Context) const;

	UE_DEPRECATED(5.8, "Use the version with the FStateTreeReadOnlyExecutionContext.")
	virtual void AppendDebugInfoString(FString& DebugString, const FStateTreeExecutionContext& Context) const final
	{
	}
#endif // WITH_GAMEPLAY_DEBUGGER
};

/**
* Base class (namespace) for all common Evaluators that are generally applicable.
* This allows schemas to safely include all Evaluators child of this struct. 
*/
USTRUCT(Meta=(Hidden))
struct FStateTreeEvaluatorCommonBase : public FStateTreeEvaluatorBase
{
	GENERATED_BODY()
};

#undef UE_API
