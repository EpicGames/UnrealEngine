// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayTagContainer.h"
#include "StructUtils/StructView.h"
#include "StateTreeIndexTypes.h"
#include "StateTreeEvents.generated.h"

#define UE_API STATETREEMODULE_API

struct FStateTreeEventQueue;

namespace UE::StateTree::Event
{
	/**
	* State Tree Events are cleared at end of each transition processing phase. (Inside TriggerTransitions()).
	* When in scope, events sent will live through the current phase and cleared on the next transition processing phase.
	* Used internally by TriggerTransitions and ParallelTree.
	*/
	struct FEventsPendingForNextTransitionProcessingScope
	{
		explicit FEventsPendingForNextTransitionProcessingScope(TNotNull<FStateTreeEventQueue*> EventQueue);

		~FEventsPendingForNextTransitionProcessingScope();

		TNotNull<FStateTreeEventQueue*> ScopedEventQueue;

		uint8 bSavedIsPendingForNextTransitionProcessingScope : 1 = false;
	};
}

/** Enum used for flow control during event iteration. */
UENUM()
enum class EStateTreeLoopEvents : uint8
{
	/** Continues to next event. */
	Next,
	/** Stops the event handling loop. */
	Break,
	/** Consumes and removes the current event. */
	Consume,
};

/**
 * StateTree event with payload.
 */
USTRUCT(BlueprintType)
struct FStateTreeEvent
{
	GENERATED_BODY()

	FStateTreeEvent() = default;

	explicit FStateTreeEvent(const FGameplayTag InTag)
		: Tag(InTag)
	{
	}
	
	explicit FStateTreeEvent(const FGameplayTag InTag, const FConstStructView InPayload, const FName InOrigin)
		: Tag(InTag)
		, Payload(InPayload)
		, Origin(InOrigin)
	{
	}

	friend inline uint32 GetTypeHash(const FStateTreeEvent& Event)
	{
		uint32 Hash = GetTypeHash(Event.Tag);
		
		if (Event.Payload.IsValid())
		{
			Hash = HashCombineFast(Hash, Event.Payload.GetScriptStruct()->GetStructTypeHash(Event.Payload.GetMemory()));
		}

		return Hash;
	}
	
	/** Tag describing the event */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Default", meta=(Categories="StateTreeEvent"))
	FGameplayTag Tag;

	/** Optional payload for the event. */ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Default")
	FInstancedStruct Payload;

	/** Optional info to describe who sent the event. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Default")
	FName Origin;

private:
	uint8 bIsPendingForNextTransitionProcessing : 1 = false;

	friend struct FStateTreeEventQueue;
};

/**
 * A struct wrapping FStateTreeEvent in shared struct, used to make it easier to refer to the events during State Tree update.
 */
USTRUCT()
struct FStateTreeSharedEvent
{
	GENERATED_BODY()

	FStateTreeSharedEvent() = default;

	explicit FStateTreeSharedEvent(const FGameplayTag InTag, const FConstStructView InPayload, const FName InOrigin)
		: Event(MakeShared<FStateTreeEvent>(InTag, InPayload, InOrigin))
	{}

	explicit FStateTreeSharedEvent(const FStateTreeEvent& InEvent)
		: Event(MakeShared<FStateTreeEvent>(InEvent))
	{}

	void AddStructReferencedObjects(FReferenceCollector& Collector);

	const FStateTreeEvent* Get() const
	{
		return Event.Get();
	}

	FStateTreeEvent* GetMutable()
	{
		return Event.Get();
	}

	const FStateTreeEvent* operator->() const
	{
		return Event.Get();
	}

	FStateTreeEvent* operator->()
	{
		return Event.Get();
	}

	const FStateTreeEvent& operator*()
	{
		check(Event.IsValid());
		return *Event.Get();
	}

	FStateTreeEvent& operator*() const
	{
		check(Event.IsValid());
		return *Event.Get();
	}

	bool IsValid() const
	{
		return Event.IsValid();
	}

	bool operator==(const FStateTreeSharedEvent& Other) const
	{
		return Event == Other.Event;
	}

protected:
	TSharedPtr<FStateTreeEvent> Event;
};

template<>
struct TStructOpsTypeTraits<FStateTreeSharedEvent> : public TStructOpsTypeTraitsBase2<FStateTreeSharedEvent>
{
	enum
	{
		WithAddStructReferencedObjects = true,
	};
};

/**
 * Event queue buffering all the events to be processed by a State Tree.
 */
USTRUCT()
struct FStateTreeEventQueue
{
	GENERATED_BODY()

	/** Maximum number of events that can be buffered. */
	static constexpr int32 MaxActiveEvents = 64;

	/** @return const view to all the events in the buffer. */
	TConstArrayView<FStateTreeSharedEvent> GetEventsView() const
	{
		return SharedEvents;
	}

	/** @return view to all the events in the buffer. */
	TArrayView<FStateTreeSharedEvent> GetMutableEventsView()
	{
		return SharedEvents;
	}

	/** Resets the events in the event queue */
	void Reset()
	{
		SharedEvents.Reset();
	}


	/** @return true if the queue has any events. */
	bool HasEvents() const
	{
		return !SharedEvents.IsEmpty();
	}
	
	/**
	 * Buffers and event to be sent to the State Tree.
	 * @param Owner Optional pointer to an owner UObject that is used for logging errors.
	 * @param Tag tag identifying the event.
	 * @param Payload Optional reference to the payload struct.
	 * @param Origin Optional name identifying the origin of the event.
	 * @return true if successfully added the event to the events queue.
	 */
	UE_API bool SendEvent(const UObject* Owner, const FGameplayTag& Tag, const FConstStructView Payload = FConstStructView(), const FName Origin = FName());

	/**
	 * Consumes and removes the specified event from the event queue.
	 * @return true if successfully found and removed the event from the queue.
	 */
	UE_API bool ConsumeEvent(const FStateTreeSharedEvent& Event);

	/**
	 * Clear all events in the queue that are not marked pending for next transition processing phase,
	 * And unmarks events that are pending for next transition processing since we are advancing to the next.
	 */
	UE_API void ClearEventsForCurrentTransitionProcessingPhase();
	
	/**
	 * Iterates over all events.
	 * @param Function a lambda which takes const FStateTreeSharedEvent& Event, and returns EStateTreeLoopEvents.
	 */
	template<typename TFunc>
	void ForEachEvent(TFunc&& Function)
	{
		for (TArray<FStateTreeSharedEvent>::TIterator It(SharedEvents); It; ++It)
		{
			const EStateTreeLoopEvents Result = Function(*It);
			if (Result == EStateTreeLoopEvents::Break)
			{
				break;
			}
			if (Result == EStateTreeLoopEvents::Consume)
			{
				It.RemoveCurrent();
			}
		}
	}

protected:
	// Used by FStateTreeExecutionState to implement deprecated functionality.
	TArray<FStateTreeSharedEvent>& GetEventsArray() { return SharedEvents; };

	UPROPERTY()
	TArray<FStateTreeSharedEvent> SharedEvents;
	
	uint8 bNewlyAddedEventsShouldBePendingForNextTransitionProcessing : 1 = false;

	friend UE::StateTree::Event::FEventsPendingForNextTransitionProcessingScope;
};

#undef UE_API
