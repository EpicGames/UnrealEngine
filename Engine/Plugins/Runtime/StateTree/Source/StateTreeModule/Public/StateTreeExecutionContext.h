// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTree.h"
#include "StateTreeExecutionExtension.h"
#include "StateTreeExecutionTypes.h"
#include "StateTreeExecutionContextTypes.h"
#include "StateTreeNodeBase.h"
#include "StateTreeNodeRef.h"
#include "StateTreeReference.h"
#include "Debugger/StateTreeTrace.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "Templates/IsInvocable.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "StateTreeAsyncExecutionContext.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6

#define UE_API STATETREEMODULE_API

struct FGameplayTag;
struct FInstancedPropertyBag;
struct FStateTreeExecutionContext;
struct FStateTreeEvaluatorBase;
struct FStateTreeTaskBase;
struct FStateTreeConditionBase;
struct FStateTreeDelegateDispatcher;
struct FStateTreeEvent;
struct FStateTreeMinimalExecutionContext;
struct FStateTreeTransitionRequest;
struct FStateTreeInstanceDebugId;
struct FStateTreeWeakExecutionContext;

namespace UE::StateTree::InstanceData
{
	struct FEvaluationScopeInstanceContainer;
}

namespace UE::StateTree::ExecutionContext
{
	UE_API bool MarkDelegateAsBroadcasted(FStateTreeDelegateDispatcher Dispatcher, const FStateTreeExecutionFrame& CurrentFrame, FStateTreeInstanceStorage& Storage);
	UE_API EStateTreeRunStatus GetPriorityRunStatus(EStateTreeRunStatus A, EStateTreeRunStatus B);
	UE_API UE::StateTree::ETaskCompletionStatus CastToTaskStatus(EStateTreeFinishTaskType FinishTask);
	UE_API EStateTreeRunStatus CastToRunStatus(EStateTreeFinishTaskType FinishTask);
	UE_API UE::StateTree::ETaskCompletionStatus CastToTaskStatus(EStateTreeRunStatus InStatus);
	UE_API EStateTreeRunStatus CastToRunStatus(UE::StateTree::ETaskCompletionStatus InStatus);

	using FTaskCompletionDelegateDispatcherContainer = TArray<FStateTreeDelegateDispatcher, TInlineAllocator<3>>;
	/** Find the dispatcher for a task completion if one matches the task node and condition. */
	UE_API FTaskCompletionDelegateDispatcherContainer GetTaskCompletionDispatcher(TNotNull<const UStateTree*> StateTree, int32 NodeIndex, UE::StateTree::ETaskCompletionStatus TaskStatus);
}

/**
 * Delegate used by the execution context to collect external data views for a given StateTree asset.
 * The caller is expected to iterate over the ExternalDataDescs array, find the matching external data,
 * and store it in the OutDataViews at the same index:
 *
 *	for (int32 Index = 0; Index < ExternalDataDescs.Num(); Index++)
 *	{
 *		const FStateTreeExternalDataDesc& Desc = ExternalDataDescs[Index];
 *		// Find data requested by Desc
 *		OutDataViews[Index] = ...;
 *	}
 */
DECLARE_DELEGATE_RetVal_FourParams(bool, FOnCollectStateTreeExternalData, const FStateTreeExecutionContext& /*Context*/, const UStateTree* /*StateTree*/, TArrayView<const FStateTreeExternalDataDesc> /*ExternalDataDescs*/, TArrayView<FStateTreeDataView> /*OutDataViews*/);

/**
 * Read-only execution context to interact with the state tree instance data. Only const and read accesses are available.
 * Multiple FStateTreeReadOnlyExecutionContext can coexist on different threads as long no other (minimal, weak, regular) execution context exists.
 * The user is responsible for preventing invalid multi-threaded access.
 */
struct FStateTreeReadOnlyExecutionContext
{
public:
	UE_API explicit FStateTreeReadOnlyExecutionContext(TNotNull<UObject*> Owner, TNotNull<const UStateTree*> StateTree, FStateTreeInstanceData& InInstanceData);
	UE_API explicit FStateTreeReadOnlyExecutionContext(TNotNull<UObject*> Owner, TNotNull<const UStateTree*> StateTree, FStateTreeInstanceStorage& Storage);
	UE_API virtual ~FStateTreeReadOnlyExecutionContext();

private:
	FStateTreeReadOnlyExecutionContext(const FStateTreeReadOnlyExecutionContext&) = delete;
	FStateTreeReadOnlyExecutionContext& operator=(const FStateTreeReadOnlyExecutionContext&) = delete;

public:
	/**
	 * Indicates if the instance is valid and would be able to run the instance of the associated StateTree asset with a regular execution context.
	 * @return True if the StateTree asset assigned to the execution context is valid
	 * (i.e., not empty) and successfully initialized (i.e., linked and all bindings resolved).
	 */
	bool IsValid() const
	{
		return RootStateTree.IsReadyToRun();
	}

	/** @return The owner of the context */
	TNotNull<UObject*> GetOwner() const
	{
		return &Owner;
	}

	/** @return The world of the owner or nullptr if the owner is not set. */
	UWorld* GetWorld() const
	{
		return Owner.GetWorld();
	}

	/** @return the StateTree asset in use by the instance. It is the root asset. */
	TNotNull<const UStateTree*> GetStateTree() const
	{
		return &RootStateTree;
	}

	/** @return true if there is a pending event with specified tag. */
	bool HasEventToProcess(const FGameplayTag Tag) const
	{
		return Storage.GetEventQueue().GetEventsView().ContainsByPredicate([Tag](const FStateTreeSharedEvent& Event)
			{
				check(Event.IsValid());
				return Event->Tag.MatchesTag(Tag);
			});
	}

	/** @return Pointer to a State or null if state not found */
	const FCompactStateTreeState* GetStateFromHandle(const FStateTreeStateHandle StateHandle) const
	{
		return RootStateTree.GetStateFromHandle(StateHandle);
	}

	/** @return the delta time for the next execution context tick. */
	UE_API FStateTreeScheduledTick GetNextScheduledTick() const;

	/** @return the tree run status. */
	UE_API EStateTreeRunStatus GetStateTreeRunStatus() const;

	/** @return the status of the last tick function */
	UE_API EStateTreeRunStatus GetLastTickStatus() const;

	/** @return reference to the list of currently active frames and states. */
	UE_API TConstArrayView<FStateTreeExecutionFrame> GetActiveFrames() const;

	/** @return the name of the active state. */
	UE_API FString GetActiveStateName() const;

	/** @return the names of all the active state. */
	UE_API TArray<FName> GetActiveStateNames() const;

#if WITH_GAMEPLAY_DEBUGGER
	/** @return Debug string describing the current state of the execution */
	UE_API FString GetDebugInfoString() const;
#endif // WITH_GAMEPLAY_DEBUGGER

#if WITH_STATETREE_DEBUG
	UE_API int32 GetStateChangeCount() const;

	UE_API void DebugPrintInternalLayout();
#endif

#if WITH_STATETREE_TRACE
	/** A unique Id used by debugging tools to identify the instance. */
	UE_API FStateTreeInstanceDebugId GetInstanceDebugId() const;

	/** @return Short description of the instance for debug/trace purposes */
	FString GetInstanceDebugDescription() const
	{
		return GetInstanceDescriptionInternal();
	}

	void SetOuterTraceId(const uint64 Id) const
	{
		OuterTraceId = Id;
	}

	uint64 GetOuterTraceId() const
	{
		return OuterTraceId;
	}

	void SetNodeCustomDebugTraceData(UE::StateTreeTrace::FNodeCustomDebugData&& DebugData) const
	{
		ensureMsgf(!NodeCustomDebugTraceData .IsSet()
			, TEXT("CustomData is not expected to be already set."
				" This might indicate nested calls to SetNodeCustomDebugTraceData without calls to a trace macro"));
		NodeCustomDebugTraceData = MoveTemp(DebugData);
	}

	UE::StateTreeTrace::FNodeCustomDebugData StealNodeCustomDebugTraceData() const
	{
		return MoveTemp(NodeCustomDebugTraceData);
	}
#else
	void SetOuterTraceId(const uint64 Id) const
	{
	}

	uint64 GetOuterTraceId() const
	{
		return 0;
	}
#endif // WITH_STATETREE_TRACE

protected:
	/** @return Description used as prefix by STATETREE_LOG and STATETREE_CLOG, Owner name by default. */
	UE_API FString GetInstanceDescriptionInternal() const;

	/** Owner of the instance data. */
	UObject& Owner;

	/** The StateTree asset the context is initialized for */
	const UStateTree& RootStateTree;

	/** Data storage of the instance data. */
	FStateTreeInstanceStorage& Storage;

#if WITH_STATETREE_TRACE
	mutable UE::StateTreeTrace::FNodeCustomDebugData NodeCustomDebugTraceData;
	mutable uint64 OuterTraceId = 0;
#endif
};

/**
 * Minimal execution context to interact with the state tree instance data.
 * A regular execution context requires the context data and external data to be valid to execute all possible operations.
 * The minimal execution context doesn't requires those data but supports only a subset of operations.
 */
struct FStateTreeMinimalExecutionContext : public FStateTreeReadOnlyExecutionContext
{
public:
	UE_DEPRECATED(5.6, "Use FStateTreeMinimalExecutionContext with the not null pointer.")
	UE_API explicit FStateTreeMinimalExecutionContext(UObject& Owner, const UStateTree& StateTree, FStateTreeInstanceData& InInstanceData);
	UE_DEPRECATED(5.6, "Use FStateTreeMinimalExecutionContext with the not null pointer.")
	UE_API explicit FStateTreeMinimalExecutionContext(UObject& Owner, const UStateTree& StateTree, FStateTreeInstanceStorage& Storage);
	UE_API explicit FStateTreeMinimalExecutionContext(TNotNull<UObject*> Owner, TNotNull<const UStateTree*> StateTree, FStateTreeInstanceData& InInstanceData);
	UE_API explicit FStateTreeMinimalExecutionContext(TNotNull<UObject*> Owner, TNotNull<const UStateTree*> StateTree, FStateTreeInstanceStorage& Storage);
	UE_API virtual ~FStateTreeMinimalExecutionContext();

private:
	FStateTreeMinimalExecutionContext(const FStateTreeMinimalExecutionContext&) = delete;
	FStateTreeMinimalExecutionContext& operator=(const FStateTreeMinimalExecutionContext&) = delete;

public:
	/** 
	 * Adds a scheduled tick request.
	 * The result of GetNextScheduledTick is affected by the request.
	 * This allows a specific task to control when the tree ticks.
	 * @note A request with a higher priority will supersede all other requests.
	 * ex: Task A request a custom time of 1FPS and Task B request a custom time of 2FPS. Both tasks will tick at 1FPS.
	 */
	UE_API UE::StateTree::FScheduledTickHandle AddScheduledTickRequest(FStateTreeScheduledTick ScheduledTick);

	/** Updates the scheduled tick of a previous request. */
	UE_API void UpdateScheduledTickRequest(UE::StateTree::FScheduledTickHandle Handle, FStateTreeScheduledTick ScheduledTick);

	/** Removes a scheduled tick request. */
	UE_API void RemoveScheduledTickRequest(UE::StateTree::FScheduledTickHandle Handle);

	/** Sends event for the StateTree. */
	UE_API void SendEvent(const FGameplayTag Tag, const FConstStructView Payload = FConstStructView(), const FName Origin = FName());

protected:
	/**
	 * Get ExecutionExtension from InstanceStorage for less indirections
	 * The user is responsible for validity of the result, re-call the getter when needed
	 * @return ExecutionExtension from InstanceStorage. could be null if Extension hasn't been set or Instance Storage has been reset.
	 */
	UE_API FStateTreeExecutionExtension* GetMutableExecutionExtension() const;

	/** Informs the owner when the instance of the tree must woke up from a scheduled tick sleep. */
	UE_API void ScheduleNextTick(UE::StateTree::ETickReason Reason = UE::StateTree::ETickReason::None);

protected:
	/** The context is processing the tree. We do not need to inform the owner that something changed. */
	bool bAllowedToScheduleNextTick = true;
};

/**
 * StateTree Execution Context is a helper that is used to update StateTree instance data.
 *
 * The context is meant to be temporary, you should not store a context across multiple frames.
 *
 * The owner is used as the owner of the instantiated UObjects in the instance data and logging,
 * it should have same or greater lifetime as the InstanceData. 
 *
 * In common case you can use the constructor to initialize the context, and us a helper struct
 * to set up the context data and external data getter:
 *
 *		FStateTreeExecutionContext Context(*GetOwner(), *StateTreeRef.GetStateTree(), InstanceData);
 *		if (SetContextRequirements(Context))
 *		{
 *			Context.Tick(DeltaTime);
 * 		}
 *
 * 
 *		bool UMyComponent::SetContextRequirements(FStateTreeExecutionContext& Context)
 *		{
 *			if (!Context.IsValid())
 *			{
 *				return false;
 *			}
 *			// Setup context data
 *			Context.SetContextDataByName(...);
 *			...
 *
 *			Context.SetCollectExternalDataCallback(FOnCollectStateTreeExternalData::CreateUObject(this, &UMyComponent::CollectExternalData);
 *
 *			return Context.AreContextDataViewsValid();
 *		}
 *
 *		bool UMyComponent::CollectExternalData(const FStateTreeExecutionContext& Context, const UStateTree* StateTree, TArrayView<const FStateTreeExternalDataDesc> ExternalDataDescs, TArrayView<FStateTreeDataView> OutDataViews)
 *		{
 *			...
 *			for (int32 Index = 0; Index < ExternalDataDescs.Num(); Index++)
 *			{
 *				const FStateTreeExternalDataDesc& Desc = ExternalDataDescs[Index];
 *				if (Desc.Struct->IsChildOf(UWorldSubsystem::StaticClass()))
 *				{
 *					UWorldSubsystem* Subsystem = World->GetSubsystemBase(Cast<UClass>(const_cast<UStruct*>(Desc.Struct.Get())));
 *					OutDataViews[Index] = FStateTreeDataView(Subsystem);
 *				}
 *				...
 *			}
 *			return true;
 *		}
 *
 * In this example the SetContextRequirements() method is used to set the context defined in the schema,
 * and the delegate FOnCollectStateTreeExternalData is used to query the external data required by the tasks and conditions.
 *
 * In case the State Tree links to other state tree assets, the collect external data might get called
 * multiple times, once for each asset.
 */
struct FStateTreeExecutionContext : public FStateTreeMinimalExecutionContext
{
public:
	UE_API FStateTreeExecutionContext(UObject& InOwner, const UStateTree& InStateTree, FStateTreeInstanceData& InInstanceData, const FOnCollectStateTreeExternalData& CollectExternalDataCallback = {}, const EStateTreeRecordTransitions RecordTransitions = EStateTreeRecordTransitions::No);
	/** Construct an execution context from a parent context and another tree. Useful to run a subtree from the parent context with the same schema. */
	UE_API FStateTreeExecutionContext(const FStateTreeExecutionContext& InContextToCopy, const UStateTree& InStateTree, FStateTreeInstanceData& InInstanceData);
	UE_API FStateTreeExecutionContext(TNotNull<UObject*> Owner, TNotNull<const UStateTree*> StateTree, FStateTreeInstanceData& InInstanceData, const FOnCollectStateTreeExternalData& CollectExternalDataCallback = {}, const EStateTreeRecordTransitions RecordTransitions = EStateTreeRecordTransitions::No);
	/** Construct an execution context from a parent context and another tree. Useful to run a subtree from the parent context with the same schema. */
	UE_API FStateTreeExecutionContext(const FStateTreeExecutionContext& InContextToCopy, TNotNull<const UStateTree*> StateTree, FStateTreeInstanceData& InInstanceData);
	UE_API virtual ~FStateTreeExecutionContext();

private:
	FStateTreeExecutionContext(const FStateTreeExecutionContext&) = delete;
	FStateTreeExecutionContext& operator=(const FStateTreeExecutionContext&) = delete;

public:
	/** Sets callback used to collect external data views during State Tree execution. */
	UE_API void SetCollectExternalDataCallback(const FOnCollectStateTreeExternalData& Callback);

	UE_DEPRECATED(5.6, "Use SetLinkedStateTreeOverrides that creates a copy.")
	/**
	 * Overrides for linked State Trees. This table is used to override State Tree references on linked states.
	 * If a linked state's tag is exact match of the tag specified on the table, the reference from the table is used instead.
	 */
	UE_API void SetLinkedStateTreeOverrides(const FStateTreeReferenceOverrides* InLinkedStateTreeOverrides);

	/**
	 * Overrides for linked State Trees. This table is used to override State Tree references on linked states.
	 * If a linked state's tag is exact match of the tag specified on the table, the reference from the table is used instead.
	 */
	UE_API void SetLinkedStateTreeOverrides(FStateTreeReferenceOverrides InLinkedStateTreeOverrides);

	/** @return the first state tree reference set by SetLinkedStateTreeOverrides that matches the StateTag. Or null if not found. */
	UE_API const FStateTreeReference* GetLinkedStateTreeOverrideForTag(const FGameplayTag StateTag) const;

	/** Structure to-be-populated and set for any StateTree using any EStateTreeDataSourceType::ExternalGlobalParameterData bindings */
	struct FExternalGlobalParameters
	{
		/* Add memory mapping, this expects InParameterMemory to resolve correctly for the SourceLeafProperty and SourceIndirection */
		UE_API bool Add(const FPropertyBindingCopyInfo& Copy, uint8* InParameterMemory);
		UE_API uint8* Find(const FPropertyBindingCopyInfo& Copy) const;
		UE_API void Reset();
	private:
		TMap<uint32, uint8*> Mappings;
	};
	UE_API void SetExternalGlobalParameters(const FExternalGlobalParameters* Parameters);

	/** @return const references to the instance data in use */
	const FStateTreeInstanceData* GetInstanceData() const
	{
		return &InstanceData;
	}

	/** @return mutable references to the instance data in use */
	FStateTreeInstanceData* GetMutableInstanceData() const
	{
		return &InstanceData;
	}

	/** @return mutable references to the instance data in use. */
	const FStateTreeEventQueue& GetEventQueue() const
	{
		return InstanceData.GetEventQueue();
	}

	/** @return mutable references to the instance data in use. */
	FStateTreeEventQueue& GetMutableEventQueue() const
	{
		return InstanceData.GetMutableEventQueue();
	}

	/** @return a weak context to interact with the state tree instance data that can be stored for later uses. */
	UE_API FStateTreeWeakExecutionContext MakeWeakExecutionContext() const;
	
	/**
	 * @return a weak reference for a task that can be stored for later uses.
	 * @note similar to GetInstanceData, the node needs to be the current processing node.
	 */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.6, "FStateTreeWeakTaskRef is no longer used.")
	UE_API FStateTreeWeakTaskRef MakeWeakTaskRef(const FStateTreeTaskBase& Node) const;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * @return a weak reference for a task that can be stored for later uses.
	 * @note similar to GetInstanceData, the instance data needs to be the current processing node.
	 */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	template <typename T>
	UE_DEPRECATED(5.6, "FStateTreeWeakTaskRef is no longer used.")
	FStateTreeWeakTaskRef MakeWeakTaskRefFromInstanceData(const T& InInstanceData) const
	{
		check(&CurrentNodeInstanceData.template GetMutable<T>() == &InInstanceData);
		return MakeWeakTaskRefInternal();
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * @return a weak reference for a task that can be stored for later uses.
	 * @note similar to GetInstanceData, the instance data needs to be the current processing node.
	 */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	template <typename T>
	UE_DEPRECATED(5.6, "FStateTreeWeakTaskRef is no longer used.")
	FStateTreeWeakTaskRef MakeWeakTaskRefFromInstanceDataPtr(const T* InInstanceData) const
	{
		check(CurrentNodeInstanceData.template GetMutablePtr<T>() == InInstanceData);
		return MakeWeakTaskRefInternal();
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	struct FStartParameters
	{
		/** Optional override of global parameters initial values. */
		FConstStructView InitialGlobalParameters;

		/** Optional extension for the execution context. */
		TInstancedStruct<FStateTreeExecutionExtension> ExecutionExtension;

		/** Optional event queue from another instance data. Marks the event queue not owned. */
		const TSharedPtr<FStateTreeEventQueue> SharedEventQueue;
		
		/** Optional override of initial seed for RandomStream. By default FPlatformTime::Cycles() will be used. */
		TOptional<int32> RandomSeed;

		struct FStateToSelectOverrideArgs
		{
			/** Optional override of tag of start state to select. */
			FGameplayTag StateTag;
			
			/** Does the tag needs to match exactly when searching the state. */
			UStateTree::EStateGameplayTagQueryMethod TagQueryMethod = UStateTree::EStateGameplayTagQueryMethod::MatchesExact;
		};

		/** Optional override of initial state to select. By default, root state will be selected. */
		TOptional<FStateToSelectOverrideArgs> SelectStateOverrideArgs;

#if WITH_EDITORONLY_DATA
		UE_DEPRECATED(5.8, "Use InitialGlobalParameters instead.")
		/** Optional override of global parameters initial values. */
		const FInstancedPropertyBag* GlobalParameters = nullptr;
#endif
	};

	/**
	 * Start executing.
	 * @return Tree execution status after the start.
	 */
	UE_API EStateTreeRunStatus Start();

	/**
	 * Start executing.
	 * @param InitialGlobalParameters Optional override of parameters initial values
	 * @return Tree execution status after the start.
	 */
	UE_API EStateTreeRunStatus Start(FConstStructView InitialGlobalParameters);
	
	/**
	 * Start executing.
	 * @return Tree execution status after the start.
	 */
	UE_API EStateTreeRunStatus Start(FStartParameters Parameter);

	UE_DEPRECATED(5.8, "Use the version with the FStartParameters")
	UE_API EStateTreeRunStatus Start(const FInstancedPropertyBag* InitialGlobalParameters, int32 RandomSeed = -1);

	/**
	 * Stop executing if the tree is running.
	 * @param CompletionStatus Status (and terminal state) reported in the transition when the tree is stopped.
	 * @return Tree execution status at stop, can be CompletionStatus, or earlier status if the tree is not running. 
	 */
	UE_API EStateTreeRunStatus Stop(const EStateTreeRunStatus CompletionStatus = EStateTreeRunStatus::Stopped);

	/**
	 * Tick the state tree logic, updates the tasks and triggers transitions.
	 * @param DeltaTime time to advance the logic.
	 * @returns tree run status after the tick.
	 */
	UE_API EStateTreeRunStatus Tick(const float DeltaTime);

	/**
	 * Tick the state tree logic partially, updates the tasks.
	 * For full update TickTriggerTransitions() should be called after.
	 * @param DeltaTime time to advance the logic.
	 * @returns tree run status after the partial tick.
	 */
	UE_API EStateTreeRunStatus TickUpdateTasks(const float DeltaTime);
	
	/**
	 * Tick the state tree logic partially, triggers the transitions.
	 * For full update TickUpdateTasks() should be called before.
	 * @returns tree run status after the partial tick.
	 */
	UE_API EStateTreeRunStatus TickTriggerTransitions();

	/**
	 * Broadcasts the delegate.
	 * It executes bound delegates immediately and triggers bound transitions (when transitions are evaluated).
	 */
	UE_API void BroadcastDelegate(const FStateTreeDelegateDispatcher& Dispatcher);

	UE_DEPRECATED(5.6, "Use BindDelegate")
	/** Adds delegate listener. */
	UE_API bool AddDelegateListener(const FStateTreeDelegateListener& Listener, FSimpleDelegate Delegate);

	/**
	 * Registers the delegate to the listener.
	 * If the listener was previously registered, then unregister it first before registering it again with the new delegate callback.
	 * The listener is bound to a dispatcher in the editor.
	 */
	UE_API void BindDelegate(const FStateTreeDelegateListener& Listener, FSimpleDelegate Delegate);

	UE_DEPRECATED(5.6, "Use UnbindDelegate")
	/** Removes delegate listener. */
	UE_API void RemoveDelegateListener(const FStateTreeDelegateListener& Listener);

	/** Unregisters the callback bound to the listener. */
	UE_API void UnbindDelegate(const FStateTreeDelegateListener& Listener);

	/**
	 * Iterates over all events.
	 * @param Function a lambda which takes const FStateTreeSharedEvent& Event, and returns EStateTreeLoopEvents.
	 */
	template<typename TFunc>
	typename TEnableIf<TIsInvocable<TFunc, FStateTreeSharedEvent>::Value, void>::Type ForEachEvent(TFunc&& Function) const
	{
		if (!EventQueue)
		{
			return;
		}
		EventQueue->ForEachEvent(Function);
	}

	/**
	 * Iterates over all events.
	 * @param Function a lambda which takes const FStateTreeEvent& Event, and returns EStateTreeLoopEvents.
	 * Less preferable than FStateTreeSharedEvent version.
	 */
	template<typename TFunc>
	typename TEnableIf<TIsInvocable<TFunc, FStateTreeEvent>::Value, void>::Type ForEachEvent(TFunc&& Function) const
	{
		if (!EventQueue)
		{
			return;
		}
		EventQueue->ForEachEvent([Function](const FStateTreeSharedEvent& Event)
		{
			return Function(*Event);
		});
	}

	/** @return events to process this tick. */
	TArrayView<FStateTreeSharedEvent> GetMutableEventsToProcessView()
	{
		return EventQueue ? EventQueue->GetMutableEventsView() : TArrayView<FStateTreeSharedEvent>();
	}

	/** @return events to process this tick. */
	TConstArrayView<FStateTreeSharedEvent> GetEventsToProcessView() const
	{
		return EventQueue ? EventQueue->GetMutableEventsView() : TArrayView<FStateTreeSharedEvent>();
	}

	/** Consumes and removes the specified event from the event queue. */
	UE_API void ConsumeEvent(const FStateTreeSharedEvent& Event);

	FStateTreeIndex16 GetCurrentlyProcessedNodeIndex() const
	{
		return FStateTreeIndex16(CurrentNodeIndex);
	}

	FStateTreeDataHandle GetCurrentlyProcessedNodeInstanceData() const
	{
		return CurrentNodeDataHandle;
	}

	UE_DEPRECATED(5.6, "Use GetCurrentlyProcessedNodeInstanceData() instead.")
	/** @return the currently processed node if applicable. */
	FStateTreeDataHandle GetCurrentlyProcessedNode() const
	{
		return CurrentNodeDataHandle;
	}

	/** @return the currently processed state if applicable. */
	FStateTreeStateHandle GetCurrentlyProcessedState() const
	{
		return CurrentlyProcessedState;
	}

	/** @return the currently processed execution frame if applicable. */
	const FStateTreeExecutionFrame* GetCurrentlyProcessedFrame() const
	{
		return CurrentlyProcessedFrame;
	}

	/** @return the currently processed execution parent frame if applicable. */
	const FStateTreeExecutionFrame* GetCurrentlyProcessedParentFrame() const
	{
		return CurrentlyProcessedParentFrame;
	}

	/** @return the currently processed temporary storage */
	TSharedPtr<UE::StateTree::ExecutionContext::ITemporaryStorage> GetCurrentlyProcessedTemporaryStorage() const
	{
		return GetCurrentlyProcessedStateSelection();
	}

	/** @return Array view to named external data descriptors associated with this context. Note: Init() must be called before calling this method. */
	TConstArrayView<FStateTreeExternalDataDesc> GetContextDataDescs() const
	{
		return RootStateTree.GetContextDataDescs();
	}

	/** Sets context data view value for specific item. */
	void SetContextData(const FStateTreeExternalDataHandle Handle, FStateTreeDataView DataView)
	{
		check(Handle.IsValid());
		check(Handle.DataHandle.GetSource() == EStateTreeDataSourceType::ContextData);
		ContextAndExternalDataViews[Handle.DataHandle.GetIndex()] = DataView;
	}

	/** Sets the context data based on name (name is defined in the schema), returns true if data was found */
	UE_API bool SetContextDataByName(const FName Name, FStateTreeDataView DataView);

	/** @return the context data based on name (name is defined in the schema) */
	UE_API FStateTreeDataView GetContextDataByName(const FName Name) const;

	/** @return True if all context data pointers are set. */ 
	UE_API bool AreContextDataViewsValid() const;

	/**
	 * Returns reference to external data based on provided handle. The return type is deduced from the handle's template type.
     * @param Handle Valid TStateTreeExternalDataHandle<> handle. 
	 * @return reference to external data based on handle or null if data is not set.
	 */ 
	template <typename T>
	typename T::DataType& GetExternalData(const T Handle) const
	{
		check(Handle.IsValid());
		check(Handle.DataHandle.GetSource() == EStateTreeDataSourceType::ExternalData);
		check(CurrentlyProcessedFrame);
		check(CurrentlyProcessedFrame->StateTree->ExternalDataDescs[Handle.DataHandle.GetIndex()].Requirement != EStateTreeExternalDataRequirement::Optional); // Optionals should query pointer instead.
		return ContextAndExternalDataViews[CurrentlyProcessedFrame->ExternalDataBaseIndex.Get() + Handle.DataHandle.GetIndex()].template GetMutable<typename T::DataType>();
	}

	/**
	 * Returns pointer to external data based on provided item handle. The return type is deduced from the handle's template type.
     * @param Handle Valid TStateTreeExternalDataHandle<> handle.
	 * @return pointer to external data based on handle or null if item is not set or handle is invalid.
	 */ 
	template <typename T>
	typename T::DataType* GetExternalDataPtr(const T Handle) const
	{
		if (Handle.IsValid())
		{
			check(CurrentlyProcessedFrame);
			check(Handle.DataHandle.GetSource() == EStateTreeDataSourceType::ExternalData);
			return ContextAndExternalDataViews[CurrentlyProcessedFrame->ExternalDataBaseIndex.Get() + Handle.DataHandle.GetIndex()].template GetMutablePtr<typename T::DataType>();
		}
		return nullptr;
	}

	FStateTreeDataView GetExternalDataView(const FStateTreeExternalDataHandle Handle)
	{
		if (Handle.IsValid())
		{
			check(CurrentlyProcessedFrame);
			check(Handle.DataHandle.GetSource() == EStateTreeDataSourceType::ExternalData);
			return ContextAndExternalDataViews[CurrentlyProcessedFrame->ExternalDataBaseIndex.Get() + Handle.DataHandle.GetIndex()];
		}
		return FStateTreeDataView();
	}

	/** @returns pointer to the instance data of specified node. */
	template <typename T>
	T* GetInstanceDataPtr(const FStateTreeNodeBase& Node) const
	{
		check(CurrentNodeDataHandle == Node.InstanceDataHandle);
		return CurrentNodeInstanceData.template GetMutablePtr<T>();
	}

	/** @returns reference to the instance data of specified node. */
	template <typename T>
	T& GetInstanceData(const FStateTreeNodeBase& Node) const
	{
		check(CurrentNodeDataHandle == Node.InstanceDataHandle);
		return CurrentNodeInstanceData.template GetMutable<T>();
	}

	/** @returns reference to the instance data of specified node. Infers the instance data type from the node's FInstanceDataType. */
	template <typename T>
	typename T::FInstanceDataType& GetInstanceData(const T& Node) const
	{
		static_assert(TIsDerivedFrom<T, FStateTreeNodeBase>::IsDerived, "Expecting Node to derive from FStateTreeNodeBase.");
		check(CurrentNodeDataHandle == Node.InstanceDataHandle);
		return CurrentNodeInstanceData.template GetMutable<typename T::FInstanceDataType>();
	}

	/** @returns pointer to the execution runtime data of specified node. */
	template <typename T>
	T* GetExecutionRuntimeDataPtr(const FStateTreeNodeBase& Node) const
	{
		check(CurrentNodeDataHandle == Node.InstanceDataHandle && Node.InstanceDataHandle.IsValid());
		return GetExecutionRuntimeDataView().template GetMutablePtr<T>();
	}

	/** @returns reference to the execution runtime data of specified node. */
	template <typename T>
	T& GetExecutionRuntimeData(const FStateTreeNodeBase& Node) const
	{
		check(CurrentNodeDataHandle == Node.InstanceDataHandle && Node.InstanceDataHandle.IsValid());
		return GetExecutionRuntimeDataView().template GetMutable<T>();
	}

	/** @returns reference to the execution runtime data of specified node. Infers the execution runtime data type from the node's FExecutionRuntimeDataType. */
	template <typename T>
	typename T::FExecutionRuntimeDataType& GetExecutionRuntimeData(const T& Node) const
	{
		static_assert(TIsDerivedFrom<T, FStateTreeNodeBase>::IsDerived, "Expecting Node to derive from FStateTreeNodeBase.");
		check(CurrentNodeDataHandle == Node.InstanceDataHandle && Node.InstanceDataHandle.IsValid());
		return GetExecutionRuntimeDataView().template GetMutable<typename T::FExecutionRuntimeDataType>();
	}

	/** @returns reference to instance data struct that can be passed to lambdas. See TStateTreeInstanceDataStructRef for usage. */
	template <typename T>
	TStateTreeInstanceDataStructRef<typename T::FInstanceDataType> GetInstanceDataStructRef(const T& Node) const
	{
		static_assert(TIsDerivedFrom<T, FStateTreeNodeBase>::IsDerived, "Expecting Node to derive from FStateTreeNodeBase.");
		check(CurrentlyProcessedFrame);
		return TStateTreeInstanceDataStructRef<typename T::FInstanceDataType>(InstanceData, *CurrentlyProcessedFrame, Node.InstanceDataHandle);
	}

	/**
	 * Requests transition to a state.
	 * If called during during transition processing (e.g. from FStateTreeTaskBase::TriggerTransitions()) the transition
	 * is attempted to be activate immediately (it can fail e.g. because of preconditions on a target state).
	 * If called outside the transition handling, the request is buffered and handled at the beginning of next transition processing.
	 * @param Request The state to transition to.
	 */
	UE_API void RequestTransition(const FStateTreeTransitionRequest& Request);

	/**
	 * Requests transition to a state.
	 * If called during during transition processing (e.g. from FStateTreeTaskBase::TriggerTransitions()) the transition
	 * is attempted to be activate immediately (it can fail e.g. because of preconditions on a target state).
	 * If called outside the transition handling, the request is buffered and handled at the beginning of next transition processing.
	 * @param TargetState The state to transition to.
	 * @param Priority The priority of the transition.
	 * @param Fallback of the transition if it fails to select the target state.
	 */
	UE_API void RequestTransition(FStateTreeStateHandle TargetState, EStateTreeTransitionPriority Priority = EStateTreeTransitionPriority::Normal, EStateTreeSelectionFallback Fallback = EStateTreeSelectionFallback::None);

	/**
	 * Finishes a task. This fails if the Task is not currently the processed node.
	 * ie. Must be called from inside a FStateTreeTaskBase EnterState, ExitState, StateCompleted, Tick, TriggerTransitions.
	 * If called during tick processing, then the state completes immediately.
	 * If called outside of the tick processing, then the request is buffered and handled on the next tick.
	 */
	UE_API void FinishTask(const FStateTreeTaskBase& Task, EStateTreeFinishTaskType FinishType);
	
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.6, "Use the weak context to finish a task async or FinishTask(FStateTreeTaskBase, EStateTreeFinishTaskType) to finish the current task.")
	/**
	 * Finishes a task.
	 * If called during tick processing, then the state completes immediately.
	 * If called outside of the tick processing, then the request is buffered and handled on the next tick.
	 */
	UE_API void FinishTask(const UE::StateTree::FFinishedTask& Task, EStateTreeFinishTaskType FinishType);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** @return data view of the specified handle relative to given frame. */
	UE_DEPRECATED(5.6, "Use UE::StateTree::InstanceData::GetDataView instead.")
	static FStateTreeDataView GetDataViewFromInstanceStorage(FStateTreeInstanceStorage& InstanceDataStorage, FStateTreeInstanceStorage* CurrentlyProcessedSharedInstanceStorage, const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& CurrentFrame, const FStateTreeDataHandle Handle)
	{
		return UE::StateTree::InstanceData::GetDataView(InstanceDataStorage, CurrentlyProcessedSharedInstanceStorage, CurrentFrame, Handle);
	}

	/** Looks for a frame in provided list of frames. */
	UE_DEPRECATED(5.8, "Use UE::StateTree::ExecutionContext::FindExecutionFrame instead.")
	static UE_API const FStateTreeExecutionFrame* FindFrame(const UStateTree* StateTree, FStateTreeStateHandle RootState, TConstArrayView<FStateTreeExecutionFrame> Frames, const FStateTreeExecutionFrame*& OutParentFrame);

protected:
	static UE_API FStateTreeExecutionFrame* FindMutableExecutionFrame(UE::StateTree::FActiveFrameID InFrameID, TArrayView<FStateTreeExecutionFrame> ActiveFrames, TArrayView<FStateTreeExecutionFrame> InTemporaryFrames);

public:

	/**
	 * Forces transition to a state. It will skip all conditions.
	 * Primarily used for replication purposes so that a client state tree stays in sync with its server counterpart.
	 * It has to be a running instance. It will not work if you didn't call Start or if the execution previously failed.
	 * @param Recorded state transition to run on the state tree.
	 * @return The new run status for the state tree.
	 */
	UE_API EStateTreeRunStatus ForceTransition(const FRecordedStateTreeTransitionResult& Transition);
	
	/** Returns the recorded transitions for this context. */
	TConstArrayView<FRecordedStateTreeTransitionResult> GetRecordedTransitions() const
	{
		return RecordedTransitions;
	}

protected:
	/** Event used during state selection. Identified by the ID of the active state that consumed it. */
	struct FSelectionEventWithID
	{
		UE::StateTree::FActiveState State;
		FStateTreeSharedEvent Event;
	};

	/** Describes a result of States Selection. */
	struct FSelectStateResult : public UE::StateTree::ExecutionContext::ITemporaryStorage
	{
		FSelectStateResult() = default;
		virtual ~FSelectStateResult() = default;

		/** The active states selected. They are in order from the root to the leaf. */
		TArray<UE::StateTree::FActiveState> SelectedStates;

		/** The selected frame ID. The frame can be in the current active list or in the TemporaryFrames list. */
		TArray<UE::StateTree::FActiveFrameID, TInlineAllocator<4>> SelectedFrames;

		/**
		 * New execution frame created during the state selection.
		 * The active state list is empty and will be filled during EnterState()
		 */
		TArray<FStateTreeExecutionFrame, TInlineAllocator<2>> TemporaryFrames;

		/** Events used during the state selection. */
		TArray<FSelectionEventWithID, TInlineAllocator<2>> SelectionEvents;

		/** The requested target of the selection. */
		UE::StateTree::FActiveState TargetState;

		/** @return a new temporary frame */
		UE_API FStateTreeExecutionFrame& MakeAndAddTemporaryFrame(UE::StateTree::FActiveFrameID FrameID, const UE::StateTree::FExecutionFrameHandle& FrameHandle, bool bIsGlobalFrame);
		UE_API FStateTreeExecutionFrame& MakeAndAddTemporaryFrameWithNewRoot(UE::StateTree::FActiveFrameID FrameID, const UE::StateTree::FExecutionFrameHandle& FrameHandle, FStateTreeExecutionFrame& OtherFrame);

		/** @return a temporary frame by ID. */
		FStateTreeExecutionFrame* FindTemporaryFrame(UE::StateTree::FActiveFrameID FrameID)
		{
			return TemporaryFrames.FindByPredicate(
				[FrameID = FrameID](const FStateTreeExecutionFrame& Frame)
				{
					return Frame.FrameID == FrameID;
				});
		}

		//~ITemporaryStorage
		virtual FFrameAndParent GetExecutionFrame(UE::StateTree::FActiveFrameID ID) override;
		virtual UE::StateTree::FActiveState GetStateHandle(UE::StateTree::FActiveStateID ID) const override;
		virtual TConstArrayView<FStateTreeExecutionFrame> GetTemporaryFrames() const override;
	};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	struct
	UE_DEPRECATED(5.7, "The selection result changed to include the state ID and frame ID")
	FStateSelectionResult
	{
		/** Max number of execution frames handled during state selection. */
		static constexpr int32 MaxExecutionFrames = 8;

		FStateSelectionResult() = default;
		explicit FStateSelectionResult(TConstArrayView<FStateTreeExecutionFrame> InFrames)
		{
			SelectedFrames = InFrames;
			FramesStateSelectionEvents.SetNum(SelectedFrames.Num());
		}

		bool IsFull() const
		{
			return SelectedFrames.Num() == MaxExecutionFrames;
		}

		void PushFrame(FStateTreeExecutionFrame Frame)
		{
			SelectedFrames.Add(Frame);
			FramesStateSelectionEvents.AddDefaulted();
		}

		void PopFrame()
		{
			SelectedFrames.Pop();
			FramesStateSelectionEvents.Pop();
		}

		bool ContainsFrames() const
		{
			return !SelectedFrames.IsEmpty();
		}

		int32 FramesNum() const
		{
			return SelectedFrames.Num();
		}

		TArrayView<FStateTreeExecutionFrame> GetSelectedFrames()
		{
			return SelectedFrames;
		}

		TConstArrayView<FStateTreeExecutionFrame> GetSelectedFrames() const
		{
			return SelectedFrames;
		}

		TArrayView<FStateTreeFrameStateSelectionEvents> GetFramesStateSelectionEvents()
		{
			return FramesStateSelectionEvents;
		}

	protected:
		TArray<FStateTreeExecutionFrame, TFixedAllocator<MaxExecutionFrames>> SelectedFrames;
		TArray<FStateTreeFrameStateSelectionEvents, TFixedAllocator<MaxExecutionFrames>> FramesStateSelectionEvents;
	};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** The result of the RequestTransition function. */
	struct FRequestTransitionResult
	{
		TSharedPtr<FSelectStateResult> Selection;
		FStateTreeTransitionResult Transition;
		FStateTreeTransitionSource Source;
	};

	UE_DEPRECATED(5.6, "Use FStateTreeExecutionExtension::GetInstanceDescription instead")
	/** @return Prefix that will be used by STATETREE_LOG and STATETREE_CLOG, Owner name by default. */
	UE_API virtual FString GetInstanceDescription() const final;

	/** Callback when delayed transition is triggered. Contexts that are event based can use this to trigger a future event. */
	virtual void BeginDelayedTransition(const FStateTreeTransitionDelayedState& DelayedState) {};

	UE_DEPRECATED(5.7, "Use the UpdateInstanceData with the FSelectStateResult argument.")
	UE_API void UpdateInstanceData(TConstArrayView<FStateTreeExecutionFrame> CurrentActiveFrames, TArrayView<FStateTreeExecutionFrame> NextActiveFrames);

	/** Allocate the instance data. Fixup the TemporarySelectedFrame with the correct data. */
	UE_API void UpdateInstanceData(const TSharedPtr<FSelectStateResult>& SelectStateResult);

	UE_DEPRECATED(5.7, "Use the EnterState with the FSelectStateResult argument.")
	UE_API EStateTreeRunStatus EnterState(FStateTreeTransitionResult& Transition);

	/**
	 * Handles logic for entering State. EnterState is called on new active Evaluators and Tasks that are part of the re-planned tree.
	 * Re-planned tree is from the transition target up to the leaf state. States that are parent to the transition target state
	 * and still active after the transition will remain intact.
	 * @return run status returned by the tasks.
	 */
	UE_API EStateTreeRunStatus EnterState(const TSharedPtr<FSelectStateResult>& SelectStateResult, const FStateTreeTransitionResult& Transition);

	UE_DEPRECATED(5.7, "Use the ExitState with the FSelectStateResult argument.")
	UE_API void ExitState(const FStateTreeTransitionResult& Transition);

	/**
	 * Handles logic for exiting State. ExitState is called on current active Evaluators and Tasks that are part of the re-planned tree.
	 * Re-planned tree is from the transition target up to the leaf state. States that are parent to the transition target state
	 * and still active after the transition will remain intact.
	 */
	UE_API void ExitState(const TSharedPtr<const FSelectStateResult>& SelectStateResult, const FStateTreeTransitionResult& Transition);

	/**
	 * Removes all delegate listeners.
	 */
	UE_API void RemoveAllDelegateListeners();

	/**
	 * Handles logic for signaling State completed. StateCompleted is called on current active Evaluators and Tasks in reverse order (from leaf to root).
	 */
	UE_API void StateCompleted();

private:
	template<bool bOnActiveInstances>
	void TickGlobalEvaluatorsForFrameInternal(float DeltaTime, const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& Frame);

protected:
	/**
	 * Stop the frame's evaluators.
	 * Should be used only on active node instances, assumes valid data handles and does not consider temporary node instances.
	 */
	UE_API void TickGlobalEvaluatorsForFrameOnActiveInstances(float DeltaTime, const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& Frame);

	/**
	 * Tick the frame's evaluators.
	 * This version validates the data handles and looks up temporary instances.
	 */
	UE_API void TickGlobalEvaluatorsForFrameWithValidation(float DeltaTime, const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& Frame);

	/**
	 * Tick evaluators and global tasks by delta time.
	 */
	UE_API EStateTreeRunStatus TickEvaluatorsAndGlobalTasks(float DeltaTime, bool bTickGlobalTasks = true);

	/**
	 * Tick the frame's evaluators and the frame's tasks.
	 */
	UE_API EStateTreeRunStatus TickEvaluatorsAndGlobalTasksForFrame(float DeltaTime, bool bTickGlobalTasks, int32 FrameIndex, const FStateTreeExecutionFrame* CurrentParentFrame, const TNotNull<FStateTreeExecutionFrame*> CurrentFrame);

	/** @return in progress transition or the latest requested transition. */
	UE_API TSharedPtr<FSelectStateResult> GetCurrentlyProcessedStateSelection() const;

private:
	/** Move the temporary data for each evaluators and tasks to instance data. */
	void MoveGlobalTemporariesToInstances(FStateTreeExecutionFrame& Frame);

	template<bool bOnActiveInstances, bool bMoveFromTemporary>
	EStateTreeRunStatus StartGlobalsForFrameInternal(const FStateTreeExecutionFrame* ParentFrame, FStateTreeExecutionFrame& Frame, FStateTreeTransitionResult& Transition);

protected:
	/**
	 * Start the frame's evaluators and the frame's tasks.
	 * Should be used only on active node instances, assumes valid data handles and does not consider temporary node instances.
	 */
	UE_API EStateTreeRunStatus StartGlobalsForFrameOnActiveInstances(const FStateTreeExecutionFrame* ParentFrame, FStateTreeExecutionFrame& Frame, FStateTreeTransitionResult& Transition);

	/**
	 * Start the frame's evaluators and the frame's tasks.
	 * This version validates the data handles and looks up temporary instances.
	 */
	UE_API EStateTreeRunStatus StartGlobalsForFrameWithValidation(const FStateTreeExecutionFrame* ParentFrame, FStateTreeExecutionFrame& Frame, FStateTreeTransitionResult& Transition);

	UE_DEPRECATED(5.7, "Use StartEvaluatorsAndGlobalTasks without the OutLastInitializedTaskIndex argument")
	UE_API EStateTreeRunStatus StartEvaluatorsAndGlobalTasks(FStateTreeIndex16& OutLastInitializedTaskIndex);

	/**
	 * Starts active global evaluators and global tasks.
	 * @return run status returned by the global tasks.
	 */
	UE_API EStateTreeRunStatus StartEvaluatorsAndGlobalTasks();

private:
	template<bool bOnActiveInstances>
	void StopGlobalsForFrameInternal(const FStateTreeExecutionFrame* ParentFrame, FStateTreeExecutionFrame& Frame, const FStateTreeTransitionResult& Transition);

protected:
	/**
	 * Stop the frame's evaluators and the frame's tasks.
	 * Should be used only on active node instances, assumes valid data handles and does not consider temporary node instances.
	 */
	UE_API void StopGlobalsForFrameOnActiveInstances(const FStateTreeExecutionFrame* ParentFrame, FStateTreeExecutionFrame& Frame, const FStateTreeTransitionResult& Transition);

	/**
	 * Stop the frame's evaluators and the frame's tasks.
	 * This version validates the data handles and looks up temporary instances.
	 */
	UE_API void StopGlobalsForFrameWithValidation(const FStateTreeExecutionFrame* ParentFrame, FStateTreeExecutionFrame& Frame, const FStateTreeTransitionResult& Transition);

	UE_DEPRECATED(5.7, "Use StopEvaluatorsAndGlobalTasks without LastInitializedTaskIndex")
	UE_API void StopEvaluatorsAndGlobalTasks(const EStateTreeRunStatus CompletionStatus, const FStateTreeIndex16 LastInitializedTaskIndex);

	/**
	 * Stops active global evaluators and active global tasks.
	 */
	UE_API void StopEvaluatorsAndGlobalTasks(const EStateTreeRunStatus CompletionStatus);

	UE_DEPRECATED(5.7, "Use StopGlobalsForFrameOnActiveInstances")
	UE_API void CallStopOnEvaluatorsAndGlobalTasks(const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& Frame, const FStateTreeTransitionResult& Transition, const FStateTreeIndex16 LastInitializedTaskIndex = FStateTreeIndex16());

	/** Starts temporary instances of global evaluators and tasks for a given frame. */
	UE_API EStateTreeRunStatus StartTemporaryEvaluatorsAndGlobalTasks(const FStateTreeExecutionFrame* CurrentParentFrame, FStateTreeExecutionFrame& CurrentFrame);

	UE_DEPRECATED(5.6, "Use the non const version of StartTemporaryEvaluatorsAndGlobalTasks")
	EStateTreeRunStatus StartTemporaryEvaluatorsAndGlobalTasks(const FStateTreeExecutionFrame* CurrentParentFrame, const FStateTreeExecutionFrame& CurrentFrame)
	{
		return StartTemporaryEvaluatorsAndGlobalTasks(CurrentParentFrame, const_cast<FStateTreeExecutionFrame&>(CurrentFrame));
	}

	UE_DEPRECATED(5.7, "Use the non const version of StopTemporaryEvaluatorsAndGlobalTasks")
	UE_API void StopTemporaryEvaluatorsAndGlobalTasks(const FStateTreeExecutionFrame* CurrentParentFrame, const FStateTreeExecutionFrame& CurrentFrame);

	/** Stops temporary frame's evaluators and frame's tasks for the provided frame. */
	UE_API void StopTemporaryEvaluatorsAndGlobalTasks(const FStateTreeExecutionFrame* CurrentParentFrame, FStateTreeExecutionFrame& CurrentFrame, EStateTreeRunStatus StartResult);

	/**
	 * Ticks tasks of all active states starting from current state by delta time.
	 * @return Run status returned by the tasks.
	 */
	UE_API EStateTreeRunStatus TickTasks(const float DeltaTime);

	struct FTickTaskResult
	{
		bool bShouldTickTasks = true;
	};
	struct FTickTaskArguments
	{
		FTickTaskArguments() = default;
		float DeltaTime = 0.f;
		int32 TasksBegin = 0;
		int32 TasksNum = 0;
		int32 Indent = 0;
		const FStateTreeExecutionFrame* ParentFrame = nullptr;
		FStateTreeExecutionFrame* Frame = nullptr;
		UE::StateTree::FActiveStateID StateID;
		UE::StateTree::FTasksCompletionStatus* TasksCompletionStatus = nullptr;
		bool bIsGlobalTasks = false;
		bool bShouldTickTasks = true;
	};
	/** Ticks tasks and updates the bindings for a specific state or frame. */
	UE_API FTickTaskResult TickTasks(const FTickTaskArguments& Args);

	/** Common functionality shared by the tick methods. */
	UE_API EStateTreeRunStatus TickPrelude();
	UE_API EStateTreeRunStatus TickPostlude();

	/** Handles task ticking part of the tick. */
	UE_API void TickUpdateTasksInternal(float DeltaTime);
	
	/** Handles transition triggering part of the tick. */
	UE_API void TickTriggerTransitionsInternal();

	/** Broadcasts task completion dispatchers when a task completes. */
	void BroadcastTaskCompletionDispatchers(TNotNull<const UStateTree*> StateTree, int32 NodeIndex, UE::StateTree::ETaskCompletionStatus TaskStatus);

	/** Gives Execution Extension a chance to react. */
	UE_API void BeginApplyTransition(const FStateTreeTransitionResult& InTransitionResult);

protected:
	using FMemoryRequirement = UE::StateTree::InstanceData::FEvaluationScopeInstanceContainer::FMemoryRequirement;

private:
	template<bool bOnActiveInstances>
	bool TestAllConditionsInternal(const FStateTreeExecutionFrame* CurrentParentFrame, const FStateTreeExecutionFrame& CurrentFrame, FStateTreeStateHandle CurrentStateHandle, const FMemoryRequirement& MemoryRequirement, const int32 ConditionsOffset, const int32 ConditionsNum, EStateTreeUpdatePhase Phase);

	enum class EConditionFunction
	{
		EnterState,
		ExitState,
		StateCompleted,
	};
	template<bool bOnActiveInstances, EConditionFunction ConditionFunction>
	void RunAllConditionsInternal(const FStateTreeExecutionFrame* CurrentParentFrame, const FStateTreeExecutionFrame& CurrentFrame, const FCompactStateTreeState& CurrentState, const FStateTreeTransitionResult& CurrentTransition);

protected:
	/**
	 * Checks all conditions at given range.
	 * Should be used only on active node instances, assumes valid data handles and does not consider temporary node instances.
	 * @return True if all conditions pass.
	 */
	UE_API bool TestAllConditionsOnActiveInstances(const FStateTreeExecutionFrame* CurrentParentFrame, const FStateTreeExecutionFrame& CurrentFrame, FStateTreeStateHandle CurrentStateHandle, const FMemoryRequirement& MemoryRequirement, const int32 ConditionsOffset, const int32 ConditionsNum, EStateTreeUpdatePhase Phase);

	/**
	 * Checks all conditions at given range.
	 * This version validates the data handles and looks up temporary instances.
	 * @return True if all conditions pass.
	 */
	UE_API bool TestAllConditionsWithValidation(const FStateTreeExecutionFrame* CurrentParentFrame, const FStateTreeExecutionFrame& CurrentFrame, FStateTreeStateHandle CurrentStateHandle, const FMemoryRequirement& MemoryRequirement, const int32 ConditionsOffset, const int32 ConditionsNum, EStateTreeUpdatePhase Phase);

	UE_DEPRECATED(5.7, "Use TestAllConditionsWithValidation or TestAllConditionsOnActiveInstances. TestAllConditionsWithValidation can get the data from a temporary frame in state selection.")
	UE_API bool TestAllConditions(const FStateTreeExecutionFrame* CurrentParentFrame, const FStateTreeExecutionFrame& CurrentFrame, const int32 ConditionsOffset, const int32 ConditionsNum);

	/**
	 * Calculate the final score of all considerations at given range.
	 * @return the final score
	 */
	UE_API float EvaluateUtilityWithValidation(const FStateTreeExecutionFrame* CurrentParentFrame, const FStateTreeExecutionFrame& CurrentFrame, FStateTreeStateHandle StateHandle, const FMemoryRequirement& MemoryRequirement, const int32 ConsiderationsBegin, const int32 ConsiderationsNum, const float StateWeight);

	UE_DEPRECATED(5.7, "Use EvaluateUtilityWithValidation. It can get the data from a temporary frame in state selection.")
	UE_API float EvaluateUtility(const FStateTreeExecutionFrame* CurrentParentFrame, const FStateTreeExecutionFrame& CurrentFrame, const int32 ConsiderationsOffset, const int32 ConsiderationsNum, const float StateWeight);

private:
	template<bool bOnActiveInstances>
	void EvaluatePropertyFunctionsInternal(const FStateTreeExecutionFrame* CurrentParentFrame, const FStateTreeExecutionFrame& CurrentFrame, FStateTreeIndex16 FuncsBegin, uint16 FuncsNum);

protected:
	/*
	 * Evaluate all function at given range.
	 * Should be used only on active node instances, assumes valid data handles and does not consider temporary node instances.
	 */
	UE_API void EvaluatePropertyFunctionsOnActiveInstances(const FStateTreeExecutionFrame* CurrentParentFrame, const FStateTreeExecutionFrame& CurrentFrame, FStateTreeIndex16 FuncsBegin, uint16 FuncsNum);

	/*
	 * Evaluate all function at given range.
	 * This version validates the data handles and looks up temporary instances.
	 */
	UE_API void EvaluatePropertyFunctionsWithValidation(const FStateTreeExecutionFrame* CurrentParentFrame, const FStateTreeExecutionFrame& CurrentFrame, FStateTreeIndex16 FuncsBegin, uint16 FuncsNum);

	UE_DEPRECATED(5.7, "Use the RequestTransitionInternal with the active state ID")
	UE_API bool RequestTransition(
		const FStateTreeExecutionFrame& CurrentFrame,
		const FStateTreeStateHandle NextState,
		const EStateTreeTransitionPriority Priority,
		const FStateTreeSharedEvent* TransitionEvent = nullptr,
		const EStateTreeSelectionFallback Fallback = EStateTreeSelectionFallback::None);


	UE_DEPRECATED(5.7, "Use the SetupNextTransition with TransitionArguments")
	UE_API void SetupNextTransition(const FStateTreeExecutionFrame& CurrentFrame, const FStateTreeStateHandle NextState, const EStateTreeTransitionPriority Priority);
	
	/** Optional arguments for a transition request. */
	struct FTransitionArguments
	{
		EStateTreeTransitionPriority Priority = EStateTreeTransitionPriority::Normal;
		FStateTreeSharedEvent TransitionEvent;
		EStateTreeSelectionFallback Fallback = EStateTreeSelectionFallback::None;
		EStateTreeStateSelectionRules SelectionRules = EStateTreeStateSelectionRules::None;
	};

	/** Requests transition to a specified state. */
	UE_API bool RequestTransitionInternal(const FStateTreeExecutionFrame& SourceFrame,
		UE::StateTree::FActiveStateID SourceStateID,
		UE::StateTree::ExecutionContext::FStateHandleContext TargetState,
		const FTransitionArguments& Args);
	
	/** Sets up NextTransition based on the provided parameters and the current execution status. */
	UE_API void SetupNextTransition(const FStateTreeExecutionFrame& SourceFrame,
		UE::StateTree::FActiveStateID SourceStateID,
		UE::StateTree::ExecutionContext::FStateHandleContext TargetState,
		const FTransitionArguments& Args);

	/** Sets up NextTransition based on the provided parameters and the current execution status. */
	UE_API void SetupNextTransition(const FStateTreeExecutionFrame& SourceFrame,
		UE::StateTree::FActiveStateID SourceStateID,
		UE::StateTree::ExecutionContext::FStateHandleContext TargetState,
		const FTransitionArguments& Args,
		FStateTreeTransitionResult& OutTransitionResult);

	/**
	 * Tick (if needed) event, delayed, delegate and, tick state completion transitions.
	 * Test the completed state transitions.
	 * @return if the transition is valid and should be entered.
	 */
	struct FTriggerTransitionsInternalArgs
	{
		TOptional<int32> CompletedSubtreeParentFrameIndex;
	};

	UE_API bool TriggerTransitions(const FTriggerTransitionsInternalArgs Args = FTriggerTransitionsInternalArgs());

	UE_DEPRECATED(5.7, "Use ForceTransition")
	/** Create a new transition result from a recorded transition result. It will fail if the recorded transition is malformed. */
	UE_API TOptional<FStateTreeTransitionResult> MakeTransitionResult(const FRecordedStateTreeTransitionResult& Transition) const;

	UE_DEPRECATED(5.7, "Use MakeRecordedTransitionResult with the FSelectStateResult argument.")
	/** Create a new recorded transition from a transition result. */
	UE_API FRecordedStateTreeTransitionResult MakeRecordedTransitionResult(const FStateTreeTransitionResult& Transition) const;

	/** Create a new recorded transition from a select state result. */
	UE_API FRecordedStateTreeTransitionResult MakeRecordedTransitionResult(const TSharedRef<FSelectStateResult>& Args, const FStateTreeTransitionResult& Transition) const;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.6, "Replaced with FStateTreeTasksCompletionStatus")
	/** Confirms that the frame and state ID are valid and the task index is correct. */
	UE_API bool IsFinishedTaskValid(const UE::StateTree::FFinishedTask& Task) const;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(5.6, "Replaced with FStateTreeTasksCompletionStatus")
	/**
	 * Removes stale entries and fills the completed states list.
	 * The tree can be in a different state from when a finished task was added to the pending list and the finished task may not be valid.
	 * @param bMarkTaskProcessed when true, marks the tasks as processed. When false, only used the already marked tasks.
	 */
	UE_API void UpdateCompletedStateList(bool bMarkTaskProcessed);

	UE_DEPRECATED(5.6, "Replaced with FStateTreeTasksCompletionStatus")
	UE_API void UpdateCompletedStateList();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.6, "Replaced with FStateTreeTasksCompletionStatus")
	/** Adds the state to the completed list from a finished task. */
	UE_API void MarkStateCompleted(UE::StateTree::FFinishedTask& FinisedTask);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(5.6, "Replaced with FStateTreeTasksCompletionStatus")
	/** In the list of completed states, returns the run status when it's completed by a global task. */
	UE_API EStateTreeRunStatus GetGlobalTasksCompletedStatesStatus() const;

	/** Forces transition to a state. It will skip all conditions. */
	UE_API bool ForceTransitionInternal(const TArrayView<const UE::StateTree::ExecutionContext::FStateHandleContext> States, const TSharedRef<FSelectStateResult>& OutSelectionResult);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.7, "Use the SelectState with the FSelectStateArguments argument.")
	UE_API bool SelectState(const FStateTreeExecutionFrame& CurrentFrame, const FStateTreeStateHandle NextState, FStateSelectionResult& OutSelectionResult, const FStateTreeSharedEvent* TransitionEvent = nullptr, const EStateTreeSelectionFallback Fallback = EStateTreeSelectionFallback::None);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** The different type of selections. */
	enum class ESelectStateBehavior : uint8
	{
		/** From a state transition. Normal rules apply. */
		StateTransition,
		/**
		 * From a force transition.
		 * The state's selection behavior and condition will not be respected.
		 * The target is the final state.
		 */
		Forced,
	};
	struct FSelectStateArguments
	{
		/** The current list of active states. */
		TArrayView<const UE::StateTree::FActiveState> ActiveStates;
		/** The state that requested the transition. */
		UE::StateTree::FActiveState SourceState;
		/** The state that we want to select. Depending on the state's selection behavior, another state can be selected. */
		UE::StateTree::ExecutionContext::FStateHandleContext TargetState;
		/** Transition event used by transition to trigger the state selection. */
		FStateTreeSharedEvent TransitionEvent;
		/** Fallback selection behavior to execute if it fails to select the desired state. */
		EStateTreeSelectionFallback Fallback = EStateTreeSelectionFallback::None;
		/** The type of selection. */
		ESelectStateBehavior Behavior = ESelectStateBehavior::StateTransition;
		/** Rule to Selection state rules. */
		EStateTreeStateSelectionRules SelectionRules = EStateTreeStateSelectionRules::None;
	};

	/**
	 * Starting at the specified state, walking towards the leaf states.
	 * @param SelectStateArgs the arguments for SelectState.
	 * @param OutSelectionResult the result of the selection.
	 * @return True if succeeded to select new active states.
	 */
	UE_API bool SelectState(const FSelectStateArguments& SelectStateArgs, const TSharedRef<FSelectStateResult>& OutSelectionResult);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.7, "Use the SelectStateInternal with FSelectStateArguments argument.")
	UE_API bool SelectStateInternal(const FStateTreeExecutionFrame* CurrentParentFrame, FStateTreeExecutionFrame& CurrentFrame, const FStateTreeExecutionFrame* CurrentFrameInActiveFrames, TConstArrayView<FStateTreeStateHandle> PathToNextState, FStateSelectionResult& OutSelectionResult, const FStateTreeSharedEvent* TransitionEvent = nullptr);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	struct FSelectStateInternalArguments
	{
		TArrayView<const UE::StateTree::FActiveState> MissingActiveStates;
		UE::StateTree::FActiveFrameID MissingSourceFrameID;
		TArrayView<const UE::StateTree::FActiveState> MissingSourceStates;
		TArrayView<const UE::StateTree::ExecutionContext::FStateHandleContext> MissingStatesToReachTarget;
	};

	/** Used internally to do the recursive part of the SelectState(). */
	UE_API bool SelectStateInternal(
		const FSelectStateArguments& Args,
		const FSelectStateInternalArguments& InternalArgs,
		const TSharedRef<FSelectStateResult>& OutSelectionResult);

	/** Used internally to select state until we reach the transition source or the transition target. */
	UE_API TOptional<bool> SelectStateFromSourceInternal(
		const FSelectStateArguments& Args,
		const FSelectStateInternalArguments& InternalArgs,
		const TSharedRef<FSelectStateResult>& OutSelectionResult,
		const FStateTreeExecutionFrame& NextFrame,
		const FCompactStateTreeState& NextState,
		FStateTreeStateHandle NextStateHandle,
		bool bNewFrameCreated);

	UE_API bool SelectStateInternal_Linked(
		const FSelectStateArguments& Args,
		const FSelectStateInternalArguments& InternalArgs,
		const TSharedRef<FSelectStateResult>& OutSelectionResult,
		TNotNull<const UStateTree*> NextStateTree,
		const FCompactStateTreeState& NextState,
		bool bShouldCreateNewState
		);

	UE_API bool SelectStateInternal_LinkedAsset(
		const FSelectStateArguments& Args,
		const FSelectStateInternalArguments& InternalArgs,
		const TSharedRef<FSelectStateResult>& OutSelectionResult,
		TNotNull<const UStateTree*> NextStateTree,
		const FCompactStateTreeState& NextState,
		const UStateTree* NextLinkedStateAsset,
		bool bShouldCreateNewState);

	UE_API bool SelectStateInternal_TrySelectChildrenInOrder(
		const FSelectStateArguments& Args,
		const FSelectStateInternalArguments& InternalArgs,
		const TSharedRef<FSelectStateResult>& OutSelectionResult,
		TNotNull<const UStateTree*> NextStateTree,
		const FCompactStateTreeState& TargetState,
		const FStateTreeStateHandle TargetStateHandle);
		
	UE_API bool SelectStateInternal_TrySelectChildrenAtRandom(
		const FSelectStateArguments& Args,
		const FSelectStateInternalArguments& InternalArgs,
		const TSharedRef<FSelectStateResult>& OutSelectionResult,
		TNotNull<const UStateTree*> NextStateTree,
		const FCompactStateTreeState& TargetState,
		const FStateTreeStateHandle TargetStateHandle);

	UE_API bool SelectStateInternal_TrySelectChildrenWithHighestUtility(
		const FSelectStateArguments& Args,
		const FSelectStateInternalArguments& InternalArgs,
		const TSharedRef<FSelectStateResult>& OutSelectionResult,
		TNotNull<const UStateTree*> NextStateTree,
		const FCompactStateTreeState& TargetState,
		const FStateTreeStateHandle TargetStateHandle);

	UE_API bool SelectStateInternal_TrySelectChildrenAtRandomWeightedByUtility(
		const FSelectStateArguments& Args,
		const FSelectStateInternalArguments& InternalArgs,
		const TSharedRef<FSelectStateResult>& OutSelectionResult,
		TNotNull<const UStateTree*> NextStateTree,
		const FCompactStateTreeState& TargetState,
		const FStateTreeStateHandle TargetStateHandle);

	UE_API bool SelectStateInternal_TryFollowTransitions(
		const FSelectStateArguments& Args,
		const FSelectStateInternalArguments& InternalArgs,
		const TSharedRef<FSelectStateResult>& OutSelectionResult,
		TNotNull<const UStateTree*> NextStateTree,
		const FCompactStateTreeState& TargetState,
		const FStateTreeStateHandle TargetStateHandle);

	/** @return StateTree execution state from the instance storage. */
	[[nodiscard]] FStateTreeExecutionState& GetExecState()
	{
		return Storage.GetMutableExecutionState();
	}

	/** @return const StateTree execution state from the instance storage. */
	[[nodiscard]] const FStateTreeExecutionState& GetExecState() const
	{
		return Storage.GetExecutionState();
	}

	/** Updates the update phase of the statetree execution state. */
	UE_API void SetUpdatePhaseInExecutionState(FStateTreeExecutionState& ExecutionState, EStateTreeUpdatePhase UpdatePhase) const;

	/** @return String describing state status for logging and debug. */
	[[nodiscard]] UE_API FString GetStateStatusString(const FStateTreeExecutionState& ExecState) const;

	/** @return String describing state name for logging and debug. */
	[[nodiscard]] UE_API FString GetSafeStateName(const FStateTreeExecutionFrame& CurrentFrame, const FStateTreeStateHandle State) const;

	/** @return String describing state name for logging and debug. */
	[[nodiscard]] UE_API FString GetSafeStateName(const UStateTree* StateTree, const FStateTreeStateHandle State) const;

	/** @return String describing full path of an activate state for logging and debug. */
	[[nodiscard]] UE_API FString DebugGetStatePath(TConstArrayView<FStateTreeExecutionFrame> ActiveFrames, const FStateTreeExecutionFrame* CurrentFrame = nullptr, const int32 ActiveStateIndex = INDEX_NONE) const;

	/** @return String describing all events that are currently being processed  for logging and debug. */
	[[nodiscard]] UE_API FString DebugGetEventsAsString() const;

	/**
	 * Move the data from the temporary buffer to the allocated instance data relative to given frame.
	 * @return instanced data of the specified handle relative to given frame.
	 */
	UE_API FStateTreeDataView MoveTemporaryToInstance(const FStateTreeExecutionFrame& Frame, const FStateTreeIndex16 NodeIndex, const FStateTreeDataHandle Handle);

	/** @return data view of the specified handle relative to given frame. */
	[[nodiscard]] UE_API FStateTreeDataView GetDataView(const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& CurrentFrame, const FStateTreeDataHandle Handle);
	[[nodiscard]] UE_API FStateTreeDataView GetDataView(const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& CurrentFrame, const FPropertyBindingCopyInfo& CopyInfo);

	/** @return data view for the execution runtime view of the current node and frame. */
	[[nodiscard]] UE_API FStateTreeDataView GetExecutionRuntimeDataView() const;

	/** @return true if handle source is valid cified handle relative to given frame. */
	[[nodiscard]] UE_API bool IsHandleSourceValid(const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& CurrentFrame, const FStateTreeDataHandle Handle) const;
	[[nodiscard]] UE_API bool IsHandleSourceValid(const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& CurrentFrame, const FPropertyBindingCopyInfo& CopyInfo) const;
	
	/** @return data view of the specified handle relative to the given frame, or tries to find a matching temporary instance. */
	[[nodiscard]] UE_API FStateTreeDataView GetDataViewOrTemporary(const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& CurrentFrame, const FStateTreeDataHandle Handle);
	[[nodiscard]] UE_API FStateTreeDataView GetDataViewOrTemporary(const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& CurrentFrame, const FPropertyBindingCopyInfo& CopyInfo);

	/** @return data view of the specified handle from temporary instance. */
	[[nodiscard]] UE_API FStateTreeDataView GetTemporaryDataView(const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& CurrentFrame, const FStateTreeDataHandle Handle);

	/**
	 * Adds a temporary instance that can be located using frame and data handle later.
	 * @returns view to the newly added instance. If NewInstanceData is Object wrapper, the new object is returned.
	 */
	[[nodiscard]] UE_API FStateTreeDataView AddTemporaryInstance(const FStateTreeExecutionFrame& Frame, const FStateTreeIndex16 OwnerNodeIndex, const FStateTreeDataHandle DataHandle, FConstStructView NewInstanceData);

	enum class ECopyBindings : uint8
	{
		EnterState,
		Tick,
		ExitState,
	};

	/** Copy the binding on all valid active instances. */
	UE_API void CopyAllBindingsOnActiveInstances(ECopyBindings CopyType);

private:
	template<bool bOnActiveInstances>
	bool CopyBatchInternal(const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& CurrentFrame, const FStateTreeDataView TargetView, const FStateTreeIndex16 BindingsBatch);

protected:
	/**
	 * Copies a batch of properties to the data in TargetView.
	 * Should be used only on active node instances, assumes valid data handles and does not consider temporary node instances.
	 */
	UE_API bool CopyBatchOnActiveInstances(const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& CurrentFrame, const FStateTreeDataView TargetView, const FStateTreeIndex16 BindingsBatch);

	/** Copies a batch of properties to the data in TargetView. This version validates the data handles and looks up temporary instances. */
	UE_API bool CopyBatchWithValidation(const FStateTreeExecutionFrame* ParentFrame, const FStateTreeExecutionFrame& CurrentFrame, const FStateTreeDataView TargetView, const FStateTreeIndex16 BindingsBatch);

	/**
	 * Collects external data for all StateTrees in active frames.
	 * @returns true if all external data are set successfully.
	 */
	UE_API bool CollectActiveExternalData();
	
	/**
	 * Collects external data for all StateTrees in active frames.
	 * @returns true if all external data are set successfully.
	 */
	UE_API bool CollectActiveExternalData(const TArrayView<FStateTreeExecutionFrame> Frames);

	/**
	 * Collects external data for specific State Tree asset. If the data is already collected, cached index is returned.
	 * @returns index in ContextAndExternalDataViews for the first external data.
	 */
	UE_API FStateTreeIndex16 CollectExternalData(const UStateTree* StateTree);

	/**
	 * Stores copy of provided parameters as State Tree global parameters.
	 * @param Parameters parameters to copy
	 * @returns true if successfully set the parameters
	 */
	UE_API bool SetGlobalParameters(FConstStructView Parameters);

	UE_DEPRECATED(5.8, "Use the version with a FConstStructView")
	UE_API bool SetGlobalParameters(const FInstancedPropertyBag& Parameters);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.7, "Use the CaptureNewStateEvents with the FSelectStateResult argument.")
	void CaptureNewStateEvents(TConstArrayView<FStateTreeExecutionFrame> PrevFrames, TConstArrayView<FStateTreeExecutionFrame> NewFrames, TArrayView<FStateTreeFrameStateSelectionEvents> FramesStateSelectionEvents);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Captures StateTree events used during the state selection.
	 */
	UE_API void CaptureNewStateEvents(const TSharedRef<const FSelectStateResult>& Args);

	/** @return a weak reference for a task that can be stored for later uses. */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.6, "FStateTreeWeakTaskRef is no longer used.")
	UE_API FStateTreeWeakTaskRef MakeWeakTaskRefInternal() const;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Instance data used during current tick. */
	FStateTreeInstanceData& InstanceData;

	UE_DEPRECATED(5.6, "Use Storage instead.")
	/** Data storage of the instance data, cached for less indirections. */
	FStateTreeInstanceStorage* InstanceDataStorage = nullptr;

	/** Events queue to use, cached for less indirections. */
	TSharedPtr<FStateTreeEventQueue> EventQueue;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.6, "Use the LinkedStateTreeOverrides copy")
	/** Pointer to linked state tree overrides. */
	const FStateTreeReferenceOverrides* LinkedStateTreeOverrides = nullptr;
#endif
	/** Current linked state tree overrides. */
	FStateTreeReferenceOverrides LinkedAssetStateTreeOverrides;
	
	/** Data view of the context data. */
	TArray<FStateTreeDataView> ContextAndExternalDataViews;

	FOnCollectStateTreeExternalData CollectExternalDataDelegate;

	struct FCollectedExternalDataCache
	{
		const UStateTree* StateTree = nullptr;
		FStateTreeIndex16 BaseIndex;
	};
	TArray<FCollectedExternalDataCache> CollectedExternalCache;
	bool bActiveExternalDataCollected = false;

	/** Holds the container. Multiple instances with the same state tree can occur in a recursive call. */
	struct FEvaluationScopeDataCache
	{
		UE::StateTree::InstanceData::FEvaluationScopeInstanceContainer* Container;
		TObjectKey<UStateTree> StateTree;
	};

	static constexpr int32 ExpectedEvaluationScopeCacheLength = 4;
	/** Data view of the evaluation scope instance data. */
	TArray<FEvaluationScopeDataCache, TInlineAllocator<ExpectedEvaluationScopeCacheLength>> EvaluationScopeInstanceCaches;

	UE_API void PushEvaluationScopeInstanceContainer(UE::StateTree::InstanceData::FEvaluationScopeInstanceContainer& Container, const FStateTreeExecutionFrame& Frame);
	UE_API void PopEvaluationScopeInstanceContainer(UE::StateTree::InstanceData::FEvaluationScopeInstanceContainer& Container);

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.6, "CompletedStateRunStatus is not used.")
	/** Run status set in FinishTask and that used after the task ticks. */
	TOptional<EStateTreeRunStatus> CompletedStateRunStatus;

	UE_DEPRECATED(5.7, "Use RequestedTransition instead.")
	/** Next transition, used by RequestTransition(). */
	FStateTreeTransitionResult NextTransition;

	UE_DEPRECATED(5.7, "Use RequestedTransition instead.")
	/** Structure describing the origin of the state transition that caused the state change. */
	FStateTreeTransitionSource NextTransitionSource;

	UE_DEPRECATED(5.8, "No longer used.")
	TOptional<int32> TriggerTransitionsFromFrameIndex;
#endif 

	/** Current selected transition. */
	TUniquePtr<FRequestTransitionResult> RequestedTransition;

	/** Current broadcasted delegates we're processing. */
	TArray<FStateTreeDelegateDispatcher> CurrentlyProcessedBroadcastedDelegates;

	/** Current frame we're processing. */
	const FStateTreeExecutionFrame* CurrentlyProcessedParentFrame = nullptr; 
	const FStateTreeExecutionFrame* CurrentlyProcessedFrame = nullptr; 

	/** Pointer to the shared instance data of the current frame we're processing. */
	FStateTreeInstanceStorage* CurrentlyProcessedSharedInstanceStorage = nullptr;

	/** Helper struct to track currently processed frame. */
	struct FCurrentlyProcessedFrameScope
	{
		FCurrentlyProcessedFrameScope(FStateTreeExecutionContext& InContext, const FStateTreeExecutionFrame* CurrentParentFrame, const FStateTreeExecutionFrame& CurrentFrame);
		FCurrentlyProcessedFrameScope(const FCurrentlyProcessedFrameScope&) = delete;
		FCurrentlyProcessedFrameScope& operator=(const FCurrentlyProcessedFrameScope&) = delete;
		~FCurrentlyProcessedFrameScope();

	private:
		FStateTreeExecutionContext& Context;
		int32 SavedFrameIndex = 0;
		FStateTreeInstanceStorage* SavedSharedInstanceDataStorage = nullptr;
		const FStateTreeExecutionFrame* SavedFrame = nullptr;
		const FStateTreeExecutionFrame* SavedParentFrame = nullptr;
	};

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.7, "The selection is now identified by the state ID.")
	/** Current state selection result when performing recursive state selection, or nullptr if not applicable. */
	const FStateSelectionResult* CurrentSelectionResult = nullptr;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	/** Current state we're processing, or invalid if not applicable. */
	FStateTreeStateHandle CurrentlyProcessedState;

	/** Helper struct to track currently processed state. */
	struct FCurrentlyProcessedStateScope
	{
		FCurrentlyProcessedStateScope(FStateTreeExecutionContext& InContext, const FStateTreeStateHandle State)
			: Context(InContext)
		{
			SavedState = Context.CurrentlyProcessedState;
			Context.CurrentlyProcessedState = State;
		}
		FCurrentlyProcessedStateScope(const FCurrentlyProcessedStateScope&) = delete;
		FCurrentlyProcessedStateScope& operator=(const FCurrentlyProcessedStateScope&) = delete;
		~FCurrentlyProcessedStateScope()
		{
			Context.CurrentlyProcessedState = SavedState;
		}

	private:
		FStateTreeExecutionContext& Context;
		FStateTreeStateHandle SavedState = FStateTreeStateHandle::Invalid;
	};

	/** Current event we're processing in transition, or invalid if not applicable. */
	const FStateTreeEvent* CurrentlyProcessedTransitionEvent = nullptr;

	/** Helper struct to track currently processed transition event. */
	struct FCurrentlyProcessedTransitionEventScope
	{
		FCurrentlyProcessedTransitionEventScope(FStateTreeExecutionContext& InContext, const FStateTreeEvent* Event)
			: Context(InContext)
		{
			check(Context.CurrentlyProcessedTransitionEvent == nullptr);
			Context.CurrentlyProcessedTransitionEvent = Event;
		}

		~FCurrentlyProcessedTransitionEventScope()
		{
			Context.CurrentlyProcessedTransitionEvent = nullptr;
		}

	private:
		FStateTreeExecutionContext& Context;
	};

	/**
	 * Current select state result we're processing during the state selection, or invalid if not applicable.
	 * Used by the GetDataView when accessing event data.
	 * @note used only during selection and will always point to CurrentlyProcessedTemporaryStorage.
	 */
	FSelectStateResult* CurrentlyProcessedStateSelectionResult  = nullptr;

	/** Current temporary storage for instance data, frames and state. Valid during the state selection, ExitState and EnterState. */
	TSharedPtr<FSelectStateResult> CurrentlyProcessedTemporaryStorage;

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.7, "FStateTreeFrameStateSelectionEvents is now unused. Use CurrentlyProcessedStateSelectionResult")
	FStateTreeFrameStateSelectionEvents* CurrentlyProcessedStateSelectionEvents = nullptr;

	struct
	UE_DEPRECATED(5.7, "FStateTreeFrameStateSelectionEvents is now unused. Use CurrentlyProcessedStateSelectionResult")
	FCurrentFrameStateSelectionEventsScope
	{
		FCurrentFrameStateSelectionEventsScope(FStateTreeExecutionContext& InContext, FStateTreeFrameStateSelectionEvents& InCurrentlyProcessedStateSelectionEvents)
			: Context(InContext)
		{
			SavedStateSelectionEvents = Context.CurrentlyProcessedStateSelectionEvents; 
			Context.CurrentlyProcessedStateSelectionEvents = &InCurrentlyProcessedStateSelectionEvents;
		}

		~FCurrentFrameStateSelectionEventsScope()
		{
			Context.CurrentlyProcessedStateSelectionEvents = SavedStateSelectionEvents;
		}

	private:
		FStateTreeExecutionContext& Context;
		FStateTreeFrameStateSelectionEvents* SavedStateSelectionEvents = nullptr;
	};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	/** True if transitions are allowed to be requested directly instead of buffering. */
	bool bAllowDirectTransitions = false;

	/** Helper struct to track when it is allowed to request transitions. */
	struct FAllowDirectTransitionsScope
	{
		FAllowDirectTransitionsScope(FStateTreeExecutionContext& InContext)
			: Context(InContext)
		{
			bSavedAllowDirectTransitions = Context.bAllowDirectTransitions; 
			Context.bAllowDirectTransitions = true;
		}

		~FAllowDirectTransitionsScope()
		{
			Context.bAllowDirectTransitions = bSavedAllowDirectTransitions;
		}

	private:
		FStateTreeExecutionContext& Context;
		bool bSavedAllowDirectTransitions = false;
	};

	/** Currently processed nodes instance data. Ideally we would pass these to the nodes directly, but do not want to change the API currently. */
	const FStateTreeNodeBase* CurrentNode = nullptr;
	int32 CurrentNodeIndex = FStateTreeIndex16::InvalidValue;
	FStateTreeDataHandle CurrentNodeDataHandle;
	FStateTreeDataView CurrentNodeInstanceData;

	/** Helper struct to set current node data. */
	struct FNodeInstanceDataScope
	{
		FNodeInstanceDataScope(FStateTreeExecutionContext& InContext, const FStateTreeNodeBase* InNode, const int32 InNodeIndex, const FStateTreeDataHandle InNodeDataHandle, const FStateTreeDataView InNodeInstanceData);
		~FNodeInstanceDataScope();

	private:
		FStateTreeExecutionContext& Context;
		const FStateTreeNodeBase* SavedNode = nullptr;
		int32 SavedNodeIndex = FStateTreeIndex16::InvalidValue;
		FStateTreeDataHandle SavedNodeDataHandle;
		FStateTreeDataView SavedNodeInstanceData;
	};

	/** If true, the state tree context will create snapshots of transition events and capture them within RecordedTransitions for later use. */
	bool bRecordTransitions = false;

	/** Captured snapshots for transition results that can be used to recreate transitions. This array is only populated if bRecordTransitions is true. */
	TArray<FRecordedStateTreeTransitionResult> RecordedTransitions;

	/** Memory mapping structure used for redirecting property-bag copies to external (raw) memory pointers */
	const FExternalGlobalParameters* ExternalGlobalParameters = nullptr;
};

/**
 * The const version of a StateTree Execution Context that prevents using the FStateTreeInstanceData with non-const member function.
 */
struct FConstStateTreeExecutionContextView
{
public:
	FConstStateTreeExecutionContextView(UObject& InOwner, const UStateTree& InStateTree, const FStateTreeInstanceData& InInstanceData)
		: ExecutionContext(InOwner, InStateTree, const_cast<FStateTreeInstanceData&>(InInstanceData))
	{}

	operator const FStateTreeExecutionContext& ()
	{
		return ExecutionContext;
	}

	const FStateTreeExecutionContext& Get() const
	{
		return ExecutionContext;
	}

private:
	FConstStateTreeExecutionContextView(const FConstStateTreeExecutionContextView&) = delete;
	FConstStateTreeExecutionContextView& operator=(const FConstStateTreeExecutionContextView&) = delete;

private:
	FStateTreeExecutionContext ExecutionContext;
};

#undef UE_API
