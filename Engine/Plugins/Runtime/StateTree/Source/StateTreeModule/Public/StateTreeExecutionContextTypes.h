// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeExecutionTypes.h"
#include "StateTreeStatePath.h"
#include "Misc/NotNull.h"

#define UE_API STATETREEMODULE_API

class UStateTree;

namespace UE::StateTree::ExecutionContext
{
	/** Helper that combines the state handle and its tree asset. */
	struct FStateHandleContext
	{
		FStateHandleContext() = default;
		FStateHandleContext(TNotNull<const UStateTree*> InStateTree, FStateTreeStateHandle InStateHandle)
			: StateTree(InStateTree)
			, StateHandle(InStateHandle)
		{
		}
		
		bool operator==(const FStateHandleContext&) const = default;
		
		const UStateTree* StateTree = nullptr;
		FStateTreeStateHandle StateHandle;
	};

	/** Interface of structure that can store temporary frames and states. */
	struct ITemporaryStorage
	{
		virtual ~ITemporaryStorage() = default;

		struct FFrameAndParent
		{
			FStateTreeExecutionFrame* Frame = nullptr;
			UE::StateTree::FActiveFrameID ParentFrameID;
		};

		virtual FFrameAndParent GetExecutionFrame(UE::StateTree::FActiveFrameID ID) = 0;
		virtual UE::StateTree::FActiveState GetStateHandle(UE::StateTree::FActiveStateID ID) const = 0;
		virtual TConstArrayView<FStateTreeExecutionFrame> GetTemporaryFrames() const = 0;
	};

	UE_API const FStateTreeExecutionFrame* FindExecutionFrame(FActiveFrameID InFrameID, TConstArrayView<FStateTreeExecutionFrame> InActiveFrames, TConstArrayView<FStateTreeExecutionFrame> InTemporaryFrames);
} // UE::StateTree::ExecutionContext

#undef UE_API
