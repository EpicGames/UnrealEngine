// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeExecutionExtension.generated.h"

#define UE_API STATETREEMODULE_API

class UObject;
class UStateTree;
struct FStateTreeInstanceStorage;
struct FStateTreeTransitionDelayedState;
struct FStateTreeReferenceOverrides;
struct FStateTreeTransitionResult;

namespace UE::StateTree
{
	enum class ETickReason : uint8;
}

/** Used by the execution context or a weak execution context to extend their functionalities. */
USTRUCT()
struct FStateTreeExecutionExtension
{
	GENERATED_BODY()

	struct FContextParameters
	{
		FContextParameters(UObject& Owner, const UStateTree& StateTree, FStateTreeInstanceStorage& InstanceData)
			: Owner(Owner)
			, StateTree(StateTree)
			, InstanceData(InstanceData)
		{}
		FContextParameters(TNotNull<UObject*> Owner, TNotNull<const UStateTree*> StateTree, FStateTreeInstanceStorage& InstanceData)
			: Owner(*Owner)
			, StateTree(*StateTree)
			, InstanceData(InstanceData)
		{
		}
		UObject& Owner;
		const UStateTree& StateTree;
		FStateTreeInstanceStorage& InstanceData;
	};

	virtual ~FStateTreeExecutionExtension() = default;

	/** Prefix that will be used by STATETREE_LOG and STATETREE_CLOG, using Entity description. */
	virtual FString GetInstanceDescription(const FContextParameters& Context) const
	{
		return Context.Owner.GetName();
	}

	struct FNextTickArguments
	{
		UE_API FNextTickArguments();
		UE_API explicit FNextTickArguments(UE::StateTree::ETickReason Reason);

		UE::StateTree::ETickReason Reason;
	};

	/** Callback when the execution context request the tree to wakeup from a schedule tick sleep. */
	virtual void ScheduleNextTick(const FContextParameters& Context, const FNextTickArguments& Args)
	{
		
	}

	UE_DEPRECATED(5.7, "Use ScheduleNextTick with the FNextTickArguments parameter.")
	virtual void ScheduleNextTick(const FContextParameters& Context) final {}

	/** Callback when the overrides are set to the execution context . */
	virtual void OnLinkedStateTreeOverridesSet(const FContextParameters& Context, const FStateTreeReferenceOverrides& Overrides)
	{
		
	}

	/** Callback before the execution context applies a transition. */
	virtual void OnBeginApplyTransition(const FContextParameters& Context, const FStateTreeTransitionResult& TransitionResult)
	{
		
	}
};

#undef UE_API
