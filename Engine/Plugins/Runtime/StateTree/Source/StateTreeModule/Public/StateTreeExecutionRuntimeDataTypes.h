// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeInstanceContainer.h"
#include "StateTreeTypes.h"
#include "UObject/ObjectKey.h"
#include "StateTreeExecutionRuntimeDataTypes.generated.h"

#define UE_API STATETREEMODULE_API

class UStateTree;

namespace UE::StateTree::InstanceData
{
	/** Helper structure that holds the execution runtime instances. */
	USTRUCT()
	struct FExecutionRuntimeData
	{
		GENERATED_BODY()

		/** The state tree of the instances. */
		TObjectKey<UStateTree> StateTree;

		UPROPERTY()
		UE::StateTree::InstanceData::FInstanceContainer Instances;
	};
}

#undef UE_API
