// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "StateTreeTypes.h"
#include "StateTreeDelegate.h"
#include "StateTreeEvents.h"
#include "StateTreeExecutionExtension.h"
#include "StateTreeStatePath.h"
#include "StateTreeTasksStatus.h"
#include "UObject/ObjectKey.h"

#include "StateTreeExecutionTypes.generated.h"

#define UE_API STATETREEMODULE_API

struct FStateTreeExecutionContext;
class UStateTree;
struct FStateTreeInstanceStorage;

/**
 * Enumeration for the different update phases.
 * This is used as context information when tracing debug events.
 */
UENUM()
enum class EStateTreeUpdatePhase : uint8
{
	Unset							= 0,
	StartTree						UMETA(DisplayName = "Start Tree"),
	StopTree						UMETA(DisplayName = "Stop Tree"),
	StartGlobalTasks				UMETA(DisplayName = "Start Global Tasks & Evaluators"),
	StartGlobalTasksForSelection	UMETA(DisplayName = "Start Global Tasks & Evaluators for selection"),
	StopGlobalTasks					UMETA(DisplayName = "Stop Global Tasks & Evaluators"),
	StopGlobalTasksForSelection		UMETA(DisplayName = "Stop Global Tasks & Evaluators for selection"),
	TickStateTree					UMETA(DisplayName = "Tick State Tree"),
	ApplyTransitions				UMETA(DisplayName = "Transition"),
	TickTransitions					UMETA(DisplayName = "Tick Transitions"),
	TriggerTransitions				UMETA(DisplayName = "Trigger Transitions"),
	TickingGlobalTasks				UMETA(DisplayName = "Tick Global Tasks & Evaluators"),
	TickingTasks					UMETA(DisplayName = "Tick Tasks"),
	TransitionConditions			UMETA(DisplayName = "Transition conditions"),
	StateSelection					UMETA(DisplayName = "Try Enter"),
	TrySelectBehavior				UMETA(DisplayName = "Try Select Behavior"),
	EnterConditions					UMETA(DisplayName = "Enter conditions"),
	EnterStates						UMETA(DisplayName = "Enter States"),
	ExitStates						UMETA(DisplayName = "Exit States"),
	StateCompleted					UMETA(DisplayName = "State(s) Completed"),
	EvaluateUtility					UMETA(DisplayName = "Evaluate Utility")
};


/** Status describing current run status of a State Tree. */
UENUM(BlueprintType)
enum class EStateTreeRunStatus : uint8
{
	/** Tree is still running. */
	Running,
	
	/** The State Tree was requested to stop without a particular success or failure state. */
	Stopped,
	
	/** Tree execution has stopped on success. */
	Succeeded,

	/** Tree execution has stopped on failure. */
	Failed,

	/** Status not set. */
	Unset,
};


/** Status describing how a task finished. */
UENUM()
enum class EStateTreeFinishTaskType : uint8
{	
	/** The task execution failed. */
	Failed,
	
	/** The task execution succeed. */
	Succeeded
};


/** State change type. Passed to EnterState() and ExitState() to indicate how the state change affects the state and Evaluator or Task is on. */
UENUM()
enum class EStateTreeStateChangeType : uint8
{
	/** Not an activation */
	None,
	
	/** The state became activated or deactivated. */
	Changed,
	
	/** The state is parent of new active state and sustained previous active state. */
	Sustained,
};


/** Defines how to assign the result of a condition to evaluate.  */
UENUM()
enum class EStateTreeConditionEvaluationMode : uint8
{
	/** Condition is evaluated to set the result. This is the normal behavior. */
	Evaluated,
	
	/** Do not evaluate the condition and force result to 'true'. */
	ForcedTrue,
	
	/** Do not evaluate the condition and force result to 'false'. */
	ForcedFalse,
};


/**
 * Handle to access an external struct or object.
 * Note: Use the templated version below. 
 */
USTRUCT()
struct FStateTreeExternalDataHandle
{
	GENERATED_BODY()

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FStateTreeExternalDataHandle() = default;
	FStateTreeExternalDataHandle(const FStateTreeExternalDataHandle& Other) = default;
	FStateTreeExternalDataHandle(FStateTreeExternalDataHandle&& Other) = default;
	FStateTreeExternalDataHandle& operator=(FStateTreeExternalDataHandle const& Other) = default;
	FStateTreeExternalDataHandle& operator=(FStateTreeExternalDataHandle&& Other) = default;
	UE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	static const FStateTreeExternalDataHandle Invalid;
	
	bool IsValid() const { return DataHandle.IsValid(); }

	UPROPERTY()
	FStateTreeDataHandle DataHandle = FStateTreeDataHandle::Invalid;
};

/**
 * Handle to access an external struct or object.
 * This reference handle can be used in StateTree tasks and evaluators to have quick access to external data.
 * The type provided to the template is used by the linker and context to pass along the type.
 *
 * USTRUCT()
 * struct FExampleTask : public FStateTreeTaskBase
 * {
 *    ...
 *
 *    bool Link(FStateTreeLinker& Linker)
 *    {
 *      Linker.LinkExternalData(ExampleSubsystemHandle);
 *      return true;
 *    }
 * 
 *    EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const EStateTreeStateChangeType ChangeType, const FStateTreeTransitionResult& Transition)
 *    {
 *      const UExampleSubsystem& ExampleSubsystem = Context.GetExternalData(ExampleSubsystemHandle);
 *      ...
 *    }
 *
 *    TStateTreeExternalDataHandle<UExampleSubsystem> ExampleSubsystemHandle;
 * }
 */
template<typename T, EStateTreeExternalDataRequirement Req = EStateTreeExternalDataRequirement::Required>
struct TStateTreeExternalDataHandle : FStateTreeExternalDataHandle
{
	typedef T DataType;
	static constexpr EStateTreeExternalDataRequirement DataRequirement = Req;
};


/**
 * Describes an external data. The data can point to a struct or object.
 * The code that handles StateTree ticking is responsible for passing in the actually data, see FStateTreeExecutionContext.
 */
USTRUCT()
struct FStateTreeExternalDataDesc
{
	GENERATED_BODY()

	FStateTreeExternalDataDesc() = default;
	FStateTreeExternalDataDesc(const UStruct* InStruct, const EStateTreeExternalDataRequirement InRequirement) : Struct(InStruct), Requirement(InRequirement) {}

	FStateTreeExternalDataDesc(const FName InName, const UStruct* InStruct, const FGuid InGuid)
		: Struct(InStruct)
		, Name(InName)
#if WITH_EDITORONLY_DATA
		, ID(InGuid)
#endif
	{}

	/** @return true if the DataView is compatible with the descriptor. */
	bool IsCompatibleWith(const FStateTreeDataView& DataView) const
	{
		if (DataView.GetStruct()->IsChildOf(Struct))
		{
			return true;
		}
		
		if (const UClass* DataDescClass = Cast<UClass>(Struct))
		{
			if (const UClass* DataViewClass = Cast<UClass>(DataView.GetStruct()))
			{
				return DataViewClass->ImplementsInterface(DataDescClass);
			}
		}
		
		return false;
	}
	
	bool operator==(const FStateTreeExternalDataDesc& Other) const
	{
		return Struct == Other.Struct && Requirement == Other.Requirement;
	}
	
	/** Class or struct of the external data. */
	UPROPERTY();
	TObjectPtr<const UStruct> Struct = nullptr;

	/**
	 * Name of the external data. Used only for bindable external data (enforced by the schema).
	 * External data linked explicitly by the nodes (i.e. LinkExternalData) are identified only
	 * by their type since they are used for unique instance of a given type.  
	 */
	UPROPERTY(VisibleAnywhere, Category = Common)
	FName Name;
	
	/** Handle/Index to the StateTreeExecutionContext data views array */
	UPROPERTY();
	FStateTreeExternalDataHandle Handle;

	/** Describes if the data is required or not. */
	UPROPERTY();
	EStateTreeExternalDataRequirement Requirement = EStateTreeExternalDataRequirement::Required;

#if WITH_EDITORONLY_DATA
	/** Unique identifier. Used only for bindable external data. */
	UPROPERTY()
	FGuid ID;
#endif
};


/** Transition request */
USTRUCT(BlueprintType)
struct FStateTreeTransitionRequest
{
	GENERATED_BODY()

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FStateTreeTransitionRequest() = default;

	explicit FStateTreeTransitionRequest(const FStateTreeStateLink& InStateLink)
		: TargetState(InStateLink.StateHandle)
		, Fallback(InStateLink.Fallback)
	{
	}

	explicit FStateTreeTransitionRequest(
		const FStateTreeStateHandle InTargetState, 
		const EStateTreeTransitionPriority InPriority = EStateTreeTransitionPriority::Normal,
		const EStateTreeSelectionFallback InFallback = EStateTreeSelectionFallback::None)
		: TargetState(InTargetState)
		, Priority(InPriority)
		, Fallback(InFallback)
	{
	}

	FStateTreeTransitionRequest(const FStateTreeTransitionRequest&) = default;
	FStateTreeTransitionRequest(FStateTreeTransitionRequest&&) = default;
	FStateTreeTransitionRequest& operator=(const FStateTreeTransitionRequest&) = default;
	FStateTreeTransitionRequest& operator=(FStateTreeTransitionRequest&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Target state of the transition. */
	UPROPERTY(EditDefaultsOnly, Category = "Default")
	FStateTreeStateHandle TargetState;
	
	/** Priority of the transition. */
	UPROPERTY(EditDefaultsOnly, Category = "Default")
	EStateTreeTransitionPriority Priority = EStateTreeTransitionPriority::Normal;

	/** Fallback of the transition if it fails to select the target state */
	UPROPERTY(EditDefaultsOnly, Category = "Default")
	EStateTreeSelectionFallback Fallback = EStateTreeSelectionFallback::None;

	/** StateTree frame that was active when the transition was requested. Filled in by the StateTree execution context. */
	UE::StateTree::FActiveFrameID SourceFrameID;
	/** StateTree state that was active when the transition was requested. Filled in by the StateTree execution context. */
	UE::StateTree::FActiveStateID SourceStateID;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.6, "Use SourceFrameID to uniquely identify a frame.")
	/** StateTree asset that was active when the transition was requested. Filled in by the StateTree execution context. */
	UPROPERTY()
	TObjectPtr<const UStateTree> SourceStateTree = nullptr;

	UE_DEPRECATED(5.6, "Use SourceFrameID to uniquely identify a frame.")
	/** Root state the execution frame where the transition was requested. Filled in by the StateTree execution context. */
	UPROPERTY()
	FStateTreeStateHandle SourceRootState = FStateTreeStateHandle::Invalid;

	UE_DEPRECATED(5.6, "Use SourceStateID to uniquely identify a state.")
	/** Source state of the transition. Filled in by the StateTree execution context. */
	UPROPERTY()
	FStateTreeStateHandle SourceState;
#endif
};


/**
 * Describes an array of active states in a State Tree.
 */
USTRUCT(BlueprintType)
struct FStateTreeActiveStates
{
	GENERATED_BODY()

	/** Max number of active states. */
	static constexpr uint8 MaxStates = 8;

	FStateTreeActiveStates() = default;
	
	UE_DEPRECATED(5.6, "Use the constructor with FActiveStateId.")
	explicit FStateTreeActiveStates(const FStateTreeStateHandle StateHandle)
	{
		Push(StateHandle, UE::StateTree::FActiveStateID::Invalid);
	}
	explicit FStateTreeActiveStates(const FStateTreeStateHandle StateHandle, UE::StateTree::FActiveStateID StateID)
	{
		Push(StateHandle, StateID);
	}
	
	/** Resets the active state array to empty. */
	void Reset()
	{
		NumStates = 0;
	}

	UE_DEPRECATED(5.6, "Use the Push override with the FActiveStateId.")
	/** Pushes new state at the back of the array and returns true if there was enough space. */
	bool Push(const FStateTreeStateHandle StateHandle)
	{
		return Push(StateHandle, UE::StateTree::FActiveStateID::Invalid);
	}

	/** Pushes new state at the back of the array and returns true if there was enough space. */
	bool Push(const FStateTreeStateHandle StateHandle, UE::StateTree::FActiveStateID StateID)
	{
		if ((NumStates + 1) > MaxStates)
		{
			return false;
		}

		States[NumStates] = StateHandle;
		StateIDs[NumStates] = StateID;
		++NumStates;
		
		return true;
	}

	UE_DEPRECATED(5.6, "Use PushFront override with FActiveStateId.")
	/** Pushes new state at the front of the array and returns true if there was enough space. */
	bool PushFront(const FStateTreeStateHandle StateHandle)
	{
		return PushFront(StateHandle, UE::StateTree::FActiveStateID::Invalid);
	}

	/** Pushes new state at the front of the array and returns true if there was enough space. */
	bool PushFront(const FStateTreeStateHandle StateHandle, UE::StateTree::FActiveStateID StateID)
	{
		if ((NumStates + 1) > MaxStates)
		{
			return false;
		}

		NumStates++;
		for (int32 Index = (int32)NumStates - 1; Index > 0; Index--)
		{
			States[Index] = States[Index - 1];
			StateIDs[Index] = StateIDs[Index - 1];
		}
		States[0] = StateHandle;
		StateIDs[0] = StateID;
		
		return true;
	}

	/** Pops a state from the back of the array and returns the popped value, or invalid handle if the array was empty. */
	FStateTreeStateHandle Pop()
	{
		if (NumStates == 0)
		{
			return FStateTreeStateHandle::Invalid;
		}

		const FStateTreeStateHandle Ret = States[NumStates - 1];
		NumStates--;
		return Ret;
	}

	/** Sets the number of states, new states are set to invalid state. */
	void SetNum(const int32 NewNum)
	{
		check(NewNum >= 0 && NewNum <= MaxStates);
		if (NewNum > (int32)NumStates)
		{
			for (int32 Index = NumStates; Index < NewNum; Index++)
			{
				States[Index] = FStateTreeStateHandle::Invalid;
				StateIDs[Index] = UE::StateTree::FActiveStateID::Invalid;
			}
		}
		NumStates = static_cast<uint8>(NewNum);
	}

	/* Returns the corresponding state handle for the active state ID. */
	FStateTreeStateHandle FindStateHandle(UE::StateTree::FActiveStateID StateId) const
	{
		for (int32 Index = (int32)NumStates - 1; Index >= 0; Index--)
		{
			if (StateIDs[Index] == StateId)
			{
				return States[Index];
			}
		}

		return FStateTreeStateHandle::Invalid;
	}

	/* Returns the corresponding state ID for the active state handle. */
	UE::StateTree::FActiveStateID FindStateID(FStateTreeStateHandle StateHandle) const
	{
		for (int32 Index = (int32)NumStates - 1; Index >= 0; Index--)
		{
			if (States[Index] == StateHandle)
			{
				return StateIDs[Index];
			}
		}

		return UE::StateTree::FActiveStateID::Invalid;
	}

	/** Returns index of a state, searching in reverse order. */
	int32 IndexOfReverse(const FStateTreeStateHandle StateHandle) const
	{
		for (int32 Index = (int32)NumStates - 1; Index >= 0; Index--)
		{
			if (States[Index] == StateHandle)
			{
				return Index;
			}
		}
		return INDEX_NONE;
	}
	
	/** Returns index of a state, searching in reverse order. */
	int32 IndexOfReverse(const UE::StateTree::FActiveStateID StateId) const
	{
		for (int32 Index = (int32)NumStates - 1; Index >= 0; Index--)
		{
			if (StateIDs[Index] == StateId)
			{
				return Index;
			}
		}
		return INDEX_NONE;
	}
	
	/** Returns true of the array contains specified state. */
	bool Contains(const FStateTreeStateHandle StateHandle) const
	{
		for (int32 Index = 0; Index < NumStates; ++Index)
		{
			if (States[Index] == StateHandle)
			{
				return true;
			}
		}
		return false;
	}

	/** Returns true of the array contains specified state within MaxNumStatesToCheck states. */
	bool Contains(const FStateTreeStateHandle StateHandle, const uint8 MaxNumStatesToCheck) const
	{
		const int32 Num = (int32)FMath::Min(NumStates, MaxNumStatesToCheck);
		for (int32 Index = 0; Index < Num; ++Index)
		{
			if (States[Index] == StateHandle)
			{
				return true;
			}
		}
		return false;
	}

	/** Returns true if the state id is inside the container. */
	bool Contains(const UE::StateTree::FActiveStateID StateId) const
	{
		for (int32 Index = (int32)NumStates - 1; Index >= 0; Index--)
		{
			if (StateIDs[Index] == StateId)
			{
				return true;
			}
		}
		return false;
	}
	
	/** Returns last state in the array, or invalid state if the array is empty. */
	FStateTreeStateHandle Last() const
	{
		return NumStates > 0 ? States[NumStates - 1] : FStateTreeStateHandle::Invalid;
	}
	
	/** Returns number of states in the array. */
	int32 Num() const
	{
		return NumStates;
	}

	/** Returns true if the index is within array bounds. */
	bool IsValidIndex(const int32 Index) const
	{
		return Index >= 0 && Index < (int32)NumStates;
	}
	
	/** Returns true if the array is empty. */
	bool IsEmpty() const
	{
		return NumStates == 0;
	}

	/** Returns a specified state in the array. */
	inline const FStateTreeStateHandle& operator[](const int32 Index) const
	{
		check(Index >= 0 && Index < (int32)NumStates);
		return States[Index];
	}

	/** Returns mutable reference to a specified state in the array. */
	inline FStateTreeStateHandle& operator[](const int32 Index)
	{
		check(Index >= 0 && Index < (int32)NumStates);
		return States[Index];
	}

	/** Returns the active states from the States array. */
	operator TArrayView<FStateTreeStateHandle>()
	{
		return TArrayView<FStateTreeStateHandle>(&States[0], Num());
	}

	/** Returns the active states from the States array. */
	operator TConstArrayView<FStateTreeStateHandle>() const
	{
		return TConstArrayView<FStateTreeStateHandle>(&States[0], Num());
	}

	/** Returns a specified state in the array, or FStateTreeStateHandle::Invalid if Index is out of array bounds. */
	FStateTreeStateHandle GetStateSafe(const int32 Index) const
	{
		return (Index >= 0 && Index < (int32)NumStates) ? States[Index] : FStateTreeStateHandle::Invalid;
	}

	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 */
	inline FStateTreeStateHandle* begin() { return &States[0]; }
	inline FStateTreeStateHandle* end  () { return &States[0] + Num(); }
	inline const FStateTreeStateHandle* begin() const { return &States[0]; }
	inline const FStateTreeStateHandle* end  () const { return &States[0] + Num(); }

	UE::StateTree::FActiveStateID StateIDs[MaxStates];

	UPROPERTY(VisibleDefaultsOnly, Category = Default)
	FStateTreeStateHandle States[MaxStates];

	UPROPERTY(VisibleDefaultsOnly, Category = Default)
	uint8 NumStates = 0;
};


UENUM()
enum class EStateTreeTransitionSourceType : uint8
{
	Unset,
	Asset,
	ExternalRequest,
	Internal
};

/**
 * Describes the origin of an applied transition.
 */
USTRUCT()
struct FStateTreeTransitionSource
{
	GENERATED_BODY()

	FStateTreeTransitionSource() = default;

	UE_DEPRECATED(5.6, "Use the version that requires a pointer to the StateTree asset.")
	explicit FStateTreeTransitionSource(const EStateTreeTransitionSourceType SourceType, const FStateTreeIndex16 TransitionIndex, const FStateTreeStateHandle TargetState, const EStateTreeTransitionPriority Priority)
	: FStateTreeTransitionSource(nullptr, SourceType, TransitionIndex, TargetState, Priority)
	{
	}

	UE_DEPRECATED(5.6, "Use the version that requires a pointer to the StateTree asset.")
	explicit FStateTreeTransitionSource(const FStateTreeIndex16 TransitionIndex, const FStateTreeStateHandle TargetState, const EStateTreeTransitionPriority Priority)
	: FStateTreeTransitionSource(nullptr, EStateTreeTransitionSourceType::Asset, TransitionIndex, TargetState, Priority)
	{
	}

	UE_DEPRECATED(5.6, "Use the version that requires a pointer to the StateTree asset.")
	explicit FStateTreeTransitionSource(const EStateTreeTransitionSourceType SourceType, const FStateTreeStateHandle TargetState, const EStateTreeTransitionPriority Priority)
	: FStateTreeTransitionSource(nullptr, SourceType, FStateTreeIndex16::Invalid, TargetState, Priority)
	{
	}

	UE_API explicit FStateTreeTransitionSource(const UStateTree* StateTree, const EStateTreeTransitionSourceType SourceType, const FStateTreeIndex16 TransitionIndex, const FStateTreeStateHandle TargetState, const EStateTreeTransitionPriority Priority);

	explicit FStateTreeTransitionSource(const UStateTree* StateTree, const FStateTreeIndex16 TransitionIndex, const FStateTreeStateHandle TargetState, const EStateTreeTransitionPriority Priority)
	: FStateTreeTransitionSource(StateTree, EStateTreeTransitionSourceType::Asset, TransitionIndex, TargetState, Priority)
	{
	}

	explicit FStateTreeTransitionSource(const UStateTree* StateTree, const EStateTreeTransitionSourceType SourceType, const FStateTreeStateHandle TargetState, const EStateTreeTransitionPriority Priority)
	: FStateTreeTransitionSource(StateTree, SourceType, FStateTreeIndex16::Invalid, TargetState, Priority)
	{
	}

	void Reset()
	{
		*this = {};
	}

	/** The StateTree asset owning the transition and state. */
	TWeakObjectPtr<const UStateTree> Asset;

	/** Describes where the transition originated. */
	EStateTreeTransitionSourceType SourceType = EStateTreeTransitionSourceType::Unset;

	/* Index of the transition if from predefined asset transitions, invalid otherwise */
	FStateTreeIndex16 TransitionIndex;

	/** Transition target state */
	FStateTreeStateHandle TargetState = FStateTreeStateHandle::Invalid;
	
	/** Priority of the transition that caused the state change. */
	EStateTreeTransitionPriority Priority = EStateTreeTransitionPriority::None;
};


#if WITH_STATETREE_TRACE
struct FStateTreeInstanceDebugId
{
	FStateTreeInstanceDebugId() = default;
	FStateTreeInstanceDebugId(const uint32 InstanceId, const uint32 SerialNumber)
		: Id(InstanceId)
		, SerialNumber(SerialNumber)
	{
	}
	explicit FStateTreeInstanceDebugId(const uint64 Id)
		: Id(static_cast<uint32>(Id >> 32))
		, SerialNumber(static_cast<uint32>(Id))
	{
	}
	
	bool IsValid() const { return Id != INDEX_NONE && SerialNumber != INDEX_NONE; }
	bool IsInvalid() const { return !IsValid(); }
	void Reset() { *this = Invalid; }

	bool operator==(const FStateTreeInstanceDebugId& Other) const
	{
		return Id == Other.Id && SerialNumber == Other.SerialNumber;
	}

	bool operator!=(const FStateTreeInstanceDebugId& Other) const
	{
		return !(*this == Other);
	}

	uint64 ToUint64() const
	{
		return (static_cast<uint64>(Id) << 32) | static_cast<uint64>(SerialNumber);
	}

	friend uint32 GetTypeHash(const FStateTreeInstanceDebugId InstanceDebugId)
	{
		return HashCombine(InstanceDebugId.Id, InstanceDebugId.SerialNumber);
	}

	friend FString LexToString(const FStateTreeInstanceDebugId InstanceDebugId)
	{
		return FString::Printf(TEXT("0x%llx"), InstanceDebugId.ToUint64());
	}

	static UE_API const FStateTreeInstanceDebugId Invalid;
	
	uint32 Id = INDEX_NONE;
	uint32 SerialNumber = INDEX_NONE;
};
#endif // WITH_STATETREE_TRACE

/** Describes current state of a delayed transition. */
USTRUCT()
struct FStateTreeTransitionDelayedState
{
	GENERATED_BODY()

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FStateTreeTransitionDelayedState() = default;
	FStateTreeTransitionDelayedState(const FStateTreeTransitionDelayedState&) = default;
	FStateTreeTransitionDelayedState(FStateTreeTransitionDelayedState&&) = default;
	FStateTreeTransitionDelayedState& operator=(const FStateTreeTransitionDelayedState&) = default;
	FStateTreeTransitionDelayedState& operator=(FStateTreeTransitionDelayedState&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** The state ID that triggers the transition. */
	UE::StateTree::FActiveStateID StateID;

	UE_DEPRECATED(5.6, "StateTree is unused. Use StateID instead.")
	UPROPERTY()
	TObjectPtr<const UStateTree> StateTree = nullptr;

	UPROPERTY()
	FStateTreeSharedEvent CapturedEvent;

	UPROPERTY()
	float TimeLeft = 0.0f;

	UPROPERTY()
	uint32 CapturedEventHash = 0u;

	UE_DEPRECATED(5.6, "StateHandle is unused. Use StateID instead.")
	UPROPERTY()
	FStateTreeStateHandle StateHandle;
	
	UPROPERTY()
	FStateTreeIndex16 TransitionIndex = FStateTreeIndex16::Invalid;
};


namespace UE::StateTree
{
	/** Describes a finished task waiting to be processed by an execution context. */
	struct
	UE_DEPRECATED(5.6, "Replaced with FStateTreeTasksCompletionStatus")
	FFinishedTask
	{
		enum class EReasonType : uint8
		{
			/** A global task finished. The FrameID and TaskIndex are valid. */
			GlobalTask,
			/** A task inside a state finished. The FrameID, StateID and TaskIndex are valid. */
			StateTask,
			/** An internal transition finish the state. The FrameID and StateID are valid. */
			InternalTransition,
		};
		FFinishedTask() = default;
		STATETREEMODULE_API FFinishedTask(FActiveFrameID FrameID, FActiveStateID StateID, FStateTreeIndex16 TaskIndex, EStateTreeRunStatus RunStatus, EReasonType Reason, bool bTickProcessed);
		/** Frame ID that identifies 1the active frame. */
		FActiveFrameID FrameID;
		/** State ID that contains the finished task. */
		FActiveStateID StateID;
		/** Task that is finished and needs to be processed. */
		FStateTreeIndex16 TaskIndex = FStateTreeIndex16::Invalid;
		/** The result of the finished task. */
		EStateTreeRunStatus RunStatus = EStateTreeRunStatus::Failed;
		/** The reason of the finished task. */
		EReasonType Reason = EReasonType::GlobalTask;
		/**
		 * Set to true if the task is completed before or during the TickTasks.
		 * Used to identify tasks that were completed and had the chance to be processed by the algo.
		 * If not processed, they won't trigger the transition in this frame.
		 */
		bool bTickProcessed = false;
	};

	/** Describes the reason behind StateTree ticking. */
	UENUM()
	enum class ETickReason : uint8
	{
		None,
		/** A scheduled tick request is active. */
		ScheduledTickRequest,
		/** The tick is forced. The schema doesn't support scheduling. */
		Forced,
		/** An active state requested a custom tick rate. */
		StateCustomTickRate,
		/** A task in an active state request ticking. */
		TaskTicking,
		/** A transition in an active state request ticking. */
		TransitionTicking,
		/** A transition request occurs via RequestTransition. */
		TransitionRequest,
		/** An event need to be cleared. */
		Event,
		/** A state completed async and transition didn't tick yet. */
		CompletedState,
		/** A active delayed transitions is pending. */
		DelayedTransition,
		/** A broadcast delegate occurs via BroadcastDelegate, and a transition is listening to the delegate. */
		Delegate,
	};
}

/*
 * Information on how a state tree should tick next.
 */
USTRUCT()
struct FStateTreeScheduledTick
{
	GENERATED_BODY()

public:
	FStateTreeScheduledTick() = default;

	bool operator==(const FStateTreeScheduledTick&) const = default;
	bool operator!=(const FStateTreeScheduledTick&) const = default;

	/** Make a scheduled tick that returns Sleep. */
	static UE_API FStateTreeScheduledTick MakeSleep();
	/** Make a scheduled tick that returns EveryFrame. */
	static UE_API FStateTreeScheduledTick MakeEveryFrames(UE::StateTree::ETickReason Reason = UE::StateTree::ETickReason::None);
	/** Make a scheduled tick that returns NextFrame. */
	static UE_API FStateTreeScheduledTick MakeNextFrame(UE::StateTree::ETickReason Reason = UE::StateTree::ETickReason::None);
	/** Make a scheduled tick that returns a tick rate. The value needs to be greater than zero. */
	static UE_API FStateTreeScheduledTick MakeCustomTickRate(float DeltaTime, UE::StateTree::ETickReason Reason = UE::StateTree::ETickReason::None);

public:
	/** @return true if it doesn't need to tick until an event/delegate/transition/... occurs. */
	UE_API bool ShouldSleep() const;
	/** @return true if it the needs to tick every frame. */
	UE_API bool ShouldTickEveryFrames() const;
	/** @return true if it usually doesn't need to tick but it needs to tick once next frame. */
	UE_API bool ShouldTickOnceNextFrame() const;
	/** @return true if it has a custom tick rate. */
	UE_API bool HasCustomTickRate() const;
	/** @return the delay in seconds between ticks. */
	UE_API float GetTickRate() const;

	/** @return the reason why tick is required. */
	UE::StateTree::ETickReason GetReason() const
	{
		return Reason;
	}

private:
	FStateTreeScheduledTick(float DeltaTime, UE::StateTree::ETickReason Reason)
		: NextDeltaTime(DeltaTime)
		, Reason(Reason)
	{}

	UPROPERTY()
	float NextDeltaTime = 0.0f;
	UE::StateTree::ETickReason Reason = UE::StateTree::ETickReason::None;
};


namespace UE::StateTree
{
/*
 * ID of a scheduled tick request.
 */
struct FScheduledTickHandle
{
public:
	FScheduledTickHandle() = default;
	FScheduledTickHandle(const FScheduledTickHandle&) = default;

	STATETREEMODULE_API static FScheduledTickHandle GenerateNewHandle();

	bool IsValid() const
	{
		return Value != 0;
	}

	bool operator==(const FScheduledTickHandle& Other) const = default;
	bool operator!=(const FScheduledTickHandle& Other) const = default;

private:
	explicit FScheduledTickHandle(uint32 InValue)
		: Value(InValue)
	{}

	uint32 Value = 0;
};
} // namespace UE::StateTree


/** Describes added delegate listeners. */
struct FStateTreeDelegateActiveListeners
{
	FStateTreeDelegateActiveListeners() = default;
	STATETREEMODULE_API ~FStateTreeDelegateActiveListeners();

	/** Adds a delegate bound in the editor to the list. Safe to be called during broadcasting.*/
	STATETREEMODULE_API void Add(const FStateTreeDelegateListener& Listener, FSimpleDelegate Delegate, UE::StateTree::FActiveFrameID FrameID, UE::StateTree::FActiveStateID StateID, FStateTreeIndex16 OwningNodeIndex);

	/** Removes a delegate bound in the editor from the list. Safe to be called during broadcasting. */
	STATETREEMODULE_API void Remove(const FStateTreeDelegateListener& Listener);

	/** Removes the listener by predicate. */
	STATETREEMODULE_API void RemoveAll(UE::StateTree::FActiveFrameID FrameID);

	/** Removes the listener that match. */
	STATETREEMODULE_API void RemoveAll(UE::StateTree::FActiveStateID StateID);

	/** Broadcasts the listener by predicate. */
	STATETREEMODULE_API void BroadcastDelegate(const FStateTreeDelegateDispatcher& Dispatcher, const FStateTreeExecutionState& Exec);

private:
	void RemoveUnbounds();

	struct FActiveListener
	{
		FActiveListener() = default;
		FActiveListener(const FStateTreeDelegateListener& Listener, FSimpleDelegate InDelegate, UE::StateTree::FActiveFrameID FrameID, UE::StateTree::FActiveStateID StateID, FStateTreeIndex16 OwningNodeIndex);

		bool IsValid() const
		{
			return Listener.IsValid() && Delegate.IsBound();
		}

		FStateTreeDelegateListener Listener;
		FSimpleDelegate Delegate;
		UE::StateTree::FActiveFrameID FrameID;
		UE::StateTree::FActiveStateID StateID;
		FStateTreeIndex16 OwningNodeIndex = FStateTreeIndex16::Invalid;
	};

	TArray<FActiveListener> Listeners;
	uint32 BroadcastingLockCount : 31 = 0;
	uint32 bContainsUnboundListeners : 1 = false;
};

namespace UE::StateTree
{
	/** Helper that identifies an execution frame with the root state and its tree asset. */
	USTRUCT()
	struct FExecutionFrameHandle
	{
		GENERATED_BODY()

		FExecutionFrameHandle() = default;
		FExecutionFrameHandle(TNotNull<const UStateTree*> InStateTree, FStateTreeStateHandle InRootState)
			: StateTree(InStateTree)
			, RootState(InRootState)
		{
		}

		bool IsValid() const
		{
			return RootState.IsValid() && StateTree != nullptr;
		}

		const UStateTree* GetStateTree() const
		{
			return StateTree;
		}

		FStateTreeStateHandle GetRootState() const
		{
			return RootState;
		}

private:
		UPROPERTY()
		TObjectPtr<const UStateTree> StateTree;

		UPROPERTY()
		FStateTreeStateHandle RootState;
	};
}

/** Describes an active branch of a State Tree. */
USTRUCT(BlueprintType)
struct FStateTreeExecutionFrame
{
	GENERATED_BODY()

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FStateTreeExecutionFrame() = default;
	FStateTreeExecutionFrame(const FStateTreeExecutionFrame&) = default;
	FStateTreeExecutionFrame(FStateTreeExecutionFrame&&) = default;
	FStateTreeExecutionFrame& operator=(const FStateTreeExecutionFrame&) = default;
	FStateTreeExecutionFrame& operator=(FStateTreeExecutionFrame&&) = default;
	UE_DEPRECATED(5.6, "The recorded frame doesn't have all the needed information to properly form an FStateTreeExecutionFrame.")
	UE_API FStateTreeExecutionFrame(const FRecordedStateTreeExecutionFrame& RecordedExecutionFrame);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(5.7, "Use the other version of IsSameRoot.")
	bool IsSameFrame(const FStateTreeExecutionFrame& OtherFrame) const
	{
		return StateTree == OtherFrame.StateTree && RootState == OtherFrame.RootState;
	}

	/** @return wherever the frame uses the state tree and root state. */
	bool HasSameRoot(const FStateTreeExecutionFrame& OtherFrame) const
	{
		return StateTree == OtherFrame.StateTree && RootState == OtherFrame.RootState;
	}

	/** @return wherever the frame uses the state tree and root state. */
	bool HasRoot(const UE::StateTree::FExecutionFrameHandle& FrameHandle) const
	{
		return StateTree == FrameHandle.GetStateTree() && RootState == FrameHandle.GetRootState();
	}

	/** @return wherever the frame uses the state tree and root state. */
	bool HasRoot(TNotNull<const UStateTree*> InStateTree, FStateTreeStateHandle InRootState) const
	{
		return StateTree == InStateTree && RootState == InRootState;
	}
	
	/** The State Tree used by the frame. */
	UPROPERTY()
	TObjectPtr<const UStateTree> StateTree = nullptr;

	/** Flag to track the completion of a global task or a task from a state in the ActiveStates. */
	UPROPERTY(Transient)
	FStateTreeTasksCompletionStatus ActiveTasksStatus;

	/** Active states in this frame */
	UPROPERTY()
	FStateTreeActiveStates ActiveStates;

	/**
	 * Handle to the root state parameter data.
	 * Exists in parent frame if this frame is entered by a linked state,
	 * Otherwise it exists in the current frame.
	 */
	UPROPERTY()
	FStateTreeDataHandle StateParameterDataHandle = FStateTreeDataHandle::Invalid;

	/**
	 * Handle to the global parameter data,
	 * Exists in parent frame if it is a state parameter.
	 * Exists in root frame if it is the root global parameter.
	 */
	UPROPERTY()
	FStateTreeDataHandle GlobalParameterDataHandle = FStateTreeDataHandle::Invalid;

	/** Unique frame ID for this frame. Can be used to identify the frame. */
	UE::StateTree::FActiveFrameID FrameID;

	/** current or parent frame ID that holds global instance data this frame can have access to. */
	UE::StateTree::FActiveFrameID GlobalInstanceDataFrameID;

	/** current or parent frame ID that holds global parameter data this frame can have access to. */
	UE::StateTree::FActiveFrameID GlobalParameterDataFrameID;

	/** current or parent frame ID that holds root state parameter this frame can have access to. */
	UE::StateTree::FActiveFrameID StateParameterDataFrameID;

	/** The root state of the frame (e.g. Root state or a subtree). */
	UPROPERTY()
	FStateTreeStateHandle RootState = FStateTreeStateHandle::Root;

	/**
	 * The evaluator or task node index that was "entered".
	 * Used during the Enter and Exit phase. A node can fail EnterState.
	 * Nodes after ActiveNodeIndex do not receive ExitState, because they didn't receive EnterState.
	 */
	FStateTreeIndex16 ActiveNodeIndex = FStateTreeIndex16::Invalid;

	/** First index of the external data for this frame. */
	UPROPERTY()
	FStateTreeIndex16 ExternalDataBaseIndex = FStateTreeIndex16::Invalid;

	/** Index within the instance data to the first global instance data (e.g. global tasks) */
	UPROPERTY()
	FStateTreeIndex16 GlobalInstanceIndexBase = FStateTreeIndex16::Invalid;

	/**
	 * Index within the instance data that GlobalParameterDataHandle is relative to,
	 * if it is state parameter it lives within the parent frame
	 * Invalid means GlobalParameterDataHandle points to the root global parameter.
	 */
	UPROPERTY()
	FStateTreeIndex16 GlobalParameterIndexBase = FStateTreeIndex16::Invalid;

	/**
	 * Index within the instance data that StateParameterDataHandle is relative to,
	 * It could be the current frame or parent frame's active instance index base, depending on if the current frame is a linked subtree.
	 * Always valid.
	 */
	UPROPERTY()
	FStateTreeIndex16 StateParameterIndexBase = FStateTreeIndex16::Invalid;

	/** Index within the instance data to the first active state's instance data (e.g. tasks) */
	UPROPERTY()
	FStateTreeIndex16 ActiveInstanceIndexBase = FStateTreeIndex16::Invalid;

	/** Index within the execution runtime data to the first execution runtime's instance data (e.g. tasks). */
	UPROPERTY()
	FStateTreeIndex16 ExecutionRuntimeIndexBase = FStateTreeIndex16::Invalid;

	/** If true, the global tasks of the State Tree should be handle in this frame. */
	UPROPERTY()
	uint8 bIsGlobalFrame : 1 = false;

	/**
	 * If true, the global tasks/evaluator received the "EnterState".
	 * Can be sustained or added via a linked state.
	 * Only call StateEnter when the state didn't previously receive a state enter.
	 */
	UPROPERTY()
	uint8 bHaveEntered : 1 = false;

#if defined(WITH_EDITORONLY_DATA) && WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.7, "States in ActiveStates are always valid.")
	uint8 NumCurrentlyActiveStates = 0;
#endif
};

/** Describes the execution state of the current State Tree instance. */
USTRUCT()
struct FStateTreeExecutionState
{
	GENERATED_BODY()

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FStateTreeExecutionState() = default;
	FStateTreeExecutionState(const FStateTreeExecutionState&) = default;
	FStateTreeExecutionState(FStateTreeExecutionState&&) = default;
	FStateTreeExecutionState& operator=(const FStateTreeExecutionState&) = default;
	FStateTreeExecutionState& operator=(FStateTreeExecutionState&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
public:
	void Reset()
	{
		*this = FStateTreeExecutionState();
	}

	/** @return the unique path of all the active states of all the active execution frames. */
	UE_API UE::StateTree::FActiveStatePath GetActiveStatePath() const;

	UE_DEPRECATED(5.6, "FindAndRemoveExpiredDelayedTransitions is not used anymore.")
	/** Finds all delayed transition states for a specific transition and removes them. Returns their copies. */
	TArray<FStateTreeTransitionDelayedState, TInlineAllocator<8>> FindAndRemoveExpiredDelayedTransitions(const UStateTree* OwnerStateTree, const FStateTreeIndex16 TransitionIndex)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		TArray<FStateTreeTransitionDelayedState, TInlineAllocator<8>> Result;
		for (TArray<FStateTreeTransitionDelayedState>::TIterator It = DelayedTransitions.CreateIterator(); It; ++It)
		{
			if (It->TimeLeft <= 0.0f && It->StateTree == OwnerStateTree && It->TransitionIndex == TransitionIndex)
			{
				Result.Emplace(MoveTemp(*It));
				It.RemoveCurrentSwap();
			}
		}

		return Result;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** @return the active frame that matches the unique frame ID. */
	UE_API const FStateTreeExecutionFrame* FindActiveFrame(UE::StateTree::FActiveFrameID FrameID) const;

	/** @return the active frame that matches the unique frame ID. */
	UE_API FStateTreeExecutionFrame* FindActiveFrame(UE::StateTree::FActiveFrameID FrameID);

	/** @return the active frame index that matches the unique frame ID. */
	UE_API int32 IndexOfActiveFrame(UE::StateTree::FActiveFrameID FrameID) const;

	/** @return whether it contains any scheduled tick requests. */
	bool HasScheduledTickRequests() const
	{
		return ScheduledTickRequests.Num() > 0;
	}

	/** @return the best/smallest scheduled tick request of all the requests. */
	FStateTreeScheduledTick GetScheduledTickRequest() const
	{
		return HasScheduledTickRequests() ? CachedScheduledTickRequest : FStateTreeScheduledTick();
	}

	/** Adds a scheduled tick request. */
	UE_API UE::StateTree::FScheduledTickHandle AddScheduledTickRequest(FStateTreeScheduledTick ScheduledTick);

	/** Updates the scheduled tick of a previous request. */
	UE_API bool UpdateScheduledTickRequest(UE::StateTree::FScheduledTickHandle Handle, FStateTreeScheduledTick ScheduledTick);

	/** Removes a request. */
	UE_API bool RemoveScheduledTickRequest(UE::StateTree::FScheduledTickHandle Handle);

private:
	UE_API void CacheScheduledTickRequest();

public:
	/** Currently active frames (and states) */
	UPROPERTY()
	TArray<FStateTreeExecutionFrame> ActiveFrames;

	/** Pending delayed transitions. */
	UPROPERTY()
	TArray<FStateTreeTransitionDelayedState> DelayedTransitions;

	/** Used by state tree random-based operations. */
	UPROPERTY()
	FRandomStream RandomStream;

	/** Active delegate listeners. */
	FStateTreeDelegateActiveListeners DelegateActiveListeners;

private:
	/** ScheduledTick */
	struct FScheduledTickRequest
	{
		UE::StateTree::FScheduledTickHandle Handle;
		FStateTreeScheduledTick ScheduledTick;
	};
	TArray<FScheduledTickRequest> ScheduledTickRequests;

	/** The current computed value from ScheduledTickRequests. Only valid when ScheduledTickRequests is not empty. */
	FStateTreeScheduledTick CachedScheduledTickRequest;

public:
#if WITH_STATETREE_TRACE
	/** Id for the active instance used for debugging. */
	mutable FStateTreeInstanceDebugId InstanceDebugId;
#endif

	/** Optional extension for the execution context. */
	UPROPERTY(Transient)
	TInstancedStruct<FStateTreeExecutionExtension> ExecutionExtension;

	/** Result of last TickTasks */
	UPROPERTY()
	EStateTreeRunStatus LastTickStatus = EStateTreeRunStatus::Failed;

	/** Running status of the instance */
	UPROPERTY()
	EStateTreeRunStatus TreeRunStatus = EStateTreeRunStatus::Unset;

	/** Completion status stored if Stop was called during the Tick and needed to be deferred. */
	UPROPERTY()
	EStateTreeRunStatus RequestedStop = EStateTreeRunStatus::Unset;

	/** Current update phase used to validate reentrant calls to the main entry points of the execution context (i.e. Start, Stop, Tick). */
	UPROPERTY()
	EStateTreeUpdatePhase CurrentPhase = EStateTreeUpdatePhase::Unset;

	/** Number of times a new state has been changed. */
	UPROPERTY()
	uint16 StateChangeCount = 0;

	/** A task that completed a state or a global task that completed a global frame. */
	UPROPERTY()
	bool bHasPendingCompletedState = false;

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.6, "Replaced with FStateTreeTasksCompletionStatus")
	/** Pending finished tasks that need processing. */
	TArray<UE::StateTree::FFinishedTask> FinishedTasks;

	UE_DEPRECATED(5.6, "Use FinishTask to completed a state.")
	/** Handle of the state that was first to report state completed (success or failure), used to trigger completion transitions. */
	UPROPERTY()
	FStateTreeIndex16 CompletedFrameIndex = FStateTreeIndex16::Invalid; 
	
	UE_DEPRECATED(5.6, "Use FinishTask to completed a state.")
	UPROPERTY()
	FStateTreeStateHandle CompletedStateHandle = FStateTreeStateHandle::Invalid;

	UE_DEPRECATED(5.6, "CurrentExecutionContext is not needed anymore. Use FrameID and StateID.")
	FStateTreeExecutionContext* CurrentExecutionContext = nullptr;

	UE_DEPRECATED(5.7, "Use FStateTreeExecutionFrame::ActiveNodeIndex instead.")
	UPROPERTY()
	FStateTreeIndex16 EnterStateFailedFrameIndex = FStateTreeIndex16::Invalid;

	UE_DEPRECATED(5.7, "Use FStateTreeExecutionFrame::ActiveNodeIndex instead.")
	UPROPERTY()
	FStateTreeIndex16 EnterStateFailedTaskIndex = FStateTreeIndex16::Invalid;

	UE_DEPRECATED(5.7, "Use FStateTreeExecutionFrame::ActiveNodeIndex instead.")
	UPROPERTY()
	FStateTreeIndex16 LastExitedNodeIndex = FStateTreeIndex16::Invalid;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif //WITH_EDITORONLY_DATA
};

/** Contains StateTree events used during State Selection for a single execution frame. */
struct
UE_DEPRECATED(5.7, "Use the StateTreeExecutionContext.RequestTransitionResult.Selection.SelectionEvents instead.")
FStateTreeFrameStateSelectionEvents
{
	TStaticArray<FStateTreeSharedEvent, FStateTreeActiveStates::MaxStates> Events;
};

/**
 * Describes a state tree transition. Source is the state where the transition started, Target describes the state where the transition pointed at,
 * and Next describes the selected state. The reason Transition and Next are different is that Transition state can be a selector state,
 * in which case the children will be visited until a leaf state is found, which will be the next state.
 */
USTRUCT(BlueprintType)
struct FStateTreeTransitionResult
{
	GENERATED_BODY()

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FStateTreeTransitionResult() = default;
	FStateTreeTransitionResult(const FStateTreeTransitionResult&) = default;
	FStateTreeTransitionResult(FStateTreeTransitionResult&&) = default;
	FStateTreeTransitionResult& operator=(const FStateTreeTransitionResult&) = default;
	FStateTreeTransitionResult& operator=(FStateTreeTransitionResult&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(5.6, "Use FStateTreeExecutionContext::MakeTransitionResult to crate a new transition.")
	UE_API FStateTreeTransitionResult(const FRecordedStateTreeTransitionResult& RecordedTransition);

	void Reset()
	{
		*this = FStateTreeTransitionResult();
	}

	/** Frame that was active when the transition was requested. */
	UE::StateTree::FActiveFrameID SourceFrameID;

	/**
	 * The state the transition was requested from.
	 * It can be invalid if the transition is requested outside the Tick.
	 */
	UE::StateTree::FActiveStateID SourceStateID;

	/** Transition target state. It can be a completion state. */
	UPROPERTY(EditDefaultsOnly, Category = "Default", BlueprintReadOnly)
	FStateTreeStateHandle TargetState = FStateTreeStateHandle::Invalid;

	/** The current state being executed. On enter/exit callbacks this is the state of the task. */
	UPROPERTY(EditDefaultsOnly, Category = "Default", BlueprintReadOnly)
	FStateTreeStateHandle CurrentState = FStateTreeStateHandle::Invalid;
	
	/** Current Run status. */
	UPROPERTY(EditDefaultsOnly, Category = "Default", BlueprintReadOnly)
	EStateTreeRunStatus CurrentRunStatus = EStateTreeRunStatus::Unset;

	/** If the change type is Sustained, then the CurrentState was reselected, or if Changed then the state was just activated. */
	UPROPERTY(EditDefaultsOnly, Category = "Default", BlueprintReadOnly)
	EStateTreeStateChangeType ChangeType = EStateTreeStateChangeType::Changed; 

	/** Priority of the transition that caused the state change. */
	UPROPERTY(EditDefaultsOnly, Category = "Default", BlueprintReadOnly)
	EStateTreeTransitionPriority Priority = EStateTreeTransitionPriority::None;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.7, "Use the StateTreeExecutionContext.RequestTransitionResult.Selection.SelectedState instead.")
	/** States selected as result of the transition. */
	UPROPERTY(Category = "Default", BlueprintReadOnly, meta = (DeprecatedProperty))
	TArray<FStateTreeExecutionFrame> NextActiveFrames;

	UE_DEPRECATED(5.7, "Use the StateTreeExecutionContext.RequestTransitionResult.Selection.SelectionEvents instead.")
	/** Events used in state selection. */
	TArray<FStateTreeFrameStateSelectionEvents> NextActiveFrameEvents;

	UE_DEPRECATED(5.6, "Use SourceFrameID instead")
	/** StateTree asset that was active when the transition was requested. */
	UPROPERTY(Category = "Default", BlueprintReadOnly, meta = (DeprecatedProperty))
	TObjectPtr<const UStateTree> SourceStateTree = nullptr;

	UE_DEPRECATED(5.6, "Use SourceFrameID instead.")
	/** Root state the execution frame where the transition was requested. */
	UPROPERTY(Category = "Default", BlueprintReadOnly, meta = (DeprecatedProperty))
	FStateTreeStateHandle SourceRootState = FStateTreeStateHandle::Invalid;

	UE_DEPRECATED(5.6, "Use SourceStateID instead")
	/** Transition source state. */
	UPROPERTY(Category = "Default", BlueprintReadOnly, meta=(DeprecatedProperty))
	FStateTreeStateHandle SourceState = FStateTreeStateHandle::Invalid;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

/*
 * Enumeration for the different transition recording types.
 * This is used by the execution context to capture transition snapshots if set to record.
*/
UENUM()
enum class EStateTreeRecordTransitions : uint8
{
	No,
	Yes
};

/*
 * Captured state tree execution frame that can be cached for recording purposes.
 * Held in FRecordedStateTreeTransitionResult for its NextActiveFrames.
 */
USTRUCT()
struct
UE_DEPRECATED(5.7, "FRecordedStateTreeExecutionFrame is not used anymore.")
FRecordedStateTreeExecutionFrame
{
	GENERATED_BODY()

	FRecordedStateTreeExecutionFrame() = default;
	UE_DEPRECATED(5.6, "Use FStateTreeExecutionContext::MakeRecordedTransitionResult to create a new recorded transition.")
	UE_API FRecordedStateTreeExecutionFrame(const FStateTreeExecutionFrame& ExecutionFrame);

	/** The State Tree used for ticking this frame. */
	UPROPERTY()
	TObjectPtr<const UStateTree> StateTree = nullptr;

	/** The root state of the frame (e.g. Root state or a subtree). */
	UPROPERTY()
	FStateTreeStateHandle RootState = FStateTreeStateHandle::Root; 
	
	/** Active states in this frame. */
	UPROPERTY()
	FStateTreeActiveStates ActiveStates;

	/** If true, the global tasks of the State Tree should be handle in this frame. */
	UPROPERTY()
	uint8 bIsGlobalFrame : 1 = false;

	/** Captured indices of the events we've recorded. */
	TStaticArray<uint8, FStateTreeActiveStates::MaxStates> EventIndices;
};

/** Captured state cached for recording purposes. */
USTRUCT()
struct FRecordedActiveState
{
	GENERATED_BODY()

	FRecordedActiveState() = default;

	/** The state tree that owns the state handle. */
	UPROPERTY()
	TObjectPtr<const UStateTree> StateTree;

	/** The active state. */
	UPROPERTY()
	FStateTreeStateHandle State;

	/** Captured events from the transition that we've recorded */
	UPROPERTY()
	int32 EventIndex = INDEX_NONE;
};

/*
 * Captured state tree transition result that can be cached for recording purposes.
 * when transitions are recorded through this structure, we can replicate them down
 * to clients to keep our state tree in sync.
 */
USTRUCT()
struct FRecordedStateTreeTransitionResult
{
	GENERATED_BODY()

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FRecordedStateTreeTransitionResult() = default;
	FRecordedStateTreeTransitionResult(const FRecordedStateTreeTransitionResult&) = default;
	FRecordedStateTreeTransitionResult(FRecordedStateTreeTransitionResult&&) = default;
	FRecordedStateTreeTransitionResult& operator=(const FRecordedStateTreeTransitionResult&) = default;
	FRecordedStateTreeTransitionResult& operator=(FRecordedStateTreeTransitionResult&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(5.6, "Use FStateTreeExecutionContext::MakeRecordedTransitionResult to create a new recorded transition.")
	UE_API FRecordedStateTreeTransitionResult(const FStateTreeTransitionResult& Transition);
	
	/** The selected states. */
	UPROPERTY()
	TArray<FRecordedActiveState> States;

	/** The selected states. */
	UPROPERTY()
	TArray<FStateTreeEvent> Events;
	
	/** Priority of the transition that caused the state change. */
	UPROPERTY()
	EStateTreeTransitionPriority Priority = EStateTreeTransitionPriority::None;

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.7, "The recorded transition doesn't record the frame. ForceTransition will recreated them if needed.")
	/** States selected as result of the transition. */
	UPROPERTY()
	TArray<FRecordedStateTreeExecutionFrame> NextActiveFrames;

	UE_DEPRECATED(5.7, "The event are recorded in SelectedStates.")
	/** Captured events from the transition that we've recorded */
	UPROPERTY()
	TArray<FStateTreeEvent> NextActiveFrameEvents;

	UE_DEPRECATED(5.7, "We can't assumed that the recorded has the same active states. The source won't be set.")
	/** Transition source state. */
	UPROPERTY()
	FStateTreeStateHandle SourceState = FStateTreeStateHandle::Invalid;

	UE_DEPRECATED(5.7, "The TargetState is always the last item. We can't assumed that the recorded has the same active states.")
	/** Transition target state. */
	UPROPERTY()
	FStateTreeStateHandle TargetState = FStateTreeStateHandle::Invalid;

	UE_DEPRECATED(5.7, "We can't assumed that the recorded has the same active states. The source won't be set.")
	/** StateTree asset that was active when the transition was requested. */
	UPROPERTY()
	TObjectPtr<const UStateTree> SourceStateTree = nullptr;

	UE_DEPRECATED(5.7, "We can't assumed that the recorded has the same active states. The source won't be set.")
	/** Root state the execution frame where the transition was requested. */
	UPROPERTY()
	FStateTreeStateHandle SourceRootState = FStateTreeStateHandle::Invalid;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif //WITH_EDITORONLY_DATA
};

#undef UE_API
