// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "StateTreeExtension.generated.h"

class UStateTree;
struct FStateTreeLinker;

/**
 * Extension for the state tree asset.
 */
UCLASS(Abstract, DefaultToInstanced, Within=StateTree, MinimalAPI)
class UStateTreeExtension : public UObject
{
	GENERATED_BODY()

public:
	/** Resolves references to other StateTree data. */
	virtual bool Link(FStateTreeLinker& Linker)
	{
		return true;
	}

protected:
	UStateTree* GetStateTree() const
	{
		return GetOuterUStateTree();
	}
};
