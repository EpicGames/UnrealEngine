// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "StateTreeFunctionLibrary.generated.h"

#define UE_API STATETREEMODULE_API

struct FStateTreeReference;
class UStateTree;

/**
 * A collection of blueprint functions for state tree.
 */
UCLASS(MinimalAPI, Abstract)
class UStateTreeFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "StateTree|Reference")
	static UE_API void SetStateTree(UPARAM(ref)FStateTreeReference& Reference, UStateTree* StateTree);

	UFUNCTION(BlueprintCallable, DisplayName = "Make State Tree Reference", Category = "StateTree|Reference", meta = (Keywords = "construct build", NativeMakeFunc, BlueprintInternalUseOnly = "true"))
	static UE_API FStateTreeReference MakeStateTreeReference(UStateTree* StateTree);

	UFUNCTION(BlueprintCallable, CustomThunk, DisplayName = "Set Parameter Property", Category = "StateTree|Reference", meta = (BlueprintInternalUseOnly = "true", CustomStructureParam = "NewValue"))
	static UE_API void K2_SetParametersProperty(UPARAM(ref)FStateTreeReference& Reference, FGuid PropertyID, UPARAM(ref) const int32& NewValue);

	UFUNCTION(BlueprintCallable, CustomThunk, DisplayName = "Get Parameter Property", Category = "StateTree|Reference", meta = (BlueprintInternalUseOnly = "true", CustomStructureParam = "ReturnValue"))
	static UE_API void K2_GetParametersProperty(UPARAM(ref)const FStateTreeReference& Reference, FGuid PropertyID, int32& ReturnValue);

private:
	DECLARE_FUNCTION(execK2_SetParametersProperty);
	DECLARE_FUNCTION(execK2_GetParametersProperty);
};

#undef UE_API
