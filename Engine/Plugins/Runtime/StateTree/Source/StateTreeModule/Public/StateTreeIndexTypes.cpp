// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeIndexTypes.h"
#include "StateTreeTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeIndexTypes)

bool FStateTreeIndex16::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_UInt16Property)
	{
		// Support loading from uint16.
		// Note: 0xffff is silently read as invalid value.
		uint16 OldValue = 0;
		Slot << OldValue;

		*this = FStateTreeIndex16(OldValue);
		return true;
	}
	else if (Tag.GetType().IsStruct(FStateTreeIndex8::StaticStruct()->GetFName()))
	{
		// Support loading from Index8.
		FStateTreeIndex8 OldValue;
		FStateTreeIndex8::StaticStruct()->SerializeItem(Slot, &OldValue, nullptr);

		int32 NewValue = OldValue.AsInt32();
		if (!IsValidIndex(NewValue))
		{
			NewValue = INDEX_NONE;
		}
		
		*this = FStateTreeIndex16(NewValue);
		
		return true;
	}
	
	return false;
}


bool FStateTreeIndex8::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	// Support loading from Index16.
	if (Tag.GetType().IsStruct(FStateTreeIndex16::StaticStruct()->GetFName()))
	{
		FStateTreeIndex16 OldValue;
		FStateTreeIndex16::StaticStruct()->SerializeItem(Slot, &OldValue, nullptr);

		int32 NewValue = OldValue.AsInt32();
		if (!IsValidIndex(NewValue))
		{
			NewValue = INDEX_NONE;
		}
		
		*this = FStateTreeIndex8(NewValue);
		
		return true;
	}
	
	return false;
}
