// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "PropertyBindingTypes.h"

#include "StateTreeIndexTypes.generated.h"

#define UE_API STATETREEMODULE_API

/** uint16 index that can be invalid. */
USTRUCT(BlueprintType)
struct FStateTreeIndex16
{
	GENERATED_BODY()

	static constexpr uint16 InvalidValue = MAX_uint16;
	static UE_API const FStateTreeIndex16 Invalid;

	friend inline uint32 GetTypeHash(const FStateTreeIndex16 Index)
	{
		return GetTypeHash(Index.Value);
	}

	/** @return true if the given index can be represented by the type. */
	static bool IsValidIndex(const int32 Index)
	{
		return Index >= 0 && Index < (int32)MAX_uint16;
	}

	FStateTreeIndex16() = default;

	/**
	 * Construct from a uint16 index where MAX_uint16 is considered an invalid index
	 * (i.e FStateTreeIndex16::InvalidValue).
	 */
	explicit FStateTreeIndex16(const uint16 InIndex) : Value(InIndex)
	{
	}

	/**
	 * Construct from a int32 index where INDEX_NONE is considered an invalid index
	 * and converted to FStateTreeIndex16::InvalidValue (i.e MAX_uint16).
	 */
	explicit FStateTreeIndex16(const int32 InIndex)
	{
		check(InIndex == INDEX_NONE || IsValidIndex(InIndex));
		Value = InIndex == INDEX_NONE ? InvalidValue : (uint16)InIndex;
	}

	/**
	 * Construct from FPropertyBindingIndex16 to facilitate transition to FPropertyBindingIndex16 in bindings
	 */
	explicit FStateTreeIndex16(const FPropertyBindingIndex16 InIndex) : Value(InIndex.Get())
	{
	}

	/**
	 * Conversion to from FPropertyBindingIndex16 to facilitate transition to FPropertyBindingIndex16 in bindings
	 * (Intentionally not explicit for the initial refactoring)
	 * @todo make this explicit and update StateTree code to use FPropertyBindingIndex16
	 */
	operator FPropertyBindingIndex16() const
	{
		return FPropertyBindingIndex16(Value);
	}

	/** @return value of the index or FStateTreeIndex16::InvalidValue (i.e. MAX_uint16) if invalid. */
	uint16 Get() const { return Value; }
	
	/** @return the index value as int32, mapping invalid value to INDEX_NONE. */
	int32 AsInt32() const { return Value == InvalidValue ? INDEX_NONE : Value; }

	/** @return true if the index is valid. */
	bool IsValid() const { return Value != InvalidValue; }

	bool operator==(const FStateTreeIndex16& RHS) const { return Value == RHS.Value; }
	bool operator!=(const FStateTreeIndex16& RHS) const { return Value != RHS.Value; }

	UE_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

protected:
	UPROPERTY()
	uint16 Value = InvalidValue;
};

template<>
struct TStructOpsTypeTraits<FStateTreeIndex16> : public TStructOpsTypeTraitsBase2<FStateTreeIndex16>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

/** uint8 index that can be invalid. */
USTRUCT(BlueprintType)
struct FStateTreeIndex8
{
	GENERATED_BODY()

	static constexpr uint8 InvalidValue = MAX_uint8;
	static UE_API const FStateTreeIndex8 Invalid;
	
	/** @return true if the given index can be represented by the type. */
	static bool IsValidIndex(const int32 Index)
	{
		return Index >= 0 && Index < (int32)MAX_uint8;
	}
	
	FStateTreeIndex8() = default;
	
	explicit FStateTreeIndex8(const int32 InIndex)
	{
		check(InIndex == INDEX_NONE || IsValidIndex(InIndex));
		Value = InIndex == INDEX_NONE ? InvalidValue : (uint8)InIndex;
	}

	/** @return value of the index. */
	uint8 Get() const { return Value; }

	/** @return the index value as int32, mapping invalid value to INDEX_NONE. */
	int32 AsInt32() const { return Value == InvalidValue ? INDEX_NONE : Value; }
	
	/** @return true if the index is valid. */
	bool IsValid() const { return Value != InvalidValue; }

	bool operator==(const FStateTreeIndex8& RHS) const { return Value == RHS.Value; }
	bool operator!=(const FStateTreeIndex8& RHS) const { return Value != RHS.Value; }

	UE_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);
	
protected:
	UPROPERTY()
	uint8 Value = InvalidValue;
};

template<>
struct TStructOpsTypeTraits<FStateTreeIndex8> : public TStructOpsTypeTraitsBase2<FStateTreeIndex8>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

#undef UE_API
