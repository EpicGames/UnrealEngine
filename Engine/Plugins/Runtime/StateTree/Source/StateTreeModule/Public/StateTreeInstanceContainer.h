// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/InstancedStructContainer.h"
#include "StateTreeTypes.h"
#include "StateTreeInstanceContainer.generated.h"

#define UE_API STATETREEMODULE_API

/** Wrapper class to store an object amongst the structs. */
USTRUCT()
struct FStateTreeInstanceObjectWrapper
{
	GENERATED_BODY()

	FStateTreeInstanceObjectWrapper() = default;
	FStateTreeInstanceObjectWrapper(UObject* Object)
		: InstanceObject(Object)
	{}
	
	UPROPERTY()
	TObjectPtr<UObject> InstanceObject = nullptr;
};

namespace UE::StateTree::InstanceData
{
	/** Container of instance data. */
	USTRUCT()
	struct FInstanceContainer
	{
		GENERATED_BODY()

		FInstanceContainer() = default;

	public:
		struct FAddArgs
		{
			/** Duplicate the object contained by object wrapper. */
			bool bDuplicateWrappedObject = true;
		};

		/** Initializes the array with specified items. */
		UE_API void Init(TNotNull<UObject*> InOwner, const FInstanceContainer& InStructs, FAddArgs Args);
		/** Initializes the array with specified items. */
		UE_API void Init(TNotNull<UObject*> InOwner, TConstArrayView<FInstancedStruct> InStructs, FAddArgs Args);
		/** Initializes the array with specified items. */
		UE_API void Init(TNotNull<UObject*> InOwner, TConstArrayView<FConstStructView> InStructs, FAddArgs Args);
		/** Append the specified items to the array. */
		UE_API int32 Append(TNotNull<UObject*> InOwner, const FInstanceContainer& InStructs, FAddArgs Args);
		/** Append the specified items to the array. */
		UE_API int32 Append(TNotNull<UObject*> InOwner, TConstArrayView<FInstancedStruct> InStructs, FAddArgs Args);
		/** Append the specified items to the array. */
		UE_API int32 Append(TNotNull<UObject*> InOwner, TConstArrayView<FConstStructView> InStructs, FAddArgs Args);

		/** @return number of items in the storage. */
		[[nodiscard]] int32 Num() const
		{
			return InstanceStructs.Num();
		}

		/** @return true if the index can be used to get data. */
		[[nodiscard]] bool IsValidIndex(const int32 Index) const
		{
			return InstanceStructs.IsValidIndex(Index);
		}

		/** @return true if item at the specified index is object type. */
		[[nodiscard]] bool IsObject(const int32 Index) const
		{
			return InstanceStructs[Index].GetScriptStruct() == TBaseStructure<FStateTreeInstanceObjectWrapper>::Get();
		}

		/** @return specified item as struct. */
		[[nodiscard]] FConstStructView GetStruct(const int32 Index) const
		{
			return InstanceStructs[Index];
		}

		/** @return specified item as mutable struct. */
		[[nodiscard]] FStructView GetMutableStruct(const int32 Index)
		{
			return InstanceStructs[Index];
		}

		/** @return specified item as object, will check() if the item is not an object. */
		[[nodiscard]] const UObject* GetObject(const int32 Index) const
		{
			const FStateTreeInstanceObjectWrapper& Wrapper = InstanceStructs[Index].Get<const FStateTreeInstanceObjectWrapper>();
			return Wrapper.InstanceObject;
		}

		/** @return specified item as mutable Object, will check() if the item is not an object. */
		[[nodiscard]] UObject* GetMutableObject(const int32 Index) const
		{
			const FStateTreeInstanceObjectWrapper& Wrapper = InstanceStructs[Index].Get<const FStateTreeInstanceObjectWrapper>();
			return Wrapper.InstanceObject;
		}

		/** @return true if all instances are valid. */
		[[nodiscard]] UE_API bool AreAllInstancesValid() const;

		/** Resets the data to empty. */
		void Reset()
		{
			InstanceStructs.Reset();
		}

		/** Returns the estimated memory allocated by the container. */
		[[nodiscard]] UE_API int32 GetAllocatedMemory() const;

		using FIterator = FInstancedStructContainer::FIterator;
		using FConstIterator = FInstancedStructContainer::FConstIterator;

		//~ For ranged for, do not use directly.
		FIterator begin()
		{
			return InstanceStructs.begin();
		}
		FIterator end()
		{
			return InstanceStructs.end();
		}

		FConstIterator begin() const
		{
			return InstanceStructs.begin();
		}
		FConstIterator end() const
		{
			return InstanceStructs.end();
		}

	private:
		UPROPERTY()
		FInstancedStructContainer InstanceStructs;
	};
} // namespace UE::StateTree::InstanceData

#undef UE_API
