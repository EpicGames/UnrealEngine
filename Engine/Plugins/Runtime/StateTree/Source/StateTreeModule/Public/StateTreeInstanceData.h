// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/InstancedStructContainer.h"
#include "Debugger/StateTreeRuntimeValidation.h"
#include "StateTreeEvents.h"
#include "StateTreeExecutionRuntimeDataTypes.h"
#include "StateTreeInstanceContainer.h"
#include "StateTreeTypes.h"
#include "StateTreeExecutionTypes.h"
#include "Misc/MTAccessDetector.h"
#include "Templates/PimplPtr.h"
#include "Templates/SharedPointer.h"
#include "StateTreeInstanceData.generated.h"

#define UE_API STATETREEMODULE_API

struct FStateTreeTransitionRequest;
struct FStateTreeExecutionState;
struct FStateTreeDelegateDispatcher;

#if WITH_STATETREE_DEBUG
namespace UE::StateTree::Debug
{
	class FRuntimeValidationInstanceData;
}
#endif

namespace UE::StateTree::InstanceData
{
	/** @return data view of the specified handle relative to the given frame. */
	[[nodiscard]] UE_API FStateTreeDataView GetDataView(
		FStateTreeInstanceStorage& InstanceStorage,
		FStateTreeInstanceStorage* SharedInstanceStorage,
		const FStateTreeExecutionFrame& CurrentFrame,
		const FStateTreeDataHandle& Handle);

	UE_DEPRECATED(5.8, "Use the version without Parent Frame instead")
	[[nodiscard]] inline FStateTreeDataView GetDataView(
		FStateTreeInstanceStorage& InstanceStorage,
		FStateTreeInstanceStorage* SharedInstanceStorage,
		const FStateTreeExecutionFrame* ParentFrame,
		const FStateTreeExecutionFrame& CurrentFrame,
		const FStateTreeDataHandle& Handle)
	{
		return GetDataView(InstanceStorage, SharedInstanceStorage, CurrentFrame, Handle);
	}

	/** @return data view of the specified handle relative to the given frame, or tries to find a matching temporary instance. */
	[[nodiscard]] UE_API FStateTreeDataView GetDataViewOrTemporary(
		FStateTreeInstanceStorage& InstanceStorage,
		FStateTreeInstanceStorage* SharedInstanceStorage,
		const FStateTreeExecutionFrame& CurrentFrame,
		const FStateTreeDataHandle& Handle);

	UE_DEPRECATED(5.8, "Use the version without Parent Frame instead")
	[[nodiscard]] inline FStateTreeDataView GetDataViewOrTemporary(
		FStateTreeInstanceStorage& InstanceStorage,
		FStateTreeInstanceStorage* SharedInstanceStorage,
		const FStateTreeExecutionFrame* ParentFrame,
		const FStateTreeExecutionFrame& CurrentFrame,
		const FStateTreeDataHandle& Handle)
	{
		return GetDataViewOrTemporary(InstanceStorage, SharedInstanceStorage, CurrentFrame, Handle);
	}

	/**
	 * Check if a data source type requires FStateTreeExecutionContext to access.
	 * Sources that require execution context include context data, external data, and evaluation scope data.
	 * @param SourceType The data source type to check.
	 * @return true if the source requires a full FStateTreeExecutionContext to access.
	 */
	UE_API bool DoesRequireExecutionContext(EStateTreeDataSourceType SourceType);

	/**
	 * Check if a data source type can be access from the UE::StateTree::InstanceData::GetDataView function.
	 * @param SourceType The data source type to check.
	 * @return true if the source doesn't requires anything other than the provided arguments.
	 */
	UE_API bool DoesRequireInstanceStorage(EStateTreeDataSourceType SourceType);
}

/**
 * Holds temporary instance data created during state selection.
 * The data is identified by Frame and DataHandle.
 */
USTRUCT()
struct FStateTreeTemporaryInstanceData
{
	GENERATED_BODY()

	FStateTreeTemporaryInstanceData() = default;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FStateTreeTemporaryInstanceData(const FStateTreeTemporaryInstanceData& Other) = default;
	FStateTreeTemporaryInstanceData(FStateTreeTemporaryInstanceData&& Other) = default;
	FStateTreeTemporaryInstanceData& operator=(FStateTreeTemporaryInstanceData const& Other) = default;
	FStateTreeTemporaryInstanceData& operator=(FStateTreeTemporaryInstanceData&& Other) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE::StateTree::FActiveFrameID FrameID;

	UPROPERTY()
	FStateTreeDataHandle DataHandle = FStateTreeDataHandle::Invalid;

	UPROPERTY()
	FStateTreeIndex16 OwnerNodeIndex = FStateTreeIndex16::Invalid; 
	
	UPROPERTY()
	FInstancedStruct Instance;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.6, "Use the frame id to identify the frame.")
	UPROPERTY()
	TObjectPtr<const UStateTree> StateTree = nullptr;

	UE_DEPRECATED(5.6, "Use the frame id to identify the frame.")
	UPROPERTY()
	FStateTreeStateHandle RootState = FStateTreeStateHandle::Invalid; 
#endif
};


struct FStateTreeInstanceStorageCustomVersion
{
	enum Type
	{
		// Before any version changes were made in the plugin
		BeforeCustomVersionWasAdded = 0,
		// Added custom serialization
		AddedCustomSerialization,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	/** The GUID for this custom version number */
	UE_API const static FGuid GUID;

private:
	FStateTreeInstanceStorageCustomVersion() = default;
};

/**
 * State Tree instance data is used to store the runtime state of a State Tree. It is used together with FStateTreeExecution context to tick the state tree.
 * You are supposed to use FStateTreeInstanceData as a property to store the instance data. That ensures that any UObject references will get GC'd correctly.
 *
 * The FStateTreeInstanceData wraps FStateTreeInstanceStorage, where the data is actually stored. This indirection is done in order to allow the FStateTreeInstanceData
 * to be bitwise relocatable (e.g. you can put it in an array), and we can still allow delegates to bind to the instance data of individual tasks.
 *
 * Since the tasks in the instance data are stored in a array that may get resized you will need to use TStateTreeInstanceDataStructRef
 * to reference a struct based task instance data. It is defined below, and has example how to use it.
 */

/** Storage for the actual instance data. */
USTRUCT()
struct FStateTreeInstanceStorage
{
	GENERATED_BODY()

	UE_API FStateTreeInstanceStorage();
	UE_API FStateTreeInstanceStorage(const FStateTreeInstanceStorage& Other);
	UE_API FStateTreeInstanceStorage(FStateTreeInstanceStorage&& Other) noexcept;

	UE_API FStateTreeInstanceStorage& operator=(const FStateTreeInstanceStorage& Other);
	UE_API FStateTreeInstanceStorage& operator=(FStateTreeInstanceStorage&& Other) noexcept;

	/** @return reference to the event queue. */
	FStateTreeEventQueue& GetMutableEventQueue()
	{
		return *EventQueue;
	}

	/** @return reference to the event queue. */
	const FStateTreeEventQueue& GetEventQueue() const
	{
		return *EventQueue;
	}

	/** @return true if the storage owns the event queue. */
	bool IsOwningEventQueue() const
	{
		return bIsOwningEventQueue;
	}

	/** @return shared pointer to the event queue. */
	const TSharedRef<FStateTreeEventQueue>& GetSharedMutableEventQueue()
	{
		return EventQueue;
	}

	/** Sets event queue from another storage. Marks the event queue not owned. */
	UE_API void SetSharedEventQueue(const TSharedRef<FStateTreeEventQueue>& InSharedEventQueue);
	
	/** 
	 * Buffers a transition request to be sent to the State Tree.
	 * @param Owner Optional pointer to an owner UObject that is used for logging errors.
	 * @param Request transition to request.
	*/
	UE_API void AddTransitionRequest(const UObject* Owner, const FStateTreeTransitionRequest& Request);

	/** Marks delegate as broadcasted. Use for transitions. */
	UE_API void MarkDelegateAsBroadcasted(const FStateTreeDelegateDispatcher& Dispatcher);

	/** @return true if a delegate was broadcasted. */
	UE_API bool IsDelegateBroadcasted(const FStateTreeDelegateDispatcher& Dispatcher) const;

	/** Resets the list of broadcasted delegates. */
	UE_API void ResetBroadcastedDelegates();

	/** Resets the list of broadcasted delegates and returns it's copy. */
	UE_API TArray<FStateTreeDelegateDispatcher> StealBroadcastedDelegates();

	/** @return true if there's any broadcasted delegates. */
	UE_API bool HasBroadcastedDelegates() const;

	/** @return currently pending transition requests. */
	TConstArrayView<FStateTreeTransitionRequest> GetTransitionRequests() const
	{
		return TransitionRequests;
	}

	/** Reset all pending transition requests. */
	UE_API void ResetTransitionRequests();

	/** @return true if all instances are valid. */
	UE_API bool AreAllInstancesValid() const;

	/** @return number of items in the storage. */
	int32 Num() const
	{
		return InstanceStructs.Num();
	}

	/** @return true if the index can be used to get data. */
	bool IsValidIndex(const int32 Index) const
	{
		return InstanceStructs.IsValidIndex(Index);
	}

	/** @return true if item at the specified index is object type. */
	bool IsObject(const int32 Index) const
	{
		return InstanceStructs[Index].GetScriptStruct() == TBaseStructure<FStateTreeInstanceObjectWrapper>::Get();
	}

	/** @return specified item as struct. */
	FConstStructView GetStruct(const int32 Index) const
	{
		return InstanceStructs[Index];
	}
	
	/** @return specified item as mutable struct. */
	FStructView GetMutableStruct(const int32 Index)
	{
		return InstanceStructs[Index];
	}

	/** @return specified item as object, will check() if the item is not an object. */
	const UObject* GetObject(const int32 Index) const
	{
		const FStateTreeInstanceObjectWrapper& Wrapper = InstanceStructs[Index].Get<const FStateTreeInstanceObjectWrapper>();
		return Wrapper.InstanceObject;
	}

	/** @return specified item as mutable Object, will check() if the item is not an object. */
	UObject* GetMutableObject(const int32 Index) const
	{
		const FStateTreeInstanceObjectWrapper& Wrapper = InstanceStructs[Index].Get<const FStateTreeInstanceObjectWrapper>();
		return Wrapper.InstanceObject;
	}

	/** @return reference to StateTree execution state. */
	const FStateTreeExecutionState& GetExecutionState() const
	{
		return ExecutionState;
	}

	/** @return reference to StateTree execution state. */
	FStateTreeExecutionState& GetMutableExecutionState()
	{
		return ExecutionState;
	}

	/** @return the storage's execution runtime data for the state tree. */
	[[nodiscard]] const UE::StateTree::InstanceData::FInstanceContainer& GetExecutionRuntimeData() const
	{
		return ExecutionRuntimeData;
	}

	/** @return the storage's execution runtime data for the state tree. */
	[[nodiscard]] UE::StateTree::InstanceData::FInstanceContainer& GetExecutionRuntimeData()
	{
		return ExecutionRuntimeData;
	}

	/**
	 * Add or reuse the execution runtime data for the state tree.
	 * Return the base index for the execution runtime data.
	 */
	[[nodiscard]] UE_API int32 AddExecutionRuntimeData(TNotNull<UObject*> Owner, UE::StateTree::FExecutionFrameHandle FrameHandle);

	/**
	 * Adds temporary instance data associated with specified frame and data handle.
	 * @returns mutable struct view to the instance.
	 */
	UE_API FStructView AddTemporaryInstance(UObject& InOwner, const FStateTreeExecutionFrame& Frame, const FStateTreeIndex16 OwnerNodeIndex, const FStateTreeDataHandle DataHandle, FConstStructView NewInstanceData);

	/** Removes a temporary instance data associated with specified frame and data handle. */
	UE_API void RemoveTemporaryInstance(const UE::StateTree::FActiveFrameID FrameID, const FStateTreeIndex16 OwnerNodeIndex, const FStateTreeDataHandle DataHandle);

	UE_DEPRECATED(5.7, "Use the version with FrameID instead.")
	FStructView GetMutableTemporaryStruct(const FStateTreeExecutionFrame& Frame, const FStateTreeDataHandle DataHandle) { return GetMutableTemporaryStruct(Frame.FrameID, DataHandle); }

	/** @returns mutable view to the specified instance data, or invalid view if not found. */
	UE_API FStructView GetMutableTemporaryStruct(UE::StateTree::FActiveFrameID FrameID, const FStateTreeDataHandle DataHandle);

	UE_DEPRECATED(5.7, "Use the version with FrameID instead.")
	UObject* GetMutableTemporaryObject(const FStateTreeExecutionFrame& Frame, const FStateTreeDataHandle DataHandle) { return GetMutableTemporaryObject(Frame.FrameID, DataHandle); }

	/** @returns mutable pointer to the specified instance data object, or invalid view if not found. Will check() if called on non-object data. */
	UE_API UObject* GetMutableTemporaryObject(UE::StateTree::FActiveFrameID FrameID, const FStateTreeDataHandle DataHandle);

	/** Empties the temporary instances. */
	UE_API void ResetTemporaryInstances();

	/** @return mutable array view to the temporary instances */
	TArrayView<FStateTreeTemporaryInstanceData> GetMutableTemporaryInstances()
	{
		return TemporaryInstances;
	}

	/** Stores copy of provided parameters as State Tree global parameters. */
	UE_API void SetGlobalParameters(FConstStructView Parameters);

	UE_DEPRECATED(5.8, "Use the version with a FConstStructView")
	UE_API void SetGlobalParameters(const FInstancedPropertyBag& Parameters);

	/** @return view to global parameters. */
	FConstStructView GetGlobalParameters() const
	{
		return GlobalParameters;
	}

	/** @return mutable view to global parameters. */
	FStructView GetMutableGlobalParameters()
	{
		return GlobalParameters;
	}

	/** @return a unique number used to make active frame id and active state id. */
	UE_API uint32 GenerateUniqueId(); //@TODO rename to GenerateUniqueID

	/** Note, called by FStateTreeInstanceData. */
	UE_API void AddStructReferencedObjects(FReferenceCollector& Collector);

	/** Resets the storage to initial state. */
	UE_API void Reset();

	/** Start the invalid multithreading read-only access detection. */
	UE_API void AcquireReadAccess();

	/** Stop the multitheading read-only access detection. */
	UE_API void ReleaseReadAccess();

	/** Start the invalid multithreading write access detection. */
	UE_API void AcquireWriteAccess();

	/** Stop the multitheading write access detection. */
	UE_API void ReleaseWriteAccess();

	/** Get the data used at runtime to confirm the inner working of the StateTree. */
	UE_API UE::StateTree::Debug::FRuntimeValidation GetRuntimeValidation() const;

private:
	/**
	 * Struct for global and active instances.
	 * The buffer format is:
	 *  for each frames
	 *    Global parameters, if it's a global frame.
	 *    Global node instances, if it's a global frame. (evaluator, global tasks)
	 *    Active state parameters
	 *    Active node instances (tasks)
	 * @note Not transient, as we use FStateTreeInstanceData to store default values for instance data.
	 */
	UPROPERTY()
	FInstancedStructContainer InstanceStructs;

	/** Execution state of the state tree instance. */
	UPROPERTY(Transient)
	FStateTreeExecutionState ExecutionState;

	/**
	 * Struct for the execution runtime data.
	 * They stay alive until the owning execution context stops.
	 */
	UPROPERTY(Transient)
	UE::StateTree::InstanceData::FInstanceContainer ExecutionRuntimeData;

	/** Info to find the index where the execution runtime data starts for a specific state tree. */
	struct FExecutionRuntimeInfo
	{
		FObjectKey StateTree;
		int32 StartIndex = 0;
	};
	TArray<FExecutionRuntimeInfo, TInlineAllocator<1>> ExecutionRuntimeDataInfos;

	/** Temporary instances */
	UPROPERTY(Transient)
	TArray<FStateTreeTemporaryInstanceData> TemporaryInstances;

	/** Events (Transient) */
	TSharedRef<FStateTreeEventQueue> EventQueue = MakeShared<FStateTreeEventQueue>();
	
	/** Array of broadcasted delegates. */
	TArray<FStateTreeDelegateDispatcher> BroadcastedDelegates;

	/** Requested transitions */
	UPROPERTY(Transient)
	TArray<FStateTreeTransitionRequest> TransitionRequests;

	/** Global parameters */
	UPROPERTY(Transient)
	FInstancedStruct GlobalParameters;

	/** Unique id.  */
	UPROPERTY(Transient)
	uint32 UniqueIdGenerator = 0;

	/**
	 * Used to detect if we are using the instance data on multiple threads in a safe way.
	 * The instance data supports multiple reader threads or a single writer thread.
	 * The detector supports recursive access.
	 */
	UE_MT_DECLARE_MRSW_RECURSIVE_ACCESS_DETECTOR(AccessDetector);

	/* True if the storage owns the event queue. */
	bool bIsOwningEventQueue = true;

#if WITH_STATETREE_DEBUG
	TPimplPtr<UE::StateTree::Debug::FRuntimeValidationInstanceData> RuntimeValidationData;
#endif

	friend struct FStateTreeInstanceData;
};

/**
 * StateTree instance data is used to store the runtime state of a StateTree.
 * The layout of the data is described in a FStateTreeInstanceDataLayout.
 *
 * Note: If FStateTreeInstanceData is placed on an struct, you must call AddStructReferencedObjects() manually,
 *		 as it is not automatically called recursively.   
 * Note: Serialization is supported only for FArchive::IsModifyingWeakAndStrongReferences(), that is replacing object references.
 */
USTRUCT()
struct FStateTreeInstanceData
{
	GENERATED_BODY()

	UE_API FStateTreeInstanceData();
	UE_API FStateTreeInstanceData(const FStateTreeInstanceData& Other);
	UE_API FStateTreeInstanceData(FStateTreeInstanceData&& Other) noexcept;

	UE_API FStateTreeInstanceData& operator=(const FStateTreeInstanceData& Other);
	UE_API FStateTreeInstanceData& operator=(FStateTreeInstanceData&& Other) noexcept;

	UE_API ~FStateTreeInstanceData();

	struct FAddArgs
	{
		STATETREEMODULE_API static FAddArgs Default;
		/** Duplicate the object contained by object wrapper. */
		bool bDuplicateWrappedObject = true;
	};

	/** Initializes the array with specified items. */
	UE_API void Init(UObject& InOwner, TConstArrayView<FInstancedStruct> InStructs, FAddArgs Args = FAddArgs::Default);
	UE_API void Init(UObject& InOwner, TConstArrayView<FConstStructView> InStructs, FAddArgs Args = FAddArgs::Default);

	/** Appends new items to the instance. */
	UE_API void Append(UObject& InOwner, TConstArrayView<FInstancedStruct> InStructs, FAddArgs Args = FAddArgs::Default);
	UE_API void Append(UObject& InOwner, TConstArrayView<FConstStructView> InStructs, FAddArgs Args = FAddArgs::Default);

	/**
	 * Appends new items to the instance, and moves existing data into the allocated instances.
	 * If the struct is of type FStateTreeInstanceObjectWrapper then this will duplicate the UObject instance.
	 */
	UE_API void Append(UObject& InOwner, TConstArrayView<FConstStructView> InStructs, TConstArrayView<FInstancedStruct*> InInstancesToMove, FAddArgs Args = FAddArgs::Default);
	/**
	 * Appends new items to the instance, and moves existing data into the allocated instances, if provided.
	 * If the struct is of type FStateTreeInstanceObjectWrapper then this will duplicate the UObject instance only if the InstanceToMove is set.
	 */
	UE_API void Append(UObject& InOwner, TConstArrayView<FConstStructView> InStructs, TConstArrayView<TOptional<FInstancedStruct*>> InInstancesToMove, FAddArgs Args = FAddArgs::Default);

	/** Shrinks the array sizes to specified lengths. Sizes must be small or equal than current size. */
	UE_API void ShrinkTo(const int32 Num);
	
	/** Shares the layout from another instance data, and copies the data over. */
	UE_API void CopyFrom(UObject& InOwner, const FStateTreeInstanceData& InOther);

	/** Resets the data to empty. */
	UE_API void Reset();

	/** @return Number of items in the instance data. */
	int32 Num() const
	{
		return GetStorage().Num();
	}

	/** @return true if the specified index is valid index into the instance data container. */
	bool IsValidIndex(const int32 Index) const
	{
		return GetStorage().IsValidIndex(Index);
	}

	/** @return true if the data at specified index is object. */
	bool IsObject(const int32 Index) const
	{
		return GetStorage().IsObject(Index);
	}

	/** @return mutable view to the struct at specified index. */
	FStructView GetMutableStruct(const int32 Index)
	{
		return GetMutableStorage().GetMutableStruct(Index);
	}

	/** @return const view to the struct at specified index. */
	FConstStructView GetStruct(const int32 Index) const
	{
		return GetStorage().GetStruct(Index);
	}

	/** @return pointer to an instance object   */
	UObject* GetMutableObject(const int32 Index)
	{
		return GetMutableStorage().GetMutableObject(Index);
	}

	/** @return const pointer to an instance object   */
	const UObject* GetObject(const int32 Index) const
	{
		return GetStorage().GetObject(Index);
	}

	/** @return pointer to StateTree execution state, or null if the instance data is not initialized. */
	const FStateTreeExecutionState* GetExecutionState() const
	{
		return &GetStorage().GetExecutionState();
	}

	/** @return mutable pointer to StateTree execution state, or null if the instance data is not initialized. */
	FStateTreeExecutionState* GetMutableExecutionState()
	{
		return &GetMutableStorage().GetMutableExecutionState();
	}

	/** @return reference to the event queue. */
	UE_API FStateTreeEventQueue& GetMutableEventQueue();
	UE_API const FStateTreeEventQueue& GetEventQueue() const;
	UE_API const TSharedRef<FStateTreeEventQueue>& GetSharedMutableEventQueue();

	/** @return true if the instance data owns its' event queue. */
	UE_API bool IsOwningEventQueue() const;

	/** Sets event queue from another instance data. Marks the event queue not owned. */
	UE_API void SetSharedEventQueue(const TSharedRef<FStateTreeEventQueue>& InSharedEventQueue);

	/** 
	 * Buffers a transition request to be sent to the State Tree.
	 * @param Owner Optional pointer to an owner UObject that is used for logging errors.
	 * @param Request transition to request.
	*/
	UE_API void AddTransitionRequest(const UObject* Owner, const FStateTreeTransitionRequest& Request);

	/** @return currently pending transition requests. */
	UE_API TConstArrayView<FStateTreeTransitionRequest> GetTransitionRequests() const;

	/** Reset all pending transition requests. */
	UE_API void ResetTransitionRequests();

	/** @return true if all instances are valid. */
	UE_API bool AreAllInstancesValid() const;

	UE_API FStateTreeInstanceStorage& GetMutableStorage();
	UE_API const FStateTreeInstanceStorage& GetStorage() const;

	UE_API TWeakPtr<FStateTreeInstanceStorage> GetWeakMutableStorage();
	UE_API TWeakPtr<const FStateTreeInstanceStorage> GetWeakStorage() const;

	UE_API int32 GetEstimatedMemoryUsage() const;
	
	/** Type traits */
	UE_API bool Identical(const FStateTreeInstanceData* Other, uint32 PortFlags) const;
	UE_API void AddStructReferencedObjects(FReferenceCollector& Collector);
	UE_API bool Serialize(FArchive& Ar);
	UE_API void GetPreloadDependencies(TArray<UObject*>& OutDeps);

	/**
	 * Adds temporary instance data associated with specified frame and data handle.
	 * @returns mutable struct view to the instance.
	 */
	FStructView AddTemporaryInstance(UObject& InOwner, const FStateTreeExecutionFrame& Frame, const FStateTreeIndex16 OwnerNodeIndex, const FStateTreeDataHandle DataHandle, FConstStructView NewInstanceData)
	{
		return GetMutableStorage().AddTemporaryInstance(InOwner, Frame, OwnerNodeIndex, DataHandle, NewInstanceData);
	}
	
	/** @returns mutable view to the specified instance data, or invalid view if not found. */
	FStructView GetMutableTemporaryStruct(const FStateTreeExecutionFrame& Frame, const FStateTreeDataHandle DataHandle)
	{
		return GetMutableStorage().GetMutableTemporaryStruct(Frame.FrameID, DataHandle);
	}

	/** @returns mutable pointer to the specified instance data object, or invalid view if not found. Will check() if called on non-object data. */
	UObject* GetMutableTemporaryObject(const FStateTreeExecutionFrame& Frame, const FStateTreeDataHandle DataHandle)
	{
		return GetMutableStorage().GetMutableTemporaryObject(Frame.FrameID, DataHandle);
	}

	/** Empties the temporary instances. */
	void ResetTemporaryInstances()
	{
		return GetMutableStorage().ResetTemporaryInstances();
	}

	/** Get the data used at runtime to confirm the inner working of the StateTree. */
	UE::StateTree::Debug::FRuntimeValidation GetRuntimeValidation() const
	{
		return GetStorage().GetRuntimeValidation();
	}

protected:
	/** Storage for the actual instance data, always stores FStateTreeInstanceStorage. */
	TSharedRef<FStateTreeInstanceStorage> InstanceStorage = MakeShared<FStateTreeInstanceStorage>();

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS

	UPROPERTY()
	TInstancedStruct<FStateTreeInstanceStorage> InstanceStorage_DEPRECATED;
	
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
};

#if WITH_EDITORONLY_DATA
namespace UE::StateTree
{
	void RegisterInstanceDataForLocalization();
}
#endif // WITH_EDITORONLY_DATA

template<>
struct TStructOpsTypeTraits<FStateTreeInstanceData> : public TStructOpsTypeTraitsBase2<FStateTreeInstanceData>
{
	enum
	{
		WithIdentical = true,
		WithAddStructReferencedObjects = true,
		WithSerializer = true,
		WithGetPreloadDependencies = true,
	};
};


/**
 * Stores indexed reference to a instance data struct.
 * The instance data structs may be relocated when the instance data composition changed. For that reason you cannot store pointers to the instance data.
 * This is often needed for example when dealing with delegate lambda's. This helper struct stores data to be able to find the instance data in the instance data array.
 * That way we can access the instance data even of the array changes, and the instance data moves in memory.
 *
 * Note that the reference is valid only during the lifetime of a task (between a call EnterState() and ExitState()). 
 *
 * You generally do not use this directly, but via FStateTreeExecutionContext.
 *
 *	EStateTreeRunStatus FTestTask::EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
 *	{
 *		FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
 *
 *		Context.GetWorld()->GetTimerManager().SetTimer(
 *	        InstanceData.TimerHandle,
 *	        [InstanceDataRef = Context.GetInstanceDataStructRef(*this)]()
 *	        {
 *	            if (FInstanceDataType* InstanceData = InstanceDataRef.GetPtr())
 *				{
 *		            ...
 *				}
 *	        },
 *	        Delay, true);
 *
 *	    return EStateTreeRunStatus::Running;
 *	}
 */
template <typename T>
struct TStateTreeInstanceDataStructRef
{
	TStateTreeInstanceDataStructRef(TWeakPtr<FStateTreeInstanceStorage> InWeakStorage, const FStateTreeExecutionFrame& CurrentFrame, const FStateTreeDataHandle InDataHandle)
		: WeakStorage(InWeakStorage)
		, FrameID(CurrentFrame.FrameID)
		, DataHandle(InDataHandle)
	{
		checkf(InDataHandle.GetSource() == EStateTreeDataSourceType::ActiveInstanceData
			|| InDataHandle.GetSource() == EStateTreeDataSourceType::ActiveInstanceDataObject
			|| InDataHandle.GetSource() == EStateTreeDataSourceType::GlobalInstanceData
			|| InDataHandle.GetSource() == EStateTreeDataSourceType::GlobalInstanceDataObject,
			TEXT("TStateTreeInstanceDataStructRef supports only instance data."));
	}

	TStateTreeInstanceDataStructRef(FStateTreeInstanceData& InInstanceData, const FStateTreeExecutionFrame& CurrentFrame, const FStateTreeDataHandle InDataHandle)
		: TStateTreeInstanceDataStructRef(InInstanceData.GetWeakMutableStorage(), CurrentFrame, InDataHandle)
	{
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TStateTreeInstanceDataStructRef(const TStateTreeInstanceDataStructRef& Other) = default;
	TStateTreeInstanceDataStructRef(TStateTreeInstanceDataStructRef&& Other) = default;
	TStateTreeInstanceDataStructRef& operator=(TStateTreeInstanceDataStructRef const& Other) = default;
	TStateTreeInstanceDataStructRef& operator=(TStateTreeInstanceDataStructRef&& Other) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	bool IsValid() const
	{
		return FrameID.IsValid() && DataHandle.IsValid();
	}

	T* GetPtr() const
	{
		TSharedPtr<FStateTreeInstanceStorage> StoragePtr = WeakStorage.Pin();
		if (StoragePtr == nullptr)
		{
			return nullptr;
		}

		const FStateTreeExecutionState& Exec = StoragePtr->GetExecutionState();
		const FStateTreeExecutionFrame* CurrentFrame = Exec.FindActiveFrame(FrameID);

		FStateTreeDataView DataView;
		if (CurrentFrame)
		{
			FStateTreeInstanceStorage& Storage = *(StoragePtr.Get());
			if (IsHandleSourceValid(Storage, *CurrentFrame, DataHandle))
			{
				DataView = GetDataView(Storage, *CurrentFrame, DataHandle);
			}
			else
			{
				DataView = Storage.GetMutableTemporaryStruct(CurrentFrame->FrameID, DataHandle);
			}
		}
		else
		{
			// When selecting a state, the frame is not in the active list.
			FStateTreeTemporaryInstanceData* ExistingInstance = StoragePtr->GetMutableTemporaryInstances().FindByPredicate([FrameID = FrameID, DataHandle = DataHandle](const FStateTreeTemporaryInstanceData& TempInstance)
				{
					return TempInstance.FrameID == FrameID
						&& TempInstance.DataHandle == DataHandle;
				});
			if (ExistingInstance)
			{
				if (const FStateTreeInstanceObjectWrapper* ObjectWrapper = ExistingInstance->Instance.GetPtr<FStateTreeInstanceObjectWrapper>())
				{
					DataView = FStateTreeDataView(ObjectWrapper->InstanceObject);
				}
				else
				{
					DataView = FStateTreeDataView(ExistingInstance->Instance);
				}
			}
		}

		if constexpr (!std::is_same_v<T, void>)
		{
			const bool bIsValidType = DataView.GetStruct() == nullptr || DataView.GetStruct()->IsChildOf<T>();
			if (!ensure(bIsValidType))
			{
				return nullptr;
			}
		}

		return static_cast<T*>(DataView.GetMutableMemory());
	}

protected:
	FStateTreeDataView GetDataView(FStateTreeInstanceStorage& Storage, const FStateTreeExecutionFrame& CurrentFrame, const FStateTreeDataHandle Handle) const
	{
		switch (Handle.GetSource())
		{
		case EStateTreeDataSourceType::GlobalInstanceData:
			return Storage.GetMutableStruct(CurrentFrame.GlobalInstanceIndexBase.Get() + Handle.GetIndex());
		case EStateTreeDataSourceType::GlobalInstanceDataObject:
			return Storage.GetMutableObject(CurrentFrame.GlobalInstanceIndexBase.Get() + Handle.GetIndex());
		case EStateTreeDataSourceType::ActiveInstanceData:
			return Storage.GetMutableStruct(CurrentFrame.ActiveInstanceIndexBase.Get() + Handle.GetIndex());
		case EStateTreeDataSourceType::ActiveInstanceDataObject:
			return Storage.GetMutableObject(CurrentFrame.ActiveInstanceIndexBase.Get() + Handle.GetIndex());
		default:
			checkf(false, TEXT("Unhandle case %s"), *UEnum::GetValueAsString(Handle.GetSource()));
		}
		return {};
	}
	
	bool IsHandleSourceValid(FStateTreeInstanceStorage& Storage, const FStateTreeExecutionFrame& CurrentFrame, const FStateTreeDataHandle Handle) const
	{
		switch (Handle.GetSource())
		{
		case EStateTreeDataSourceType::GlobalInstanceData:
		case EStateTreeDataSourceType::GlobalInstanceDataObject:
			return CurrentFrame.GlobalInstanceIndexBase.IsValid()
				&& Storage.IsValidIndex(CurrentFrame.GlobalInstanceIndexBase.Get() + Handle.GetIndex());

		case EStateTreeDataSourceType::ActiveInstanceData:
		case EStateTreeDataSourceType::ActiveInstanceDataObject:
			return CurrentFrame.ActiveInstanceIndexBase.IsValid()
				&& CurrentFrame.ActiveStates.Contains(Handle.GetState())
				&& Storage.IsValidIndex(CurrentFrame.ActiveInstanceIndexBase.Get() + Handle.GetIndex());
		default:
			checkf(false, TEXT("Unhandle case %s"), *UEnum::GetValueAsString(Handle.GetSource()));
		}
		return false;
	}
	
	TWeakPtr<FStateTreeInstanceStorage> WeakStorage = nullptr;
	UE::StateTree::FActiveFrameID FrameID;
	FStateTreeDataHandle DataHandle = FStateTreeDataHandle::Invalid;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.6, "Use the frame id to identify the frame.")
	TWeakObjectPtr<const UStateTree> WeakStateTree = nullptr;
	UE_DEPRECATED(5.6, "Use the frame id to identify the frame.")
	FStateTreeStateHandle RootState = FStateTreeStateHandle::Invalid;
#endif
};

#undef UE_API
