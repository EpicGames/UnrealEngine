// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTree.h"
#include "StateTreeSchema.h"
#include "StateTreeExecutionTypes.h"
#include "Templates/Casts.h"
#include "StateTreeLinker.generated.h"

#define UE_API STATETREEMODULE_API

UENUM()
enum class EStateTreeLinkerStatus : uint8
{
	Succeeded,
	Failed,
};

/**
 * The StateTree linker is used to resolved references to various StateTree data at load time.
 * @see TStateTreeExternalDataHandle<> for example usage.
 */
struct FStateTreeLinker
{
	UE_API explicit FStateTreeLinker(TNotNull<const UStateTree*> InStateTree);

	UE_DEPRECATED(5.7, "Use the constructor with the StateTree pointer.")
	explicit FStateTreeLinker(const UStateTreeSchema* InSchema)
		: Schema(InSchema)
	{}
	
	/** @returns the linking status. */
	EStateTreeLinkerStatus GetStatus() const
	{
		return Status;
	}

	/** @returns the StateTree asset. */
	const UStateTree* GetStateTree() const
	{
		return StateTree.Get();
	}
	
	/**
	 * Links reference to an external UObject.
	 * @param Handle Reference to TStateTreeExternalDataHandle<> with UOBJECT type to link to.
	 */
	template <typename T>
	typename TEnableIf<TIsDerivedFrom<typename T::DataType, UObject>::IsDerived, void>::Type LinkExternalData(T& Handle)
	{
		LinkExternalData(Handle, T::DataType::StaticClass(), T::DataRequirement);
	}

	/**
	 * Links reference to an external UStruct.
	 * @param Handle Reference to TStateTreeExternalDataHandle<> with USTRUCT type to link to.
	 */
	template <typename T>
	typename TEnableIf<!TIsDerivedFrom<typename T::DataType, UObject>::IsDerived && !TIsIInterface<typename T::DataType>::Value, void>::Type LinkExternalData(T& Handle)
	{
		LinkExternalData(Handle, T::DataType::StaticStruct(), T::DataRequirement);
	}

	/**
	 * Links reference to an external IInterface.
	 * @param Handle Reference to TStateTreeExternalDataHandle<> with IINTERFACE type to link to.
	 */
	template <typename T>
	typename TEnableIf<TIsIInterface<typename T::DataType>::Value, void>::Type LinkExternalData(T& Handle)
	{
		LinkExternalData(Handle, T::DataType::UClassType::StaticClass(), T::DataRequirement);
	}

	/**
	 * Links reference to an external Object or Struct.
	 * This function should only be used when TStateTreeExternalDataHandle<> cannot be used, i.e. the Struct is based on some data.
	 * @param Handle Reference to link to.
	 * @param Struct Expected type of the Object or Struct to link to.
	 * @param Requirement Describes if the external data is expected to be required or optional.
	 */
	UE_API void LinkExternalData(FStateTreeExternalDataHandle& Handle, const UStruct* Struct, const EStateTreeExternalDataRequirement Requirement);

	/** @return linked external data descriptors. */
	const TArrayView<const FStateTreeExternalDataDesc> GetExternalDataDescs() const
	{
		return ExternalDataDescs;
	}

protected:
	TStrongObjectPtr<const UStateTree> StateTree;
	TStrongObjectPtr<const UStateTreeSchema> Schema;
	EStateTreeLinkerStatus Status = EStateTreeLinkerStatus::Succeeded;
	TArray<FStateTreeExternalDataDesc> ExternalDataDescs;
};

#undef UE_API
