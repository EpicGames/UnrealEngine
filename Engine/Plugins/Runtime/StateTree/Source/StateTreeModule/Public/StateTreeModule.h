// Copyright Epic Games, Inc. All Rights Reserved.

#pragma  once

#include "Misc/NotNull.h"
#include "Modules/ModuleManager.h"

class IStateTreeDebugInfoProvider;
class UStateTree;
struct FStateTreeIndex16;

namespace UE::Trace
{
	class FStoreClient;
}

#define UE_API STATETREEMODULE_API

/**
* The public interface to this module
*/
class IStateTreeModule : public IModuleInterface
{

public:

	/**
	* Singleton-like access to this module's interface.  This is just for convenience!
	* Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	*
	* @return Returns singleton instance, loading the module on demand if needed
	*/
	static IStateTreeModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IStateTreeModule>("StateTreeModule");
	}

	/**
	* Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	*
	* @return True if the module is loaded and ready to use
	*/
	static bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("StateTreeModule");
	}

	/**
	 * Creates new tracing connection if necessary and enables StateTree debugging related channels (frame + statetree).
	 * If traces are already active we keep track of all channels previously activated to restore them on stop.
	 * @param OutTraceId In case a connection is already active this indicates its id, 0 otherwise.
	 * @return True if a new trace connection was created, false otherwise (already active or not created)
	 * @note Even if it returns false this can still have enabled StateTree related channels, OutTraceId will indicate
	 * which trace to use and StopTraces should be called to restore to previous setup.
	 */
	virtual bool StartTraces(int32& OutTraceId) = 0;

	/**
	 * Stops the trace service if it was not already connected when StartTraces was called.
	 * Restores previously enabled channels if necessary.
	 */
	virtual void StopTraces() = 0;

	/**
	 * Indicates if the statetree specific traces are active (explicitly started by StartTraces).
	 * @return True is StartTraces was called, false otherwise.
	 */
	virtual bool IsTracing() const = 0;

#if WITH_STATETREE_TRACE_DEBUGGER
	/**
	 * Returns debug info provider. Creates on get if needed.
	 * @return ptr to debug info provider to query against. 
	 */
	virtual TNotNull<const IStateTreeDebugInfoProvider*> GetDebugInfoProvider() = 0;

	/**
	 * Sets debug info provider.
	 * @param ptr to debug info provider to query against. 
	 */
	virtual void SetDebugInfoProvider(TSharedPtr<IStateTreeDebugInfoProvider> InDebugInfoProvider) = 0;

	/**
	 * Gets the store client.
	 */
	virtual UE::Trace::FStoreClient* GetStoreClient() = 0;
#endif // WITH_STATETREE_TRACE_DEBUGGER
};


/**
 * Allows runtime to use editor data for debugging wben available & fallback to runtime only if needed.
 */
class IStateTreeDebugInfoProvider
{
public:
	virtual ~IStateTreeDebugInfoProvider() = default;

	/** Get text desc for the node */
	virtual FText GetNodeDescription(TNotNull<const UStateTree*> StateTree, FStateTreeIndex16 NodeIndex) const = 0;
};

#undef UE_API
