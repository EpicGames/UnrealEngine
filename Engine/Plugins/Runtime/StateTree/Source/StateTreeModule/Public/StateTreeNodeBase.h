// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeTypes.h"
#include "StateTreeNodeBase.generated.h"

struct FStateTreeBindableStructDesc;
struct FStateTreeLinker;
struct FPropertyBindingPath;
struct FStateTreePropertyPath;
struct IStateTreeBindingLookup;

#if WITH_STATETREE_TRACE
#define SET_NODE_CUSTOM_TRACE_TEXT(Context, MergePolicy, Format, ...) \
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(StateTreeDebugChannel)) \
	{ \
		Context.SetNodeCustomDebugTraceData( \
			UE::StateTreeTrace::FNodeCustomDebugData(FString::Printf(Format, ##__VA_ARGS__) \
													, ::UE::StateTreeTrace::FNodeCustomDebugData::EMergePolicy::MergePolicy)); \
	}
#else
#define SET_NODE_CUSTOM_TRACE_TEXT(...)
#endif //WITH_STATETREE_TRACE

#if WITH_EDITOR
namespace UE::StateTree
{
	struct ICompileNodeContext
	{
		virtual ~ICompileNodeContext() = default;
		virtual void AddValidationError(const FText& Message) = 0;
		virtual FStateTreeDataView GetInstanceDataView() const = 0;
		virtual bool HasBindingForProperty(const FName PropertyName) const = 0;
	};
}
#endif

/**
 * Enum describing in what format a text is expected to be returned.
 *
 * - Normal text should be used for values
 * - Bold text should generally be used for actions, like name a of a task "<b>Play Animation</> {AnimName}".
 * - Subdued should be generally used for secondary/structural information, like "{Left} <s>equals</> {Right}".
 */
UENUM(BlueprintType)
enum class EStateTreeNodeFormatting : uint8
{
	/**
	 * The returned text can contain following right text formatting (no nesting)
	 *	- <b>Bold</> (bolder font is used)
	 *	- <s>Subdued</> (normal font with lighter color) */
	RichText,
	
	/** The text should be unformatted */
	Text,
};

/**
 * Base struct of StateTree Conditions, Considerations, Evaluators, and Tasks.
 */
USTRUCT()
struct FStateTreeNodeBase
{
	GENERATED_BODY()

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FStateTreeNodeBase() = default;
	FStateTreeNodeBase(const FStateTreeNodeBase&) = default;
	FStateTreeNodeBase(FStateTreeNodeBase&&) = default;
	FStateTreeNodeBase& operator=(const FStateTreeNodeBase&) = default;
	FStateTreeNodeBase& operator=(FStateTreeNodeBase&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	struct FNoInstanceDataType
	{ };

	/**
	 * The instance data type. The implementation node should set its type.
	 * The type should match GetInstanceDataType().
	 */
	using FInstanceDataType = FNoInstanceDataType;

	/**
	 * The execution runtime data type.
	 * Execution runtime is optional. If needed, the implementation node should set its type.
	 * The type should match GetExecutionRuntimeDataType().
	 */
	using FExecutionRuntimeDataType = FNoInstanceDataType;

	virtual ~FStateTreeNodeBase() = default;

	/** @return Struct that represents the runtime data of the node. */
	virtual const UStruct* GetInstanceDataType() const
	{
		return nullptr;
	}

	/**
	 * Execution runtime data is always valid between an FStateTreeExecutionContext::Start and FStateTreeExecutionContext::Stop.
	 * If the node is no longer active but the instance (FStateTreeInstanceData) is active, the data is still valid and will persist.
	 * @note Be careful about UObject reference, the objects are garbage collected like normal references.
	 * @return the struct that represents the persistent runtime data of the node.
	 */
	virtual const UStruct* GetExecutionRuntimeDataType() const
	{
		return nullptr;
	}

	/**
	 * Called when the StateTree asset is linked. Allows to resolve references to other StateTree data.
	 * @see TStateTreeExternalDataHandle
	 * @param Linker Reference to the linker
	 * @return true if linking succeeded. 
	 */
	[[nodiscard]] virtual bool Link(FStateTreeLinker& Linker)
	{
		return true;
	}

#if WITH_EDITOR
	/**
	 * Called during State Tree compilation, allows to modify and validate the node and instance data.
	 * The method is called with node and instance that is duplicated during compilation and used at runtime (it's different than the data used in editor).
	 * @param ValidationMessages Any messages to report during validation. Displayed as errors if the validation result is Invalid, else as warnings.
	 * @param CompileContext
	 * @return Validation result based on if the validation succeeded or not. Returning Invalid will fail compilation and messages will be displayed as errors.
	 */
	virtual EDataValidationResult Compile(UE::StateTree::ICompileNodeContext& CompileContext)
	{
		return EDataValidationResult::NotValidated;
	}

	/**
	 * Called during State Tree compilation, allows to modify and validate the node and instance data.
	 * The method is called with node and instance that is duplicated during compilation and used at runtime (it's different than the data used in editor).  
	 * @param InstanceDataView Pointer to the instance data.
	 * @param ValidationMessages Any messages to report during validation. Displayed as errors if the validation result is Invalid, else as warnings.
	 * @return Validation result based on if the validation succeeded or not. Returning Invalid will fail compilation and messages will be displayed as errors.
	 */
	UE_DEPRECATED(5.6, "Use the version with Binding infos instead")
	virtual EDataValidationResult Compile(FStateTreeDataView InstanceDataView, TArray<FText>& ValidationMessages) final { return EDataValidationResult::NotValidated; }

	/**
	 * Returns description for the node, use in the UI.
	 * The UI description is selected as follows: 
	 * - Node Name, if not empty
	 * - Description if not empty
	 * - Display name of the node struct
	 * @param ID ID of the item, can be used make property paths to this item.
	 * @param InstanceDataView View to the instance data, can be struct or class.
	 * @param BindingLookup Reference to binding lookup which can be used to reason about property paths.
	 * @param Formatting Requested formatting (whether rich or plain text should be returned).
	 */
	virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const
	{
		return FText::GetEmpty();
	}

	/**
	 * @returns name of the icon in format:
	 *		StyleSetName | StyleName [ | SmallStyleName | StatusOverlayStyleName]
	 *		SmallStyleName and StatusOverlayStyleName are optional.
	 *		Example: "StateTreeEditorStyle|Node.Animation" 
	 */
	virtual FName GetIconName() const
	{
		return FName();
	}

	/** @return the color to be used with the icon. */
	virtual FColor GetIconColor() const
	{
		return UE::StateTree::Colors::DarkGrey;
	}

	/**
	 * Called when binding of any of the properties in the node changes.
	 * @param ID ID of the item, can be used make property paths to this item.
	 * @param InstanceDataView view to the instance data, can be struct or class.
	 * @param SourcePath Source path of the new binding.
	 * @param TargetPath Target path of the new binding (the property in the condition).
	 * @param BindingLookup Reference to binding lookup which can be used to reason about property paths.
	 */
	virtual void OnBindingChanged(const FGuid& ID, FStateTreeDataView InstanceDataView, const FPropertyBindingPath& SourcePath, const FPropertyBindingPath& TargetPath, const IStateTreeBindingLookup& BindingLookup) {}
	UE_DEPRECATED(5.6, "Use the version taking FPropertyBindingPath instead")
	virtual void OnBindingChanged(const FGuid& ID, FStateTreeDataView InstanceDataView, const FStateTreePropertyPath& SourcePath, const FStateTreePropertyPath& TargetPath, const IStateTreeBindingLookup& BindingLookup) final {}

	/**
	 * Called when a property of the node has been modified externally
	 * @param PropertyChangedEvent The event for the changed property. PropertyChain's active properties are set relative to node.
	 * @param InstanceData view to the instance data, can be struct or class.
	 */
	virtual void PostEditNodeChangeChainProperty(const FPropertyChangedChainEvent& PropertyChangedEvent, FStateTreeDataView InstanceDataView) {}

	/**
	 * Called when a property of node's instance data has been modified externally
	 * @param PropertyChangedEvent The event for the changed property. PropertyChain's active properties are set relative to instance data.
	 * @param InstanceData view to the instance data, can be struct or class.
	 */
	virtual void PostEditInstanceDataChangeChainProperty(const FPropertyChangedChainEvent& PropertyChangedEvent, FStateTreeDataView InstanceDataView) {}
#endif

	/**
	 * Called after the state tree asset that contains this node is loaded from disk.
	 * @param InstanceDataView view to the instance data, can be struct or class.
	 */
	virtual void PostLoad(FStateTreeDataView InstanceDataView) {}

	/** Name of the node. */
	UPROPERTY(EditDefaultsOnly, Category = "", meta=(EditCondition = "false", EditConditionHides))
	FName Name;

	/** Property binding copy batch handle. */
	UPROPERTY()
	FStateTreeIndex16 BindingsBatch = FStateTreeIndex16::Invalid;

	/** Property output binding copy batch handle. */
	UPROPERTY()
	FStateTreeIndex16 OutputBindingsBatch = FStateTreeIndex16::Invalid;

	/** Index of template instance data for the node. Can point to Shared or Default instance data in StateTree depending on node type. */
	UPROPERTY()
	FStateTreeIndex16 InstanceTemplateIndex = FStateTreeIndex16::Invalid;

	/** Index of template execution runtime data for the node. */
	UPROPERTY()
	FStateTreeIndex16 ExecutionRuntimeTemplateIndex = FStateTreeIndex16::Invalid;

	/** Data handle to access the instance data. */
	UPROPERTY()
	FStateTreeDataHandle InstanceDataHandle = FStateTreeDataHandle::Invalid;
};
