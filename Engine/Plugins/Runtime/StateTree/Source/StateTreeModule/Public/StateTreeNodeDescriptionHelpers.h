// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "AITypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#include "Internationalization/Text.h"
#include "StateTreeTypes.h"
#include "StateTreePropertyBindings.h"

enum class EGenericAICheck : uint8;
enum class EStateTreeNodeFormatting : uint8;
struct FGameplayTagContainer;
struct FGameplayTagQuery;

namespace UE::StateTree::DescHelpers
{
#if WITH_EDITOR || WITH_STATETREE_TRACE

/** @return description for a EComparisonOperator. */
extern STATETREEMODULE_API FText GetOperatorText(const EComparisonOperator Operator, EStateTreeNodeFormatting Formatting);

UE_DEPRECATED(5.8, "Use GetOperatorText with EComparisonOperator instead.")
/** @return description for a EComparisonOperator. */
extern STATETREEMODULE_API FText GetOperatorText(const EGenericAICheck Operator, EStateTreeNodeFormatting Formatting);

/** @return description for condition inversion (returns "Not" plus a space). */
extern STATETREEMODULE_API FText GetInvertText(bool bInvert, EStateTreeNodeFormatting Formatting);

/** @return description of a boolean value. */
extern STATETREEMODULE_API FText GetBoolText(bool bValue, EStateTreeNodeFormatting Formatting);

/** @return description of a float interval. */
extern STATETREEMODULE_API FText GetIntervalText(const FFloatInterval& Interval, EStateTreeNodeFormatting Formatting);

/** @return description of a float interval. */
extern STATETREEMODULE_API FText GetIntervalText(float Min, float Max, EStateTreeNodeFormatting Formatting);

/** @return description of a float interval. */
extern STATETREEMODULE_API FText GetIntervalText(const FText& MinValueText, const FText& MaxValueText, EStateTreeNodeFormatting Formatting);

/** @return description for a Gameplay Tag Container. If the length of container description is longer than ApproxMaxLength, the it truncated and ... as added to the end. */
extern STATETREEMODULE_API FText GetGameplayTagContainerAsText(const FGameplayTagContainer& TagContainer, const int ApproxMaxLength = 60);

/** @return description for a Gameplay Tag Query. If the query description is longer than ApproxMaxLength, the it truncated and ... as added to the end. */
extern STATETREEMODULE_API FText GetGameplayTagQueryAsText(const FGameplayTagQuery& TagQuery, const int ApproxMaxLength = 120);

/** @return description for exact match, used for Gameplay Tag matching functions (returns "Exactly" plus space). */
extern STATETREEMODULE_API FText GetExactMatchText(bool bExactMatch, EStateTreeNodeFormatting Formatting);

/** @return description of a vector value. */
extern STATETREEMODULE_API FText GetText(const FVector& Value, EStateTreeNodeFormatting Formatting);

/** @return description of a float value. */
extern STATETREEMODULE_API FText GetText(float Value, EStateTreeNodeFormatting Formatting);

/** @return description of an int value. */
extern STATETREEMODULE_API FText GetText(int32 Value, EStateTreeNodeFormatting Formatting);

/** @return description of a UObject value. */
extern STATETREEMODULE_API FText GetText(const UObject* Value, EStateTreeNodeFormatting Formatting);

extern STATETREEMODULE_API FText GetMathOperationText(const FText& OperationText, const FText& LeftText, const FText& RightText, EStateTreeNodeFormatting Formatting);

extern STATETREEMODULE_API FText GetSingleParamFunctionText(const FText& FunctionText, const FText& ParamText, EStateTreeNodeFormatting Formatting);

#endif // WITH_EDITOR || WITH_STATETREE_TRACE

#if WITH_EDITOR

/** @return description in the form of (Left OperationText Right).
*	Expect TInstanceDataType to have a member Left and Right whose types have an overloaded UE::StateTree::DescHelpers::GetText function.
*/
template <typename TInstanceDataType>
FText GetDescriptionForMathOperation(FText OperationText, const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting)
{
	const TInstanceDataType& InstanceData = InstanceDataView.Get<TInstanceDataType>();

	FText LeftValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(TInstanceDataType, Left)), Formatting);
	if (LeftValue.IsEmpty())
	{
		LeftValue = UE::StateTree::DescHelpers::GetText(InstanceData.Left, Formatting);
	}

	FText RightValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(TInstanceDataType, Right)), Formatting);
	if (RightValue.IsEmpty())
	{
		RightValue = UE::StateTree::DescHelpers::GetText(InstanceData.Right, Formatting);
	}

	return GetMathOperationText(OperationText, LeftValue, RightValue, Formatting);
}

/** @return description in the form of OperationText(Input).
 *	Expect TInstanceDataType to have a member input whose type has an overloaded UE::StateTree::DescHelpers::GetText function.
 */
template <typename TInstanceDataType>
FText GetDescriptionForSingleParameterFunc(FText OperationText, const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting)
{
	const TInstanceDataType& InstanceData = InstanceDataView.Get<TInstanceDataType>();

	FText InputValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(TInstanceDataType, Input)), Formatting);
	if (InputValue.IsEmpty())
	{
		InputValue = UE::StateTree::DescHelpers::GetText(InstanceData.Input, Formatting);
	}

	return GetSingleParamFunctionText(OperationText, InputValue, Formatting);
}
#endif // WITH_EDITOR
} // namespace UE::StateTree::DescHelpers