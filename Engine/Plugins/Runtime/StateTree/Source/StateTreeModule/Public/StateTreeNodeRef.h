// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeTypes.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/WeakObjectPtr.h"

class UStateTree;
struct FStateTreeTaskBase;

/**
 * A reference to a task that can utilized in a async callback. Use FStateTreeWeakTaskRef to store the reference and Pin it to get the strong version. Similar to StrongPtr and WeakPtr.
 */
struct UE_DEPRECATED(5.6, "FStateTreeStrongTaskRef is deprecated. We now use TaskIndex in WeakExecutionContext.") FStateTreeStrongTaskRef
{
	explicit FStateTreeStrongTaskRef() = default;
#if WITH_STATETREE_DEBUG
	explicit FStateTreeStrongTaskRef(TStrongObjectPtr<const UStateTree> StateTree, const FStateTreeTaskBase* Task, FStateTreeIndex16 NodeIndex, FGuid NodeId);
#else
	explicit FStateTreeStrongTaskRef(TStrongObjectPtr<const UStateTree> StateTree, const FStateTreeTaskBase* Task, FStateTreeIndex16 NodeIndex);
#endif

	STATETREEMODULE_API const UStateTree* GetStateTree() const;
	STATETREEMODULE_API const FStateTreeTaskBase* GetTask() const;
	FStateTreeIndex16 GetTaskIndex() const
	{
		return NodeIndex;
	}

	operator bool() const
	{
		return IsValid();
	}

	STATETREEMODULE_API bool IsValid() const;

private:
	TStrongObjectPtr<const UStateTree> StateTree;
	const FStateTreeTaskBase* Task = nullptr;
	FStateTreeIndex16 NodeIndex = FStateTreeIndex16::Invalid;
#if WITH_STATETREE_DEBUG
	FGuid NodeId;
#endif
};


/**
 * A reference to a task that can be retrieve. Similar to StrongPtr and WeakPtr.
 */
struct UE_DEPRECATED(5.6, "FStateTreeWeakTaskRef is deprecated. We now use TaskIndex in WeakExecutionContext.") FStateTreeWeakTaskRef
{
	explicit FStateTreeWeakTaskRef() = default;
	STATETREEMODULE_API explicit FStateTreeWeakTaskRef(TNotNull<const UStateTree*> StateTree, FStateTreeIndex16 TaskIndex);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	STATETREEMODULE_API FStateTreeStrongTaskRef Pin() const;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	void Release()
	{
		*this = FStateTreeWeakTaskRef();
	}

private:
	TWeakObjectPtr<const UStateTree> StateTree;
	FStateTreeIndex16 NodeIndex = FStateTreeIndex16::Invalid;
#if WITH_STATETREE_DEBUG
	FGuid NodeId;
#endif
};
