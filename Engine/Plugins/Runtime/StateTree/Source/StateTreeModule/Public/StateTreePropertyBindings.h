// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyBindingBindableStructDescriptor.h"
#include "PropertyBindingPath.h"
#include "PropertyBindingBinding.h"
#include "PropertyBindingBindingCollection.h"
#include "StateTreeNodeBase.h"
#include "StateTreeTypes.h"
#include "StateTreePropertyRefHelpers.h"
#include "StructUtils/StructView.h"
#include "StateTreeIndexTypes.h"
#include "StateTreeDelegate.h"
#include "StateTreePropertyBindings.generated.h"

#define UE_API STATETREEMODULE_API

class FProperty;
struct FStateTreePropertyBindingCompiler;
struct FStateTreePropertyRef;
class UStateTree;
enum class EStateTreeNodeFormatting : uint8;

UENUM()
enum class EStateTreeBindableStructSource : uint8
{
	/** Source is StateTree context object */
	Context,
	/** Source is StateTree parameter */
	Parameter,
	/** Source is StateTree evaluator */
	Evaluator,
	/** Source is StateTree global task */
	GlobalTask,
	/** Source is State parameter */
	StateParameter,
	/** Source is State task */
	Task,
	/** Source is State condition */
	Condition,
	/** Source is State utility consideration */
	Consideration,
	/** Source is StateTree event used by transition */
	TransitionEvent,
	/** Source is StateTree event used by state selection */
	StateEvent,
	/** Source is Property Function */
	PropertyFunction,
	/** Source is Transition */
	Transition,
};

namespace UE::StateTree
{
	/** Can that binding type accept a task instance data for a source. */
	[[nodiscard]] UE_API bool AcceptTaskInstanceData(EStateTreeBindableStructSource Target);
}

/**
 * Descriptor for a struct or class that can be a binding source or target.
 * Each struct has unique identifier, which is used to distinguish them, and name that is mostly for debugging and UI.
 */
USTRUCT()
struct FStateTreeBindableStructDesc : public FPropertyBindingBindableStructDescriptor
{
	GENERATED_BODY()

	FStateTreeBindableStructDesc() = default;

#if WITH_EDITORONLY_DATA
	FStateTreeBindableStructDesc(const FString& InStatePath, const FName InName, const UStruct* InStruct, const FStateTreeDataHandle InDataHandle, const EStateTreeBindableStructSource InDataSource, const FGuid InGuid)
		: FPropertyBindingBindableStructDescriptor(InName, InStruct, InGuid)
		, DataHandle(InDataHandle)
		, DataSource(InDataSource)
		, StatePath(InStatePath)
	{
	}

	virtual FString GetSection() const override
	{
		return StatePath;
	}
#endif // WITH_EDITORONLY_DATA

	UE_API virtual FString ToString() const override;

	/** Runtime data the struct represents. */
	UPROPERTY()
	FStateTreeDataHandle DataHandle = FStateTreeDataHandle::Invalid;

	/** Type of the source. */
	UPROPERTY()
	EStateTreeBindableStructSource DataSource = EStateTreeBindableStructSource::Context;

#if WITH_EDITORONLY_DATA
	/** In Editor path to State containing the data. */
	UPROPERTY(Transient)
	FString StatePath;
#endif
};

USTRUCT()
struct UE_DEPRECATED(all, "Use FPropertyBindingPath instead.") FStateTreeEditorPropertyPath
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	/** Handle of the struct this property path is relative to. */
	UPROPERTY()
	FGuid StructID;

	/** Property path segments */
	UPROPERTY()
	TArray<FString> Path;

	bool IsValid() const
	{
		return StructID.IsValid();
	}
#endif // WITH_EDITORONLY_DATA
};

/**
 * Representation of a property binding in StateTree
 */
USTRUCT()
struct FStateTreePropertyPathBinding : public FPropertyBindingBinding
{
	GENERATED_BODY()

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FStateTreePropertyPathBinding() = default;

	FStateTreePropertyPathBinding(const FPropertyBindingPath& InSourcePath, const FPropertyBindingPath& InTargetPath, const bool bInIsOutputBinding)
		: FPropertyBindingBinding(InSourcePath, InTargetPath)
		, bIsOutputBinding(bInIsOutputBinding)
	{
	}

	UE_DEPRECATED(5.7, "Use the version with bInIsOutputBinding instead.")
	FStateTreePropertyPathBinding(const FPropertyBindingPath& InSourcePath, const FPropertyBindingPath& InTargetPath)
		: FStateTreePropertyPathBinding(InSourcePath, InTargetPath, false)
	{
	}

	FStateTreePropertyPathBinding(const FStateTreeDataHandle InSourceDataHandle, const FPropertyBindingPath& InSourcePath, const FPropertyBindingPath& InTargetPath, const bool bInIsOutputBinding)
		: FPropertyBindingBinding(InSourcePath, InTargetPath)
		, SourceDataHandle(InSourceDataHandle)
		, bIsOutputBinding(bInIsOutputBinding)
	{
	}

	UE_DEPRECATED(5.7, "Use the version with bInIsOutputBinding instead.")
	FStateTreePropertyPathBinding(const FStateTreeDataHandle InSourceDataHandle, const FPropertyBindingPath& InSourcePath, const FPropertyBindingPath& InTargetPath)
		: FStateTreePropertyPathBinding(InSourceDataHandle, InSourcePath, InTargetPath, false)
	{
	}

#if WITH_EDITOR
	FStateTreePropertyPathBinding(FConstStructView InFunctionNodeStruct, const FPropertyBindingPath& InSourcePath, const FPropertyBindingPath& InTargetPath)
		: FPropertyBindingBinding(InFunctionNodeStruct, InSourcePath, InTargetPath)
	{
	}
#endif // WITH_EDITOR

UE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITORONLY_DATA
	void PostSerialize(const FArchive& Ar);
#endif // WITH_EDITORONLY_DATA

	void SetSourceDataHandle(const FStateTreeDataHandle NewSourceDataHandle)
	{
		SourceDataHandle = NewSourceDataHandle;
	}

	FStateTreeDataHandle GetSourceDataHandle() const
	{
		return SourceDataHandle;
	}

	void SetIsOutputBinding(const bool bInIsOutputBinding)
	{
		bIsOutputBinding = bInIsOutputBinding;
	}

	bool IsOutputBinding() const
	{
		return bIsOutputBinding;
	}

	void SetCompletionBinding(const UE::StateTree::ETaskCompletionCondition Condition)
	{
		TaskCompletionCondition = Condition;
	}

	TOptional<UE::StateTree::ETaskCompletionCondition> GetCompletionBinding() const
	{
		return TaskCompletionCondition;
	}

	bool IsTaskCompletionBinding() const
	{
		return TaskCompletionCondition.IsSet();
	}

protected:
	virtual FConstStructView GetSourceDataHandleStruct() const override
	{
		return FConstStructView::Make(SourceDataHandle);
	}

private:
	/** Describes how to get the source data pointer for the binding. */
	UPROPERTY()
	FStateTreeDataHandle SourceDataHandle = FStateTreeDataHandle::Invalid;

	/** Whether this binding is reversed(i.e., copying from target to source). */
	UPROPERTY()
	bool bIsOutputBinding = false;

	/**
	 * Whether this is a task completion binding.
	 * The SourcePath.StructID is set to the task node ID (template)
	 * The SourcePath.Segments is empty.
	 */
	UPROPERTY()
	TOptional<UE::StateTree::ETaskCompletionCondition> TaskCompletionCondition;

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY()
	FStateTreeEditorPropertyPath SourcePath_DEPRECATED;

	UPROPERTY()
	FStateTreeEditorPropertyPath TargetPath_DEPRECATED;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
};

#if WITH_EDITORONLY_DATA
template<>
struct TStructOpsTypeTraits<FStateTreePropertyPathBinding> : public TStructOpsTypeTraitsBase2<FStateTreePropertyPathBinding>
{
	enum 
	{
		WithPostSerialize = true,
	};
};
#endif // WITH_EDITORONLY_DATA

/**
 * Representation of a property reference binding in StateTree.
 */
USTRUCT()
struct FStateTreePropertyRefPath
{
	GENERATED_BODY()

	FStateTreePropertyRefPath() = default;

	FStateTreePropertyRefPath(FStateTreeDataHandle InSourceDataHandle, const FPropertyBindingPath& InSourcePath)
		: SourcePropertyPath(InSourcePath)
		, SourceDataHandle(InSourceDataHandle)
	{
	}

	const FPropertyBindingPath& GetSourcePath() const { return SourcePropertyPath; }

	FPropertyBindingPath& GetMutableSourcePath() { return SourcePropertyPath; }

	void SetSourceDataHandle(const FStateTreeDataHandle NewSourceDataHandle) { SourceDataHandle = NewSourceDataHandle; }
	FStateTreeDataHandle GetSourceDataHandle() const { return SourceDataHandle; }

private:
	/** Source property path of the reference */
	UPROPERTY()
	FPropertyBindingPath SourcePropertyPath;

	/** Describes how to get the source data pointer */
	UPROPERTY()
	FStateTreeDataHandle SourceDataHandle = FStateTreeDataHandle::Invalid;
};

/**
 * Describes access to referenced property.
 */
USTRUCT()
struct FStateTreePropertyAccess
{
	GENERATED_BODY()

	/** Source property access. */
	UPROPERTY()
	FPropertyBindingPropertyIndirection SourceIndirection;

	/** Cached pointer to the leaf property of the access. */
	const FProperty* SourceLeafProperty = nullptr;

	/** Type of the source data, used for validation. */
	UPROPERTY(Transient)
	TObjectPtr<const UStruct> SourceStructType = nullptr;

	/** Describes how to get the source data pointer. */
	UPROPERTY()
	FStateTreeDataHandle SourceDataHandle = FStateTreeDataHandle::Invalid;
};

/**
 * Runtime storage and execution of property bindings.
 */
USTRUCT()
struct FStateTreePropertyBindings : public FPropertyBindingBindingCollection
{
	GENERATED_BODY()

	UE_API FStateTreePropertyBindings();

	/**
	 * Clears all bindings.
	 */
	UE_API virtual void OnReset() override;

	/**
	 * @return Referenced property access for provided PropertyRef.
	 */
	UE_API const FStateTreePropertyAccess* GetPropertyAccess(const FStateTreePropertyRef& Reference) const;

	/**
	 * Pointer to referenced property 
	 * @param SourceView Data view to referenced property's owner.
	 * @param PropertyAccess Access to the property for which we want to obtain a pointer.
	 * @return Pointer to referenced property if it's type match, nullptr otherwise.
	 */
	template<class T>
	T* GetMutablePropertyPtr(FStateTreeDataView SourceView, const FStateTreePropertyAccess& PropertyAccess) const;

	//~ Begin FPropertyBindingBindingCollection overrides
	UE_API virtual int32 GetNumBindableStructDescriptors() const override;
	UE_API virtual const FPropertyBindingBindableStructDescriptor* GetBindableStructDescriptorFromHandle(FConstStructView InSourceHandleView) const override;
	UE_API virtual void VisitSourceStructDescriptorInternal(TFunctionRef<EVisitResult(const FPropertyBindingBindableStructDescriptor& Descriptor)> InFunction) const override;
	[[nodiscard]] UE_API virtual bool ResolveBindingCopyInfo(const FPropertyBindingBinding& InResolvedBinding, const FPropertyBindingPathIndirection& InBindingSourceLeafIndirection, const FPropertyBindingPathIndirection& InBindingTargetLeafIndirection, FPropertyBindingCopyInfo& OutCopyInfo) override;
	//~ End FPropertyBindingBindingCollection overrides
protected:

	UE_API const FPropertyBindingBindableStructDescriptor* GetBindableStructDescriptorFromHandle(FStateTreeDataHandle InSourceHandle) const;

	//~ Begin FPropertyBindingBindingCollection overrides
	[[nodiscard]] UE_API virtual bool OnResolvingPaths() override;

	UE_API virtual int32 GetNumBindings() const override;
	UE_API virtual void ForEachBinding(TFunctionRef<void(const FPropertyBindingBinding& Binding)> InFunction) const override;
	UE_API virtual void ForEachBinding(const FPropertyBindingIndex16 InBegin, const FPropertyBindingIndex16 InEnd, const TFunctionRef<void(const FPropertyBindingBinding& Binding, const int32 BindingIndex)> InFunction) const override;
	UE_API virtual void ForEachMutableBinding(TFunctionRef<void(FPropertyBindingBinding& Binding)> InFunction) override;
	UE_API virtual void VisitBindings(TFunctionRef<EVisitResult(const FPropertyBindingBinding& Binding)> InFunction) const override;
	UE_API virtual void VisitBindings(const FPropertyBindingIndex16 InBegin, const FPropertyBindingIndex16 InEnd, TFunctionRef<EVisitResult(const FPropertyBindingBinding& Binding, const int32 BindingIndex)> InFunction) const override;
	UE_API virtual void VisitMutableBindings(TFunctionRef<EVisitResult(FPropertyBindingBinding& Binding)> InFunction) override;

#if WITH_EDITOR
	UE_API virtual FPropertyBindingBinding* AddBindingInternal(const FPropertyBindingPath& InSourcePath, const FPropertyBindingPath& InTargetPath) override;
	UE_API virtual void RemoveBindingsInternal(TFunctionRef<bool(FPropertyBindingBinding&)> InPredicate) override;
	UE_API virtual bool HasBindingInternal(TFunctionRef<bool(const FPropertyBindingBinding&)> InPredicate) const override;
	UE_API virtual const FPropertyBindingBinding* FindBindingInternal(TFunctionRef<bool(const FPropertyBindingBinding&)> InPredicate) const override;
#endif // WITH_EDITOR
	//~ End FPropertyBindingBindingCollection overrides
private:
	/** Array of expected source structs. */
	UPROPERTY()
	TArray<FStateTreeBindableStructDesc> SourceStructs;

	/** Array of property bindings, resolved into arrays of copies before use. */
	UPROPERTY()
	TArray<FStateTreePropertyPathBinding> PropertyPathBindings;

	/** Array of referenced property paths */
	UPROPERTY()
	TArray<FStateTreePropertyRefPath> PropertyReferencePaths;

	/** Array of individually accessed properties */
	UPROPERTY()
	TArray<FStateTreePropertyAccess> PropertyAccesses;

	friend FStateTreePropertyBindingCompiler;
	friend UStateTree;
};

template <class T>
T* FStateTreePropertyBindings::GetMutablePropertyPtr(FStateTreeDataView SourceView, const FStateTreePropertyAccess& PropertyAccess) const
{
	check(SourceView.GetStruct() == PropertyAccess.SourceStructType);

	if (!UE::StateTree::PropertyRefHelpers::Validator<std::remove_cv_t<T>>::IsValid(*PropertyAccess.SourceLeafProperty))
	{
		return nullptr;
	}

	return reinterpret_cast<T*>(Super::GetAddress(SourceView, PropertyAccess.SourceIndirection, PropertyAccess.SourceLeafProperty));
}

/**
 * Helper interface to reason about bound properties. The implementation is in the editor plugin.
 */
struct IStateTreeBindingLookup
{
	virtual ~IStateTreeBindingLookup() = default;

	/** @return Source path for given target path, or null if binding does not exists. */
	virtual const FPropertyBindingPath* GetPropertyBindingSource(const FPropertyBindingPath& InTargetPath) const = 0;

	/** @return Display name given property path. */
	virtual FText GetPropertyPathDisplayName(const FPropertyBindingPath& InPath, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const = 0;

	/** @return Leaf property based on property path. */
	virtual const FProperty* GetPropertyPathLeafProperty(const FPropertyBindingPath& InPath) const = 0;

	/** @return Display name of binding source, or empty if binding does not exists. */
	virtual FText GetBindingSourceDisplayName(const FPropertyBindingPath& InTargetPath, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const = 0;
};


namespace UE::StateTree
{
	/** @return desc and path as a display string. */
	extern STATETREEMODULE_API FString GetDescAndPathAsString(const FStateTreeBindableStructDesc& Desc, const FPropertyBindingPath& Path);

#if WITH_EDITOR
	/**
	 * Returns property usage based on the Category metadata of given property.
	 * @param Property Handle to property where value is got from.
	 * @return found usage type, or EStateTreePropertyUsage::Invalid if not found.
	 */
	STATETREEMODULE_API EStateTreePropertyUsage GetUsageFromMetaData(const FProperty* Property);

	/** @return struct's property which is the only one marked as Output. Returns null otherwise. */
	STATETREEMODULE_API const FProperty* GetStructSingleOutputProperty(const UStruct& InStruct);
#endif
} // UE::StateTree

#undef UE_API
