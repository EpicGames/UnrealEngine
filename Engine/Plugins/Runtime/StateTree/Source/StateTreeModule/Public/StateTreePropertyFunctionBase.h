// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeNodeBase.h"
#include "StateTreePropertyFunctionBase.generated.h"

#define UE_API STATETREEMODULE_API

struct FStateTreeExecutionContext;

/**
 * Base struct for all property functions.
 * PropertyFunction is a node which is executed just before evaluating owner's bindings.
 *
 * The property function's instance data is expected to have one property marked as output.
 * This property is used to find which properties the function can be used for,
 * and that property is hidden in the UI. It is expected that there's just one output property.
 *
 * Example:
 *
 *	USTRUCT()
 *	struct FStateTreeBooleanOperationPropertyFunctionInstanceData
 *	{
 *		GENERATED_BODY()
 *
 *		UPROPERTY(EditAnywhere, Category = Param)
 *		bool bLeft = false;
 *
 *		UPROPERTY(EditAnywhere, Category = Param)
 *		bool bRight = false;
 *
 *      // This property is used to find which properties the function can be used for.
 *		UPROPERTY(EditAnywhere, Category = Output)
 *		bool bResult = false;
 *	};
 *
 */
USTRUCT(meta = (Hidden))
struct FStateTreePropertyFunctionBase : public FStateTreeNodeBase
{
	GENERATED_BODY()

	/**
	 * Called right before evaluating bindings for the owning node.
	 * @param Context Reference to current execution context.
	 */
	virtual void Execute(FStateTreeExecutionContext& Context) const {}

#if WITH_EDITOR
	virtual FName GetIconName() const override
	{
		return FName("StateTreeEditorStyle|Node.Function");
	}

	UE_API virtual FColor GetIconColor() const override;
#endif
};

USTRUCT(meta = (Hidden))
struct FStateTreePropertyFunctionCommonBase : public FStateTreePropertyFunctionBase
{
	GENERATED_BODY()
};

#undef UE_API
