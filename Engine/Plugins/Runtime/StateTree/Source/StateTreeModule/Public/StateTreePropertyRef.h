// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeAsyncExecutionContext.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeIndexTypes.h"
#include "StateTreeInstanceData.h"
#include "Templates/Tuple.h"
#include "StateTreePropertyRef.generated.h"

#define UE_API STATETREEMODULE_API

struct FStateTreePropertyRef;

namespace UE::StateTree::PropertyRefHelpers
{
	namespace Private
	{
		struct FResolvePropertyReferenceIndirectionsResult
		{
			FStateTreeDataView SourceView;
			const FStateTreePropertyAccess* PropertyAccess = nullptr;
			const FStateTreePropertyBindings* PropertyBindings = nullptr;
		};
		UE_API TOptional<FResolvePropertyReferenceIndirectionsResult> ResolvePropertyReferenceIndirections(
			const FStateTreePropertyRef& InPropertyRef,
			FStateTreeInstanceStorage& InInstanceDataStorage,
			const ExecutionContext::ITemporaryStorage* InTemporaryStorage,
			const FStateTreeExecutionFrame& InExecutionFrame);
	}

	template <class T>
	UE_DEPRECATED(5.8, "Use the version with TemporaryStorage instead of ParentFrame")
	T* GetMutablePtrToProperty(const FStateTreePropertyRef& PropertyRef, FStateTreeInstanceStorage& InstanceDataStorage, const FStateTreeExecutionFrame& ExecutionFrame, const FStateTreeExecutionFrame* ParentExecutionFrame, const FProperty** OutSourceProperty = nullptr)
	{
		constexpr ExecutionContext::ITemporaryStorage* TemporaryStorage = nullptr;
		return GetMutablePtrToProperty<T>(PropertyRef, InstanceDataStorage, TemporaryStorage, ExecutionFrame, OutSourceProperty);
	}

	/**
	 * @param InPropertyRef Property's reference to get pointer to.
	 * @param InInstanceDataStorage Instance Data Storage.
	 * @param InTemporaryStorage Temporary Data Storage.
	 * @param InExecutionFrame Execution frame owning referenced property.
	 * @param OutSourceProperty On success, returns referenced property.
	 * @return Pointer to referenced property value if succeeded.
	 */
	template <class T>
	T* GetMutablePtrToProperty(const FStateTreePropertyRef& InPropertyRef, FStateTreeInstanceStorage& InInstanceDataStorage, const ExecutionContext::ITemporaryStorage* InTemporaryStorage, const FStateTreeExecutionFrame& InExecutionFrame, const FProperty** OutSourceProperty = nullptr)
	{
		using namespace Private;
		if (TOptional<FResolvePropertyReferenceIndirectionsResult> OptionalResult = ResolvePropertyReferenceIndirections(InPropertyRef, InInstanceDataStorage, InTemporaryStorage, InExecutionFrame))
		{
			const FResolvePropertyReferenceIndirectionsResult& Result = OptionalResult.GetValue();
			check(Result.PropertyAccess && Result.PropertyBindings);

			if (OutSourceProperty)
			{
				*OutSourceProperty = Result.PropertyAccess->SourceLeafProperty;
			}

			return Result.PropertyBindings->GetMutablePropertyPtr<T>(Result.SourceView, *Result.PropertyAccess);
		}

		return nullptr;
	}

	template <class... T>
	UE_DEPRECATED(5.8, "Use the version with ITemporaryStorage instead of ParentFrame")
	TTuple<T*...> GetMutablePtrTupleToProperty(const FStateTreePropertyRef& PropertyRef, FStateTreeInstanceStorage& InstanceDataStorage, const FStateTreeExecutionFrame& ExecutionFrame, const FStateTreeExecutionFrame* ParentExecutionFrame)
	{
		constexpr ExecutionContext::ITemporaryStorage* TemporaryStorage = nullptr;
		return GetMutablePtrTupleToProperty<T...>(PropertyRef, InstanceDataStorage, TemporaryStorage, ExecutionFrame);
	}

	/**
	 * @param InPropertyRef Property's reference to get pointer to.
	 * @param InInstanceDataStorage Instance Data Storage
	 * @param InTemporaryStorage Temporary Data Storage
	 * @param InExecutionFrame Execution frame owning referenced property
	 * @return A tuple of pointer to referenced property if succeeded.
	 */
	template <class... T>
	TTuple<T*...> GetMutablePtrTupleToProperty(const FStateTreePropertyRef& InPropertyRef, FStateTreeInstanceStorage& InInstanceDataStorage, const ExecutionContext::ITemporaryStorage* InTemporaryStorage, const FStateTreeExecutionFrame& InExecutionFrame)
	{
		using namespace Private;
		if (TOptional<FResolvePropertyReferenceIndirectionsResult> OptionalResult = ResolvePropertyReferenceIndirections(InPropertyRef, InInstanceDataStorage, InTemporaryStorage, InExecutionFrame))
		{
			const FResolvePropertyReferenceIndirectionsResult& Result = OptionalResult.GetValue();
			check(Result.PropertyAccess && Result.PropertyBindings);

			return TTuple<T*...>(Result.PropertyBindings->GetMutablePropertyPtr<T>(Result.SourceView, *Result.PropertyAccess)...);
		}

		return TTuple<T*...>{};
	}
} // namespace UE::StateTree::PropertyRefHelpers

/**
 * Property ref allows to get a pointer to selected property in StateTree.
 * The expected type of the reference should be set in "RefType" meta specifier.
 * PropertyRef is recommended to be used to avoid copying large struct. Otherwise, Output Binding offers a simpler workflow and designers can author the write-back inside Editor.   
 * Meta specifiers for the type:
 *  - RefType = "<type>"
 *		- Specifies a comma separated list of type of property to reference
 *		- Supported types are: bool, byte, int32, int64, float, double, Name, String, Text, UObject pointers, and structs
 *		- Structs and Objects must use full path name
 *		- If multiple types are specified, GetMutablePtrTuple can be used to access the correct type
 *  - IsRefToArray
 *		- If specified, the reference is to an TArray<RefType>
 *	- CanRefToArray
 *		- If specified, the reference can bind to a Reftype or TArray<RefType>
 *  - Optional
 *		- If specified, the reference can be left unbound, otherwise the compiler report error if the reference is not bound
 *
 * Example:
 *
 *  // Reference to float
 *	UPROPERTY(EditAnywhere, meta = (RefType = "float"))
 *	FStateTreePropertyRef RefToFloat;
 *
 *  // Reference to FTestStructBase
 *	UPROPERTY(EditAnywhere, meta = (RefType = "/Script/ModuleName.TestStructBase"))
 *	FStateTreePropertyRef RefToTest;
 *
 *  // Reference to TArray<FTestStructBase>
 *	UPROPERTY(EditAnywhere, meta = (RefType = "/Script/ModuleName.TestStructBase", IsRefToArray))
 *	FStateTreePropertyRef RefToArrayOfTests;
 *
 *  // Reference to Vector, TArray<FVector>, AActor*, TArray<AActor*>
 *	UPROPERTY(EditAnywhere, meta = (RefType = "/Script/CoreUObject.Vector, /Script/Engine.Actor", CanRefToArray))
 *	FStateTreePropertyRef RefToLocationLikeTypes;
 */
USTRUCT()
struct FStateTreePropertyRef
{
	GENERATED_BODY()

	FStateTreePropertyRef() = default;

	/** @return pointer to the property if possible, nullptr otherwise. */
	template <class T>
	T* GetMutablePtr(const FStateTreeExecutionContext& Context) const
	{
		const FStateTreeExecutionFrame* CurrentlyProcessedFrame = Context.GetCurrentlyProcessedFrame();
		check(CurrentlyProcessedFrame);

		return UE::StateTree::PropertyRefHelpers::GetMutablePtrToProperty<T>(*this, Context.GetMutableInstanceData()->GetMutableStorage(), Context.GetCurrentlyProcessedTemporaryStorage().Get(), *CurrentlyProcessedFrame);
	}

	/** @return pointer to the property if possible, nullptr otherwise. */
	template<class T, bool bWithWriteAccess>
	std::conditional_t<bWithWriteAccess, T*, const T*> GetPtrFromStrongExecutionContext(const TStateTreeStrongExecutionContext<bWithWriteAccess>& Context)
	{
		UE::StateTree::Async::FActivePathInfo ActivePath = Context.GetActivePathInfo();
		if (ActivePath.IsValid())
		{
			return UE::StateTree::PropertyRefHelpers::GetMutablePtrToProperty<T>(*this, *Context.Storage, Context.TemporaryStorage.Get(), *ActivePath.Frame);
		}

		return nullptr;
	}

	/** @return a tuple of pointers of the given types to the property if possible, nullptr otherwise. */
	template <class... T>
	TTuple<T*...> GetMutablePtrTuple(const FStateTreeExecutionContext& Context) const
	{
		const FStateTreeExecutionFrame* CurrentlyProcessedFrame = Context.GetCurrentlyProcessedFrame();
		check(CurrentlyProcessedFrame);

		return UE::StateTree::PropertyRefHelpers::GetMutablePtrTupleToProperty<T...>(*this, Context.GetMutableInstanceData()->GetMutableStorage(), Context.GetCurrentlyProcessedTemporaryStorage().Get(), *CurrentlyProcessedFrame, Context.GetCurrentlyProcessedParentFrame());
	}

	/** @return a tuple of pointers of the given types to the property if possible, nullptr otherwise. */
	template <class... T, bool bWithWriteAccess>
	std::conditional_t<bWithWriteAccess, TTuple<T*...>, TTuple<const T*...>> GetPtrTupleFromStrongExecutionContext(const TStateTreeStrongExecutionContext<bWithWriteAccess>& Context) const
	{
		UE::StateTree::Async::FActivePathInfo ActivePath = Context.GetActivePathInfo();
		if (ActivePath.IsValid())
		{
			return UE::StateTree::PropertyRefHelpers::GetMutablePtrTupleToProperty<T...>(*this, *Context.Storage, Context.TemporaryStorage.Get(), *ActivePath.Frame);
		}

		return {};
	}

	/**
	 * Used internally.
	 * @return index to referenced property access
	 */
	FStateTreeIndex16 GetRefAccessIndex() const
	{
		return RefAccessIndex;
	}

	//~ Begin TStructOpsTypeTraits interface
	bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
	{
		static const FName StateTreeStructRefName("StateTreeStructRef");
		if (Tag.GetType().IsStruct(StateTreeStructRefName))
		{
			// Serialize the data, but we don't have anything to do with it.
			// StructRef and PropertyRef are used for input and existing bindings will set them if needed.
			FStateTreeStructRef TempStructRef;
			FStateTreeStructRef::StaticStruct()->SerializeItem(Slot, &TempStructRef, nullptr);
			return true;
		}

		return false;
	}
	//~ End TStructOpsTypeTraits interface

private:
	UPROPERTY()
	FStateTreeIndex16 RefAccessIndex;

	friend FStateTreePropertyBindingCompiler;
};

template <>
struct TStructOpsTypeTraits<FStateTreePropertyRef> : public TStructOpsTypeTraitsBase2<FStateTreePropertyRef>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

/**
 * TStateTreePropertyRef is a type-safe FStateTreePropertyRef wrapper against a single given type.
 * @note When used as a property, this automatically defines PropertyRef property meta-data.
 *
 * Example:
 *
 *  // Reference to float
 *	UPROPERTY(EditAnywhere)
 *	TStateTreePropertyRef<float> RefToFloat;
 *
 *  // Reference to FTestStructBase
 *	UPROPERTY(EditAnywhere)
 *	TStateTreePropertyRef<FTestStructBase> RefToTest;
 *
 *  // Reference to TArray<FTestStructBase>
 *	UPROPERTY(EditAnywhere)
 *	TStateTreePropertyRef<TArray<FTestStructBase>> RefToArrayOfTests;
 *
 *  // Reference to FTestStructBase or TArray<FTestStructBase>
 *	UPROPERTY(EditAnywhere, meta = (CanRefToArray))
 *	TStateTreePropertyRef<FTestStructBase> RefToSingleOrArrayOfTests;
 */
template <class TRef>
struct TStateTreePropertyRef
{
	/** @return pointer to the property if possible, nullptr otherwise. */
	TRef* GetMutablePtr(const FStateTreeExecutionContext& Context) const
	{
		return PropertyRef.GetMutablePtr<TRef>(Context);
	}

	/** @return pointer to the property if possible, nullptr otherwise. */
	template<bool bWithWriteAccess>
	std::conditional_t<bWithWriteAccess, TRef*, const TRef*> GetPtrFromStrongExecutionContext(const TStateTreeStrongExecutionContext<bWithWriteAccess>& Context)
	{
		return PropertyRef.GetPtrFromStrongExecutionContext<TRef, bWithWriteAccess>(Context);
	}

	/** @return a tuple of pointer to the property of the type or array of type, nullptr otherwise. */
	TTuple<TRef*, TArray<TRef>*> GetMutablePtrTuple(const FStateTreeExecutionContext& Context) const
	{
		return PropertyRef.GetMutablePtrTuple<TRef, TArray<TRef>>(Context);
	}

	/** @return a tuple of pointers of the given types to the property if possible, nullptr otherwise. */
	template <bool bWithWriteAccess>
	std::conditional_t<bWithWriteAccess, TTuple<TRef*, TArray<TRef>*>, TTuple<const TRef*, const TArray<TRef>*>> GetPtrTupleFromStrongExecutionContext(const TStateTreeStrongExecutionContext<bWithWriteAccess>& Context) const
	{
		return PropertyRef.GetPtrTupleFromStrongExecutionContext<TRef, TArray<TRef>>(Context);
	}

	/**
	 * Used internally.
	 * @return internal property ref
	 */
	FStateTreePropertyRef GetInternalPropertyRef() const
	{
		return PropertyRef;
	}

private:
	FStateTreePropertyRef PropertyRef;
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
struct UE_DEPRECATED(5.8, "PropertyRefExternalHandle is deprecated. Use PropertyRef with ExecutionContext/StrongExecutionContext instead, which is a general async pattern for data access using Frame/StateID.") FStateTreePropertyRefExternalHandle
{
	FStateTreePropertyRefExternalHandle(FStateTreePropertyRef InPropertyRef, FStateTreeExecutionContext& InContext)
		: WeakInstanceStorage(InContext.GetMutableInstanceData()->GetWeakMutableStorage())
		, WeakStateTree(InContext.GetCurrentlyProcessedFrame()->StateTree)
		, RootState(InContext.GetCurrentlyProcessedFrame()->RootState)
		, PropertyRef(InPropertyRef)
	{
	}

	/** @return pointer to the property if possible, nullptr otherwise. */
	template <class TRef>
	TRef* GetMutablePtr() const
	{
		return GetMutablePtrTuple<TRef>().template Get<0>();
	}

	/** @return a tuple of pointers of the given types to the property if possible, nullptr otherwise. */
	template <class... TRef>
	TTuple<TRef*...> GetMutablePtrTuple() const
	{
		if (!WeakInstanceStorage.IsValid())
		{
			return TTuple<TRef*...>{};
		}

		FStateTreeInstanceStorage& InstanceStorage = *WeakInstanceStorage.Pin();
		TConstArrayView<FStateTreeExecutionFrame> ActiveFrames = InstanceStorage.GetExecutionState().ActiveFrames;
		const FStateTreeExecutionFrame* ParentFrame = nullptr;
		const FStateTreeExecutionFrame* Frame = FStateTreeExecutionContext::FindFrame(WeakStateTree.Get(), RootState, ActiveFrames, ParentFrame);

		if (Frame == nullptr)
		{
			return TTuple<TRef*...>{};
		}

		return UE::StateTree::PropertyRefHelpers::GetMutablePtrTupleToProperty<TRef...>(PropertyRef, InstanceStorage, *Frame, ParentFrame);
	}

protected:
	TWeakPtr<FStateTreeInstanceStorage> WeakInstanceStorage;
	TWeakObjectPtr<const UStateTree> WeakStateTree = nullptr;
	FStateTreeStateHandle RootState = FStateTreeStateHandle::Invalid;
	FStateTreePropertyRef PropertyRef;
};

template <class TRef>
struct UE_DEPRECATED(5.8, "PropertyRefExternalHandle is deprecated. Use PropertyRef with ExecutionContext/StrongExecutionContext instead, which is a general async pattern for data access using Frame/StateID.") TStateTreePropertyRefExternalHandle : public FStateTreePropertyRefExternalHandle
{
	using FStateTreePropertyRefExternalHandle::FStateTreePropertyRefExternalHandle;
	TStateTreePropertyRefExternalHandle(TStateTreePropertyRef<TRef> InPropertyRef, FStateTreeExecutionContext& InContext)
		: FStateTreePropertyRefExternalHandle(InPropertyRef.GetInternalPropertyRef(), InContext)
	{
	}

	/** @return pointer to the property if possible, nullptr otherwise. */
	TRef* GetMutablePtr() const
	{
		return FStateTreePropertyRefExternalHandle::GetMutablePtr<TRef>();
	}

	/** @return a tuple of pointer to the property of the type or array of type, nullptr otherwise. */
	TTuple<TRef*, TArray<TRef>*> GetMutablePtrTuple() const
	{
		return FStateTreePropertyRefExternalHandle::GetMutablePtrTuple<TRef, TArray<TRef>>();
	}

private:
	using FStateTreePropertyRefExternalHandle::GetMutablePtr;
	using FStateTreePropertyRefExternalHandle::GetMutablePtrTuple;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

UENUM()
enum class EStateTreePropertyRefType : uint8
{
	None,
	Bool,
	Byte,
	Int32,
	Int64,
	Float,
	Double,
	Name,
	String,
	Text,
	Enum,
	Struct,
	Object,
	SoftObject,
	Class,
	SoftClass,
};

/**
 * FStateTreeBlueprintPropertyRef is a PropertyRef intended to be used in State Tree Blueprint nodes like tasks, conditions or evaluators, but also as a StateTree parameter.
 */
USTRUCT(BlueprintType, DisplayName = "State Tree Property Ref")
struct FStateTreeBlueprintPropertyRef : public FStateTreePropertyRef
{
	GENERATED_BODY()

	FStateTreeBlueprintPropertyRef() = default;

	/** Returns PropertyRef's type */
	EStateTreePropertyRefType GetRefType() const { return RefType; }

	/** Returns true if referenced property is an array. */
	bool IsRefToArray() const { return bIsRefToArray; }

	/** Returns selected ScriptStruct, Class or Enum. */
	UObject* GetTypeObject() const { return TypeObject; }

	/** Returns true if PropertyRef was marked as optional. */
	bool IsOptional() const { return bIsOptional; }

private:
	/** Specifies the type of property to reference */
	UPROPERTY(EditAnywhere, Category = "InternalType")
	EStateTreePropertyRefType RefType = EStateTreePropertyRefType::None;

	/** If specified, the reference is to an TArray<RefType> */
	UPROPERTY(EditAnywhere, Category = "InternalType")
	uint8 bIsRefToArray : 1 = false;

	/** If specified, the reference can be left unbound, otherwise the State Tree compiler report error if the reference is not bound. */
	UPROPERTY(EditAnywhere, Category = "Parameter")
	uint8 bIsOptional : 1 = false;

	/** Specifies the type of property to reference together with RefType, used for Enums, Structs, Objects and Classes. */
	UPROPERTY(EditAnywhere, Category= "InternalType")
	TObjectPtr<UObject> TypeObject = nullptr;

	friend class FStateTreeBlueprintPropertyRefDetails;
};

#undef UE_API
