// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "UObject/Field.h"
#include "UObject/StrProperty.h"
#include "UObject/TextProperty.h"
#include "UObject/EnumProperty.h"

#if WITH_EDITOR
#include "Containers/ContainersFwd.h"
#include "Templates/SharedPointerFwd.h"
#endif

class FProperty;
struct FPropertyBindingPathIndirection;
struct FBindingChainElement;
struct FStateTreeBindableStructDesc;
struct FStateTreePropertyPathIndirection;
struct FEdGraphPinType;
struct FStateTreeBlueprintPropertyRef;
enum class EStateTreePropertyRefType : uint8;

namespace UE::StateTree::PropertyRefHelpers
{
#if WITH_EDITOR
	STATETREEMODULE_API extern const FName IsRefToArrayName;
	STATETREEMODULE_API extern const FName CanRefToArrayName;
	STATETREEMODULE_API extern const FName RefTypeName;
	/**
	 * @param RefProperty Property of PropertyRef type.
	 * @param SourceProperty Property to check its type compatibility.
	 * @param PropertyRefAddress Address of PropertyRef
	 * @param SourceAddress Address of checked property value.
	 * @return true if SourceProperty type is compatible with PropertyRef.
	 */
	bool STATETREEMODULE_API IsPropertyRefCompatibleWithProperty(const FProperty& RefProperty, const FProperty& SourceProperty, const void* PropertyRefAddress, const void* SourceAddress);

	/**
	 * @param SourcePropertyPathIndirections Path indirections of the referenced property.
	 * @param SourceStruct Bindable owner of referenced property.
	 * @return true if property can be referenced by PropertyRef.
	 */
	bool STATETREEMODULE_API IsPropertyAccessibleForPropertyRef(TConstArrayView<FPropertyBindingPathIndirection> SourcePropertyPathIndirections, FStateTreeBindableStructDesc SourceStruct);

	UE_DEPRECATED(5.6, "Use the overload taking FPropertyBindingPathIndirection instead")
	bool STATETREEMODULE_API IsPropertyAccessibleForPropertyRef(TConstArrayView<FStateTreePropertyPathIndirection> SourcePropertyPathIndirections, FStateTreeBindableStructDesc SourceStruct);

	/**
	 * @param SourceProperty Referenced property.
	 * @param BindingChain Binding chain to referenced property.
	 * @param SourceStruct Bindable owner of referenced property.
	 * @return true if property can be referenced by PropertyRef.
	 */
	bool STATETREEMODULE_API IsPropertyAccessibleForPropertyRef(const FProperty& SourceProperty, TConstArrayView<FBindingChainElement> BindingChain, FStateTreeBindableStructDesc SourceStruct);

	/**
	 * @param RefProperty Property of PropertyRef type.
	 * @param PropertyRefAddress Address of PropertyRef.
	 * @return true if PropertyRef is marked as optional.
	 */
	bool STATETREEMODULE_API IsPropertyRefMarkedAsOptional(const FProperty& RefProperty, const void* PropertyRefAddress);

	/**
	 * @param RefProperty Property of PropertyRef type
	 * @return PinTypes for PropertyRef's internal types
	 */
	TArray<FEdGraphPinType, TInlineAllocator<1>> STATETREEMODULE_API GetPropertyRefInternalTypesAsPins(const FProperty& RefProperty);

	/**
	 * @param RefProperty Property of PropertyRef type
	 * @param PropertyRefAddress Address of PropertyRef.
	 * @return PinType for PropertyRef's internal type
	 */
	FEdGraphPinType STATETREEMODULE_API GetPropertyRefInternalTypeAsPin(const FProperty& RefProperty, const void* PropertyRefAddress);

	/**
	 * @param InPropertyRef PropertyRef to get internal type from.
	 * @return PinType for Blueprint PropertyRef's internal type
	 */
	FEdGraphPinType STATETREEMODULE_API GetBlueprintPropertyRefInternalTypeAsPin(const FStateTreeBlueprintPropertyRef& InPropertyRef);

	/**
	 * @param PinType Pin to get type from.
	 * @param OutRefType PropertyRef's referenced type.
	 * @param bOutIsArray True if PropertyRef references an array property.
	 * @param OutObjectType Referenced type's specific object.
	 */
	void STATETREEMODULE_API GetBlueprintPropertyRefInternalTypeFromPin(const FEdGraphPinType& PinType, EStateTreePropertyRefType& OutRefType, bool& bOutIsArray, UObject*& OutObjectType);
#endif

	/**
	 * @param Property Property to check
	 * @return true if Property is a PropertyRef
	 */
	bool STATETREEMODULE_API IsPropertyRef(const FProperty& Property);

	/**
	 * @param SourceProperty Property to check it's type compatibility.
	 * @param PropertyRefAddress Address of PropertyRef
	 * @return true if SourceProperty type is compatible with Blueprint PropertyRef.
	 */
	bool STATETREEMODULE_API IsBlueprintPropertyRefCompatibleWithProperty(const FProperty& SourceProperty, const void* PropertyRefAddress);

	template<class T, class = void>
	struct Validator
	{};

	template<>
	struct Validator<void>
	{
		static bool IsValid(const FProperty& Property) { return true; }
	};

	template<>
	struct Validator<FBoolProperty::TCppType>
	{
		static bool IsValid(const FProperty& Property){ return Property.IsA<FBoolProperty>(); };
	};

	template<>
	struct Validator<FByteProperty::TCppType>
	{
		static bool IsValid(const FProperty& Property){ return Property.IsA<FByteProperty>(); };
	};

	template<>
	struct Validator<FIntProperty::TCppType>
	{
		static bool IsValid(const FProperty& Property){ return Property.IsA<FIntProperty>(); };
	};

	template<>
	struct Validator<FInt64Property::TCppType>
	{
		static bool IsValid(const FProperty& Property){ return Property.IsA<FInt64Property>(); };
	};

	template<>
	struct Validator<FFloatProperty::TCppType>
	{
		static bool IsValid(const FProperty& Property){ return Property.IsA<FFloatProperty>(); };
	};

	template<>
	struct Validator<FDoubleProperty::TCppType>
	{
		static bool IsValid(const FProperty& Property){ return Property.IsA<FDoubleProperty>(); };
	};

	template<>
	struct Validator<FNameProperty::TCppType>
	{
		static bool IsValid(const FProperty& Property){ return Property.IsA<FNameProperty>(); };
	};

	template<>
	struct Validator<FStrProperty::TCppType>
	{
		static bool IsValid(const FProperty& Property){ return Property.IsA<FStrProperty>(); };
	};

	template<>
	struct Validator<FTextProperty::TCppType>
	{
		static bool IsValid(const FProperty& Property){ return Property.IsA<FTextProperty>(); };
	};

	template<class T>
	struct Validator<T, typename TEnableIf<TIsTArray_V<T>, void>::Type>
	{
		static bool IsValid(const FProperty& Property)
		{
			if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(&Property))
			{
				return Validator<typename T::ElementType>::IsValid(*ArrayProperty->Inner);
			}

			return false;
		}
	};

	template<typename T>
	struct Validator<TEnumAsByte<T>>
	{
		static bool IsValid(const FProperty& Property)
		{
			return Property.IsA<FByteProperty>();
		}
	};

	/* Checks if provided property is compatible with selected ScriptStruct. */
	bool STATETREEMODULE_API IsPropertyCompatibleWithStruct(const FProperty& Property, const class UScriptStruct& Struct);

	template<class T>
	struct Validator<T, decltype(TBaseStructure<T>::Get, void())>
	{
		static bool IsValid(const FProperty& Property)
		{		
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(&Property))
			{
				return StructProperty->Struct->IsChildOf(TBaseStructure<T>::Get());
			}

			return false;
		}
	};

	/* Checks if provided property is compatible with selected Class. */
	bool STATETREEMODULE_API IsPropertyCompatibleWithClass(const FProperty& Property, const class UClass& Class);

	template<class T>
	struct Validator<T, typename TEnableIf<TIsDerivedFrom<typename TRemovePointer<T>::Type, UObject>::IsDerived>::Type>
	{
		static bool IsValid(const FProperty& Property)
		{		
			if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(&Property))
			{
				return ObjectProperty->PropertyClass == TRemovePointer<T>::Type::StaticClass();
			}

			return false;
		}
	};

	/* Checks if provided property is compatible with selected Enum. */
	bool STATETREEMODULE_API IsPropertyCompatibleWithEnum(const FProperty& Property, const class UEnum& Enum);

	template<class T>
	struct Validator<T, typename TEnableIf<TIsEnum<T>::Value>::Type>
	{
		static bool IsValid(const FProperty& Property)
		{		
			if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(&Property))
			{
				return EnumProperty->GetEnum() == StaticEnum<T>();
			}

			return false;
		}
	};
}