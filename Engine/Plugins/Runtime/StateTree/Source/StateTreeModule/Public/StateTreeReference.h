// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/PropertyBag.h"
#include "GameplayTagContainer.h"
#include "StateTreeReference.generated.h"

#define UE_API STATETREEMODULE_API

class UStateTree;

/**
 * Struct to hold reference to a StateTree asset along with values to parameterized it.
 */
USTRUCT(BlueprintType, meta = (DisableSplitPin, HasNativeMake = "/Script/StateTreeModule.StateTreeFunctionLibrary.MakeStateTreeReference"))
struct FStateTreeReference
{
	GENERATED_BODY()

	/** @return true if the reference is set. */
	bool IsValid() const
	{
		return StateTree != nullptr;
	}
	
	/** Sets the StateTree asset and referenced parameters. */
	void SetStateTree(UStateTree* NewStateTree)
	{
		StateTree = NewStateTree;
		SyncParameters();
	}

	/** @return const pointer to the referenced StateTree asset. */
	const UStateTree* GetStateTree() const
	{
		return StateTree;
	}

	/** @return pointer to the referenced StateTree asset. */
	UStateTree* GetMutableStateTree()
	{
		return StateTree;
	}

	/** @return the parameters for the referenced StateTree asset. */
	FConstStructView GetGlobalParameters() const
	{
		ConditionallySyncParameters();
		return Parameters.GetValue();
	}

	/** @return the parameters for the referenced StateTree asset. */
	FStructView GetMutableGlobalParameters()
	{
		ConditionallySyncParameters();
		return Parameters.GetMutableValue();
	}

	/** @return reference to the parameters for the referenced StateTree asset. */
	const FInstancedPropertyBag& GetParameters() const
	{
		ConditionallySyncParameters();
		return Parameters;
	}

	/** @return reference to the parameters for the referenced StateTree asset. */
	FInstancedPropertyBag& GetMutableParameters()
	{
		ConditionallySyncParameters();
		return Parameters;
	}

	/**
	 * Enforce self parameters to be compatible with those exposed by the selected StateTree asset.
	 */
	UE_API void SyncParameters();

	/**
	 * Indicates if current parameters are compatible with those available in the selected StateTree asset.
	 * @return true when parameters requires to be synced to be compatible with those available in the selected StateTree asset, false otherwise.
	 */
	UE_API bool RequiresParametersSync() const;

	/** Sync parameters to match the asset if required. */
	UE_API void ConditionallySyncParameters() const;

	/** @return true if the property of specified ID is overridden. */
	bool IsPropertyOverridden(const FGuid PropertyID) const
	{
		return PropertyOverrides.Contains(PropertyID);
	}

	/** Sets the override status of specified property by ID. */
	UE_API void SetPropertyOverridden(const FGuid PropertyID, const bool bIsOverridden);

	UE_API bool Serialize(FStructuredArchive::FSlot Slot);
	UE_API void PostSerialize(const FArchive& Ar);

protected:
	UPROPERTY(EditAnywhere, Category = "StateTree")
	TObjectPtr<UStateTree> StateTree = nullptr;

	UPROPERTY(EditAnywhere, Category = "StateTree", meta = (FixedLayout))
	FInstancedPropertyBag Parameters;

	/** Array of overridden properties. Non-overridden properties will inherit the values from the StateTree default parameters. */
	UPROPERTY(EditAnywhere, Category = "StateTree")
	TArray<FGuid> PropertyOverrides;

	friend class FStateTreeReferenceDetails;
};

template<>
struct TStructOpsTypeTraits<FStateTreeReference> : public TStructOpsTypeTraitsBase2<FStateTreeReference>
{
	enum
	{
		WithStructuredSerializer = true,
		WithPostSerialize = true,
	};
};


/**
 * Item describing a state tree override for a state with a specific tag.
 */
USTRUCT()
struct FStateTreeReferenceOverrideItem
{
	GENERATED_BODY()

	FStateTreeReferenceOverrideItem() = default;
	FStateTreeReferenceOverrideItem(const FGameplayTag InStateTag, const FStateTreeReference& InStateTreeReference)
		: StateTag(InStateTag)
		, StateTreeReference(InStateTreeReference)
	{
	}
	FStateTreeReferenceOverrideItem(const FGameplayTag InStateTag, FStateTreeReference&& InStateTreeReference)
		: StateTag(InStateTag)
		, StateTreeReference(MoveTemp(InStateTreeReference))
	{
	}
	
	FGameplayTag GetStateTag() const
	{
		return StateTag;
	}

	const FStateTreeReference& GetStateTreeReference() const
	{
		return StateTreeReference;
	}

	friend uint32 GetTypeHash(const FStateTreeReferenceOverrideItem& Item)
	{
		return HashCombine(GetTypeHash(Item.StateTag), GetTypeHash(Item.StateTreeReference.GetStateTree()));
	}
	
private:
	/** Exact tag used to match against a tag on a linked State Tree state. */
	UPROPERTY(EditAnywhere, Category = "StateTree")
	FGameplayTag StateTag;

	/** State Tree and parameters to replace the linked state asset with. */
	UPROPERTY(EditAnywhere, Category = "StateTree", DisplayName = "State Tree", meta=(SchemaCanBeOverriden))
	FStateTreeReference StateTreeReference;
	
	friend class FStateTreeReferenceOverridesDetails;
};

/**
 * Overrides for linked State Trees. This table is used to override State Tree references on linked states.
 * If a linked state's tag is exact match of the tag specified on the table, the reference from the table is used instead.
 */
USTRUCT()
struct FStateTreeReferenceOverrides
{
	GENERATED_BODY()

	/** Removes all overrides. */
	void Reset()
	{
		OverrideItems.Empty();
	}

	/** Adds or replaces override for a selected tag. */
	void AddOverride(const FGameplayTag StateTag, const FStateTreeReference& StateTreeReference)
	{
		FStateTreeReferenceOverrideItem* FoundOverride = OverrideItems.FindByPredicate([StateTag](const FStateTreeReferenceOverrideItem& Override)
		{
			return Override.GetStateTag() == StateTag;
		});

		if (FoundOverride)
		{
			*FoundOverride = FStateTreeReferenceOverrideItem(StateTag, StateTreeReference);
		}
		else
		{
			OverrideItems.Emplace(StateTag, StateTreeReference);
		}
	}

	/** Adds or replaces override for a selected tag. */
	void AddOverride(FStateTreeReferenceOverrideItem OverrideItem)
	{
		FStateTreeReferenceOverrideItem* FoundOverride = OverrideItems.FindByPredicate([&OverrideItem](const FStateTreeReferenceOverrideItem& Override)
			{
				return Override.GetStateTag() == OverrideItem.GetStateTag();
			});

		if (FoundOverride)
		{
			*FoundOverride = MoveTemp(OverrideItem);
		}
		else
		{
			OverrideItems.Emplace(MoveTemp(OverrideItem));
		}
	}

	/** Returns true if removing an override succeeded. */
	bool RemoveOverride(const FGameplayTag StateTag)
	{
		const int32 Index = OverrideItems.IndexOfByPredicate([StateTag](const FStateTreeReferenceOverrideItem& Override)
		{
			return Override.GetStateTag() == StateTag;
		});

		if (Index != INDEX_NONE)
		{
			OverrideItems.RemoveAtSwap(Index);
			return true;
		}

		return false;
	}

	TConstArrayView<FStateTreeReferenceOverrideItem> GetOverrideItems() const
	{
		return OverrideItems;
	}

	friend uint32 GetTypeHash(const FStateTreeReferenceOverrides& Overrides)
	{
		return GetTypeHash(Overrides.OverrideItems);
	}

private:
	UPROPERTY(EditAnywhere, Category = "StateTree")
	TArray<FStateTreeReferenceOverrideItem> OverrideItems;

	friend class FStateTreeReferenceOverridesDetails;
};

#undef UE_API
