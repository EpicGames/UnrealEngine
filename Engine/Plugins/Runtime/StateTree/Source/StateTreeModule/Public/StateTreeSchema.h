// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/IConsoleManager.h"
#include "StateTreeSchema.generated.h"

#define UE_API STATETREEMODULE_API

struct FStateTreeExternalDataDesc;
struct FStateTreeLinker;
enum class EStateTreeDataSourceType : uint8;
enum class EStateTreeParameterDataType : uint8;
enum class EStateTreeStateSelectionBehavior : uint8;
enum class EStateTreeStateSelectionRules : uint32;
enum class EStateTreeStateType : uint8;


/**
 * Schema describing which inputs, evaluators, and tasks a StateTree can contain.
 * Each StateTree asset saves the schema class name in asset data tags, which can be
 * used to limit which StatTree assets can be selected per use case, i.e.:
 *
 *	UPROPERTY(EditDefaultsOnly, Category = AI, meta=(RequiredAssetDataTags="Schema=StateTreeSchema_SupaDupa"))
 *	UStateTree* StateTree;
 *
 */
UCLASS(MinimalAPI, Abstract)
class UStateTreeSchema : public UObject
{
	GENERATED_BODY()

public:

	/** @return True if specified struct is supported */
	virtual bool IsStructAllowed(const UScriptStruct* InScriptStruct) const
	{
		return false;
	}

	/** @return True if specified class is supported */
	virtual bool IsClassAllowed(const UClass* InScriptStruct) const
	{
		return false;
	}

	/** @return True if specified struct/class is supported as external data */
	virtual bool IsExternalItemAllowed(const UStruct& InStruct) const
	{
		return false;
	}

	/** @return True if the execution context can sleep or the next tick delayed. */
	virtual bool IsScheduledTickAllowed() const
	{
		return false;
	}

	/** @return True if the state selection behavior is supported. */
	virtual bool IsStateSelectionAllowed(EStateTreeStateSelectionBehavior InBehavior) const
	{
		return true;
	}

	/** @return True if the state type is supported. */
	virtual bool IsStateTypeAllowed(EStateTreeStateType InStateType) const
	{
		return true;
	}

	/**
	 * Helper function to check if a class is any of the Blueprint extendable item classes (Eval, Task, Condition).
	 * Can be used to quickly accept all of those classes in IsClassAllowed().
	 * @return True if the class is a StateTree item Blueprint base class.
	 */
	UE_API static bool IsChildOfBlueprintBase(const UClass* InClass);

	/** @return List of context objects (UObjects or UScriptStructs) enforced by the schema. They must be provided at runtime through the execution context. */
	virtual TConstArrayView<FStateTreeExternalDataDesc> GetContextDataDescs() const
	{
		return {};
	}

	/** @return the global parameter type used by the schema. */
	UE_API virtual EStateTreeParameterDataType GetGlobalParameterDataType() const;

	/** @return the selection rules used by the schema. */
	UE_API virtual EStateTreeStateSelectionRules GetStateSelectionRules() const;

	/** Resolves schema references to other StateTree data. */
	virtual bool Link(FStateTreeLinker& Linker)
	{
		return true;
	}

#if WITH_EDITOR
	/** @return True if enter conditions are allowed. */
	virtual bool AllowEnterConditions() const
	{
		return true;
	}

	/** @return True if utility considerations are allowed. */
	virtual bool AllowUtilityConsiderations() const
	{
		return true;
	}

	/** @return True if evaluators are allowed. */
	virtual bool AllowEvaluators() const
	{
		return true;
	}

	/** @return True if multiple tasks are allowed. */
	virtual bool AllowMultipleTasks() const
	{
		return true;
	}

	/** @return True if global parameters for the State Tree are allowed. */
	virtual bool AllowGlobalParameters() const
	{
		return true;
	}

	/** @return True if modifying the tasks completion is allowed. If not allowed, "any" will be used.*/
	virtual bool AllowTasksCompletion() const
	{
		return true;
	}

	/** @return Whether the compilation of the state tree is allowed to be queued. */
	virtual bool AllowQueuedCompilation() const
	{
		return true;
	}
#endif // WITH_EDITOR
};

#undef UE_API
