// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "StateTreeSettings.generated.h"

/**
 * Default StateTree settings
 */
UCLASS(MinimalAPI, config = StateTree, defaultconfig, DisplayName = "StateTree")
class UStateTreeSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	static UStateTreeSettings& Get() { return *CastChecked<UStateTreeSettings>(StaticClass()->GetDefaultObject()); }

	/**
	 * Editor targets relies on PIE and StateTreeEditor to start/stop traces.
	 * This is to start traces automatically when launching Standalone, Client or Server builds. 
	 * It's also possible to do it manually using 'statetree.startdebuggertraces' and 'statetree.stopdebuggertraces' in the console.
	 */
	UPROPERTY(EditDefaultsOnly, Category = StateTree, config)
	bool bAutoStartDebuggerTracesOnNonEditorTargets = false;
};
