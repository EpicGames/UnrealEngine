// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Misc/NotNull.h"
#include "StateTreeTypes.h"

#define UE_API STATETREEMODULE_API

class UStateTree;
struct FStateTreeExecutionFrame;

namespace UE::StateTree
{
	/** A unique ID for FStateTreeExecutionFrame. */
	struct FActiveFrameID
	{
	private:
		static constexpr uint32 InvalidValue = 0;

	public:
		UE_API static const FActiveFrameID Invalid;

		explicit FActiveFrameID() = default;
		explicit FActiveFrameID(uint32 NewID)
			: Value(NewID)
		{
			check(NewID != InvalidValue);
		}

		[[nodiscard]] bool operator==(const FActiveFrameID&) const = default;
		[[nodiscard]] bool operator!=(const FActiveFrameID&) const = default;

		[[nodiscard]] bool IsValid() const
		{
			return Value != InvalidValue;
		}
		
	private:
		uint32 Value = InvalidValue;
	};
	
	/** A unique ID for FStateTreeActiveStates state. */
	struct FActiveStateID
	{
	private:
		static constexpr uint32 InvalidValue = 0;

	public:
		UE_API static const FActiveStateID Invalid;

		explicit FActiveStateID() = default;
		explicit FActiveStateID(uint32 NewID)
			: Value(NewID)
		{
			check(NewID != InvalidValue);
		}
		
		[[nodiscard]] bool operator==(const FActiveStateID&) const = default;
		[[nodiscard]] bool operator!=(const FActiveStateID&) const = default;

		[[nodiscard]] bool IsValid() const
		{
			return Value != InvalidValue;
		}
	
	private:
		uint32 Value = InvalidValue;
	};

	/** A state entry in the state path. */
	struct FActiveState
	{
		FActiveState() = default;
		inline explicit FActiveState(FActiveFrameID InFrameID, FActiveStateID InStateID, FStateTreeStateHandle InHandle);

		[[nodiscard]] bool operator==(const FActiveState&) const = default;
		[[nodiscard]] bool operator!=(const FActiveState&) const = default;

		[[nodiscard]] bool IsValid() const
		{
			return StateHandle.IsValid();
		}

		[[nodiscard]] FActiveFrameID GetFrameID() const
		{
			return FrameID;
		}

		[[nodiscard]] FActiveStateID GetStateID() const
		{
			return StateID;
		}

		[[nodiscard]] FStateTreeStateHandle GetStateHandle() const
		{
			return StateHandle;
		}

	private:
		/** The unique ID of the state for this instance. */
		FActiveFrameID FrameID;
		/** The unique ID of the state for this instance. */
		FActiveStateID StateID;
		/** The index of the state handle the state tree asset. */
		FStateTreeStateHandle StateHandle;
	};

	/**
	 * Describe the state list used to get to a specific state.
	 * Since a state can be a subtree and can be linked to other states, the path to activate the subtree can be different.
	 * A state can enter, then exit, then reenter. It is considered the same state path but not the same unique state path.
	 * For a tree like
	 * RootA(0)
	 *   StateA(1)
	 *   StateB(2)
	 *     LinkedStateA(3)
	 *   StateC(4)
	 *     LinkedStateA(5)
	 * SubTreeA(6)
	 *   StateD(7)
	 * The path to get to StateD can be RootA.StateB.LinkedStateA.SubTreeA.StateD(0.2.3.6.7)
	 */
	class FActiveStatePath
	{
	public:
		explicit FActiveStatePath() = default;
		UE_API explicit FActiveStatePath(TNotNull<const UStateTree*> InStateTree, const TArrayView<const FActiveState> InElements);
		UE_API explicit FActiveStatePath(TNotNull<const UStateTree*> InStateTree, TArray<FActiveState> InElements);

		/** @return the number of elements in the path. */
		[[nodiscard]] int32 Num() const
		{
			return States.Num();
		}

		/** @return a view of the states in the path. */
		[[nodiscard]] const TArrayView<const FActiveState> GetView() const
		{
			return States;
		}

		/** @return true if both paths match exactly. */
		[[nodiscard]] UE_API static bool Matches(const TArrayView<const FActiveState>& A, const TArrayView<const FActiveState>& B);

		/** @return true if both paths match exactly. */
		[[nodiscard]] UE_API bool Matches(const FActiveStatePath& Other) const;

		/** @return true if the last element in the path matches. */
		[[nodiscard]] UE_API static bool Matches(const TArrayView<const FActiveState>& Path, FActiveState Other);

		/** @return true if the last element in the path matches. */
		[[nodiscard]] UE_API bool Matches(FActiveState Other) const;

		/** @return true if the last element in the path matches. */
		[[nodiscard]] UE_API static bool Matches(const TArrayView<const FActiveState>& Path, FActiveStateID Other);

		/** @return true if the last element in the path matches. */
		[[nodiscard]] UE_API bool Matches(FActiveStateID Other) const;

		/** @return the elements that are in both paths. */
		[[nodiscard]] UE_API static const TArrayView<const FActiveState> Intersect(const TArrayView<const FActiveState>& A, const TArrayView<const FActiveState>& B);

		/** @return the path where the 2 paths intersect. */
		[[nodiscard]] UE_API const TArrayView<const FActiveState> Intersect(const FActiveStatePath& Other) const;

		/** @return true if the path starts with and contains the other path. */
		[[nodiscard]]UE_API  static bool StartsWith(const TArrayView<const FActiveState>& Path, const TArrayView<const FActiveState>& B);

		/** @return true if this path starts with and contains the other path. */
		[[nodiscard]] UE_API bool StartsWith(const FActiveStatePath& Other) const;

		/** @return true if the path contains anywhere the other element. */
		[[nodiscard]] UE_API static bool Contains(const TArrayView<const FActiveState>& Path, FActiveState Other);
		
		/** @return true if this path contains anywhere the other element. */
		[[nodiscard]] UE_API bool Contains(FActiveState Other) const;

		/** @return true if the path contains anywhere the other element. */
		[[nodiscard]] UE_API static bool Contains(const TArrayView<const FActiveState>& Path, FActiveStateID Other);

		/** @return true if this path contains anywhere the other element. */
		[[nodiscard]] UE_API bool Contains(FActiveStateID Other) const;

		/** @return true the index of element inside the path. */
		[[nodiscard]] UE_API static int32 IndexOf(const TArrayView<const FActiveState>& Path, FActiveState Other);

		/** @return true the index of element inside the path. */
		[[nodiscard]]UE_API int32 IndexOf(FActiveState Other) const;

		/** @return true the index of element inside the path. */
		[[nodiscard]] UE_API static int32 IndexOf(const TArrayView<const FActiveState>& Path, FActiveStateID Other);

		/** @return true the index of element inside the path. */
		[[nodiscard]] UE_API int32 IndexOf(FActiveStateID Other) const;

		/** @return debugging information about the path. */
		[[nodiscard]] UE_API static TValueOrError<FString, void> Describe(TNotNull<const UStateTree*> StateTree, const TArrayView<const FActiveState>& Path);
		
		/** @return debugging information about the path. */
		[[nodiscard]] UE_API TValueOrError<FString, void> Describe() const;

	private:
		TWeakObjectPtr<const UStateTree> StateTree;
		TArray<FActiveState> States;
	};

	// Inlined functions
	FActiveState::FActiveState(FActiveFrameID InFrameID, FActiveStateID InStateID, FStateTreeStateHandle InHandle)
		: FrameID(InFrameID)
		, StateID(InStateID)
		, StateHandle(InHandle)
	{
	}
}

#undef UE_API
