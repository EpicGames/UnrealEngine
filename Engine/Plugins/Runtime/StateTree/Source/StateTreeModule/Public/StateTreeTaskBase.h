// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeNodeBase.h"
#include "StateTreeExecutionTypes.h"
#include "StateTreeTaskBase.generated.h"

#define UE_API STATETREEMODULE_API

struct FStateTreeExecutionContext;
struct FStateTreeReadOnlyExecutionContext;

/**
 * Base struct for StateTree Tasks.
 * Tasks are logic executed in an active state.
 */
USTRUCT(meta = (Hidden))
struct FStateTreeTaskBase : public FStateTreeNodeBase
{
	GENERATED_BODY()

	FStateTreeTaskBase()
		: bShouldStateChangeOnReselect(true)
		, bShouldCallTick(true)
		, bShouldCallTickOnlyOnEvents(false)
		, bShouldCopyBoundPropertiesOnTick(true)
		, bShouldCopyBoundPropertiesOnExitState(true)
		, bShouldAffectTransitions(false)
		, bConsideredForScheduling(true)
		, bTaskEnabled(true)
		, bHasTaskCompletionDelegateDispatcher(false)
#if WITH_EDITORONLY_DATA
		, bConsideredForCompletion(true)
		, bCanEditConsideredForCompletion(true)
#endif
	{
	}

	/**
	 * Called when a new state is entered and task is part of active states.
	 * @param Context Reference to current execution context.
	 * @param Transition Describes the states involved in the transition
	 * @return Succeed/Failed will end the state immediately and trigger to select new state, Running will carry on to tick the state.
	 */
	virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
	{
		return EStateTreeRunStatus::Running;
	}

	/**
	 * Called when a current state is exited and task is part of active states.
	 * @param Context Reference to current execution context.
	 * @param Transition Describes the states involved in the transition
	 */
	virtual void ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
	{
	}

	/**
	 * Called right after a state has been completed, but before new state has been selected. StateCompleted is called in reverse order to allow to propagate state to other Tasks that
	 * are executed earlier in the tree. Note that StateCompleted is not called if conditional transition changes the state.
	 * @param Context Reference to current execution context.
	 * @param CompletionStatus Describes the running status of the completed state (Succeeded/Failed).
	 * @param CompletedActiveStates Active states at the time of completion.
	 */
	virtual void StateCompleted(FStateTreeExecutionContext& Context, const EStateTreeRunStatus CompletionStatus, const FStateTreeActiveStates& CompletedActiveStates) const
	{
	}

	/**
	 * Called during state tree tick when the task is on active state.
	 * Note: The method is called only if bShouldCallTick or bShouldCallTickOnlyOnEvents is set.
	 * @param Context Reference to current execution context.
	 * @param DeltaTime Time since last StateTree tick.
	 * @return Running status of the state: Running if still in progress, Succeeded if execution is done and succeeded, Failed if execution is done and failed.
	 */
	virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const
	{
		return EStateTreeRunStatus::Running;
	}

	/**
	 * Called when state tree triggers transitions. This method is called during transition handling, before state's tick and event transitions are handled.
	 * Note: the method is called only if bShouldAffectTransitions is set.
	 * @param Context Reference to current execution context.
	 */
	virtual void TriggerTransitions(FStateTreeExecutionContext& Context) const
	{
	};

#if WITH_EDITOR
	virtual FName GetIconName() const override
	{
		return FName("StateTreeEditorStyle|Node.Task");
	}
	virtual FColor GetIconColor() const override
	{
		return UE::StateTree::Colors::Grey;
	}
#endif

#if WITH_GAMEPLAY_DEBUGGER
	UE_API virtual FString GetDebugInfo(const FStateTreeReadOnlyExecutionContext& Context) const;
#endif

	/**
	 * If set to true, the task will receive EnterState/ExitState even if the state was previously active.
	 * Generally this should be true for action type tasks, like playing animation,
	 * and false on state like tasks like claiming a resource that is expected to be acquired on child states.
	 * Default value is true.
	 */
	uint8 bShouldStateChangeOnReselect : 1;

	/** If set to true, Tick() is called. Not ticking implies no property copy. Default true. */
	uint8 bShouldCallTick : 1;
	/** If set to true, Tick() is called only when there are events. No effect if bShouldCallTick is true. Not ticking implies no property copy. Default false. */
	uint8 bShouldCallTickOnlyOnEvents : 1;

	/** If set to true, copy the values of bound properties before calling Tick(). Default true. */
	uint8 bShouldCopyBoundPropertiesOnTick : 1;
	/** If set to true, copy the values of bound properties before calling ExitState(). Default true. */
	uint8 bShouldCopyBoundPropertiesOnExitState : 1;

	/** If set to true, TriggerTransitions() is called during transition handling. Default false. */
	uint8 bShouldAffectTransitions : 1;

	/**
	 * If set to true, the task is considered for scheduled tick. It will use these flags: bShouldCallTick, bShouldCallTickOnlyOnEvents, and bShouldAffectTransitions.
	 * It doesn't affect how the task ticks.
	 * Default true.
	 */
	uint8 bConsideredForScheduling : 1;

	/** True if the node is Enabled (i.e. not explicitly disabled in the asset). */
	UPROPERTY()
	uint8 bTaskEnabled : 1;

	/** True if the node is bound to a task completion delegate listener. */
	UPROPERTY()
	uint8 bHasTaskCompletionDelegateDispatcher : 1;

	UPROPERTY()
	EStateTreeTransitionPriority TransitionHandlingPriority = EStateTreeTransitionPriority::Normal;

#if WITH_EDITORONLY_DATA
	/**
	 * True if the task is considered for completion.
	 * False if the task runs in the background without affecting the state completion.
	 */
	UPROPERTY()
	uint8 bConsideredForCompletion : 1;

	/** True if the user can edit bConsideredForCompletion in the editor. */
	UPROPERTY()
	uint8 bCanEditConsideredForCompletion : 1;
#endif
};

/**
 * Base class (namespace) for all common Tasks that are generally applicable.
 * This allows schemas to safely include all conditions child of this struct. 
 */
USTRUCT(meta = (Hidden))
struct FStateTreeTaskCommonBase : public FStateTreeTaskBase
{
	GENERATED_BODY()
};

#undef UE_API
