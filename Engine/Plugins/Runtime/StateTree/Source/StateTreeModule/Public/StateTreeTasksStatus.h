// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotNull.h"
#include "Traits/IsTriviallyRelocatable.h"
#include "StateTreeTasksStatus.generated.h"

class FArchive;
struct FCompactStateTreeFrame;
struct FCompactStateTreeState;
class UPackageMap;
class UStateTree;

UENUM()
enum class EStateTreeTaskCompletionType : uint8
{
	/** All tasks need to complete for the group to completes. */
	All,
	/** Any task completes the group. */
	Any,
};

namespace UE::StateTree
{
	enum class ETaskCompletionStatus : uint8
	{
		/** The task is running. */
		Running = 0,
		/**
		 * The task stopped without a particular reason. (ie. Aborted).
		 * Use for backward compatibility. Prefer Succeeded and Failed.
		 */
		Stopped = 1,
		/** The task stopped with a success. */
		Succeeded = 2,
		/** The task stopped with a failure. */
		Failed = 3,
	};
	/** Number of entry in the ETaskCompletionStatus. */
	static constexpr int32 NumberOfTaskStatus = 4;

	/** */
	template<typename T>
	struct TTasksCompletionStatus
	{
	public:
		static constexpr int32 MaxNumTasks = sizeof(T)*8;

	public:
		TTasksCompletionStatus(TNotNull<T*> InFirstCompletionBits, TNotNull<T*> InSecondCompletionBits, T InCompletionMask, int32 InBitIndex, EStateTreeTaskCompletionType InTaskControl)
			: FirstCompletionBits(InFirstCompletionBits)
			, SecondCompletionBits(InSecondCompletionBits)
			, CompletionMask(InCompletionMask)
			, BitIndex(InBitIndex)
			, TaskControl(InTaskControl)
		{}

	public:
		/**
		 * @param StateTaskIndex The index of the task in the state.
		 * @return the status of a task.
		 */
		[[nodiscard]] ETaskCompletionStatus GetStatus(int32 StateTaskIndex) const
		{
			StateTaskIndex += BitIndex;
			check(StateTaskIndex < MaxNumTasks);
			return static_cast<ETaskCompletionStatus>(GetStatusInternal(StateTaskIndex));
		}

		/**
		 * The completion status of all tasks or any task, in priority order.
		 * If any of the tasks fail, the completion status will be Failed regardless of any other tasks.
		 * In case the type is Any, it will return Succeeded before Stopped and before Running.
		 * @return the completion status of all tasks.
		 */
		[[nodiscard]] ETaskCompletionStatus GetCompletionStatus() const
		{
			const T FirstCompletionBitsMasked = (*FirstCompletionBits) & CompletionMask;
			const T SecondCompletionBitsMasked = (*SecondCompletionBits) & CompletionMask;
			if ((SecondCompletionBitsMasked & FirstCompletionBitsMasked) != 0)
			{
				return ETaskCompletionStatus::Failed;
			}
			if (TaskControl == EStateTreeTaskCompletionType::All)
			{
				if (SecondCompletionBitsMasked == CompletionMask)
				{
					return ETaskCompletionStatus::Succeeded;
				}
				if (FirstCompletionBitsMasked == CompletionMask)
				{
					return ETaskCompletionStatus::Stopped;
				}
				// if we have a mix of Succeeded and Stopped, return Succeeded.
				return (FirstCompletionBitsMasked | SecondCompletionBitsMasked) == CompletionMask ? ETaskCompletionStatus::Succeeded : ETaskCompletionStatus::Running;
			}
			else
			{
				if (SecondCompletionBitsMasked != 0)
				{
					return ETaskCompletionStatus::Succeeded;
				}
				return FirstCompletionBitsMasked != 0 ? ETaskCompletionStatus::Stopped : ETaskCompletionStatus::Running;
			}
		}

		/**
		 * @param StateTaskIndex The index of the task in the state.
		 * @return true when the task is considered for GetCompletionStatus, HasAllCompleted, HasAnyCompleted, HasAnyFailed, IsCompleted
		 */
		[[nodiscard]] bool IsConsideredForCompletion(int32 StateTaskIndex) const
		{
			StateTaskIndex += BitIndex;
			check(StateTaskIndex < MaxNumTasks);
			const T TaskMask = T(1) << StateTaskIndex;
			return (TaskMask & CompletionMask) != 0;
		}

		/**
		 * @param StateTaskIndex The index of the task in the state.
		 * @return true when the task status is running.
		 */
		[[nodiscard]] bool IsRunning(int32 StateTaskIndex) const
		{
			StateTaskIndex += BitIndex;
			check(StateTaskIndex < MaxNumTasks);
			const T TaskMask = T(1) << StateTaskIndex;
			return (((*FirstCompletionBits) | (*SecondCompletionBits)) & TaskMask) == 0;
		}

		/**
		 * @param StateTaskIndex The index of the task in the state.
		 * @return true when the task status is failed.
		 */
		[[nodiscard]] bool HasFailed(int32 StateTaskIndex) const
		{
			StateTaskIndex += BitIndex;
			check(StateTaskIndex < MaxNumTasks);
			const T TaskMask = T(1) << StateTaskIndex;
			return ((*FirstCompletionBits) & (*SecondCompletionBits) & TaskMask) != 0;
		}

		/** @return true when there's any failure or all tasks succeeded or stopped. */
		[[nodiscard]] bool HasAllCompleted() const
		{
			const bool bHasAnyFailure = ((*FirstCompletionBits) & (*SecondCompletionBits) & CompletionMask) != 0;
			const T CompletionResult = (((*FirstCompletionBits) | (*SecondCompletionBits)) & CompletionMask);
			return bHasAnyFailure || CompletionResult == CompletionMask;
		}

		/** @return true when there's any failure or any tasks succeeded or stopped. */
		[[nodiscard]] bool HasAnyCompleted() const
		{
			return (((*FirstCompletionBits) | (*SecondCompletionBits)) & CompletionMask) != 0;
		}

		/** @return true when there's any failure. */
		[[nodiscard]] bool HasAnyFailed() const
		{
			return ((*FirstCompletionBits) & (*SecondCompletionBits) & CompletionMask) != 0;
		}

		/** @return true when there's any failure or respect the task control. */
		[[nodiscard]] bool IsCompleted() const
		{
			return TaskControl == EStateTreeTaskCompletionType::All ? HasAllCompleted() : HasAnyCompleted();
		}

		/**
		 * Set the status of a task.
		 * @param StateTaskIndex The index of the task in the state.
		 */
		void SetStatus(int32 StateTaskIndex, ETaskCompletionStatus NewStatus)
		{
			StateTaskIndex += BitIndex;
			check(StateTaskIndex < MaxNumTasks);
			SetStatusInternal(StateTaskIndex, NewStatus);
		}

		/**
		 * Set the status of a task respecting the previous status value.
		 * The priority is Failed, Succeeded, Stopped, Running.
		 * @param StateTaskIndex The index of the task in the state.
		 */
		ETaskCompletionStatus SetStatusWithPriority(int32 StateTaskIndex, ETaskCompletionStatus NewStatus)
		{
			StateTaskIndex += BitIndex;
			check(StateTaskIndex < MaxNumTasks);
			uint8 CurrentStatus = GetStatusInternal(StateTaskIndex);
			if (static_cast<uint8>(NewStatus) > CurrentStatus)
			{
				SetStatusInternal(StateTaskIndex, NewStatus);
				CurrentStatus = static_cast<uint8>(NewStatus);
			}
			return static_cast<ETaskCompletionStatus>(CurrentStatus);
		}

		struct FSetStatusResult
		{
			ETaskCompletionStatus CurrentStatus = ETaskCompletionStatus::Running;
			bool bChanged = false;
		};

		/**
		 * Set the status of a task respecting the previous status value and report if it changed.
		 * The priority is Failed, Succeeded, Stopped, Running.
		 * @param StateTaskIndex The index of the task in the state.
		 * @param NewStatus The new status to set.
		 * @return Result containing current status and whether it changed.
		 */
		FSetStatusResult SetTaskStatusWithPriority(int32 StateTaskIndex, ETaskCompletionStatus NewStatus)
		{
			StateTaskIndex += BitIndex;
			check(StateTaskIndex < MaxNumTasks);
			const uint8 OldStatus = GetStatusInternal(StateTaskIndex);
			uint8 CurrentStatus = OldStatus;
			bool bChanged = false;
			if (static_cast<uint8>(NewStatus) > OldStatus)
			{
				SetStatusInternal(StateTaskIndex, NewStatus);
				CurrentStatus = static_cast<uint8>(NewStatus);
				bChanged = true;
			}
			return FSetStatusResult{ static_cast<ETaskCompletionStatus>(CurrentStatus), bChanged };
		}

		/** Set the status of all tasks in the completion mask. */
		void SetCompletionStatus(ETaskCompletionStatus NewStatus)
		{
			const T ClearMask = ~CompletionMask;
			(*FirstCompletionBits) &= ClearMask;
			(*SecondCompletionBits) &= ClearMask;
			if ((static_cast<uint8>(NewStatus) & 0x1) != 0)
			{
				(*FirstCompletionBits) |= CompletionMask;
			}
			if ((static_cast<uint8>(NewStatus) & 0x2) != 0)
			{
				(*SecondCompletionBits) |= CompletionMask;
			}
		}

		/** Set the status of all tasks to running. */
		void ResetStatus(int32 NumberOfTasksInTheCompletionMask)
		{
			T ClearBuffer = 0;
			if (NumberOfTasksInTheCompletionMask == 0)
			{
				// All mask as at least one bit to mark the state.
				ClearBuffer = 1 << BitIndex;
			}
			else if (NumberOfTasksInTheCompletionMask == MaxNumTasks)
			{
				// Prevent buffer overflow.
				ClearBuffer = (T)(-1);
			}
			else
			{
				ClearBuffer = ((1 << NumberOfTasksInTheCompletionMask) - 1) << BitIndex;
			}
			const T ClearMask = ~ClearBuffer;
			(*FirstCompletionBits) &= ClearMask;
			(*SecondCompletionBits) &= ClearMask;
		}

	private:
		uint8 GetStatusInternal(int32 Index) const
		{
			T Result = ((*SecondCompletionBits) >> Index) & 0x1;
			Result <<= 1;
			Result += ((*FirstCompletionBits) >> Index) & 0x1;
			return static_cast<uint8>(Result);
		}

		void SetStatusInternal(int32 Index, ETaskCompletionStatus NewStatus)
		{
			// Clear the current value, then set the new value.
			const T ClearMask = ~(T(1) << Index);
			(*FirstCompletionBits) &= ClearMask;
			(*SecondCompletionBits) &= ClearMask;
			(*FirstCompletionBits) |= (static_cast<T>(NewStatus) & 0x1) << Index;
			(*SecondCompletionBits) |= ((static_cast<T>(NewStatus) & 0x2) >> 1) << Index;
		}

		/** The first buffer. 00 is Running and 01 is Stopped. */
		T* FirstCompletionBits;
		/** The second buffer. 10 is Succeeded and 11 is Failed. */
		T* SecondCompletionBits;
		/** The mask that represents each task considered by the state/frame for completion. */
		T CompletionMask;
		/** The offset, in bits, of the first task inside the mask. */
		int32 BitIndex;
		/** How the mask is tested to complete the state/frame. */
		EStateTreeTaskCompletionType TaskControl;
	};

	using FTasksCompletionStatus = TTasksCompletionStatus<uint32>;
	using FConstTasksCompletionStatus = TTasksCompletionStatus<const uint32>;
}

/**
 * Container for task status for all the active states and global tasks.
 * Each task needs 2 bits of information. The information is in 2 different int32. 1 bit per int32 instead of 2bits inside the same int32.
 * The int32 are sequential: The first masks takes the 2 first int32 and (if needed) the second mask takes the 3rd and 4th int32.
 * A state/global has at least 1 entry (2bits), even if there are no tasks on the state/global. It is to represent if the state completes (ex: when no tasks are ticked the state completes).
 * The bits, from different states, are packed until there are too many to fit inside the same int32.
 * When it can't be packed, all the bits from that overflow state are moved to the next int32.
 * When possible (when the number of tasks is below 32), the buffer will be inlined. Otherwise, it will use dynamic memory.
 * We allocate the worst case scenario when the frame is created.
 * 
 * Ex: For the tree:
 *     (0)Global task: 8 tasks. (1)State Root: 6 tasks. (2)StateA: no task (it takes one bit). (3)StateB: 10 tasks.
 *     (4)State: 8 tasks, not enough space go to the next int32. (5)State: 1 task, takes one bit.
 *     The mask looks like:
 * [------33333333333211111100000000|-----------------------544444444]
 *     The buffer will be 4 int32.
 *     The first 2 int32 are for the (0)Global task, (1)State Root, (2)State, (3)StateB
 *     The next 2 int32 are for the (4)State, (5)State
 * 
 * The first bit of each buffer is combined to represent the ETaskCompletionStatus.
 * Ex: (on int8 instead of int32 to be shorter in this description)
 *     There are 3 tasks. The bits 0-2 are used. The other bits are never set/read. We use the "completion mask" to filter them.
 * [00001100|00001010]
 *     Task 1 has the value 00 (0 from the first buffer and 0 from the second buffer). It's Running.
 *     Task 2 has the value 01 (0 from the first buffer and 1 from the second buffer). It's Stopped.
 *     Task 3 has the value 10 (1 from the first buffer and 0 from the second buffer). It's Succeeded.
 */
USTRUCT()
struct FStateTreeTasksCompletionStatus
{
	GENERATED_BODY()

	using FMaskType = uint32;
	static constexpr int32 MaxNumberOfTasksPerGroup = sizeof(FMaskType)*8;
	/** 32 global global tasks + 32 tasks per states for a max of 8 states(max 8 states per frame). */
	static constexpr int32 MaxTotalAmountOfTasks = MaxNumberOfTasksPerGroup + MaxNumberOfTasksPerGroup * 8;

public:
	FStateTreeTasksCompletionStatus() = default;
	STATETREEMODULE_API explicit FStateTreeTasksCompletionStatus(const FCompactStateTreeFrame& Frame);
	STATETREEMODULE_API ~FStateTreeTasksCompletionStatus();

	STATETREEMODULE_API FStateTreeTasksCompletionStatus(const FStateTreeTasksCompletionStatus&);
	STATETREEMODULE_API FStateTreeTasksCompletionStatus(FStateTreeTasksCompletionStatus&&);
	STATETREEMODULE_API FStateTreeTasksCompletionStatus& operator=(const FStateTreeTasksCompletionStatus&);
	STATETREEMODULE_API FStateTreeTasksCompletionStatus& operator=(FStateTreeTasksCompletionStatus&&);

public:
	[[nodiscard]] STATETREEMODULE_API UE::StateTree::FTasksCompletionStatus GetStatus(const FCompactStateTreeState& State);
	[[nodiscard]] STATETREEMODULE_API UE::StateTree::FConstTasksCompletionStatus GetStatus(const FCompactStateTreeState& State) const;

	[[nodiscard]] STATETREEMODULE_API UE::StateTree::FTasksCompletionStatus GetStatus(TNotNull<const UStateTree*> StateTree);
	[[nodiscard]] STATETREEMODULE_API UE::StateTree::FConstTasksCompletionStatus GetStatus(TNotNull<const UStateTree*> StateTree) const;

	/** @return true when the status is initialized correctly. */
	[[nodiscard]] bool IsValid() const
	{
		return BufferNum > 0;
	}

	STATETREEMODULE_API void Push(const FCompactStateTreeState& State);

	STATETREEMODULE_API bool Serialize(FArchive& Ar);
	STATETREEMODULE_API bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess);

private:
	static constexpr int32 MaxNumberOfTaskForInlineBuffer = sizeof(FMaskType*) * 8 / 2;

	inline bool UseInlineBuffer() const
	{
		return BufferNum <= 1;
	}

	void MallocBufferIfNeeded();
	void CopyBuffer(const FStateTreeTasksCompletionStatus& Other);

	template<typename TTasksCompletionStatusType>
	inline TTasksCompletionStatusType GetStatusInternal(FMaskType Mask, uint8 BufferIndex, uint8 BitsOffset, EStateTreeTaskCompletionType Control);

	/**
	 * Dynamic or inlined container for a FMaskType.
	 * The actual memory used is double of BufferNum because each task has 2 bits of information.
	 */
	FMaskType* Buffer = nullptr;
	/**
	 * Number of requested FMaskType.
	 * If <= 1, it will use Buffer as an inlined buffer.
	 */
	uint8 BufferNum = 0;
};

template<>
struct TStructOpsTypeTraits<FStateTreeTasksCompletionStatus> : public TStructOpsTypeTraitsBase2<FStateTreeTasksCompletionStatus>
{
	enum
	{
		WithCopy = true,
		WithSerializer = true,
		WithNetSerializer = true,
	};
};

// FStateTreeTasksCompletionStatus is not trivially relocatable. Flag their use in containers and other contexts.
template <UE::CDerivedFrom<FStateTreeTasksCompletionStatus> T>
struct TIsTriviallyRelocatable<T>
{
	static inline constexpr bool Value = false;
};
