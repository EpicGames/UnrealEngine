// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "PropertyBindingPath.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6

#include "PropertyBindingDataView.h"
#include "StructUtils/PropertyBag.h"
#include "GameplayTagContainer.h"
#include "StateTreeDelegate.h"
#include "StateTreeIndexTypes.h"
#include "StateTreeTasksStatus.h"
#include "StateTreeTypes.generated.h"

STATETREEMODULE_API DECLARE_LOG_CATEGORY_EXTERN(LogStateTree, Warning, All);

#ifndef WITH_STATETREE_DEBUG
#define WITH_STATETREE_DEBUG (!(UE_BUILD_SHIPPING || UE_BUILD_SHIPPING_WITH_EDITOR || UE_BUILD_TEST) && 1)
#endif // WITH_STATETREE_DEBUG

class UStateTree;
struct FStateTreeEvent;

namespace UE::StateTree
{
	inline constexpr int32 MaxExpressionIndent = 4;

	inline const FName SchemaTag(TEXT("Schema"));

	inline const FName SchemaCanBeOverridenTag(TEXT("SchemaCanBeOverriden"));

	namespace Colors
	{
		// Common and consistent colors to be used with State Tree nodes.
		extern const STATETREEMODULE_API FColor Grey;
		extern const STATETREEMODULE_API FColor DarkGrey;
		extern const STATETREEMODULE_API FColor Red;
		extern const STATETREEMODULE_API FColor DarkRed;
		extern const STATETREEMODULE_API FColor Orange;
		extern const STATETREEMODULE_API FColor DarkOrange;
		extern const STATETREEMODULE_API FColor Yellow;
		extern const STATETREEMODULE_API FColor DarkYellow;
		extern const STATETREEMODULE_API FColor Green;
		extern const STATETREEMODULE_API FColor DarkGreen;
		extern const STATETREEMODULE_API FColor Cyan;
		extern const STATETREEMODULE_API FColor DarkCyan;
		extern const STATETREEMODULE_API FColor Blue;
		extern const STATETREEMODULE_API FColor DarkBlue;
		extern const STATETREEMODULE_API FColor Purple;
		extern const STATETREEMODULE_API FColor DarkPurple;
		extern const STATETREEMODULE_API FColor Magenta;
		extern const STATETREEMODULE_API FColor DarkMagenta;
		extern const STATETREEMODULE_API FColor Bronze;
		extern const STATETREEMODULE_API FColor DarkBronze;
	} // Colors

	UENUM()
	enum class EComparisonOperator : uint8
	{
		Less,
		LessOrEqual,
		Equal,
		NotEqual,
		GreaterOrEqual,
		Greater,
	};

}; // UE::StateTree

enum class EStateTreeRunStatus : uint8;

/** Transitions behavior. */
UENUM()
enum class EStateTreeTransitionType : uint8
{
	/** No transition will take place. */
	None,

	/** Stop State Tree or sub-tree and mark execution succeeded. */
	Succeeded,
	
	/** Stop State Tree or sub-tree and mark execution failed. */
	Failed,
	
	/** Transition to the specified state. */
	GotoState,

	/** Transition to the parent state. */
	Parent,
	
	/** Transition to the next sibling state. */
	NextState,

	/** Transition to the next selectable sibling state. */
	NextSelectableState,

	/** Transition to the next sibling state of the parent state. */
	NextParent,

	/** Transition to the next selectable sibling state of the parent state. */
	NextSelectableParent,

	NotSet UE_DEPRECATED(all, "Use None instead."),
};

/** The rules used by the execution context for selecting states. */
UENUM()
enum class EStateTreeStateSelectionRules : uint32
{
	/** Previous (UE 5.6) rules */
	None = 0,
	/**
	 * If the transition source state or the transition target state (or any child states of the source or target) is completed, then the completed states are new (recreated if needed).
	 * When a new state is created, all the following states in the selection path are also new (recreated if needed).
	 * Without this rule, the state will remain active even if it's completed. The state might never trigger a complete transition.
	 */
	CompletedTransitionStatesCreateNewStates = 1 << 0,
	/**
	 * The transition will fail if there is a completed state before the transition source state.
	 * Without this rule, a transition that triggers on completion might take an extra tick to execute.
	 */
	CompletedStateBeforeTransitionSourceFailsTransition = 1 << 1,
	/**
	 * When selecting a parent state(from a child), the target child states are new (recreated if needed).
	 * Without this rule, the "reselected" states are sustained.
	 */
	ReselectedStateCreatesNewStates = 1 << 2,

	Default = CompletedTransitionStatesCreateNewStates | CompletedStateBeforeTransitionSourceFailsTransition,
};
ENUM_CLASS_FLAGS(EStateTreeStateSelectionRules);

/** Operand in an expression */
UENUM()
enum class EStateTreeExpressionOperand : uint8
{
	/** Copy result */
	Copy UMETA(Hidden),
	
	/** Combine results with AND for Condition, Min(a, b) for Consideration */
	And,
	
	/** Combine results with OR for Condition, Max(a, b) for Consideration */
	Or,

	/** Combine results with (a * b) for Consideration, not applicable for Condition */
	Multiply,
};

UENUM()
enum class EStateTreeStateType : uint8
{
	/** A State containing tasks and child states. */
	State,
	
	/** A State containing just child states. */
	Group,
	
	/** A State that is linked to another state in the tree (the execution continues on the linked state). */
	Linked,

	/** A State that is linked to another StateTree asset (the execution continues on the Root state of the linked asset). */
	LinkedAsset,

	/** A subtree that can be linked to. */
	Subtree,
};

UENUM()
enum class EStateTreeStateSelectionBehavior : uint8
{
	/** The State cannot be directly selected. */
	None,

	/** When state is considered for selection, it is selected even if it has child states. */
	TryEnterState UMETA(DisplayName = "Try Enter"),

	/** When state is considered for selection, try to select the first child state (in order they appear in the child list). If no child states are present, behaves like SelectState. */
	TrySelectChildrenInOrder UMETA(DisplayName = "Try Select Children In Order"),
	
	/** When state is considered for selection, shuffle the order of child states and try to select the first one. If no child states are present, behaves like SelectState. */
	TrySelectChildrenAtRandom UMETA(DisplayName = "Try Select Children At Random"),
	
	/** When state is considered for selection, try to select the child state with highest utility score. If there is a tie, it will try to select in order. */
	TrySelectChildrenWithHighestUtility UMETA(DisplayName = "Try Select Children With Highest Utility"),

	/** When state is considered for selection, randomly pick one of its child states. The probability of selecting each child state is its normalized utility score */
	TrySelectChildrenAtRandomWeightedByUtility UMETA(DisplayName = "Try Select Children At Random Weighted By Utility"),

	/** When state is considered for selection, try to trigger the transitions instead. */
	TryFollowTransitions UMETA(DisplayName = "Try Follow Transitions"),

	// Olds names that needs to be kept forever to ensure asset serialization to work correctly when UENUM() switched from serializing int to names.
	TrySelectChildrenAtUniformRandom UE_DEPRECATED(all, "Use TrySelectChildrenAtRandom Instead") = TrySelectChildrenAtRandom UMETA(Hidden),
	TrySelectChildrenBasedOnRelativeUtility UE_DEPRECATED(all, "Use TrySelectChildrenAtRandomWeightedByUtility Instead") = TrySelectChildrenAtRandomWeightedByUtility UMETA(Hidden)
};

namespace UE::StateTree
{
	static bool IsSelectionBehaviorUsingUtility(const EStateTreeStateSelectionBehavior Value)
	{
		switch(Value)
		{
			case EStateTreeStateSelectionBehavior::TrySelectChildrenWithHighestUtility:
			case EStateTreeStateSelectionBehavior::TrySelectChildrenAtRandomWeightedByUtility:
				return true;
			default:
				return false;
		}
	}
}


/** Transitions trigger. */
UENUM()
enum class EStateTreeTransitionTrigger : uint8
{
	None = 0 UMETA(Hidden),

	/** Try trigger transition when a state succeeded or failed. */
	OnStateCompleted = 0x1 | 0x2,

	/** Try trigger transition when a state succeeded. */
    OnStateSucceeded = 0x1,

	/** Try trigger transition when a state failed. */
    OnStateFailed = 0x2,

	/** Try trigger transition each State Tree tick. */
    OnTick = 0x4,
	
	/** Try trigger transition on specific State Tree event. */
	OnEvent = 0x8,

	/** Try trigger transition on specific State Tree delegate. */
	OnDelegate = 0x10,

	MAX
};
ENUM_CLASS_FLAGS(EStateTreeTransitionTrigger)


/** Transition priority. When multiple transitions trigger at the same time, the first transition of highest priority is selected. */
UENUM(BlueprintType)
enum class EStateTreeTransitionPriority : uint8
{
	None UMETA(Hidden),

	/** Low priority. */
	Low,

	/** Normal priority. */
	Normal,
	
	/** Medium priority. */
	Medium,
	
	/** High priority. */
	High,
	
	/** Critical priority. */
	Critical,
};

inline bool operator<(const EStateTreeTransitionPriority Lhs, const EStateTreeTransitionPriority Rhs) { return static_cast<uint8>(Lhs) < static_cast<uint8>(Rhs); }
inline bool operator>(const EStateTreeTransitionPriority Lhs, const EStateTreeTransitionPriority Rhs) { return static_cast<uint8>(Lhs) > static_cast<uint8>(Rhs); }
inline bool operator<=(const EStateTreeTransitionPriority Lhs, const EStateTreeTransitionPriority Rhs) { return static_cast<uint8>(Lhs) <= static_cast<uint8>(Rhs); }
inline bool operator>=(const EStateTreeTransitionPriority Lhs, const EStateTreeTransitionPriority Rhs) { return static_cast<uint8>(Lhs) >= static_cast<uint8>(Rhs); }
inline bool operator==(const EStateTreeTransitionPriority Lhs, const EStateTreeTransitionPriority Rhs) { return static_cast<uint8>(Lhs) == static_cast<uint8>(Rhs); }
inline bool operator!=(const EStateTreeTransitionPriority Lhs, const EStateTreeTransitionPriority Rhs) { return static_cast<uint8>(Lhs) != static_cast<uint8>(Rhs); }

/** Handle to a StateTree state */
USTRUCT(BlueprintType)
struct FStateTreeStateHandle
{
	GENERATED_BODY()

	static constexpr uint16 InvalidIndex = uint16(-1);		// Index value indicating invalid state.
	static constexpr uint16 SucceededIndex = uint16(-2);	// Index value indicating a Succeeded state.
	static constexpr uint16 FailedIndex = uint16(-3);		// Index value indicating a Failed state.
	static constexpr uint16 StoppedIndex = uint16(-4);		// Index value indicating a Stopped state.
	
	static STATETREEMODULE_API const FStateTreeStateHandle Invalid;
	static STATETREEMODULE_API const FStateTreeStateHandle Succeeded;
	static STATETREEMODULE_API const FStateTreeStateHandle Failed;
	static STATETREEMODULE_API const FStateTreeStateHandle Stopped;
	static STATETREEMODULE_API const FStateTreeStateHandle Root;

	/** @return true if the given index can be represented by the type. */
	static bool IsValidIndex(const int32 Index)
	{
		return Index >= 0 && Index < (int32)MAX_uint16;
	}

	friend inline uint32 GetTypeHash(const FStateTreeStateHandle& Handle)
	{
		return GetTypeHash(Handle.Index);
	}

	FStateTreeStateHandle() = default;
	explicit FStateTreeStateHandle(const uint16 InIndex) : Index(InIndex) {}
	explicit FStateTreeStateHandle(const int32 InIndex) : Index()
	{
		check(InIndex == INDEX_NONE || IsValidIndex(InIndex));
		Index = InIndex == INDEX_NONE ? InvalidIndex : static_cast<uint16>(InIndex);	
	}

	bool IsValid() const { return Index != InvalidIndex; }
	void Invalidate() { Index = InvalidIndex; }
	bool IsCompletionState() const { return Index == SucceededIndex || Index == FailedIndex || Index == StoppedIndex; }
	STATETREEMODULE_API EStateTreeRunStatus ToCompletionStatus() const;
	static STATETREEMODULE_API FStateTreeStateHandle FromCompletionStatus(const EStateTreeRunStatus Status);

	bool operator==(const FStateTreeStateHandle& RHS) const { return Index == RHS.Index; }
	bool operator!=(const FStateTreeStateHandle& RHS) const { return Index != RHS.Index; }

	STATETREEMODULE_API FString Describe() const;

	UPROPERTY()
	uint16 Index = InvalidIndex;
};


/** Data type the FStateTreeDataHandle is pointing at. */
UENUM(BlueprintType)
enum class EStateTreeDataSourceType : uint8
{
	None UMETA(Hidden),

	/** Global Tasks, Evaluators. */
	GlobalInstanceData,

	/** Global Tasks, Evaluators. */
	GlobalInstanceDataObject,

	/** Active State Tasks */
	ActiveInstanceData,

	/** Active State Tasks */
	ActiveInstanceDataObject,

	/** Conditions, considerations and function bindings. */
	SharedInstanceData,

	/** Conditions, considerations and function bindings*/
	SharedInstanceDataObject,

	/** Temporary data constructor, used and destroyed immediately. Used by conditions, considerations and function bindings.*/
	EvaluationScopeInstanceData,

	/** Temporary data constructor, used and destroyed immediately. Used by conditions, considerations and function bindings.*/
	EvaluationScopeInstanceDataObject,

	/** Node instance data for the duration of the execution. */
	ExecutionRuntimeData,

	/** Node instance data for the duration of the execution. */
	ExecutionRuntimeDataObject,

	/** Node instance data for the duration of the execution. Discover at runtime if it's a struct or an object. */
	ExecutionRuntimeDataAny,

	/** Context Data, Tree Parameters */
	ContextData,

	/** External Data required by the nodes. */
	ExternalData,

	/** Global parameters */
	GlobalParameterData,

	/** Parameters for subtree (may resolve to a linked state's parameters or default params) */
	SubtreeParameterData,

	/** Parameters for regular and linked states */
	StateParameterData,

	/** Event used in transition. */
	TransitionEvent,

	/** Event used in state selection. */
	StateEvent,

	/** Global parameters provided externally */
	ExternalGlobalParameterData,
};

/** Data type the global parameter type uses. */
UENUM()
enum class EStateTreeParameterDataType : uint8
{
	GlobalParameterData,
	ExternalGlobalParameterData,
};

namespace UE::StateTree
{
	static EStateTreeDataSourceType CastToDataSourceType(EStateTreeParameterDataType Value)
	{
		return Value == EStateTreeParameterDataType::ExternalGlobalParameterData ? EStateTreeDataSourceType::ExternalGlobalParameterData : EStateTreeDataSourceType::GlobalParameterData;
	}
}

/** Handle to a StateTree data */
USTRUCT(BlueprintType)
struct FStateTreeDataHandle
{
	GENERATED_BODY()

	static STATETREEMODULE_API const FStateTreeDataHandle Invalid;
	static constexpr uint16 InvalidIndex = MAX_uint16;

	/** @return true if the given index can be represented by the type. */
	static bool IsValidIndex(const int32 Index)
	{
		return Index >= 0 && Index < (int32)InvalidIndex;
	}

	bool HasValidIndex() const
	{
		return IsValidIndex(Index);
	}

	friend inline uint32 GetTypeHash(const FStateTreeDataHandle& Handle)
	{
		uint32 Hash = GetTypeHash(Handle.Index);
		Hash = HashCombineFast(Hash, GetTypeHash(Handle.Source));
		Hash = HashCombineFast(Hash, GetTypeHash(Handle.StateHandle));
		return Hash;
	}

	FStateTreeDataHandle() = default;

	explicit FStateTreeDataHandle(const EStateTreeDataSourceType InSource, const uint16 InIndex, const FStateTreeStateHandle InStateHandle = FStateTreeStateHandle::Invalid)
		: Source(InSource)
		, Index(InIndex)
		, StateHandle(InStateHandle)
	{
		// Require valid state for active instance data
		check(Source != EStateTreeDataSourceType::ActiveInstanceData || (Source == EStateTreeDataSourceType::ActiveInstanceData && StateHandle.IsValid()));
		check(Source != EStateTreeDataSourceType::ActiveInstanceDataObject || (Source == EStateTreeDataSourceType::ActiveInstanceDataObject && StateHandle.IsValid()));
		check(Source == EStateTreeDataSourceType::GlobalParameterData || Source == EStateTreeDataSourceType::ExternalGlobalParameterData || IsValidIndex(InIndex));
	}

	explicit FStateTreeDataHandle(const EStateTreeDataSourceType InSource, const int32 InIndex, const FStateTreeStateHandle InStateHandle = FStateTreeStateHandle::Invalid)
		: Source(InSource)
		, StateHandle(InStateHandle)
	{
		// Require valid state for active instance data
		check(Source != EStateTreeDataSourceType::ActiveInstanceData || (Source == EStateTreeDataSourceType::ActiveInstanceData && StateHandle.IsValid()));
		check(Source != EStateTreeDataSourceType::ActiveInstanceDataObject || (Source == EStateTreeDataSourceType::ActiveInstanceDataObject && StateHandle.IsValid()));
		check(Source == EStateTreeDataSourceType::GlobalParameterData || Source == EStateTreeDataSourceType::ExternalGlobalParameterData || IsValidIndex(InIndex));
		Index = static_cast<uint16>(InIndex);
	}

	explicit FStateTreeDataHandle(const EStateTreeDataSourceType InSource)
		: FStateTreeDataHandle(InSource, FStateTreeDataHandle::InvalidIndex)
	{}

	bool IsValid() const
	{
		return Source != EStateTreeDataSourceType::None;
	}

	void Reset()
	{
		Source = EStateTreeDataSourceType::None;
		Index = InvalidIndex;
		StateHandle = FStateTreeStateHandle::Invalid;
	}

	bool operator==(const FStateTreeDataHandle& RHS) const
	{
		return Source == RHS.Source && Index == RHS.Index && StateHandle == RHS.StateHandle;
	}

	bool operator!=(const FStateTreeDataHandle& RHS) const
	{
		return !(*this == RHS);
	}

	EStateTreeDataSourceType GetSource() const
	{
		return Source;
	}

	int32 GetIndex() const
	{
		return Index;
	}

	FStateTreeStateHandle GetState() const
	{
		return StateHandle;
	}

	STATETREEMODULE_API bool IsObjectSource() const;

	STATETREEMODULE_API FStateTreeDataHandle ToObjectSource() const;
	STATETREEMODULE_API FStateTreeDataHandle ToStructSource() const;

	STATETREEMODULE_API FString Describe() const;

private:
	UPROPERTY()
	EStateTreeDataSourceType Source = EStateTreeDataSourceType::None;

	UPROPERTY()
	uint16 Index = InvalidIndex;

	UPROPERTY()
	FStateTreeStateHandle StateHandle = FStateTreeStateHandle::Invalid; 
};

namespace UE::StateTree::InstanceData
{
	/** The memory requirement for the container allocation. */
	struct FEvaluationScopeMemoryRequirement
	{
		int32 Size = 0;
		int32 Alignment = 0;
		int32 NumberOfElements = 0;
		bool HasMemory() const
		{
			return Size > 0;
		}
	};
}

/**
 * Time duration with random variance. Stored compactly as two uint16s, which gives time range of about 650 seconds.
 * The variance is symmetric (+-) around the specified duration.
 */
USTRUCT()
struct FStateTreeRandomTimeDuration
{
	GENERATED_BODY()

	/** Reset duration to empty. */
	void Reset()
	{
		Duration = 0;
		RandomVariance = 0;
	}

	/** Sets the time duration with random variance. */
	void Set(const float InDuration, const float InRandomVariance)
	{
		Duration = Quantize(InDuration);
    	RandomVariance = Quantize(InRandomVariance);
	}

	/** @return the fixed duration. */
	float GetDuration() const
    {
		return Duration / Scale;
    }

	/** @return the maximum random variance. */
	float GetRandomVariance() const
	{
		return RandomVariance / Scale;
	}

	/** @return True of the duration is empty (always returns 0). */
	bool IsEmpty() const
	{
		return Duration == 0 && RandomVariance == 0;
	}
	
	/** @return Returns random duration around Duration, varied by +-RandomVariation. */
	float GetRandomDuration(const FRandomStream& RandomStream) const
	{
		const int32 MinVal = FMath::Max(0, static_cast<int32>(Duration) - static_cast<int32>(RandomVariance));
		const int32 MaxVal = static_cast<int32>(Duration) + static_cast<int32>(RandomVariance);
		return static_cast<decltype(Scale)>(RandomStream.RandRange(MinVal, MaxVal)) / Scale;
	}

protected:
	static constexpr float Scale = 100.0f;

	uint16 Quantize(const float Value) const
	{
		return (uint16)FMath::Clamp(FMath::RoundToInt32(Value * Scale), 0, (int32)MAX_uint16);
	}
	
	UPROPERTY(EditDefaultsOnly, Category = Default)
	uint16 Duration = 0;

	UPROPERTY(EditDefaultsOnly, Category = Default)
	uint16 RandomVariance = 0;
};

/** Fallback behavior indicating what to do after failing to select a state */
UENUM()
enum class EStateTreeSelectionFallback : uint8
{
	/** No fallback */
	None,

	/** Find next selectable sibling, if any, and select it */
	NextSelectableSibling,
};

/**
 *  Runtime representation of an event description.
 */
USTRUCT()
struct FCompactEventDesc
{
	GENERATED_BODY()

	/** Event Payload Struct. */
	UPROPERTY()
	TObjectPtr<const UScriptStruct> PayloadStruct = nullptr;

	/** Event Tag. */
	UPROPERTY()
	FGameplayTag Tag;

	/** Returns true if describes an event correctly. */
	bool IsValid() const
	{
		return Tag.IsValid() || PayloadStruct;
	}

	/** Returns true if described events is a subset of events described by another EventDesc. */
	bool IsSubsetOfAnotherDesc(const FCompactEventDesc& Desc) const
	{
		if (Tag.IsValid() && Desc.Tag.IsValid())
		{
			if (!Desc.Tag.MatchesTag(Tag) || !Tag.MatchesTag(Desc.Tag))
			{
				return false;
			}
		}

		if (PayloadStruct && Desc.PayloadStruct)
		{
			return PayloadStruct->IsChildOf(Desc.PayloadStruct);
		}

		return true;
	}

	/** Returns true provided event matches description. */
	STATETREEMODULE_API bool DoesEventMatchDesc(const FStateTreeEvent& Event) const;
};

/**
 * Additional rules for if state was changed or sustained per transition. 
 * This is exposed to designers as "Reactivation" w/ "Override" & "Denied" options.
 */
UENUM()
enum class EStateTreeTransitionChangeTypeRules : uint8
{
	ForceChanged,
	ForceSustained,
	Default,
};

/**
 *  Runtime representation of a StateTree transition.
 */
USTRUCT()
struct FCompactStateTransition
{
	GENERATED_BODY()

	explicit FCompactStateTransition()
	{
	}

	/** @return True if the transition has delay. */
	bool HasDelay() const
	{
		return !Delay.IsEmpty();
	}

	/** @return flags for state selection based on this transition's configuration */
	EStateTreeStateSelectionRules GetStateSelectionRulesForTransition(EStateTreeStateSelectionRules ModifiedStateSelectionRules) const
	{
		switch (ChangeTypeTargetStateRule)
		{
		case EStateTreeTransitionChangeTypeRules::ForceChanged:
			return ModifiedStateSelectionRules | EStateTreeStateSelectionRules::ReselectedStateCreatesNewStates;
		case EStateTreeTransitionChangeTypeRules::ForceSustained:
			return ModifiedStateSelectionRules & ~EStateTreeStateSelectionRules::ReselectedStateCreatesNewStates;
		case EStateTreeTransitionChangeTypeRules::Default:
		default:
			return ModifiedStateSelectionRules;
		}
	}

	/** Event Description */
	UPROPERTY()
	FCompactEventDesc RequiredEvent;

	/** Delegate dispatcher the transition is waiting for. */
	UPROPERTY()
	FStateTreeDelegateDispatcher RequiredDelegateDispatcher;

	/** The amount of memory used by the condition when it uses the evaluation scope data type. */
	UE::StateTree::InstanceData::FEvaluationScopeMemoryRequirement ConditionEvaluationScopeMemoryRequirement;

	/** Index to first condition to test */
	UPROPERTY()
	uint16 ConditionsBegin = 0;

	/** Target state of the transition */
	UPROPERTY()
	FStateTreeStateHandle State = FStateTreeStateHandle::Invalid;

	/** Transition delay. */
	UPROPERTY()
	FStateTreeRandomTimeDuration Delay;
	
	/* Type of the transition trigger. */
	UPROPERTY()
	EStateTreeTransitionTrigger Trigger = EStateTreeTransitionTrigger::None;

	/* Priority of the transition. */
	UPROPERTY()
	EStateTreeTransitionPriority Priority = EStateTreeTransitionPriority::Normal;

	/** Fallback of the transition if it fails to select the target state */
	UPROPERTY()
	EStateTreeSelectionFallback Fallback = EStateTreeSelectionFallback::None;

	/** Number of conditions to test. */
	UPROPERTY()
	uint8 ConditionsNum = 0;

	/** Additional rules for state changed or sustain. Their instance data will be reinstantiated. */
	UPROPERTY()
	EStateTreeTransitionChangeTypeRules ChangeTypeTargetStateRule = EStateTreeTransitionChangeTypeRules::Default;

	/** Indicates if the transition is enabled and should be considered. */
	UPROPERTY()
	uint8 bTransitionEnabled : 1 = true;

	/** If set to true, the required event is consumed (later state selection cannot react to it) if state selection can be made. */
	UPROPERTY()
	uint8 bConsumeEventOnSelect : 1 = true;
};

/**
 *  Runtime representation of a StateTree frame.
 */
USTRUCT()
struct FCompactStateTreeFrame
{
	GENERATED_BODY()

	/** The root state of the frame (e.g. Root state or a subtree). */
	UPROPERTY()
	FStateTreeStateHandle RootState = FStateTreeStateHandle::Invalid;

	/**
	 * The max number of masks needed by the frame.
	 * It is the worst case of all the possible active state combinations for the frame.
	 * It includes the global tasks.
	 */
	UPROPERTY()
	uint8 NumberOfTasksStatusMasks = 0;
};

/**
 *  Runtime representation of a StateTree state.
 */
USTRUCT()
struct FCompactStateTreeState
{
	GENERATED_BODY()
	
	/** @return Index to the next sibling state. */
	uint16 GetNextSibling() const
	{
		return ChildrenEnd;
	}

	/** @return True if the state has any child states */
	bool HasChildren() const
	{
		return ChildrenEnd > ChildrenBegin;
	}

	/** @return True if the state has any tasks that need ticking. */
	bool DoesRequestTickTasks(bool bHasEvent) const
	{
		return bCachedRequestTick || (bHasEvent && bCachedRequestTickOnlyOnEvents);
	}

	/** @return True if the state has any tasks that ticks. */
	bool ShouldTickTasks(bool bHasEvent) const
	{
		return bHasTickTasks || (bHasEvent && bHasTickTasksOnlyOnEvents);
	}

	/** @return True if the state has any transitions that need ticking. */
	bool ShouldTickTransitions(bool bHasEvent, bool bHasBroadcastedDelegates) const
	{
		return bHasTickTriggerTransitions || (bHasEvent && bHasEventTriggerTransitions) || (bHasBroadcastedDelegates && bHasDelegateTriggerTransitions);
	}

	/**
	 * A state can complete with Stopped, Succeeded, and Failed.
	 * The state can contain transitions that trigger on any completes, or only if the state succeeded, or only if the state failed.
	 * @param bSucceeded Whenever the state completion succeeded.
	 * @param bFailed Whenever the state completion failed.
	 * @return True if the state has any transitions that tick on completion.
	 */
	bool ShouldTickCompletionTransitions(bool bSucceeded, bool bFailed) const
	{
		return bHasCompletedTriggerTransitions
			|| (bHasSucceededTriggerTransitions && bSucceeded)
			|| (bHasFailedTriggerTransitions && bFailed);
	}

	/** Description of an event required to enter the state. */
	UPROPERTY()
	FCompactEventDesc RequiredEventToEnter;

	/** Name of the State */
	UPROPERTY()
	FName Name;

	/** GameplayTag describing the State */
	UPROPERTY()
	FGameplayTag Tag;

	/** The amount of memory used by the condition when it uses the evaluation scope data type. */
	UE::StateTree::InstanceData::FEvaluationScopeMemoryRequirement EnterConditionEvaluationScopeMemoryRequirement;

	/** The amount of memory used by the consideration when it uses the evaluation scope data type. */
	UE::StateTree::InstanceData::FEvaluationScopeMemoryRequirement ConsiderationEvaluationScopeMemoryRequirement;

	UPROPERTY()
	TObjectPtr<UStateTree> LinkedAsset = nullptr;

	/** Linked state handle if the state type is linked state. */
	UPROPERTY()
	FStateTreeStateHandle LinkedState = FStateTreeStateHandle::Invalid; 

	/** Parent state handle, invalid if root state. */
	UPROPERTY()
	FStateTreeStateHandle Parent = FStateTreeStateHandle::Invalid;

	/** Index to first child state */
	UPROPERTY()
	uint16 ChildrenBegin = 0;

	/** Index one past the last child state. */
	UPROPERTY()
	uint16 ChildrenEnd = 0;

	/** Index to first state enter condition */
	UPROPERTY()
	uint16 EnterConditionsBegin = 0;

	/** Index to first state utility consideration */
	UPROPERTY()
	uint16 UtilityConsiderationsBegin = 0;

	/** Index to first transition */
	UPROPERTY()
	uint16 TransitionsBegin = 0;

	/** Index to first task */
	UPROPERTY()
	uint16 TasksBegin = 0;

	/** Index to state instance data. */
	UPROPERTY()
	FStateTreeIndex16 ParameterTemplateIndex = FStateTreeIndex16::Invalid;

	UPROPERTY()
	FStateTreeDataHandle ParameterDataHandle = FStateTreeDataHandle::Invalid;

	UPROPERTY()
	FStateTreeIndex16 ParameterBindingsBatch = FStateTreeIndex16::Invalid;

	UPROPERTY()
	FStateTreeIndex16 EventDataIndex = FStateTreeIndex16::Invalid;

	/** Weight used to scale the normalized final utility score for this state */
	UPROPERTY()
	float Weight = 1.f;

	/**
	 * Tick rate in seconds the state tasks and transitions should tick
	 * If set the state cannot sleep.
	 * Must be greater than or equal to 0.
	 */
	UPROPERTY()
	float CustomTickRate = 0.f;

	/** Mask used to test the tasks completion. */
	UPROPERTY()
	uint32 CompletionTasksMask = 0;

	/**
	 * Index in the mask buffer used by the state. 
	 * CompletionTasksMaskBufferIndex = (Final Task Bit) / (size of mask:32)
	 */
	UPROPERTY()
	uint8 CompletionTasksMaskBufferIndex = 0;
	
	/**
	 * Offset, in bits, of the first flag inside the mask.
	 * Used to access the bit from the state task index.
	 * CompletionTasksMaskBitsOffset = (Final Task Bit) % (size of mask:32)
	 */
	UPROPERTY()
	uint8 CompletionTasksMaskBitsOffset = 0;

	/** How the tasks control the completion of the state. */
	UPROPERTY()
	EStateTreeTaskCompletionType CompletionTasksControl = EStateTreeTaskCompletionType::Any;

	/** Number of enter conditions */
	UPROPERTY()
	uint8 EnterConditionsNum = 0;
	
	/** Number of utility considerations */
	UPROPERTY()
	uint8 UtilityConsiderationsNum = 0;

	/** Number of transitions */
	UPROPERTY()
	uint8 TransitionsNum = 0;

	/** Number of tasks */
	UPROPERTY()
	uint8 TasksNum = 0;

	/**
	 * Number of enabled tasks
	 * todo: this should be removed once we finished only compiling enabled elements for StateTree Compiler 
	 */
	UPROPERTY()
	uint8 EnabledTasksNum = 0;

	/** Number of instance data */
	UPROPERTY()
	uint8 InstanceDataNum = 0;

	/** Distance to root state. */
	UPROPERTY()
	uint8 Depth = 0;

	/** Type of the state */
	UPROPERTY()
	EStateTreeStateType Type = EStateTreeStateType::State;

	/** What to do when the state is considered for selection. */
	UPROPERTY()
	EStateTreeStateSelectionBehavior SelectionBehavior = EStateTreeStateSelectionBehavior::TrySelectChildrenInOrder;

	/** Whether the linked state supports Linked Asset override. */
	UPROPERTY()
	uint8 bCanOverrideLinkedAssetAtRuntime : 1 = true;

	/** True if the state contains tasks that should be called during transition handling. */
	UPROPERTY()
	uint8 bHasTransitionTasks : 1 = false;

	/** True if the state contains conditions which require call to enter/completed/exit state. */
	UPROPERTY()
	uint8 bHasStateChangeConditions : 1 = false;

	/** True if any task has bShouldCallTick. */
	uint8 bHasTickTasks : 1 = false;

	/**
	 * True if any task has bShouldCallTickOnlyOnEvents.
	 * No effect if bHasTickTasks is true.
	 */
	uint8 bHasTickTasksOnlyOnEvents : 1 = false;
	
	
	/** True if any state tasks request a tick every frame. */
	uint8 bCachedRequestTick : 1 = false;

	/**
	 * True if any state tasks request a tick every frame but only if there are events.
	 * No effect if bCachedRequestTick is true.
	 */
	uint8 bCachedRequestTickOnlyOnEvents : 1 = false;

	/** True if the state contains transitions with tick trigger. */
	UPROPERTY()
	uint8 bHasTickTriggerTransitions : 1 = false;

	/** True if the state contains transitions with event trigger. */
	UPROPERTY()	
	uint8 bHasEventTriggerTransitions : 1 = false;

	/** True if the state contains transitions with delegate trigger. */
	UPROPERTY()	
	uint8 bHasDelegateTriggerTransitions : 1 = false;

	/** True if the state contains transitions with completed trigger. */
	UPROPERTY()
	uint8 bHasCompletedTriggerTransitions : 1 = false;

	/** True if the state contains transitions with Succeeded trigger. */
	UPROPERTY()
	uint8 bHasSucceededTriggerTransitions : 1 = false;
	
	/** True if the state contains transitions with Succeeded trigger. */
	UPROPERTY()
	uint8 bHasFailedTriggerTransitions : 1 = false;

	/** Should state's required event and enter conditions be evaluated when transition leads directly to it's child. */
	UPROPERTY()
	uint8 bCheckPrerequisitesWhenActivatingChildDirectly : 1 = false;

	/** True if the state is Enabled (i.e. not explicitly marked as disabled). */
	UPROPERTY()
	uint8 bEnabled : 1 = true;

	/** If set to true, the required event is consumed (later state selection cannot react to it) if state selection can be made. */
	UPROPERTY()
	uint8 bConsumeEventOnSelect : 1 = true;

	/** If set to true, the state has a custom tick rate. */
	UPROPERTY()
	uint8 bHasCustomTickRate : 1 = false;

	/**
	 * If true, state parameter bindings are copied on each task tick.
	 * Parameters bindings are always copied on state enter and state exit.
	 */
	UPROPERTY()
	uint8 bCopyParameterBindingsOnTick : 1 = false;
};

USTRUCT()
struct FCompactStateTreeParameters
{
	GENERATED_BODY()

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FCompactStateTreeParameters() = default;
	FCompactStateTreeParameters(const FCompactStateTreeParameters&) = default;
	FCompactStateTreeParameters(FCompactStateTreeParameters&&) = default;
	FCompactStateTreeParameters& operator=(const FCompactStateTreeParameters&) = default;
	FCompactStateTreeParameters& operator=(FCompactStateTreeParameters&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	explicit FCompactStateTreeParameters(FInstancedPropertyBag InParameters)
		: Parameters(MoveTemp(InParameters))
	{
	}

	explicit FCompactStateTreeParameters(FInstancedStruct InParameters)
		: InstancedParameters(MoveTemp(InParameters))
	{
	}

public:
	FConstStructView GetValue() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return Parameters.IsValid() ? Parameters.GetValue() : FConstStructView(InstancedParameters);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	FStructView GetMutableValue()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return Parameters.IsValid() ? Parameters.GetMutableValue() : FStructView(InstancedParameters);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

private:
	UE_DEPRECATED_FORGAME(5.8, "Direct access to Parameter is not valid. Use GetValue or GetMutableValue.")
	/** To hold a PropertyBag for internal state parameters. */
	UPROPERTY()
	FInstancedPropertyBag Parameters;

	/** To hold a UScriptStruct for a linked state tree asset. */
	UPROPERTY()
	FInstancedStruct InstancedParameters;
};

UENUM()
enum class EStateTreeExternalDataRequirement : uint8
{
	Required,	// StateTree cannot be executed if the data is not present.
	Optional,	// Data is optional for StateTree execution.
};


UENUM()
enum class EStateTreePropertyUsage : uint8
{
	Invalid,
	Context,
	Input,
	Parameter,
	Output,
};

/**
 * Pair of state guid and its associated state handle created at compilation.
 */
USTRUCT()
struct FStateTreeStateIdToHandle
{
	GENERATED_BODY()

	FStateTreeStateIdToHandle() = default;
	explicit FStateTreeStateIdToHandle(const FGuid& Id, const FStateTreeStateHandle Handle)
		: Id(Id)
		, Handle(Handle)
	{
	}

	UPROPERTY();
	FGuid Id;

	UPROPERTY();
	FStateTreeStateHandle Handle;
};

/**
 * Pair of node id and its associated node index created at compilation.
 */
USTRUCT()
struct FStateTreeNodeIdToIndex
{
	GENERATED_BODY()

	FStateTreeNodeIdToIndex() = default;
	explicit FStateTreeNodeIdToIndex(const FGuid& Id, const FStateTreeIndex16 Index)
		: Id(Id)
		, Index(Index)
	{
	}

	UPROPERTY();
	FGuid Id;

	UPROPERTY();
	FStateTreeIndex16 Index;
};

/**
 * Pair of transition id and its associated compact transition index created at compilation.
 */
USTRUCT()
struct FStateTreeTransitionIdToIndex
{
	GENERATED_BODY()

	FStateTreeTransitionIdToIndex() = default;
	explicit FStateTreeTransitionIdToIndex(const FGuid& Id, const FStateTreeIndex16 Index)
		: Id(Id)
		, Index(Index)
	{
	}

	UPROPERTY();
	FGuid Id;
	
	UPROPERTY();
	FStateTreeIndex16 Index;
};


/**
 * StateTree struct ref allows to get a reference/pointer to a specified type via property binding.
 * It is useful for referencing larger properties to avoid copies of the data, or to be able to write to a bounds property.
 *
 * The expected type of the reference should be set in "BaseStruct" meta tag.
 *
 * Example:
 *
 *	USTRUCT()
 *	struct FAwesomeTaskInstanceData
 *	{
 *		GENERATED_BODY()
 *
 *		UPROPERTY(VisibleAnywhere, Category = Input, meta = (BaseStruct = "/Script/AwesomeModule.AwesomeData"))
 *		FStateTreeStructRef Data;
 *	};
 *
 *
 *	if (const FAwesomeData* Awesome = InstanceData.Data.GetPtr<FAwesomeData>())
 *	{
 *		...
 *	}
 */
USTRUCT(BlueprintType)
struct FStateTreeStructRef
{
	GENERATED_BODY()

	FStateTreeStructRef() = default;

	/** @return true if the reference is valid (safe to use the reference getters). */
	bool IsValid() const
	{
		return Data.IsValid();
	}

	/** Sets the struct ref (used by property copy) */
	void Set(FStructView NewData)
	{
		Data = NewData;
	}

	/** Returns const reference to the struct, this getter assumes that all data is valid. */
	template <typename T>
	const T& Get() const
	{
		return Data.template Get<T>();
	}

	/** Returns const pointer to the struct, or nullptr if cast is not valid. */
	template <typename T>
	const T* GetPtr() const
	{
		return Data.template GetPtr<T>();
	}

	/** Returns mutable reference to the struct, this getter assumes that all data is valid. */
	template <typename T>
    T& GetMutable()
	{
		return Data.template Get<T>();
	}

	/** Returns mutable pointer to the struct, or nullptr if cast is not valid. */
	template <typename T>
    T* GetMutablePtr()
	{
		return Data.template GetPtr<T>();
	}

	/** @return Struct describing the data type. */
	const UScriptStruct* GetScriptStruct() const
	{
		return Data.GetScriptStruct();
	}

	//~ For StructOpsTypeTraits
	bool ExportTextItem(FString& ValueStr, const FStateTreeStructRef& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const;

protected:
	FStructView Data;
};

inline bool FStateTreeStructRef::ExportTextItem(FString& ValueStr, const FStateTreeStructRef& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	if (Data.IsValid())
	{
		Data.GetScriptStruct()->ExportText(ValueStr, Data.GetMemory(), Data.GetMemory(), Parent, PortFlags, ExportRootScope);
	}

	constexpr bool bSkipGenericExport = true;
	return bSkipGenericExport;
}

template<>
struct TStructOpsTypeTraits<FStateTreeStructRef> : TStructOpsTypeTraitsBase2<FStateTreeStructRef>
{
	enum
	{
		WithExportTextItem = true,
	};
};


/**
 * Helper struct to facilitate transition to FPropertyBindingDataView in the StateTree module.
 * @see FPropertyBindingDataView
 */
struct FStateTreeDataView : public FPropertyBindingDataView
{
	using FPropertyBindingDataView::FPropertyBindingDataView;
};

/**
 * Link to another state in StateTree
 */
USTRUCT(BlueprintType)
struct FStateTreeStateLink
{
	GENERATED_BODY()

	FStateTreeStateLink() = default;

#if WITH_EDITORONLY_DATA
	FStateTreeStateLink(const EStateTreeTransitionType InType) : LinkType(InType) {}
#endif // WITH_EDITORONLY_DATA
	
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FStateTreeStateLink(const FStateTreeStateLink& Other) = default;
	FStateTreeStateLink(FStateTreeStateLink&& Other) = default;
	FStateTreeStateLink& operator=(FStateTreeStateLink const& Other) = default;
	FStateTreeStateLink& operator=(FStateTreeStateLink&& Other) = default;
STATETREEMODULE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	bool Serialize(FStructuredArchive::FSlot Slot);
	STATETREEMODULE_API void PostSerialize(const FArchive& Ar);

#if WITH_EDITORONLY_DATA
	/** Name of the state at the time of linking, used for error reporting. */
	UPROPERTY(EditDefaultsOnly, Category = "Link")
	FName Name;

	/** ID of the state linked to. */
	UPROPERTY(EditDefaultsOnly, Category = "Link")
	FGuid ID;

	/** Type of the transition, used at edit time to describe e.g. next state (which is calculated at compile time). */
	UPROPERTY(EditDefaultsOnly, Category = "Link")
	EStateTreeTransitionType LinkType = EStateTreeTransitionType::None;

	UE_DEPRECATED(all, "Use LinkType instead.")
	UPROPERTY()
	EStateTreeTransitionType Type_DEPRECATED = EStateTreeTransitionType::GotoState;
#endif // WITH_EDITORONLY_DATA

	/** Handle of the linked state. */
	UPROPERTY()
	FStateTreeStateHandle StateHandle;

	/** Fallback of the transition if it fails to select the target state */
	UPROPERTY()
	EStateTreeSelectionFallback Fallback = EStateTreeSelectionFallback::None;
};

template<>
struct TStructOpsTypeTraits<FStateTreeStateLink> : public TStructOpsTypeTraitsBase2<FStateTreeStateLink>
{
	enum
	{
		WithStructuredSerializer = true,
		WithPostSerialize = true,
	};
};

/**
 * Helper macro to define instance data as a trivial type.
 * The type can be safely copied using memcpy and don't require special copy construction or destruction. It requires a default constructor.
 * The type defined default values.
 * ex: struct { int32 A = 3; };
 * This is not appropriate for element types that require non-trivial copy constructor/assignment  or non-trivial move constructor/assignment.
 * ex: struct { TObjectPtr<UObject> A; }; // No. TObjectPtr has a non-trivial copy constructor when UE_OBJECT_PTR_GC_BARRIER is enabled.
 */
#define UE_STATETREE_CONSTRUCTED_TRIVIALLY_COPIED_NO_DESTRUCTOR_INSTANCEDATA(Type) \
template <> struct TIsPODType<Type> { enum { Value = true }; }; \
template<> \
struct TStructOpsTypeTraits<Type> : public TStructOpsTypeTraitsBase2<Type> \
{ \
	enum \
	{ \
		WithNoDestructor = true, \
	}; \
};

/**
 * Helper macro to define instance data as a trivial type.
 * The type can be safely copied using memcpy and don't require special construction or destruction.
 * The memory will be memset to 0.
 * ex: struct { int32 A = 0; };
 * This is not appropriate for element types that require construction.
 * ex: struct { FFrameRate A; }; // No. FFrameRate has a constructor.
 */
#define UE_STATETREE_ZEROED_TRIVIALLY_COPIED_NO_DESTRUCTOR_INSTANCEDATA(Type) \
template <> struct TIsPODType<Type> { enum { Value = true }; }; \
template<> \
struct TStructOpsTypeTraits<Type> : public TStructOpsTypeTraitsBase2<Type> \
{ \
	enum \
	{ \
		WithZeroConstructor = true, \
		WithNoDestructor = true, \
	}; \
};

#define STATETREE_POD_INSTANCEDATA(Type) UE_DEPRECATED_MACRO(5.8, "Use UE_STATETREE_CONSTRUCTED_TRIVIALLY_COPIED_NO_DESTRUCTOR_INSTANCEDATA or UE_STATETREE_ZEROED_TRIVIALLY_COPIED_NO_DESTRUCTOR_INSTANCEDATA."); UE_STATETREE_ZEROED_TRIVIALLY_COPIED_NO_DESTRUCTOR_INSTANCEDATA(Type)
