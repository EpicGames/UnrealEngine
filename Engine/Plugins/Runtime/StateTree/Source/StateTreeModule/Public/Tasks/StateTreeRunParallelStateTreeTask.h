// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeAsyncExecutionContext.h"
#include "StateTreeInstanceData.h"
#include "StateTreeReference.h"
#include "StateTreeTaskBase.h"
#include "StateTreeRunParallelStateTreeTask.generated.h"

#define UE_API STATETREEMODULE_API

USTRUCT()
struct FStateTreeRunParallelStateTreeTaskInstanceData
{
	GENERATED_BODY()

public:
	/** Copies the global parameters from the parent tree to the parallel tree's root parameters. */
	UE_API void UpdateGlobalParameters();

public:
	/** State tree and parameters that will be run when this task is started. */
	UPROPERTY(EditAnywhere, Category = "Parameter", DisplayName = "State Tree", meta=(SchemaCanBeOverriden))
	FStateTreeReference StateTree;

	UPROPERTY(Transient)
	FStateTreeInstanceData TreeInstanceData;

	UPROPERTY(Transient)
	TObjectPtr<const UStateTree> RunningStateTree = nullptr;

	/** The handle of the scheduled tick. */
	UE::StateTree::FScheduledTickHandle ScheduledTickHandle;
};

USTRUCT()
struct FStateTreeRunParallelStateTreeExecutionExtension : public FStateTreeExecutionExtension
{
	GENERATED_BODY()

public:
	virtual void ScheduleNextTick(const FContextParameters& Context, const FNextTickArguments& Args) override;

	FStateTreeWeakExecutionContext WeakExecutionContext;
	UE::StateTree::FScheduledTickHandle ScheduledTickHandle;
};

/**
* Task that will run another state tree in the current state while allowing the current tree to continue selection and process of child state.
* It will succeed, fail or run depending on the result of the parallel tree.
* Less efficient then Linked Asset state, it has the advantage of allowing multiple trees to run in parallel.
*/
USTRUCT(meta = (DisplayName = "Run Parallel Tree", Category = "Common"))
struct FStateTreeRunParallelStateTreeTask : public FStateTreeTaskCommonBase
{
	GENERATED_BODY()
	using FInstanceDataType = FStateTreeRunParallelStateTreeTaskInstanceData;
	
	UE_API FStateTreeRunParallelStateTreeTask();

#if WITH_EDITORONLY_DATA
	// Sets event handling priority
	void SetEventHandlingPriority(const EStateTreeTransitionPriority NewPriority)
	{
		EventHandlingPriority = NewPriority;
	}
#endif	
	
protected:
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	UE_API virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transitions) const override;
	UE_API virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override;
	UE_API virtual void TriggerTransitions(FStateTreeExecutionContext& Context) const override;
	UE_API virtual void ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;

	UE_API virtual void PostLoad(FStateTreeDataView InstanceDataView) override;

#if WITH_EDITOR
	UE_API virtual EDataValidationResult Compile(UE::StateTree::ICompileNodeContext& Context) override;
	UE_API virtual void PostEditInstanceDataChangeChainProperty(const FPropertyChangedChainEvent& PropertyChangedEvent, FStateTreeDataView InstanceDataView) override;
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
	virtual FName GetIconName() const override
	{
		return FName("StateTreeEditorStyle|Node.RunParallel");
	}
	virtual FColor GetIconColor() const override
	{
		return UE::StateTree::Colors::Grey;
	}
#endif // WITH_EDITOR

	UE_API const FStateTreeReference& GetStateTreeToRun(FStateTreeExecutionContext& Context, FInstanceDataType& InstanceData) const;

public:
	/** If set the task will look at the linked state tree override to replace the state tree it's running. */
	UPROPERTY(EditAnywhere, Category = Parameter)
	FGameplayTag StateTreeOverrideTag;

	/**
	 * If set to true, copy the values of the state tree global parameters before calling Tick().
	 * Only copy if the state tree was not overridden.
	 * Enable when you have bound properties that you want to update during the execution of the state tree.
	 */
	UPROPERTY(EditAnywhere, Category = Parameter)
	bool bShouldCopyParametersOnTick = false;

	/**
	 * If set to true, copy the values of the state tree global parameters before calling ExitState()
	 * Only copy if the state tree was not overridden.
	 * Enable when you have bound properties that you want to update during the execution of the state tree.
	 */
	UPROPERTY(EditAnywhere, Category = Parameter)
	bool bShouldCopyParametersOnExitState = false;

protected:
#if WITH_EDITORONLY_DATA
	/**
	 * At what priority the events should be handled in the parallel State Tree.
	 * If set to 'Normal' the order of the States in the State Tree will define the handling order.
	 * If the priority is set to Low, the main tree is let to handle the transitions first.
	 * If set to High or above, the parallel tree has change to handle events first.
	 * If multiple tasks has same priority, the State order of the States defines the handling order.
	 * The tree handling order is: States and handle from leaf to root, tasks before and handled before transitions per State.
	 */
	UPROPERTY(EditAnywhere, Category = Parameter)
	EStateTreeTransitionPriority EventHandlingPriority = EStateTreeTransitionPriority::Normal;
#endif // WITH_EDITORONLY_DATA	
};

#undef UE_API
