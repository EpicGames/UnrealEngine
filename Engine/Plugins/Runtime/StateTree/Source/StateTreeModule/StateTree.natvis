<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

  <!--
  *
  * StateTree Visualizers
  *
  -->

  <Type Name="FStateTreeInstanceData">
    <DisplayString>{{ Instances={InstanceStorage.Object->InstanceStructs.NumItems}, TemporaryInstances={InstanceStorage.Object->TemporaryInstances.ArrayNum}, Events={InstanceStorage.Object->EventQueue.Object->SharedEvents.ArrayNum}, TransitionRequests={InstanceStorage.Object->TransitionRequests.ArrayNum} }}</DisplayString>
    <Expand>
      <ExpandedItem>*InstanceStorage.Object</ExpandedItem>
    </Expand>
  </Type>

  <Type Name="FStateTreeActiveStates">
    <DisplayString Condition="NumStates == 0">Empty</DisplayString>
    <DisplayString Condition="NumStates &gt; 0">{{ NumItems={NumItems} }} </DisplayString>
    <Expand>
      <IndexListItems Condition="NumStates != 0">
        <Size>NumStates</Size>
        <ValueNode>States[$i]</ValueNode>
      </IndexListItems>
    </Expand>
  </Type>

  <Type Name="FCompactStateTreeState">
    <DisplayString>{Name}</DisplayString>
  </Type>
  
  <Type Name="FStateTreeStateHandle">
    <DisplayString Condition="Index == 0">Root</DisplayString>
    <DisplayString Condition="Index == 0xffff">Invalid</DisplayString>
    <DisplayString Condition="Index == 0xfffe">Succeeded</DisplayString>
    <DisplayString Condition="Index == 0xfffd">Failed</DisplayString>
    <DisplayString Condition="Index == 0xfffc">Stopped</DisplayString>
    <DisplayString Condition="Index &gt; 0 &amp;&amp; Index &lt; 0xfffc">Index={Index}</DisplayString>
  </Type>

  <Type Name="FStateTreeIndex16">
    <DisplayString Condition="Value == 0xffff">Invalid</DisplayString>
    <DisplayString Condition="Value != 0xffff">{Value}</DisplayString>
  </Type>

  <Type Name="FStateTreeIndex8">
    <DisplayString Condition="Value == 0xff">Invalid</DisplayString>
    <DisplayString Condition="Value != 0xff">{Value}</DisplayString>
  </Type>

  <Type Name="FStateTreeExecutionFrame">
    <Intrinsic Name="stateTree" Category="Method" Expression="*(UStateTree**)&amp;StateTree.ObjectPtr.Handle" />
    <Intrinsic Name="stateName" Category="Method" Expression="((FCompactStateTreeState*)stateTree()-&gt;States.AllocatorInstance.Data)[stateHandle.Index].Name">
      <Parameter Name="stateHandle" Type="FStateTreeStateHandle" />
    </Intrinsic>
    <DisplayString Condition="stateTree() == nullptr">Empty</DisplayString>
    <DisplayString Condition="stateTree() != nullptr">{{ StateTree={stateTree()->NamePrivate}, Root={stateName(RootState)}, NumActiveStates={ActiveStates.NumStates} }} </DisplayString>
  </Type>

  <!-- StateTreeDebugger visualizer -->
  <Type Name="FScrubState">
    <DisplayString>(ScrubTime={ScrubTime},CollectionIdx={EventCollectionIndex},FrameIdx={TraceFrameIndex},SpanIdx={FrameSpanIndex},ActiveStatesIdx={ActiveStatesIndex},BoundState={ScrubTimeBoundState},Events={EventCollections})</DisplayString>
  </Type>

</AutoVisualizer>
