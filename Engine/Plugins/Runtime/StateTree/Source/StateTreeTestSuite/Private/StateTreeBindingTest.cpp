// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeTest.h"
#include "StateTreeTestBase.h"
#include "StateTreeTestTypes.h"

#include "StateTreeCompiler.h"
#include "StateTreeCompilerLog.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeEditorData.h"
#include "StateTreeInstanceData.h"
#include "Conditions/StateTreeCommonConditions.h"

#define LOCTEXT_NAMESPACE "AITestSuite_StateTreeTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::StateTree::Tests
{

struct FStateTreeTest_BindingsCompiler : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		FStateTreeCompilerLog Log;
		FStateTreePropertyBindings Bindings;
		FStateTreePropertyBindingCompiler BindingCompiler;

		const bool bInitResult = BindingCompiler.Init(Bindings, Log);
		AITEST_TRUE(TEXT("Expect init to succeed"), bInitResult);

		FStateTreeBindableStructDesc SourceADesc;
		SourceADesc.Name = FName(TEXT("SourceA"));
		SourceADesc.Struct = TBaseStructure<FStateTreeTest_PropertyCopy>::Get();
		SourceADesc.DataSource = EStateTreeBindableStructSource::Parameter;
		SourceADesc.DataHandle = FStateTreeDataHandle(EStateTreeDataSourceType::ContextData, 0); // Used as index to SourceViews below.
		SourceADesc.ID = FGuid::NewGuid();

		FStateTreeBindableStructDesc SourceBDesc;
		SourceBDesc.Name = FName(TEXT("SourceB"));
		SourceBDesc.Struct = TBaseStructure<FStateTreeTest_PropertyCopy>::Get();
		SourceBDesc.DataSource = EStateTreeBindableStructSource::Parameter;
		SourceBDesc.DataHandle = FStateTreeDataHandle(EStateTreeDataSourceType::ContextData, 1); // Used as index to SourceViews below.
		SourceBDesc.ID = FGuid::NewGuid();

		FStateTreeBindableStructDesc TargetDesc;
		TargetDesc.Name = FName(TEXT("Target"));
		TargetDesc.Struct = TBaseStructure<FStateTreeTest_PropertyCopy>::Get();
		TargetDesc.DataSource = EStateTreeBindableStructSource::Parameter;
		TargetDesc.ID = FGuid::NewGuid();
		
		const int32 SourceAIndex = BindingCompiler.AddSourceStruct(SourceADesc);
		const int32 SourceBIndex = BindingCompiler.AddSourceStruct(SourceBDesc);

		TArray<FStateTreePropertyPathBinding> PropertyBindings;
		PropertyBindings.Add(MakeBinding(SourceBDesc.ID, TEXT("Item"), TargetDesc.ID, TEXT("Array[1]")));
		PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("Item.B"), TargetDesc.ID, TEXT("Array[1].B")));
		PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("Array"), TargetDesc.ID, TEXT("Array")));

		PropertyBindings.Add(MakeBinding(SourceBDesc.ID, TEXT("Item"), TargetDesc.ID, TEXT("FixedArray[1]")));
		PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("Item.B"), TargetDesc.ID, TEXT("FixedArray[1].B")));
		PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("FixedArray"), TargetDesc.ID, TEXT("FixedArray")));

		PropertyBindings.Add(MakeBinding(SourceBDesc.ID, TEXT("Item"), TargetDesc.ID, TEXT("CArray[1]")));
		PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("Item.B"), TargetDesc.ID, TEXT("CArray[1].B")));
		PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("CArray"), TargetDesc.ID, TEXT("CArray")));

		int32 CopyBatchIndex = INDEX_NONE;
		{
			const bool bCompileBatchResult = BindingCompiler.CompileBatch(TargetDesc, PropertyBindings, FStateTreeIndex16::Invalid, FStateTreeIndex16::Invalid, CopyBatchIndex);
			AITEST_TRUE(TEXT("CompileBatch should succeed"), bCompileBatchResult);
			AITEST_NOT_EQUAL(TEXT("CopyBatchIndex should not be INDEX_NONE"), CopyBatchIndex, (int32)INDEX_NONE);
		}

		PropertyBindings.Reset();
		PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayOfFloat"), TargetDesc.ID, TEXT("ArrayOfDouble")));
		PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayOfDouble"), TargetDesc.ID, TEXT("ArrayOfFloat")));
		int32 CopyBatchIndex2 = INDEX_NONE;
		{
			const bool bCompileBatchResult = BindingCompiler.CompileBatch(TargetDesc, PropertyBindings, FStateTreeIndex16::Invalid, FStateTreeIndex16::Invalid, CopyBatchIndex2);
			AITEST_TRUE(TEXT("CompileBatch 2 should succeed"), bCompileBatchResult);
			AITEST_NOT_EQUAL(TEXT("CopyBatchIndex2 should not be INDEX_NONE"), CopyBatchIndex2, (int32)INDEX_NONE);
		}

		BindingCompiler.Finalize();

		const bool bResolveResult = Bindings.ResolvePaths();
		AITEST_TRUE(TEXT("ResolvePaths should succeed"), bResolveResult);

		FStateTreeTest_PropertyCopy SourceA;
		SourceA.Item.B = 123;
		SourceA.Array.AddDefaulted_GetRef().A = 1;
		SourceA.Array.AddDefaulted_GetRef().B = 2;

		constexpr int32 FixedArraySize = 4;
		SourceA.FixedArray.SetNum(FixedArraySize, EAllowShrinking::No);
		SourceA.FixedArray[0].A = 1;
		SourceA.FixedArray[1].B = 2;

		SourceA.CArray[0].A = 1;
		SourceA.CArray[0].B = 2;

		SourceA.ArrayOfFloat.Add(1.0f);
		SourceA.ArrayOfFloat.Add(2.0f);
		SourceA.ArrayOfFloat.Add(3.0f);
		SourceA.ArrayOfDouble.Add(11.0);
		SourceA.ArrayOfDouble.Add(12.0);

		FStateTreeTest_PropertyCopy SourceB;
		SourceB.Item.A = 41;
		SourceB.Item.B = 42;
		SourceB.FixedArray.SetNum(FixedArraySize, EAllowShrinking::No);

		FStateTreeTest_PropertyCopy Target;
		Target.FixedArray.SetNum(FixedArraySize, EAllowShrinking::No);

		AITEST_TRUE(TEXT("SourceAIndex should be less than max number of source structs."), SourceAIndex < Bindings.GetNumBindableStructDescriptors());
		AITEST_TRUE(TEXT("SourceBIndex should be less than max number of source structs."), SourceBIndex < Bindings.GetNumBindableStructDescriptors());

		TArray<FStateTreeDataView> SourceViews;
		SourceViews.SetNum(Bindings.GetNumBindableStructDescriptors());
		SourceViews[SourceAIndex] = FStateTreeDataView(FStructView::Make(SourceA));
		SourceViews[SourceBIndex] = FStateTreeDataView(FStructView::Make(SourceB));
		FPropertyBindingDataView TargetView(FStructView::Make(Target));

		{
			bool bCopyResult = true;
			for (const FPropertyBindingCopyInfo& Copy : Bindings.Super::GetBatchCopies(FPropertyBindingIndex16(CopyBatchIndex)))
			{
				bCopyResult &= Bindings.Super::CopyProperty(Copy, SourceViews[Copy.SourceDataHandle.Get<FStateTreeDataHandle>().GetIndex()], TargetView);
			}
			AITEST_TRUE(TEXT("CopyTo should succeed"), bCopyResult);
		}

		// Due to binding sorting, we expect them to execute in this order (sorted based on target access, earliest to latest)
		// SourceA.CArray -> Target.CArray
		// SourceB.Item -> Target.CArray[1]
		// SourceA.Item.B -> Target.CArray[1].B
		// SourceA.FixedArray -> Target.FixedArray
		// SourceB.Item -> Target.FixedArray[1]
		// SourceA.Item.B -> Target.FixedArray[1].B
		// SourceA.Array -> Target.Array
		// SourceB.Item -> Target.Array[1]
		// SourceA.Item.B -> Target.Array[1].B

		AITEST_EQUAL(TEXT("Expect TargetArray to be copied from SourceA"), Target.Array.Num(), SourceA.Array.Num());
		AITEST_EQUAL(TEXT("Expect Target.Array[0].A copied from SourceA.Array[0].A"), Target.Array[0].A, SourceA.Array[0].A);
		AITEST_EQUAL(TEXT("Expect Target.Array[0].B copied from SourceA.Array[0].B"), Target.Array[0].B, SourceA.Array[0].B);
		AITEST_EQUAL(TEXT("Expect Target.Array[1].A copied from SourceB.Item.A"), Target.Array[1].A, SourceB.Item.A);
		AITEST_EQUAL(TEXT("Expect Target.Array[1].B copied from SourceA.Item.B"), Target.Array[1].B, SourceA.Item.B);

		AITEST_EQUAL(TEXT("Expect TargetArray to be copied from SourceA"), Target.FixedArray.Num(), SourceA.FixedArray.Num());
		AITEST_EQUAL(TEXT("Expect Target.FixedArray[0].A copied from SourceA.FixedArray[0].A"), Target.FixedArray[0].A, SourceA.FixedArray[0].A);
		AITEST_EQUAL(TEXT("Expect Target.FixedArray[0].B copied from SourceA.FixedArray[0].B"), Target.FixedArray[0].B, SourceA.FixedArray[0].B);
		AITEST_EQUAL(TEXT("Expect Target.FixedArray[1].A copied from SourceB.Item.A"), Target.FixedArray[1].A, SourceB.Item.A);
		AITEST_EQUAL(TEXT("Expect Target.FixedArray[1].B copied from SourceA.Item.B"), Target.FixedArray[1].B, SourceA.Item.B);
		AITEST_EQUAL(TEXT("Expect Target.FixedArray to not have changed size"), Target.FixedArray.Num(), FixedArraySize);

		AITEST_EQUAL(TEXT("Expect Target.CArray[0].A copied from SourceA.CArray[0].A"), Target.CArray[0].A, SourceA.CArray[0].A);
		AITEST_EQUAL(TEXT("Expect Target.CArray[0].B copied from SourceA.CArray[0].B"), Target.CArray[0].B, SourceA.CArray[0].B);
		AITEST_EQUAL(TEXT("Expect Target.CArray[1].A copied from SourceB.Item.A"), Target.CArray[1].A, SourceB.Item.A);
		AITEST_EQUAL(TEXT("Expect Target.CArray[1].B copied from SourceA.Item.B"), Target.CArray[1].B, SourceA.Item.B);
		
		const int32 NumAllocated_FStateTreeTest_PropertyStructB_BeforeReset = FStateTreeTest_PropertyStructB::NumConstructed;
		bool bResetResult = Bindings.FPropertyBindingBindingCollection::ResetObjects(FPropertyBindingIndex16(CopyBatchIndex), TargetView);
		AITEST_TRUE(TEXT("ResetObjects should succeed"), bResetResult);
		AITEST_EQUAL(TEXT("Expect Target dynamic array to be empty"), Target.Array.Num(), 0);
		AITEST_EQUAL(TEXT("Expect Target fixed size Array to not have changed size."), Target.FixedArray.Num(), FixedArraySize);
		AITEST_NOT_EQUAL(TEXT("Expect the count of constructed FStateTreeTest_PropertyStructB to be smaller after calling ResetObjects"), FStateTreeTest_PropertyStructB::NumConstructed, NumAllocated_FStateTreeTest_PropertyStructB_BeforeReset);


		{
			bool bCopyResult = true;
			for (const FPropertyBindingCopyInfo& Copy : Bindings.Super::GetBatchCopies(FPropertyBindingIndex16(CopyBatchIndex2)))
			{
				bCopyResult &= Bindings.Super::CopyProperty(Copy, SourceViews[Copy.SourceDataHandle.Get<FStateTreeDataHandle>().GetIndex()], TargetView);
			}
			AITEST_TRUE(TEXT("CopyTo should succeed"), bCopyResult);
		}

		// SourceA.ArrayOfFloat -> Target.ArrayOfDouble
		// SourceA.ArrayOfDouble -> Target.ArrayOfFloat
		bool bDoubleEqualFloat = Target.ArrayOfDouble.Num() == SourceA.ArrayOfFloat.Num();
		if (bDoubleEqualFloat)
		{
			for (int32 Index = 0; Index < Target.ArrayOfDouble.Num(); ++Index)
			{
				if ((int32)SourceA.ArrayOfFloat[Index] != (int32)Target.ArrayOfDouble[Index])
				{
					bDoubleEqualFloat = false;
					break;
				}
			}
		}
		AITEST_TRUE(TEXT("Expect Target.ArrayOfDouble copied from SourceA.ArrayOfFloat"), bDoubleEqualFloat);

		bool bFloatEqualDouble = Target.ArrayOfFloat.Num() == SourceA.ArrayOfDouble.Num();
		if (bFloatEqualDouble)
		{
			for (int32 Index = 0; Index < Target.ArrayOfFloat.Num(); ++Index)
			{
				if ((int32)SourceA.ArrayOfDouble[Index] != (int32)Target.ArrayOfFloat[Index])
				{
					bFloatEqualDouble = false;
					break;
				}
			}
		}
		AITEST_TRUE(TEXT("Expect Target.ArrayOfFloat copied from SourceA.ArrayOfDouble"), bFloatEqualDouble);

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_BindingsCompiler, "System.StateTree.Binding.BindingsCompiler");

struct FStateTreeTest_BindingsCompiler_WithObjectPtr : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		FStateTreeCompilerLog Log;
		FStateTreePropertyBindings Bindings;
		FStateTreePropertyBindingCompiler BindingCompiler;

		const bool bInitResult = BindingCompiler.Init(Bindings, Log);
		AITEST_TRUE(TEXT("Expect init to succeed"), bInitResult);

		FStateTreeBindableStructDesc SourceADesc;
		SourceADesc.Name = FName(TEXT("SourceA"));
		SourceADesc.Struct = TBaseStructure<FStateTreeTest_PropertyStructObjectPtr>::Get();
		SourceADesc.DataSource = EStateTreeBindableStructSource::Parameter;
		SourceADesc.DataHandle = FStateTreeDataHandle(EStateTreeDataSourceType::ContextData, 0); // Used as index to SourceViews below.
		SourceADesc.ID = FGuid::NewGuid();

		FStateTreeBindableStructDesc TargetDesc;
		TargetDesc.Name = FName(TEXT("Target"));
		TargetDesc.Struct = TBaseStructure<FStateTreeTest_PropertyStructObjectPtr>::Get();
		TargetDesc.DataSource = EStateTreeBindableStructSource::Parameter;
		TargetDesc.ID = FGuid::NewGuid();

		const int32 SourceAIndex = BindingCompiler.AddSourceStruct(SourceADesc);

		auto CompileBatch=[this, &BindingCompiler , &TargetDesc](TArrayView<FStateTreePropertyPathBinding> PropertyBindings, int32& CopyBatchIndex)
			{
				CopyBatchIndex = INDEX_NONE;
				const bool bCompileBatchResult = BindingCompiler.CompileBatch(TargetDesc, PropertyBindings, FStateTreeIndex16::Invalid, FStateTreeIndex16::Invalid, CopyBatchIndex);
				AITEST_TRUE(TEXT("CompileBatch should succeed"), bCompileBatchResult);
				AITEST_NOT_EQUAL(TEXT("CopyBatchIndex should not be INDEX_NONE"), CopyBatchIndex, (int32)INDEX_NONE);
				return true;
			};

		// A to ...
		int32 CopyBatchIndex1 = INDEX_NONE;
		{
			TArray<FStateTreePropertyPathBinding> PropertyBindings;
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("A"), TargetDesc.ID, TEXT("A")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("A"), TargetDesc.ID, TEXT("B")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("A"), TargetDesc.ID, TEXT("C")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("A"), TargetDesc.ID, TEXT("ArrayA[1]")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("A"), TargetDesc.ID, TEXT("ArrayB[1]")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("A"), TargetDesc.ID, TEXT("ArrayC[1]")));
			if (!CompileBatch(PropertyBindings, CopyBatchIndex1))
			{
				return false;
			}
		}
		// ArrayA[] to ...
		int32 CopyBatchIndex2 = INDEX_NONE;
		{
			TArray<FStateTreePropertyPathBinding> PropertyBindings;
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayA[2]"), TargetDesc.ID, TEXT("A")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayA[2]"), TargetDesc.ID, TEXT("B")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayA[2]"), TargetDesc.ID, TEXT("C")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayA[2]"), TargetDesc.ID, TEXT("ArrayA[1]")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayA[2]"), TargetDesc.ID, TEXT("ArrayB[1]")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayA[2]"), TargetDesc.ID, TEXT("ArrayC[1]")));
			if (!CompileBatch(PropertyBindings, CopyBatchIndex2))
			{
				return false;
			}
		}
		// B to ...
		int32 CopyBatchIndex3 = INDEX_NONE;
		{
			TArray<FStateTreePropertyPathBinding> PropertyBindings;
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("B"), TargetDesc.ID, TEXT("A")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("B"), TargetDesc.ID, TEXT("B")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("B"), TargetDesc.ID, TEXT("C")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("B"), TargetDesc.ID, TEXT("ArrayA[1]")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("B"), TargetDesc.ID, TEXT("ArrayB[1]")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("B"), TargetDesc.ID, TEXT("ArrayC[1]")));
			if (!CompileBatch(PropertyBindings, CopyBatchIndex3))
			{
				return false;
			}
		}
		// ArrayB[] to ...
		int32 CopyBatchIndex4 = INDEX_NONE;
		{
			TArray<FStateTreePropertyPathBinding> PropertyBindings;
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayB[2]"), TargetDesc.ID, TEXT("A")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayB[2]"), TargetDesc.ID, TEXT("B")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayB[2]"), TargetDesc.ID, TEXT("C")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayB[2]"), TargetDesc.ID, TEXT("ArrayA[1]")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayB[2]"), TargetDesc.ID, TEXT("ArrayB[1]")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayB[2]"), TargetDesc.ID, TEXT("ArrayC[1]")));
			if (!CompileBatch(PropertyBindings, CopyBatchIndex4))
			{
				return false;
			}
		}
		// C to ...
		int32 CopyBatchIndex5 = INDEX_NONE;
		{
			TArray<FStateTreePropertyPathBinding> PropertyBindings;
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("C"), TargetDesc.ID, TEXT("A")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("C"), TargetDesc.ID, TEXT("B")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("C"), TargetDesc.ID, TEXT("C")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("C"), TargetDesc.ID, TEXT("ArrayA[1]")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("C"), TargetDesc.ID, TEXT("ArrayB[1]")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("C"), TargetDesc.ID, TEXT("ArrayC[1]")));
			if (!CompileBatch(PropertyBindings, CopyBatchIndex5))
			{
				return false;
			}
		}
		// ArrayC[] to ...
		int32 CopyBatchIndex6 = INDEX_NONE;
		{
			TArray<FStateTreePropertyPathBinding> PropertyBindings;
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayC[2]"), TargetDesc.ID, TEXT("A")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayC[2]"), TargetDesc.ID, TEXT("B")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayC[2]"), TargetDesc.ID, TEXT("C")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayC[2]"), TargetDesc.ID, TEXT("ArrayA[1]")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayC[2]"), TargetDesc.ID, TEXT("ArrayB[1]")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayC[2]"), TargetDesc.ID, TEXT("ArrayC[1]")));
			if (!CompileBatch(PropertyBindings, CopyBatchIndex6))
			{
				return false;
			}
		}
		// ArrayA to Array...
		int32 CopyBatchIndex9 = INDEX_NONE;
		{
			TArray<FStateTreePropertyPathBinding> PropertyBindings;
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayA"), TargetDesc.ID, TEXT("ArrayA")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayA"), TargetDesc.ID, TEXT("ArrayB")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayA"), TargetDesc.ID, TEXT("ArrayC")));
			if (!CompileBatch(PropertyBindings, CopyBatchIndex9))
			{
				return false;
			}
		}
		// ArrayB to Array...
		int32 CopyBatchIndex10 = INDEX_NONE;
		{
			TArray<FStateTreePropertyPathBinding> PropertyBindings;
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayB"), TargetDesc.ID, TEXT("ArrayA")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayB"), TargetDesc.ID, TEXT("ArrayB")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayB"), TargetDesc.ID, TEXT("ArrayC")));
			if (!CompileBatch(PropertyBindings, CopyBatchIndex10))
			{
				return false;
			}
		}
		// ArrayC to Array...
		int32 CopyBatchIndex11 = INDEX_NONE;
		{
			TArray<FStateTreePropertyPathBinding> PropertyBindings;
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayC"), TargetDesc.ID, TEXT("ArrayA")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayC"), TargetDesc.ID, TEXT("ArrayB")));
			PropertyBindings.Add(MakeBinding(SourceADesc.ID, TEXT("ArrayC"), TargetDesc.ID, TEXT("ArrayC")));
			if (!CompileBatch(PropertyBindings, CopyBatchIndex11))
			{
				return false;
			}
		}
		BindingCompiler.Finalize();

		const bool bResolveResult = Bindings.ResolvePaths();
		AITEST_TRUE(TEXT("ResolvePaths should succeed"), bResolveResult);

		FStateTreeTest_PropertyStructObjectPtr SourceA;
		SourceA.A = TBaseStructure<FVector>::Get();
		SourceA.B = TBaseStructure<FRotator>::Get();
		SourceA.C = TBaseStructure<FMatrix>::Get();
		constexpr int32 NumberOfElements = 4;
		SourceA.ArrayA.SetNum(NumberOfElements, EAllowShrinking::No);
		SourceA.ArrayB.SetNum(NumberOfElements, EAllowShrinking::No);
		SourceA.ArrayC.SetNum(NumberOfElements, EAllowShrinking::No);
		
		SourceA.ArrayA[2] = TBaseStructure<FVector>::Get();
		SourceA.ArrayB[2] = TBaseStructure<FRotator>::Get();
		SourceA.ArrayC[2] = TBaseStructure<FMatrix>::Get();

		FStateTreeTest_PropertyStruct SourceB;

		FStateTreeTest_PropertyStructObjectPtr Target;
		Target.ArrayA.SetNum(2, EAllowShrinking::No);
		Target.ArrayB.SetNum(2, EAllowShrinking::No);
		Target.ArrayC.SetNum(2, EAllowShrinking::No);

		auto ResetTargetData = [&Target]()
			{
				Target.A = nullptr;
				Target.B = nullptr;
				Target.C = nullptr;
				for (int32 Index = 0; Index < Target.ArrayA.Num(); ++Index)
				{
					Target.ArrayA[Index] = nullptr;
					Target.ArrayB[Index] = nullptr;
					Target.ArrayC[Index] = nullptr;
				}
				return true;
			};
		ResetTargetData();

		auto TestCopy = [this, &Target](UObject* Object, const FStringView SourceStr)
			{
				AITEST_EQUAL(FString::Printf(TEXT("Expect A to be copied from %s"), SourceStr.GetData()), Target.A.Get(), Object);
				AITEST_EQUAL(FString::Printf(TEXT("Expect B to be copied from %s"), SourceStr.GetData()), Target.B.Get(), Object);
				AITEST_EQUAL(FString::Printf(TEXT("Expect C to be copied from %s"), SourceStr.GetData()), Target.C.Get(), Object);
				AITEST_EQUAL(FString::Printf(TEXT("Expect A[] to be copied from %s"), SourceStr.GetData()), Target.ArrayA[1].Get(), Object);
				AITEST_EQUAL(FString::Printf(TEXT("Expect B[] to be copied from %s"), SourceStr.GetData()), Target.ArrayB[1].Get(), Object);
				AITEST_EQUAL(FString::Printf(TEXT("Expect C[] to be copied from %s"), SourceStr.GetData()), Target.ArrayC[1].Get(), Object);
				return true;
			};

		AITEST_TRUE(TEXT("SourceAIndex should be less than max number of source structs."), SourceAIndex < Bindings.GetNumBindableStructDescriptors());

		TArray<FStateTreeDataView> SourceViews;
		SourceViews.SetNum(Bindings.GetNumBindableStructDescriptors());
		SourceViews[SourceAIndex] = FStateTreeDataView(FStructView::Make(SourceA));
		FPropertyBindingDataView TargetView(FStructView::Make(Target));

		auto CopyAndTestBatch = [this, &ResetTargetData, &TestCopy, &Bindings, &SourceViews, &TargetView](int32 CopyBatchIndex, UObject* Object, FStringView SourceStr)
			{
				ResetTargetData();
				bool bCopyResult = true;
				for (const FPropertyBindingCopyInfo& Copy : Bindings.Super::GetBatchCopies(FPropertyBindingIndex16(CopyBatchIndex)))
				{
					bCopyResult &= Bindings.Super::CopyProperty(Copy, SourceViews[Copy.SourceDataHandle.Get<FStateTreeDataHandle>().GetIndex()], TargetView);
				}
				AITEST_TRUE(TEXT("CopyTo should succeed"), bCopyResult);
				return TestCopy(Object, SourceStr);
			};

		if (!CopyAndTestBatch(CopyBatchIndex1, SourceA.A.Get(), TEXT("SourceA.A")))
		{
			return false;
		}
		if (!CopyAndTestBatch(CopyBatchIndex2, SourceA.A.Get(), TEXT("SourceA.ArrayA")))
		{
			return false;
		}
		if (!CopyAndTestBatch(CopyBatchIndex3, SourceA.B.Get(), TEXT("SourceA.B")))
		{
			return false;
		}
		if (!CopyAndTestBatch(CopyBatchIndex4, SourceA.B.Get(), TEXT("SourceA.ArrayB")))
		{
			return false;
		}
		if (!CopyAndTestBatch(CopyBatchIndex5, SourceA.C.Get(), TEXT("SourceA.C")))
		{
			return false;
		}
		if (!CopyAndTestBatch(CopyBatchIndex6, SourceA.C.Get(), TEXT("SourceA.ArrayC")))
		{
			return false;
		}

		// Test promote array
		for (int32 Index = 0; Index < NumberOfElements; ++Index)
		{
			SourceA.ArrayA[Index] = TBaseStructure<FVector>::Get();
			SourceA.ArrayB[Index] = TBaseStructure<FRotator>::Get();
			SourceA.ArrayC[Index] = TBaseStructure<FMatrix>::Get();
		}

		auto TestCopyArray = [this, &Target, NumberOfElements](UObject* Object, const FStringView SourceStr)
			{
				for (int32 Index = 0; Index < NumberOfElements; ++Index)
				{
					AITEST_EQUAL(FString::Printf(TEXT("Expect A[] to be copied from %s"), SourceStr.GetData()), Target.ArrayA[Index].Get(), Object);
					AITEST_EQUAL(FString::Printf(TEXT("Expect B[] to be copied from %s"), SourceStr.GetData()), Target.ArrayB[Index].Get(), Object);
					AITEST_EQUAL(FString::Printf(TEXT("Expect C[] to be copied from %s"), SourceStr.GetData()), Target.ArrayC[Index].Get(), Object);
				}
				return true;
			};

		auto CopyAndTestBatchArray = [this, &ResetTargetData, &TestCopyArray, &Bindings, &SourceViews, &TargetView](int32 CopyBatchIndex, UObject* Object, FStringView SourceStr)
			{
				ResetTargetData();
				bool bCopyResult = true;
				for (const FPropertyBindingCopyInfo& Copy : Bindings.Super::GetBatchCopies(FPropertyBindingIndex16(CopyBatchIndex)))
				{
					bCopyResult &= Bindings.Super::CopyProperty(Copy, SourceViews[Copy.SourceDataHandle.Get<FStateTreeDataHandle>().GetIndex()], TargetView);
				}
				AITEST_TRUE(TEXT("CopyTo should succeed"), bCopyResult);
				return TestCopyArray(Object, SourceStr);
			};
		if (!CopyAndTestBatchArray(CopyBatchIndex9, SourceA.A.Get(), TEXT("SourceA.ArrayA")))
		{
			return false;
		}
		if (!CopyAndTestBatchArray(CopyBatchIndex10, SourceA.B.Get(), TEXT("SourceA.ArrayB")))
		{
			return false;
		}
		if (!CopyAndTestBatchArray(CopyBatchIndex11, SourceA.C.Get(), TEXT("SourceA.ArrayC")))
		{
			return false;
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_BindingsCompiler_WithObjectPtr, "System.StateTree.Binding.BindingsCompilerWithObjectPtr");

struct FStateTreeTest_PropertyFunctions : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		FPropertyBindingPathSegment PathSegmentToFuncResult = FPropertyBindingPathSegment(TEXT("Result"));

		// Condition with property function binding.
		{
			TStateTreeEditorNode<FStateTreeCompareIntCondition>& EnterCond = Root.AddEnterCondition<FStateTreeCompareIntCondition>(UE::StateTree::EComparisonOperator::Equal);
			EnterCond.GetInstanceData().Right = 1;
			EditorData.AddPropertyBinding(CastChecked<UScriptStruct>(FTestPropertyFunction::StaticStruct()), {PathSegmentToFuncResult}, FPropertyBindingPath(EnterCond.ID, TEXT("Left")));
		}

		// Task with multiple nested property function bindings.
		auto& TaskA = Root.AddTask<FTestTask_PrintAndResetValue>(FName(TEXT("TaskA")));
		constexpr int32 TaskAPropertyFunctionsAmount = 10;
		{
			EditorData.AddPropertyBinding(CastChecked<UScriptStruct>(FTestPropertyFunction::StaticStruct()), {PathSegmentToFuncResult}, FPropertyBindingPath(TaskA.ID, TEXT("Value")));
		
			for (int32 i = 0; i < TaskAPropertyFunctionsAmount - 1; ++i)
			{
				const FStateTreePropertyPathBinding& LastBinding = EditorData.GetPropertyEditorBindings()->GetBindings().Last();
				const FGuid LastBindingPropertyFuncID = LastBinding.GetPropertyFunctionNode().Get<const FStateTreeEditorNode>().ID;
				EditorData.AddPropertyBinding(CastChecked<UScriptStruct>(FTestPropertyFunction::StaticStruct()), {PathSegmentToFuncResult}, FPropertyBindingPath(LastBindingPropertyFuncID, TEXT("Input")));
			}
		}

		// Task bound to state parameter with multiple nested property function bindings.
		auto& TaskB = Root.AddTask<FTestTask_PrintAndResetValue>(FName(TEXT("TaskB")));
		constexpr int32 ParameterPropertyFunctionsAmount = 5;
		{
			Root.Parameters.Parameters.AddProperty(FName(TEXT("Int")), EPropertyBagPropertyType::Int32);
			const FPropertyBindingPath PathToProperty = FPropertyBindingPath(Root.Parameters.ID, TEXT("Int"));
			EditorData.AddPropertyBinding(PathToProperty, FPropertyBindingPath(TaskB.ID, TEXT("Value")));
			EditorData.AddPropertyBinding(CastChecked<UScriptStruct>(FTestPropertyFunction::StaticStruct()), {PathSegmentToFuncResult}, PathToProperty);
		
			for (int32 i = 0; i < ParameterPropertyFunctionsAmount - 1; ++i)
			{
				const FStateTreePropertyPathBinding& LastBinding = EditorData.GetPropertyEditorBindings()->GetBindings().Last();
				const FGuid LastBindingPropertyFuncID = LastBinding.GetPropertyFunctionNode().Get<const FStateTreeEditorNode>().ID;
				EditorData.AddPropertyBinding(CastChecked<UScriptStruct>(FTestPropertyFunction::StaticStruct()), {PathSegmentToFuncResult}, FPropertyBindingPath(LastBindingPropertyFuncID, TEXT("Input")));
			}
		}

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		Exec.Start();
		AITEST_TRUE(*FString::Printf(TEXT("StateTree TaskA should enter state with value %d"), TaskAPropertyFunctionsAmount), Exec.Expect(TaskA.GetName(), *FString::Printf(TEXT("EnterState%d"), TaskAPropertyFunctionsAmount)));
		AITEST_TRUE(*FString::Printf(TEXT("StateTree TaskB should enter state with value %d"), ParameterPropertyFunctionsAmount), Exec.Expect(TaskB.GetName(), *FString::Printf(TEXT("EnterState%d"), ParameterPropertyFunctionsAmount)));
		Exec.LogClear();

		Exec.Tick(0.1f);
		AITEST_TRUE(*FString::Printf(TEXT("StateTree TaskA should tick with value %d"), TaskAPropertyFunctionsAmount), Exec.Expect(TaskA.GetName(), *FString::Printf(TEXT("Tick%d"), TaskAPropertyFunctionsAmount)));
		AITEST_TRUE(*FString::Printf(TEXT("StateTree TaskB should tick with value %d"), ParameterPropertyFunctionsAmount), Exec.Expect(TaskB.GetName(), *FString::Printf(TEXT("Tick%d"), ParameterPropertyFunctionsAmount)));
		Exec.LogClear();

		Exec.Stop(EStateTreeRunStatus::Stopped);
		AITEST_TRUE(*FString::Printf(TEXT("StateTree TaskA should exit state with value %d"), TaskAPropertyFunctionsAmount), Exec.Expect(TaskA.GetName(), *FString::Printf(TEXT("ExitState%d"), TaskAPropertyFunctionsAmount)));
		AITEST_TRUE(*FString::Printf(TEXT("StateTree TaskB should exit state with value %d"), ParameterPropertyFunctionsAmount), Exec.Expect(TaskB.GetName(), *FString::Printf(TEXT("ExitState%d"), ParameterPropertyFunctionsAmount)));
		Exec.LogClear();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_PropertyFunctions, "System.StateTree.Binding.PropertyFunctions");

struct FStateTreeTest_CopyObjects : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		FStateTreeCompilerLog Log;
		FStateTreePropertyBindings Bindings;
		FStateTreePropertyBindingCompiler BindingCompiler;

		const bool bInitResult = BindingCompiler.Init(Bindings, Log);
		AITEST_TRUE(TEXT("Expect init to succeed"), bInitResult);

		FStateTreeBindableStructDesc SourceDesc;
		SourceDesc.Name = FName(TEXT("Source"));
		SourceDesc.Struct = TBaseStructure<FStateTreeTest_PropertyCopyObjects>::Get();
		SourceDesc.DataSource = EStateTreeBindableStructSource::Parameter;
		SourceDesc.DataHandle = FStateTreeDataHandle(EStateTreeDataSourceType::ContextData, 0); // Used as index to SourceViews below.
		SourceDesc.ID = FGuid::NewGuid();

		FStateTreeBindableStructDesc TargetADesc;
		TargetADesc.Name = FName(TEXT("TargetA"));
		TargetADesc.Struct = TBaseStructure<FStateTreeTest_PropertyCopyObjects>::Get();
		TargetADesc.DataSource = EStateTreeBindableStructSource::Parameter;
		TargetADesc.ID = FGuid::NewGuid();

		FStateTreeBindableStructDesc TargetBDesc;
		TargetBDesc.Name = FName(TEXT("TargetB"));
		TargetBDesc.Struct = TBaseStructure<FStateTreeTest_PropertyCopyObjects>::Get();
		TargetBDesc.DataSource = EStateTreeBindableStructSource::Parameter;
		TargetBDesc.ID = FGuid::NewGuid();

		const int32 SourceIndex = BindingCompiler.AddSourceStruct(SourceDesc);

		TArray<FStateTreePropertyPathBinding> PropertyBindings;
		// One-to-one copy from source to target A
		PropertyBindings.Add(MakeBinding(SourceDesc.ID, TEXT("Object"), TargetADesc.ID, TEXT("Object")));
		PropertyBindings.Add(MakeBinding(SourceDesc.ID, TEXT("SoftObject"), TargetADesc.ID, TEXT("SoftObject")));
		PropertyBindings.Add(MakeBinding(SourceDesc.ID, TEXT("Class"), TargetADesc.ID, TEXT("Class")));
		PropertyBindings.Add(MakeBinding(SourceDesc.ID, TEXT("SoftClass"), TargetADesc.ID, TEXT("SoftClass")));

		// Cross copy from source to target B
		PropertyBindings.Add(MakeBinding(SourceDesc.ID, TEXT("SoftObject"), TargetBDesc.ID, TEXT("Object")));
		PropertyBindings.Add(MakeBinding(SourceDesc.ID, TEXT("Object"), TargetBDesc.ID, TEXT("SoftObject")));
		PropertyBindings.Add(MakeBinding(SourceDesc.ID, TEXT("SoftClass"), TargetBDesc.ID, TEXT("Class")));
		PropertyBindings.Add(MakeBinding(SourceDesc.ID, TEXT("Class"), TargetBDesc.ID, TEXT("SoftClass")));
		
		int32 TargetACopyBatchIndex = INDEX_NONE;
		const bool bCompileBatchResultA = BindingCompiler.CompileBatch(TargetADesc, PropertyBindings, FStateTreeIndex16::Invalid, FStateTreeIndex16::Invalid, TargetACopyBatchIndex);
		AITEST_TRUE(TEXT("CompileBatchResultA should succeed"), bCompileBatchResultA);
		AITEST_NOT_EQUAL(TEXT("TargetACopyBatchIndex should not be INDEX_NONE"), TargetACopyBatchIndex, (int32)INDEX_NONE);

		int32 TargetBCopyBatchIndex = INDEX_NONE;
		const bool bCompileBatchResultB = BindingCompiler.CompileBatch(TargetBDesc, PropertyBindings, FStateTreeIndex16::Invalid, FStateTreeIndex16::Invalid, TargetBCopyBatchIndex);
		AITEST_TRUE(TEXT("CompileBatchResultB should succeed"), bCompileBatchResultB);
		AITEST_NOT_EQUAL(TEXT("TargetBCopyBatchIndex should not be INDEX_NONE"), TargetBCopyBatchIndex, (int32)INDEX_NONE);

		BindingCompiler.Finalize();

		const bool bResolveResult = Bindings.ResolvePaths();
		AITEST_TRUE(TEXT("ResolvePaths should succeed"), bResolveResult);

		UStateTreeTest_PropertyObject* ObjectA = NewObject<UStateTreeTest_PropertyObject>();
		UStateTreeTest_PropertyObject2* ObjectB = NewObject<UStateTreeTest_PropertyObject2>();
		
		FStateTreeTest_PropertyCopyObjects Source;
		Source.Object = ObjectA;
		Source.SoftObject = ObjectB;
		Source.Class = UStateTreeTest_PropertyObject::StaticClass();
		Source.SoftClass = UStateTreeTest_PropertyObject::StaticClass();

		AITEST_TRUE(TEXT("SourceIndex should be less than max number of source structs."), SourceIndex < Bindings.GetNumBindableStructDescriptors());

		TArray<FStateTreeDataView> SourceViews;
		SourceViews.SetNum(Bindings.GetNumBindableStructDescriptors());
		SourceViews[SourceIndex] = FStateTreeDataView(FStructView::Make(Source));

		FStateTreeTest_PropertyCopyObjects TargetA;
		bool bCopyResultA = true;
		for (const FPropertyBindingCopyInfo& Copy : Bindings.Super::GetBatchCopies(FStateTreeIndex16(TargetACopyBatchIndex)))
		{
			bCopyResultA &= Bindings.Super::CopyProperty(Copy, SourceViews[Copy.SourceDataHandle.Get<FStateTreeDataHandle>().GetIndex()], FStructView::Make(TargetA));
		}
		AITEST_TRUE(TEXT("CopyTo should succeed"), bCopyResultA);

		AITEST_TRUE(TEXT("Expect TargetA.Object == Source.Object"), TargetA.Object == Source.Object);
		AITEST_TRUE(TEXT("Expect TargetA.SoftObject == Source.SoftObject"), TargetA.SoftObject == Source.SoftObject);
		AITEST_TRUE(TEXT("Expect TargetA.Class == Source.Class"), TargetA.Class == Source.Class);
		AITEST_TRUE(TEXT("Expect TargetA.SoftClass == Source.SoftClass"), TargetA.SoftClass == Source.SoftClass);

		// Copying to TargetB should not affect TargetA
		TargetA.Object = nullptr;
		
		FStateTreeTest_PropertyCopyObjects TargetB;
		bool bCopyResultB = true;
		for (const FPropertyBindingCopyInfo& Copy : Bindings.Super::GetBatchCopies(FStateTreeIndex16(TargetBCopyBatchIndex)))
		{
			bCopyResultB &= Bindings.Super::CopyProperty(Copy, SourceViews[Copy.SourceDataHandle.Get<FStateTreeDataHandle>().GetIndex()], FStructView::Make(TargetB));
		}
		AITEST_TRUE(TEXT("CopyTo should succeed"), bCopyResultB);

		AITEST_TRUE(TEXT("Expect TargetB.Object == Source.SoftObject"), TSoftObjectPtr<UObject>(TargetB.Object) == Source.SoftObject);
		AITEST_TRUE(TEXT("Expect TargetB.SoftObject == Source.Object"), TargetB.SoftObject == TSoftObjectPtr<UObject>(Source.Object));
		AITEST_TRUE(TEXT("Expect TargetB.Class == Source.SoftClass"), TSoftClassPtr<UObject>(TargetB.Class) == Source.SoftClass);
		AITEST_TRUE(TEXT("Expect TargetB.SoftClass == Source.Class"), TargetB.SoftClass == TSoftClassPtr<UObject>(Source.Class));

		AITEST_TRUE(TEXT("Expect TargetA.Object == nullptr after copy of TargetB"), TargetA.Object == nullptr);

		// Collect ObjectA and ObjectB, soft object paths should still copy ok.
		ObjectA = nullptr;
		ObjectB = nullptr;
		Source.Object = nullptr;

		// @todo: Avoid relying on GC within this test.
		TryCollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

		FStateTreeTest_PropertyCopyObjects TargetC;
		bool bCopyResultC = true;
		for (const FPropertyBindingCopyInfo& Copy : Bindings.Super::GetBatchCopies(FStateTreeIndex16(TargetACopyBatchIndex)))
		{
			bCopyResultB &= Bindings.Super::CopyProperty(Copy, SourceViews[Copy.SourceDataHandle.Get<FStateTreeDataHandle>().GetIndex()], FStructView::Make(TargetC));
		}

		
		AITEST_TRUE(TEXT("CopyTo should succeed"), bCopyResultC);
		AITEST_TRUE(TEXT("Expect TargetC.SoftObject == Source.SoftObject after GC"), TargetC.SoftObject == Source.SoftObject);

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_CopyObjects, "System.StateTree.Binding.CopyObjects");

struct FStateTreeTest_References : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		FStateTreeCompilerLog Log;
		FStateTreePropertyBindings Bindings;
		FStateTreePropertyBindingCompiler BindingCompiler;

		const bool bInitResult = BindingCompiler.Init(Bindings, Log);
		AITEST_TRUE(TEXT("Expect init to succeed"), bInitResult);

		FStateTreeBindableStructDesc SourceDesc;
		SourceDesc.Name = FName(TEXT("Source"));
		SourceDesc.Struct = TBaseStructure<FStateTreeTest_PropertyRefSourceStruct>::Get();
		SourceDesc.DataSource = EStateTreeBindableStructSource::Parameter;
		SourceDesc.DataHandle = FStateTreeDataHandle(EStateTreeDataSourceType::ContextData, 0);
		SourceDesc.ID = FGuid::NewGuid();
		BindingCompiler.AddSourceStruct(SourceDesc);

		FStateTreeBindableStructDesc TargetDesc;
		TargetDesc.Name = FName(TEXT("Target"));
		TargetDesc.Struct = TBaseStructure<FStateTreeTest_PropertyRefTargetStruct>::Get();
		TargetDesc.DataSource = EStateTreeBindableStructSource::Parameter;
		TargetDesc.ID = FGuid::NewGuid();

		TArray<FStateTreePropertyPathBinding> PropertyBindings;
		PropertyBindings.Add(MakeBinding(SourceDesc.ID, TEXT("Item"), TargetDesc.ID, TEXT("RefToStruct")));
		PropertyBindings.Add(MakeBinding(SourceDesc.ID, TEXT("Item.A"), TargetDesc.ID, TEXT("RefToInt")));
		PropertyBindings.Add(MakeBinding(SourceDesc.ID, TEXT("Array"), TargetDesc.ID, TEXT("RefToStructArray")));

		FStateTreeTest_PropertyRefSourceStruct Source;
		FStateTreeDataView SourceView = FStateTreeDataView(FStructView::Make(Source));

		FStateTreeTest_PropertyRefTargetStruct Target;
		FStateTreeDataView TargetView(FStructView::Make(Target));

		TMap<FGuid, const FStateTreeDataView> IDToStructValue;
		IDToStructValue.Emplace(SourceDesc.ID, SourceView);
		IDToStructValue.Emplace(TargetDesc.ID, TargetView);

		const bool bCompileReferencesResult = BindingCompiler.CompileReferences(TargetDesc, PropertyBindings, TargetView, IDToStructValue);
		AITEST_TRUE(TEXT("CompileReferences should succeed"), bCompileReferencesResult);

		BindingCompiler.Finalize();

		const bool bResolveResult = Bindings.ResolvePaths();
		AITEST_TRUE(TEXT("ResolvePaths should succeed"), bResolveResult);

		{
			const FStateTreePropertyAccess* PropertyAccess = Bindings.GetPropertyAccess(Target.RefToStruct);
			AITEST_NOT_NULL(TEXT("GetPropertyAccess should succeed"), PropertyAccess);

			FStateTreeTest_PropertyStruct* Reference = Bindings.GetMutablePropertyPtr<FStateTreeTest_PropertyStruct>(SourceView, *PropertyAccess);
			AITEST_EQUAL(TEXT("Expect RefToStruct to point to SourceA.Item"), Reference, &Source.Item);
		}

		{
			const FStateTreePropertyAccess* PropertyAccess = Bindings.GetPropertyAccess(Target.RefToInt);
			AITEST_NOT_NULL(TEXT("GetPropertyAccess should succeed"), PropertyAccess);

			int32* Reference = Bindings.GetMutablePropertyPtr<int32>(SourceView, *PropertyAccess);
			AITEST_EQUAL(TEXT("Expect RefToInt to point to SourceA.Item.A"), Reference, &Source.Item);
		}

		{
			const FStateTreePropertyAccess* PropertyAccess = Bindings.GetPropertyAccess(Target.RefToStructArray);
			AITEST_NOT_NULL(TEXT("GetPropertyAccess should succeed"), PropertyAccess);

			TArray<FStateTreeTest_PropertyStruct>* Reference = Bindings.GetMutablePropertyPtr<TArray<FStateTreeTest_PropertyStruct>>(SourceView, *PropertyAccess);
			AITEST_EQUAL(TEXT("Expect RefToStructArray to point to SourceA.Array"), Reference, &Source.Array);
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_References, "System.StateTree.Binding.References");

struct FStateTreeTest_ReferencesConstness : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		FStateTreeCompilerLog Log;
		FStateTreePropertyBindings Bindings;
		FStateTreePropertyBindingCompiler BindingCompiler;

		const bool bInitResult = BindingCompiler.Init(Bindings, Log);
		AITEST_TRUE(TEXT("Expect init to succeed"), bInitResult);

		FStateTreeBindableStructDesc SourceAsTaskDesc;
		SourceAsTaskDesc.Name = FName(TEXT("SourceTask"));
		SourceAsTaskDesc.Struct = TBaseStructure<FStateTreeTest_PropertyRefSourceStruct>::Get();
		SourceAsTaskDesc.DataSource = EStateTreeBindableStructSource::Task;
		SourceAsTaskDesc.DataHandle = FStateTreeDataHandle(EStateTreeDataSourceType::ContextData, 0);
		SourceAsTaskDesc.ID = FGuid::NewGuid();
		BindingCompiler.AddSourceStruct(SourceAsTaskDesc);

		FStateTreeBindableStructDesc SourceAsContextDesc;
		SourceAsContextDesc.Name = FName(TEXT("SourceContext"));
		SourceAsContextDesc.Struct = TBaseStructure<FStateTreeTest_PropertyRefSourceStruct>::Get();
		SourceAsContextDesc.DataSource = EStateTreeBindableStructSource::Context;
		SourceAsContextDesc.DataHandle = FStateTreeDataHandle(EStateTreeDataSourceType::ContextData, 0);
		SourceAsContextDesc.ID = FGuid::NewGuid();
		BindingCompiler.AddSourceStruct(SourceAsContextDesc);

		FStateTreeBindableStructDesc TargetDesc;
		TargetDesc.Name = FName(TEXT("Target"));
		TargetDesc.Struct = TBaseStructure<FStateTreeTest_PropertyRefTargetStruct>::Get();
		TargetDesc.DataSource = EStateTreeBindableStructSource::Parameter;
		TargetDesc.ID = FGuid::NewGuid();

		FStateTreePropertyPathBinding TaskPropertyBinding = MakeBinding(SourceAsTaskDesc.ID, TEXT("Item"), TargetDesc.ID, TEXT("RefToStruct"));
		FStateTreePropertyPathBinding TaskOutputPropertyBinding = MakeBinding(SourceAsTaskDesc.ID, TEXT("OutputItem"), TargetDesc.ID, TEXT("RefToStruct"));

		FStateTreePropertyPathBinding ContextPropertyBinding = MakeBinding(SourceAsTaskDesc.ID, TEXT("Item"), TargetDesc.ID, TEXT("RefToStruct"));
		FStateTreePropertyPathBinding ContextOutputPropertyBinding = MakeBinding(SourceAsTaskDesc.ID, TEXT("Item"), TargetDesc.ID, TEXT("RefToStruct"));

		FStateTreeTest_PropertyRefSourceStruct SourceAsTask;
		FStateTreeDataView SourceAsTaskView(FStructView::Make(SourceAsTask));

		FStateTreeTest_PropertyRefSourceStruct SourceAsContext;
		FStateTreeDataView SourceAsContextView(FStructView::Make(SourceAsContext));

		FStateTreeTest_PropertyRefTargetStruct Target;
		FStateTreeDataView TargetView(FStructView::Make(Target));

		TMap<FGuid, const FStateTreeDataView> IDToStructValue;
		IDToStructValue.Emplace(SourceAsTaskDesc.ID, SourceAsTaskView);
		IDToStructValue.Emplace(SourceAsContextDesc.ID, SourceAsContextView);
		IDToStructValue.Emplace(TargetDesc.ID, TargetView);

		{
			const bool bCompileReferenceResult = BindingCompiler.CompileReferences(TargetDesc, { TaskPropertyBinding }, TargetView, IDToStructValue);
			AITEST_FALSE(TEXT("CompileReferences should fail"), bCompileReferenceResult);
		}

		{
			const bool bCompileReferenceResult = BindingCompiler.CompileReferences(TargetDesc, { TaskOutputPropertyBinding }, TargetView, IDToStructValue);
			AITEST_TRUE(TEXT("CompileReferences should succeed"), bCompileReferenceResult);
		}

		{
			const bool bCompileReferenceResult = BindingCompiler.CompileReferences(TargetDesc, { ContextPropertyBinding }, TargetView, IDToStructValue);
			AITEST_FALSE(TEXT("CompileReferences should fail"), bCompileReferenceResult);
		}

		{
			const bool bCompileReferenceResult = BindingCompiler.CompileReferences(TargetDesc, { ContextOutputPropertyBinding }, TargetView, IDToStructValue);
			AITEST_FALSE(TEXT("CompileReferences should fail"), bCompileReferenceResult);
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_ReferencesConstness, "System.StateTree.Binding.ReferencesConstness");

struct FStateTreeTest_ReferencesOnNode : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		// Root IntA, IntB
		//     State1 PropertyRefOnNode -> IntA, PropertyRefOnInstance -> IntB
		//         State2 PropertyRefOnNode -> PropertyRefOnNode(State1), PropertyRefOnInstance -> PropertyRefOnInstance(State1)

		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		UStateTreeState& State1 = Root.AddChildState(FName(TEXT("State1")));
		UStateTreeState& State2 = State1.AddChildState(FName(TEXT("State2")));

		TStateTreeEditorNode<FTestTask_IntegersOutput>& RootTaskNode = Root.AddTask<FTestTask_IntegersOutput>(FName(TEXT("RootTask")));
		TStateTreeEditorNode<FTestTask_PropertyRefOnNodeAndInstance>& State1TaskNode = State1.AddTask<FTestTask_PropertyRefOnNodeAndInstance>(FName(TEXT("State1Task")));
		TStateTreeEditorNode<FTestTask_PropertyRefOnNodeAndInstance>& State2TaskNode = State2.AddTask<FTestTask_PropertyRefOnNodeAndInstance>(FName(TEXT("State2Task")));

		EditorData.AddPropertyBinding(
			FPropertyBindingPath(RootTaskNode.ID, GET_MEMBER_NAME_CHECKED(FTestTask_IntegersOutput_InstanceData, IntA)),
			FPropertyBindingPath(State1TaskNode.GetNodeID(), GET_MEMBER_NAME_CHECKED(FTestTask_PropertyRefOnNodeAndInstance, RefOnNode)));
		EditorData.AddPropertyBinding(
			FPropertyBindingPath(RootTaskNode.ID, GET_MEMBER_NAME_CHECKED(FTestTask_IntegersOutput_InstanceData, IntB)),
			FPropertyBindingPath(State1TaskNode.ID, GET_MEMBER_NAME_CHECKED(FTestTask_PropertyRefOnNodeAndInstance_InstanceData, RefOnInstance)));
		EditorData.AddPropertyBinding(
			FPropertyBindingPath(State1TaskNode.GetNodeID(), GET_MEMBER_NAME_CHECKED(FTestTask_PropertyRefOnNodeAndInstance, RefOnNode)),
			FPropertyBindingPath(State2TaskNode.GetNodeID(), GET_MEMBER_NAME_CHECKED(FTestTask_PropertyRefOnNodeAndInstance, RefOnNode)));
		EditorData.AddPropertyBinding(
			FPropertyBindingPath(State1TaskNode.ID, GET_MEMBER_NAME_CHECKED(FTestTask_PropertyRefOnNodeAndInstance_InstanceData, RefOnInstance)),
			FPropertyBindingPath(State2TaskNode.ID, GET_MEMBER_NAME_CHECKED(FTestTask_PropertyRefOnNodeAndInstance_InstanceData, RefOnInstance)));

		void* IntAAddress = nullptr;
		void* IntBAddress = nullptr;
		void* State1RefOnNodeAddress = nullptr;
		void* State1RefOnInstanceAddress = nullptr;
		void* State2RefOnNodeAddress = nullptr;
		void* State2RefOnInstanceAddress = nullptr;

		RootTaskNode.GetNode().CustomEnterStateFunc = 
			[&IntAAddress, &IntBAddress](FStateTreeExecutionContext& ExecContext, const FTestTask_IntegersOutput& Node)
			{
				auto& InstanceData = ExecContext.GetInstanceData<FTestTask_IntegersOutput::FInstanceDataType>(Node);
				IntAAddress = &InstanceData.IntA;
				IntBAddress = &InstanceData.IntB;
			};

		State1TaskNode.GetNode().CustomEnterStateFunc =
			[&State1RefOnNodeAddress, &State1RefOnInstanceAddress](FStateTreeExecutionContext& ExecContext, const FTestTask_PropertyRefOnNodeAndInstance& Node)
			{
				auto& InstanceData = ExecContext.GetInstanceData<FTestTask_PropertyRefOnNodeAndInstance::FInstanceDataType>(Node);

				State1RefOnNodeAddress = Node.RefOnNode.GetMutablePtr<int32>(ExecContext);
				State1RefOnInstanceAddress = InstanceData.RefOnInstance.GetMutablePtr<int32>(ExecContext);
			};

		State2TaskNode.GetNode().CustomEnterStateFunc =
			[&State2RefOnNodeAddress, &State2RefOnInstanceAddress](FStateTreeExecutionContext& ExecContext, const FTestTask_PropertyRefOnNodeAndInstance& Node)
			{
				auto& InstanceData = ExecContext.GetInstanceData<FTestTask_PropertyRefOnNodeAndInstance::FInstanceDataType>(Node);

				State2RefOnNodeAddress = Node.RefOnNode.GetMutablePtr<int32>(ExecContext);
				State2RefOnInstanceAddress = InstanceData.RefOnInstance.GetMutablePtr<int32>(ExecContext);
			};

		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE("StateTree should get compiled", bResult);
		}

		{
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE("StateTree should init", bInitSucceeded);

			Exec.Start();
			AITEST_TRUE("Expected Root to be active.", Exec.ExpectInActiveStates("Root", "State1", "State2"));
			AITEST_TRUE("int addresses should have been initialized", IntAAddress && IntBAddress);
			AITEST_EQUAL("Ref on Node should fetch IntA", IntAAddress, State1RefOnNodeAddress);
			AITEST_EQUAL("Chained refs on Node should point to the same address", State1RefOnNodeAddress, State2RefOnNodeAddress);
			AITEST_EQUAL("Ref on Instance should fetch IntB", IntBAddress, State1RefOnInstanceAddress);
			AITEST_EQUAL("Chained refs on Instance should point to the same address", State1RefOnInstanceAddress, State2RefOnInstanceAddress);
			Exec.LogClear();

			Exec.Stop();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_ReferencesOnNode, "System.StateTree.Binding.ReferencesOnNode");

struct FStateTreeTest_ChainedReferences : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		// TreeA(Global Parameter: 10)
		//	RootA
		//    LinkedStateB -> SubtreeB (Parameter : PropertyRef linked to Global Parameter)
		//  SubtreeB (Parameter : PropertyRef)
		//    LinkedStateC -> SubtreeC (Parameter : PropertyRef)
		//  SubtreeC (Parameter : PropertyRef)
		//    LinkedAssetStateD -> TreeD (Parameter : PropertyRef)
		// TreeD(Global Parameter: PropertyRef)
		//  RootE
		//   LinkedStateF -> SubtreeF
		//  SubtreeF
		//    LinkedStateG -> SubtreeG
		//  SubtreeG(ConditionG with PropertyRef, TaskG with PropertyRef)

		const void* GlobalParamAddressFromCondition = nullptr;
		const void* GlobalParamAddressFromTask = nullptr;

		auto AddPropertyRefPropertyToPropertyBag = [](FInstancedPropertyBag& PropertyBag, FName VarName)
			{
				PropertyBag.AddProperty(VarName, EPropertyBagPropertyType::Struct, FStateTreePropertyRef::StaticStruct());
				const FPropertyBagPropertyDesc* PropertyDesc = PropertyBag.FindPropertyDescByName(VarName);
				check(PropertyDesc && PropertyDesc->CachedProperty);
				const_cast<FProperty*>(PropertyDesc->CachedProperty)->SetMetaData(TEXT("Optional"), nullptr);
				const_cast<FProperty*>(PropertyDesc->CachedProperty)->SetMetaData(TEXT("RefType"), TEXT("int32"));

				return PropertyDesc->ID;
			};

		constexpr auto PropertyRefName = TEXT("PropertyRef");

		FGuid StateTreeDPropertyRefParameterID;
		UStateTree& StateTreeD = NewStateTree();
		{
			UStateTreeEditorData& EditorDataD = *Cast<UStateTreeEditorData>(StateTreeD.EditorData);
			{
				StateTreeDPropertyRefParameterID = AddPropertyRefPropertyToPropertyBag(const_cast<FInstancedPropertyBag&>(EditorDataD.GetRootParametersPropertyBag()), PropertyRefName);

				auto& RootE = EditorDataD.AddSubTree(TEXT("RootE"));

				auto& SubtreeF = EditorDataD.AddSubTree(TEXT("SubtreeF"), EStateTreeStateType::Subtree);

				auto& SubtreeG = EditorDataD.AddSubTree(TEXT("SubtreeG"), EStateTreeStateType::Subtree);
				{
					constexpr auto NodePropertyRefName = TEXT("RefOnNode");

					TStateTreeEditorNode<FTestCondition_PropertyRefOnNodeAndInstance>& ConditionNode = SubtreeG.AddEnterCondition<FTestCondition_PropertyRefOnNodeAndInstance>(TEXT("SubtreeGStateCondition"));
					{
						ConditionNode.GetNode().CustomTestConditionFunc = [&GlobalParamAddressFromCondition](FStateTreeExecutionContext& Context, const FTestCondition_PropertyRefOnNodeAndInstance& Node)
						{
								GlobalParamAddressFromCondition = Node.RefOnNode.GetMutablePtr<int32>(Context);
						};

						EditorDataD.AddPropertyBinding(
							FPropertyBindingPath(EditorDataD.GetRootParametersGuid(), PropertyRefName),
							FPropertyBindingPath(ConditionNode.GetNodeID(), NodePropertyRefName)
						);
					}

					TStateTreeEditorNode<FTestTask_PropertyRefOnNodeAndInstance>& TaskNode = SubtreeG.AddTask<FTestTask_PropertyRefOnNodeAndInstance>(TEXT("SubtreeGStateTask"));
					{
						TaskNode.GetNode().CustomEnterStateFunc = [&GlobalParamAddressFromTask](FStateTreeExecutionContext& Context, const FTestTask_PropertyRefOnNodeAndInstance& Node)
							{
								GlobalParamAddressFromTask = Node.RefOnNode.GetMutablePtr<int32>(Context);
							};

						EditorDataD.AddPropertyBinding(
							FPropertyBindingPath(EditorDataD.GetRootParametersGuid(), PropertyRefName),
							FPropertyBindingPath(TaskNode.GetNodeID(), NodePropertyRefName)
						);
					}
				}

				auto& LinkedF = RootE.AddChildState(TEXT("LinkedF"), EStateTreeStateType::Linked);
				LinkedF.SetLinkedState(SubtreeF.GetLinkToState());

				auto& LinkedG = SubtreeF.AddChildState(TEXT("LinkedG"), EStateTreeStateType::Linked);
				LinkedG.SetLinkedState(SubtreeG.GetLinkToState());
			}
		}

		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTreeD);
			AITEST_TRUE(TEXT("StateTreeD should get compiled"), bResult);
		}

		constexpr auto GlobalIntParameterName = TEXT("GlobalInt");
		UStateTree& StateTreeA = NewStateTree();
		{
			UStateTreeEditorData& EditorDataA = *Cast<UStateTreeEditorData>(StateTreeA.EditorData);
			{
				const_cast<FInstancedPropertyBag&>(EditorDataA.GetRootParametersPropertyBag()).AddProperty(GlobalIntParameterName, EPropertyBagPropertyType::Int32);
				const_cast<FInstancedPropertyBag&>(EditorDataA.GetRootParametersPropertyBag()).SetValueInt32(GlobalIntParameterName, 10);

				auto& RootA = EditorDataA.AddSubTree(TEXT("RootA"));

				FGuid SubtreeBPropertyRefParameterID;
				auto& SubtreeB = EditorDataA.AddSubTree(TEXT("SubtreeB"), EStateTreeStateType::Subtree);
				{
					SubtreeBPropertyRefParameterID = AddPropertyRefPropertyToPropertyBag(SubtreeB.Parameters.Parameters, PropertyRefName);
				}

				FGuid SubtreeCPropertyRefParameterID;
				auto& SubtreeC = EditorDataA.AddSubTree(TEXT("SubtreeC"), EStateTreeStateType::Subtree);
				{
					SubtreeCPropertyRefParameterID = AddPropertyRefPropertyToPropertyBag(SubtreeC.Parameters.Parameters, PropertyRefName);
				}

				auto& LinkedB = RootA.AddChildState(TEXT("LinkedB"), EStateTreeStateType::Linked);
				{
					LinkedB.SetLinkedState(SubtreeB.GetLinkToState());

					constexpr bool bIsOverridden = true;
					LinkedB.SetParametersPropertyOverridden(SubtreeBPropertyRefParameterID, bIsOverridden);

					EditorDataA.AddPropertyBinding(
						FPropertyBindingPath(EditorDataA.GetRootParametersGuid(), GlobalIntParameterName),
						FPropertyBindingPath(LinkedB.Parameters.ID, PropertyRefName));
				}

				auto& LinkedC = SubtreeB.AddChildState(TEXT("LinkedC"), EStateTreeStateType::Linked);
				{
					LinkedC.SetLinkedState(SubtreeC.GetLinkToState());

					constexpr bool bIsOverridden = true;
					LinkedC.SetParametersPropertyOverridden(SubtreeCPropertyRefParameterID, bIsOverridden);

					EditorDataA.AddPropertyBinding(
						FPropertyBindingPath(SubtreeB.Parameters.ID, PropertyRefName),
						FPropertyBindingPath(LinkedC.Parameters.ID, PropertyRefName));
				}

				auto& LinkedAssetD = SubtreeC.AddChildState(TEXT("LinkedAssetD"), EStateTreeStateType::LinkedAsset);
				{
					LinkedAssetD.SetLinkedStateAsset(&StateTreeD);

					constexpr bool bIsOverridden = true;
					LinkedAssetD.SetParametersPropertyOverridden(StateTreeDPropertyRefParameterID, bIsOverridden);

					EditorDataA.AddPropertyBinding(
						FPropertyBindingPath(SubtreeC.Parameters.ID, PropertyRefName),
						FPropertyBindingPath(LinkedAssetD.Parameters.ID, PropertyRefName));
				}
			}
		}

		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTreeA);
			AITEST_TRUE(TEXT("StateTreeA should get compiled"), bResult);
		}


		{
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTreeA, StateTreeA, InstanceData);

			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

			EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;

			Status = Exec.Start();

			const void* GlobalIntParameterActualAddress =
				InstanceData.GetStorage().GetGlobalParameters().GetMemory() + StateTreeA.GetDefaultParameters().FindPropertyDescByName(GlobalIntParameterName)->CachedProperty->GetOffset_ForInternal();

			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("RootA", "LinkedB", "SubtreeB", "LinkedC", "SubtreeC", "LinkedAssetD", "RootE", "LinkedF", "SubtreeF", "LinkedG", "SubtreeG"));

			AITEST_TRUE(TEXT("Condition Node should correctly fetch global parameter address."), GlobalIntParameterActualAddress == GlobalParamAddressFromCondition);
			AITEST_TRUE(TEXT("Task Node should correctly fetch global parameter address."), GlobalIntParameterActualAddress == GlobalParamAddressFromTask);

			Exec.LogClear();

			Exec.Stop();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_ChainedReferences, "System.StateTree.Binding.ChainedReferences");

struct FStateTreeTest_MutableArray : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		//Tree 1
		//	Root
		//		StateA -> Succeeded(Root)

		FStateTreeCompilerLog Log;
		FStateTreePropertyBindings Bindings;
		FStateTreePropertyBindingCompiler BindingCompiler;

		UStateTree& StateTree = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
			{
				// Parameters
				FInstancedPropertyBag& RootPropertyBag = GetRootPropertyBag(EditorData);
				RootPropertyBag.AddProperty("Value", EPropertyBagPropertyType::Int32);
				RootPropertyBag.SetValueInt32("Value", -111);
				RootPropertyBag.AddContainerProperty("ArrayValue", FPropertyBagContainerTypes(EPropertyBagContainerType::Array), EPropertyBagPropertyType::Int32, nullptr);
				FPropertyBagArrayRef ValueArrayRef = RootPropertyBag.GetMutableArrayRef("ArrayValue").GetValue();
				ValueArrayRef.EmptyAndAddValues(4);
				ValueArrayRef.SetValueInt32(0, -11);
				ValueArrayRef.SetValueInt32(1, -22);
				ValueArrayRef.SetValueInt32(2, -33);
				ValueArrayRef.SetValueInt32(3, -44);

				// Global
				TStateTreeEditorNode<FTestTask_PrintValue>& TaskA = EditorData.AddGlobalTask<FTestTask_PrintValue>("Tree1GlobalTaskA");
				TaskA.GetInstanceData().Value = -2;
				TaskA.GetInstanceData().ArrayValue = {-1, -2};
				EditorData.AddPropertyBinding(FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("Value")), FPropertyBindingPath(TaskA.ID, TEXT("Value")));
				EditorData.AddPropertyBinding(FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("ArrayValue")), FPropertyBindingPath(TaskA.ID, TEXT("ArrayValue")));

			}
			UStateTreeState& Root = EditorData.AddSubTree("Tree1StateRoot");
			{
				UStateTreeState& State = Root.AddChildState("Tree1StateA", EStateTreeStateType::State);

				FStateTreeTransition& Transition = State.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::Succeeded);
				Transition.bDelayTransition = true;
				Transition.DelayDuration = 60.0;

				FPropertyBindingPath RootParametersArrayValue3(EditorData.GetRootParametersGuid());
				RootParametersArrayValue3.AddPathSegment("ArrayValue", 3);

				TStateTreeEditorNode<FTestTask_PrintAndResetValue>& TaskA = State.AddTask<FTestTask_PrintAndResetValue>("Tree1StateATaskA");
				TaskA.GetInstanceData().Value = -2;
				TaskA.GetInstanceData().ArrayValue = { -1, -2, -3, -4 };
				TaskA.GetNode().ResetValue = 22;
				TaskA.GetNode().ResetArrayValue = {200, 201, 202, 203, 204, 205};

				FPropertyBindingPath TaskAArrayValue3(TaskA.ID);
				TaskAArrayValue3.AddPathSegment("ArrayValue", 3);

				EditorData.AddPropertyBinding(FPropertyBindingPath(EditorData.GetRootParametersGuid(), "Value"), FPropertyBindingPath(TaskA.ID, TEXT("Value")));
				EditorData.AddPropertyBinding(RootParametersArrayValue3, TaskAArrayValue3);
				
				TStateTreeEditorNode<FTestTask_PrintAndResetValue>& TaskB = State.AddTask<FTestTask_PrintAndResetValue>("Tree1StateATaskB");
				TaskB.GetInstanceData().Value = -2;
				TaskB.GetInstanceData().ArrayValue = { -1, -2, -3, -4 };
				TaskB.GetNode().ResetValue = 33;
				TaskB.GetNode().ResetArrayValue = { 300, 301, 302, 303, 304, 305, 306, 307, 308, 309, 310, 311, 312, 313, 314, 315 };

				FPropertyBindingPath TaskBArrayValue3(TaskB.ID);
				TaskBArrayValue3.AddPathSegment("ArrayValue", 3);

				EditorData.AddPropertyBinding(FPropertyBindingPath(TaskA.ID, TEXT("Value")), FPropertyBindingPath(TaskB.ID, TEXT("Value")));
				EditorData.AddPropertyBinding(TaskAArrayValue3, TaskBArrayValue3);
			}
		}
		{
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE(TEXT("StateTree2 should get compiled"), bResult);
		}
		{
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);

			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

			EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
			FInstancedPropertyBag GlobalParameters = StateTree.GetDefaultParameters();
			{
				GlobalParameters.SetValueInt32("Value", 11);
				FPropertyBagArrayRef ValueArrayRef = GlobalParameters.GetMutableArrayRef("ArrayValue").GetValue();
				ValueArrayRef.EmptyAndAddValues(2);
				ValueArrayRef.SetValueInt32(0, 911);
				ValueArrayRef.SetValueInt32(1, 922);
				Status = Exec.Start(GlobalParameters.GetValue());
			}
			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Start should enter Global tasks"), Exec.Expect("Tree1GlobalTaskA", TEXT("EnterState11"))
				.Then("Tree1GlobalTaskA", TEXT("EnterState:{911, 922}")) // should copy the full array
				.Then("Tree1StateATaskA", TEXT("EnterState11"))
				.Then("Tree1StateATaskA", TEXT("EnterState:{-1, -2, -3, -4}")) // should not copy anything since [3] is out of scope
				.Then("Tree1StateATaskB", TEXT("EnterState22")) // TaskA set the value to 22  and {200, 201, 202, 203, 204, 205} (in EnterTask)
				.Then("Tree1StateATaskB", TEXT("EnterState:{-1, -2, -3, 203}"))
			);
			Exec.LogClear();

			Exec.Stop();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_MutableArray, "System.StateTree.Binding.MutableArray");

struct FStateTreeTest_TransitionTaskWithBinding : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		//Tree 1
		//	Root
		//		StateA -> Succeeded(Root)

		FStateTreeCompilerLog Log;
		FStateTreePropertyBindings Bindings;
		FStateTreePropertyBindingCompiler BindingCompiler;

		UStateTree& StateTree = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
			{
				// Parameters
				FInstancedPropertyBag& RootPropertyBag = GetRootPropertyBag(EditorData);
				RootPropertyBag.AddProperty("Value", EPropertyBagPropertyType::Int32);
				RootPropertyBag.SetValueInt32("Value", -111);

				// Global
				TStateTreeEditorNode<FTestTask_PrintValue>& TaskA = EditorData.AddGlobalTask<FTestTask_PrintValue>("Tree1GlobalTaskA");
				TaskA.GetInstanceData().Value = -2;
				EditorData.AddPropertyBinding(FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("Value")), FPropertyBindingPath(TaskA.ID, TEXT("Value")));

				TStateTreeEditorNode<FTestTask_PrintValue_TransitionTick>& TaskB = EditorData.AddGlobalTask<FTestTask_PrintValue_TransitionTick>("Tree1GlobalTaskB");
				TaskB.GetInstanceData().Value = -2;
				EditorData.AddPropertyBinding(FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("Value")), FPropertyBindingPath(TaskB.ID, TEXT("Value")));

				TStateTreeEditorNode<FTestTask_PrintValue_TransitionNoTick>& TaskC = EditorData.AddGlobalTask<FTestTask_PrintValue_TransitionNoTick>("Tree1GlobalTaskC");
				TaskC.GetInstanceData().Value = -2;
				EditorData.AddPropertyBinding(FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("Value")), FPropertyBindingPath(TaskC.ID, TEXT("Value")));
			}
			UStateTreeState& Root = EditorData.AddSubTree("Tree1StateRoot");
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& TaskA = Root.AddTask<FTestTask_PrintValue>("Tree1StateRootTaskA");
				TaskA.GetInstanceData().Value = -2;
				EditorData.AddPropertyBinding(FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("Value")), FPropertyBindingPath(TaskA.ID, TEXT("Value")));
				
				TStateTreeEditorNode<FTestTask_PrintValue_TransitionTick>& TaskB = Root.AddTask<FTestTask_PrintValue_TransitionTick>("Tree1StateRootTaskB");
				TaskB.GetInstanceData().Value = -2;
				EditorData.AddPropertyBinding(FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("Value")), FPropertyBindingPath(TaskB.ID, TEXT("Value")));
				
				TStateTreeEditorNode<FTestTask_PrintValue_TransitionNoTick>& TaskC = Root.AddTask<FTestTask_PrintValue_TransitionNoTick>("Tree1StateRootTaskC");
				TaskC.GetInstanceData().Value = -2;
				EditorData.AddPropertyBinding(FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("Value")), FPropertyBindingPath(TaskC.ID, TEXT("Value")));
			}
			{
				UStateTreeState& State = Root.AddChildState("Tree1StateA", EStateTreeStateType::State);

				FStateTreeTransition& Transition = State.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::Succeeded);
				Transition.bDelayTransition = true;
				Transition.DelayDuration = 5.0;

				TStateTreeEditorNode<FTestTask_PrintValue>& TaskA = State.AddTask<FTestTask_PrintValue>("Tree1StateATaskA");
				TaskA.GetInstanceData().Value = -2;
				EditorData.AddPropertyBinding(FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("Value")), FPropertyBindingPath(TaskA.ID, TEXT("Value")));

				TStateTreeEditorNode<FTestTask_PrintValue_TransitionTick>& TaskB = State.AddTask<FTestTask_PrintValue_TransitionTick>("Tree1StateATaskB");
				TaskB.GetInstanceData().Value = -2;
				EditorData.AddPropertyBinding(FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("Value")), FPropertyBindingPath(TaskB.ID, TEXT("Value")));

				TStateTreeEditorNode<FTestTask_PrintValue_TransitionNoTick>& TaskC = State.AddTask<FTestTask_PrintValue_TransitionNoTick>("Tree1StateATaskC");
				TaskC.GetInstanceData().Value = -2;
				EditorData.AddPropertyBinding(FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("Value")), FPropertyBindingPath(TaskC.ID, TEXT("Value")));
			}
		}
		{
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE(TEXT("StateTree2 should get compiled"), bResult);
		}
		{
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);

			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

			EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
			FInstancedPropertyBag GlobalParameters = StateTree.GetDefaultParameters();

			{
				GlobalParameters.SetValueInt32("Value", 99);
				Status = Exec.Start(GlobalParameters.GetValue());
			}
			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1StateA"));
			AITEST_TRUE(TEXT("Start should enter Global tasks"), Exec.Expect("Tree1GlobalTaskA", TEXT("EnterState99"))
				.Then("Tree1GlobalTaskB", TEXT("EnterState99"))
				.Then("Tree1GlobalTaskC", TEXT("EnterState99"))
				.Then("Tree1StateRootTaskA", TEXT("EnterState99"))
				.Then("Tree1StateRootTaskB", TEXT("EnterState99"))
				.Then("Tree1StateRootTaskC", TEXT("EnterState99"))
				.Then("Tree1StateATaskA", TEXT("EnterState99"))
				.Then("Tree1StateATaskB", TEXT("EnterState99"))
				.Then("Tree1StateATaskC", TEXT("EnterState99"))
			);
			Exec.LogClear();

			GlobalParameters.SetValueInt32("Value", 88);
			InstanceData.GetMutableStorage().SetGlobalParameters(GlobalParameters.GetValue());

			Status = Exec.Tick(1.0f);
			AITEST_EQUAL(TEXT("2nd Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1StateA"));
			AITEST_TRUE(TEXT("2nd Tick should tick tasks"), Exec.Expect("Tree1GlobalTaskA", TEXT("Tick88"))
				.Then("Tree1GlobalTaskB", TEXT("Tick88"))
				.Then("Tree1StateRootTaskA", TEXT("Tick88"))
				.Then("Tree1StateRootTaskB", TEXT("Tick88"))
				.Then("Tree1StateATaskA", TEXT("Tick88"))
				.Then("Tree1StateATaskB", TEXT("Tick88"))
				.Then("Tree1StateATaskC", TEXT("TriggerTransitions88"))
				.Then("Tree1StateATaskB", TEXT("TriggerTransitions88"))
				.Then("Tree1StateRootTaskC", TEXT("TriggerTransitions88"))
				.Then("Tree1StateRootTaskB", TEXT("TriggerTransitions88"))
				.Then("Tree1GlobalTaskC", TEXT("TriggerTransitions88"))
				.Then("Tree1GlobalTaskB", TEXT("TriggerTransitions88"))
			);
			Exec.LogClear();

			Exec.Stop();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_TransitionTaskWithBinding, "System.StateTree.Binding.TransitionTaskWithBinding");

struct FStateTreeTest_BindingStructRef : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		//	Root (global variable, )
		//		State1 (ref to root variable)
		//			State2
		//			State3 (with a lot of instance data to realloc the internal buffer)
		//				State4 (with a lot of instance data to realloc the internal buffer)

		const FStateTreeTest_PropertyStruct* TaskA_State_PropertyStructPtr = nullptr;
		const FStateTreeTest_PropertyStruct* TaskB_State_PropertyStructPtr = nullptr;
		const FStateTreeTest_PropertyStruct* TaskA_Task_PropertyStructPtr = nullptr;
		const FStateTreeTest_PropertyStruct* TaskB_Task_PropertyStructPtr = nullptr;

		FStateTreeCompilerLog Log;
		FStateTreePropertyBindings Bindings;
		FStateTreePropertyBindingCompiler BindingCompiler;

		UStateTree& StateTree = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

			UStateTreeState& Root = EditorData.AddSubTree("Tree1StateRoot");
			UStateTreeState& State1 = Root.AddChildState("Tree1State1", EStateTreeStateType::State);
			UStateTreeState& State2 = State1.AddChildState("Tree1State2", EStateTreeStateType::State);
			UStateTreeState& State3 = State1.AddChildState("Tree1State3", EStateTreeStateType::State);
			UStateTreeState& State4 = State3.AddChildState("Tree1State4", EStateTreeStateType::State);
			{
				Root.Parameters.Parameters.AddProperty("RootParam", EPropertyBagPropertyType::Struct, FStateTreeTest_PropertyStruct::StaticStruct());
				FStateTreeTest_PropertyStruct PropertyStruct;
				PropertyStruct.A = 111;
				PropertyStruct.B = 222;
				PropertyStruct.StructB.B = 333;
				Root.Parameters.Parameters.SetValueStruct("RootParam", PropertyStruct);
			}
			{
				FPropertyBagPropertyDesc Desc = FPropertyBagPropertyDesc("StateParam", EPropertyBagPropertyType::Struct, FStateTreeStructRef::StaticStruct());
				Desc.MetaData.Emplace(FName("BaseStruct"), TEXT("/Script/StateTreeTestSuite.StateTreeTest_PropertyStruct"));
				State1.Parameters.Parameters.AddProperties({ Desc }, /*bOverwrite*/false);

				EditorData.AddPropertyBinding(FPropertyBindingPath(Root.Parameters.ID, TEXT("RootParam")), FPropertyBindingPath(State1.Parameters.ID, TEXT("StateParam")));
			}
			{
				TStateTreeEditorNode<FTestTask_PrintValue_StructRef_NoBindingUpdate>& TaskA = State1.AddTask<FTestTask_PrintValue_StructRef_NoBindingUpdate>("Tree1State1TaskA");
				TaskA.GetNode().CustomTickFunc = [&TaskA_State_PropertyStructPtr, &TaskA_Task_PropertyStructPtr](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						TaskA_State_PropertyStructPtr = nullptr;
						TaskA_Task_PropertyStructPtr = nullptr;

						FConstStructView RootParameters = Context.GetInstanceData()->GetStorage().GetStruct(0);
						if (const FCompactStateTreeParameters* Parameters = RootParameters.GetPtr<const FCompactStateTreeParameters>())
						{
							if (Parameters->GetValue().GetScriptStruct() != nullptr)
							{
								if (const FStructProperty* StructProperty = CastField<const FStructProperty>(Parameters->GetValue().GetScriptStruct()->FindPropertyByName("RootParam")))
								{
									if (StructProperty->Struct == TBaseStructure<FStateTreeTest_PropertyStruct>())
									{
										TaskA_State_PropertyStructPtr = StructProperty->ContainerPtrToValuePtr<const FStateTreeTest_PropertyStruct>(Parameters->GetValue().GetMemory());
									}
								}
							}
						}

						FTestTask_PrintValue_StructRef_NoBindingUpdate::FInstanceDataType& InstanceData = Context.GetInstanceData(*static_cast<const FTestTask_PrintValue_StructRef_NoBindingUpdate*>(Task));
						TaskA_Task_PropertyStructPtr = &InstanceData.PropertyStruct;
					};
				TaskA.GetInstanceData().PropertyStruct.A = 11;
				TaskA.GetInstanceData().PropertyStruct.B = 22;
				TaskA.GetInstanceData().PropertyStruct.StructB.B = 33;

				TStateTreeEditorNode<FTestTask_PrintValue_StructRef_NoBindingUpdate>& TaskB = State1.AddTask<FTestTask_PrintValue_StructRef_NoBindingUpdate>("Tree1State1TaskB");
				TaskB.GetNode().CustomTickFunc = [&TaskB_State_PropertyStructPtr, &TaskB_Task_PropertyStructPtr](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						TaskB_State_PropertyStructPtr = nullptr;

						FStateTreeTestLog& TestLog = Context.GetExternalData(Task->LogHandle);

						FConstStructView State1Parameters = Context.GetInstanceData()->GetStorage().GetStruct(1);
						if (const FCompactStateTreeParameters* Parameters = State1Parameters.GetPtr<const FCompactStateTreeParameters>())
						{
							if (Parameters->GetValue().GetScriptStruct() != nullptr)
							{
								if (const FStructProperty* StructProperty = CastField<const FStructProperty>(Parameters->GetValue().GetScriptStruct()->FindPropertyByName("StateParam")))
								{
									if (StructProperty->Struct == TBaseStructure<FStateTreeStructRef>())
									{
										if (const FStateTreeStructRef* StructRef = StructProperty->ContainerPtrToValuePtr<const FStateTreeStructRef>(Parameters->GetValue().GetMemory()))
										{
											TaskB_State_PropertyStructPtr = StructRef->GetPtr<const FStateTreeTest_PropertyStruct>();
										}
									}
								}
							}

							if (const FStateTreeStructRef* PropertyStruct = Parameters->GetValue().GetPtr<FStateTreeStructRef>())
							{
								TaskB_State_PropertyStructPtr = PropertyStruct->GetPtr<const FStateTreeTest_PropertyStruct>();
							}
						}

						FTestTask_PrintValue_StructRef_NoBindingUpdate::FInstanceDataType& InstanceData = Context.GetInstanceData(*static_cast<const FTestTask_PrintValue_StructRef_NoBindingUpdate*>(Task));
						TaskB_Task_PropertyStructPtr = InstanceData.StructRef.GetPtr<const FStateTreeTest_PropertyStruct>();
					};
				EditorData.AddPropertyBinding(FPropertyBindingPath(TaskA.ID, TEXT("PropertyStruct")), FPropertyBindingPath(TaskB.ID, TEXT("StructRef")));
			}
			{
				State2.SelectionBehavior = EStateTreeStateSelectionBehavior::TryEnterState;
				State2.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::GotoState, &State3);
			}
			{
				for (int32 Index = 0; Index < 32; ++Index)
				{
					State3.AddTask<FTestTask_PrintValue>("Tree1State3TaskX");
				}
			}
			{
				for (int32 Index = 0; Index < 32; ++Index)
				{
					State4.AddTask<FTestTask_PrintValue>("Tree1State4TaskX");
				}
			}

		}
		{
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE(TEXT("StateTree2 should get compiled"), bResult);
		}
		{
			FStateTreeInstanceData InstanceData;
			{
				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
				const bool bInitSucceeded = Exec.IsValid();
				AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);
			}
			{
				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
				EStateTreeRunStatus Status = Exec.Start();
				AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1State1", "Tree1State2"));
			}
			{
				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
				EStateTreeRunStatus Status = Exec.Tick(0.1f);
				AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1State1", "Tree1State3", "Tree1State4"));
				AITEST_TRUE(TEXT("Both task should point to the same StructRef"), TaskA_State_PropertyStructPtr != nullptr && TaskA_State_PropertyStructPtr == TaskB_State_PropertyStructPtr);
				AITEST_TRUE(TEXT("Both task should point to the same StructRef"), TaskA_Task_PropertyStructPtr != nullptr && TaskA_Task_PropertyStructPtr == TaskB_Task_PropertyStructPtr);
			}
			{
				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
				EStateTreeRunStatus Status = Exec.Tick(0.1f);
				AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1State1", "Tree1State3", "Tree1State4"));
				AITEST_TRUE(TEXT("Both task should point to the same StructRef"), TaskA_State_PropertyStructPtr != nullptr && TaskA_State_PropertyStructPtr == TaskB_State_PropertyStructPtr);
				//@TODO: Deprecate FStateTreeStructRef. If the binding is not updated before it is used it can access invalid data. Here the array of instance data grows.
				AITEST_FALSE(TEXT("Both task should point to the same StructRef"), TaskA_Task_PropertyStructPtr != nullptr && TaskA_Task_PropertyStructPtr == TaskB_Task_PropertyStructPtr);
			}
			{
				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
				Exec.Stop();
			}
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_BindingStructRef, "System.StateTree.Binding.StructRef");

struct FStateTreeTest_BindingEnum : FStateTreeTestBase
{
	// The test check if all enum type supported by UHT are also supported by the binding systems.
	//Also test if the StateTreePropertyRef with enum works.
	//The value will be set from Root to State1. The default value of Root are set to B. The default value of State1 are set to C.
	//We expect the value in State1 to be set to B after the State1::OnEnter
	virtual bool InstantTest() override
	{
		//	Root (global variable, )
		//		State1 (binding and ref to root variable)

		UStateTree& StateTree = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

			UStateTreeState& Root = EditorData.AddSubTree("Tree1StateRoot");
			UStateTreeState& State1 = Root.AddChildState("Tree1State1", EStateTreeStateType::State);
			{
				using namespace UE::StateTree::Test;
				Root.Parameters.Parameters.AddProperty("StructWithEnum", EPropertyBagPropertyType::Struct, FStructWithEnum::StaticStruct());
				FStructWithEnum StructValue;
				StructValue.CppClassEnumByte = ECppClassEnumByte::B;
				StructValue.CppClassEnumInt = ECppClassEnumInt::B;
				StructValue.NamespaceEnumByte = ENamespaceEnumByte::B;
				StructValue.EnumByte = EEnumByte::EEnumByte_B;
				Root.Parameters.Parameters.SetValueStruct("StructWithEnum", StructValue);

				Root.Parameters.Parameters.AddProperty("CppClassEnumByte", EPropertyBagPropertyType::Enum, StaticEnum<ECppClassEnumByte>());
				Root.Parameters.Parameters.SetValueEnum("CppClassEnumByte", ECppClassEnumByte::B);
				Root.Parameters.Parameters.AddProperty("CppClassEnumInt", EPropertyBagPropertyType::Enum, StaticEnum<ECppClassEnumInt>());
				Root.Parameters.Parameters.SetValueEnum("CppClassEnumInt", ECppClassEnumInt::B);
				Root.Parameters.Parameters.AddProperty("NamespaceEnumByte", EPropertyBagPropertyType::Enum, StaticEnum<ENamespaceEnumByte::Type>());
				Root.Parameters.Parameters.SetValueEnum("NamespaceEnumByte", ENamespaceEnumByte::B);
				Root.Parameters.Parameters.AddProperty("EnumByte", EPropertyBagPropertyType::Enum, StaticEnum<EEnumByte>());
				Root.Parameters.Parameters.SetValueEnum("EnumByte", EEnumByte::EEnumByte_B);
			}
			{
				using namespace UE::StateTree::Test;
				State1.Parameters.Parameters = Root.Parameters.Parameters;
				State1.Parameters.Parameters.AddProperty("StructWithEnumRef1", EPropertyBagPropertyType::Struct, FStructWithEnumRef::StaticStruct());
				State1.Parameters.Parameters.AddProperty("StructWithEnumRef2", EPropertyBagPropertyType::Struct, FStructWithEnumRef::StaticStruct());

				FStructWithEnum StructValue;
				StructValue.CppClassEnumByte = ECppClassEnumByte::C;
				StructValue.CppClassEnumInt = ECppClassEnumInt::C;
				StructValue.NamespaceEnumByte = ENamespaceEnumByte::C;
				StructValue.EnumByte = EEnumByte::EEnumByte_C;
				State1.Parameters.Parameters.SetValueStruct("StructWithEnum", StructValue);
				State1.Parameters.Parameters.SetValueEnum("CppClassEnumByte", ECppClassEnumByte::C);
				State1.Parameters.Parameters.SetValueEnum("CppClassEnumInt", ECppClassEnumInt::C);
				State1.Parameters.Parameters.SetValueEnum("NamespaceEnumByte", ENamespaceEnumByte::C);
				State1.Parameters.Parameters.SetValueEnum("EnumByte", EEnumByte::EEnumByte_C);

				EditorData.AddPropertyBinding(FPropertyBindingPath(Root.Parameters.ID, "StructWithEnum"), FPropertyBindingPath(State1.Parameters.ID, "StructWithEnum"));
				EditorData.AddPropertyBinding(FPropertyBindingPath(Root.Parameters.ID, "CppClassEnumByte"), FPropertyBindingPath(State1.Parameters.ID, "CppClassEnumByte"));
				EditorData.AddPropertyBinding(FPropertyBindingPath(Root.Parameters.ID, "CppClassEnumInt"), FPropertyBindingPath(State1.Parameters.ID, "CppClassEnumInt"));
				EditorData.AddPropertyBinding(FPropertyBindingPath(Root.Parameters.ID, "NamespaceEnumByte"), FPropertyBindingPath(State1.Parameters.ID, "NamespaceEnumByte"));
				EditorData.AddPropertyBinding(FPropertyBindingPath(Root.Parameters.ID, "EnumByte"), FPropertyBindingPath(State1.Parameters.ID, "EnumByte"));
				EditorData.AddPropertyBinding(FPropertyBindingPath(Root.Parameters.ID, "CppClassEnumByte"), FPropertyBindingPath(State1.Parameters.ID, {FPropertyBindingPathSegment("StructWithEnumRef1"), FPropertyBindingPathSegment("CppClassEnumByte")}));
				EditorData.AddPropertyBinding(FPropertyBindingPath(Root.Parameters.ID, "CppClassEnumInt"), FPropertyBindingPath(State1.Parameters.ID, {FPropertyBindingPathSegment("StructWithEnumRef1"), FPropertyBindingPathSegment("CppClassEnumInt")}));
				EditorData.AddPropertyBinding(FPropertyBindingPath(Root.Parameters.ID, "NamespaceEnumByte"), FPropertyBindingPath(State1.Parameters.ID, { FPropertyBindingPathSegment("StructWithEnumRef1"), FPropertyBindingPathSegment("NamespaceEnumByte") }));
				EditorData.AddPropertyBinding(FPropertyBindingPath(Root.Parameters.ID, "EnumByte"), FPropertyBindingPath(State1.Parameters.ID, { FPropertyBindingPathSegment("StructWithEnumRef1"), FPropertyBindingPathSegment("EnumByte") }));
				EditorData.AddPropertyBinding(FPropertyBindingPath(Root.Parameters.ID, { FPropertyBindingPathSegment("StructWithEnum"), FPropertyBindingPathSegment("CppClassEnumByte") }), FPropertyBindingPath(State1.Parameters.ID, {FPropertyBindingPathSegment("StructWithEnumRef2"), FPropertyBindingPathSegment("CppClassEnumByte")}));
				EditorData.AddPropertyBinding(FPropertyBindingPath(Root.Parameters.ID, { FPropertyBindingPathSegment("StructWithEnum"), FPropertyBindingPathSegment("CppClassEnumInt") }), FPropertyBindingPath(State1.Parameters.ID, {FPropertyBindingPathSegment("StructWithEnumRef2"), FPropertyBindingPathSegment("CppClassEnumInt")}));
				EditorData.AddPropertyBinding(FPropertyBindingPath(Root.Parameters.ID, { FPropertyBindingPathSegment("StructWithEnum"), FPropertyBindingPathSegment("NamespaceEnumByte") }), FPropertyBindingPath(State1.Parameters.ID, { FPropertyBindingPathSegment("StructWithEnumRef2"), FPropertyBindingPathSegment("NamespaceEnumByte") }));
				EditorData.AddPropertyBinding(FPropertyBindingPath(Root.Parameters.ID, { FPropertyBindingPathSegment("StructWithEnum"), FPropertyBindingPathSegment("EnumByte") }), FPropertyBindingPath(State1.Parameters.ID, { FPropertyBindingPathSegment("StructWithEnumRef2"), FPropertyBindingPathSegment("EnumByte") }));
			}
		}
		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE(TEXT("StateTree1 should compiled"), bResult);
		}

		const FStateTreeStateHandle State1Handle = FStateTreeStateHandle(1);
		const FCompactStateTreeState* CompactState1 = StateTree.GetStateFromHandle(State1Handle);
		AITEST_TRUE(TEXT("Invalid state"), CompactState1 != nullptr);
		CA_ASSUME(CompactState1); // Suppress code analyzer warning C6011

		{
			FStateTreeInstanceData InstanceData;
			{
				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
				const bool bInitSucceeded = Exec.IsValid();
				AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);
				const EStateTreeRunStatus Status = Exec.Start();
				AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			}
			{
				AITEST_TRUE(TEXT("A single frame"), InstanceData.GetExecutionState()->ActiveFrames.Num() == 1);
				FStateTreeDataView State1DataView = UE::StateTree::InstanceData::GetDataView(
					InstanceData.GetMutableStorage(),
					nullptr,
					InstanceData.GetExecutionState()->ActiveFrames[0],
					CompactState1->ParameterDataHandle);

				const UStruct* Struct = State1DataView.GetStruct();
				const void* Memory = State1DataView.GetMemory();
				AITEST_TRUE(TEXT("Valid data view"), Struct != nullptr && Memory != nullptr);
				CA_ASSUME(Struct); // Suppress code analyzer warning C6011
				CA_ASSUME(Memory); // Suppress code analyzer warning C6011
				
				{
					using namespace UE::StateTree::Test;
					const FStructProperty* Property = CastField<const FStructProperty>(Struct->FindPropertyByName("StructWithEnum"));
					AITEST_TRUE(TEXT("StructProperty"), Property != nullptr);
					CA_ASSUME(Property); // Suppress code analyzer warning C6011
					FStructWithEnum StructValue;
					Property->GetValue_InContainer(Memory, &StructValue);
					AITEST_TRUE(TEXT("StructWithEnum"), StructValue.CppClassEnumByte == ECppClassEnumByte::B);
					AITEST_TRUE(TEXT("StructWithEnum"), StructValue.CppClassEnumInt == ECppClassEnumInt::B);
					AITEST_TRUE(TEXT("StructWithEnum"), StructValue.NamespaceEnumByte == ENamespaceEnumByte::B);
					AITEST_TRUE(TEXT("StructWithEnum"), StructValue.EnumByte == EEnumByte::EEnumByte_B);
				}
				{
					using namespace UE::StateTree::Test;
					const FEnumProperty* EnumProperty = CastField<const FEnumProperty>(Struct->FindPropertyByName("CppClassEnumByte"));
					AITEST_TRUE(TEXT("CppClassEnumByte"), EnumProperty != nullptr);
					CA_ASSUME(EnumProperty); // Suppress code analyzer warning C6011
					AITEST_TRUE(TEXT("FByteProperty"), CastField<const FByteProperty>(EnumProperty->GetUnderlyingProperty()) != nullptr);
					ECppClassEnumByte Value;
					EnumProperty->GetValue_InContainer(Memory, &Value);
					AITEST_TRUE(TEXT("Value"), Value == ECppClassEnumByte::B);
				}
				//@todo doesn't support uint32
				//{
				//	using namespace UE::StateTree::Test;
				//	const FEnumProperty* EnumProperty = CastField<const FEnumProperty>(Struct->FindPropertyByName("CppClassEnumInt"));
				//	AITEST_TRUE(TEXT("EnumProperty"), EnumProperty != nullptr);
				//	AITEST_TRUE(TEXT("FUInt32Property"), CastField<const FUInt32Property>(EnumProperty->GetUnderlyingProperty()) != nullptr);
				//	ECppClassEnumInt Value;
				//	EnumProperty->GetValue_InContainer(Memory, &Value);
				//	AITEST_TRUE(TEXT("Value"), Value == ECppClassEnumInt::B);
				//}
				{
					using namespace UE::StateTree::Test;
					const FProperty* Property = CastField<const FByteProperty>(Struct->FindPropertyByName("NamespaceEnumByte"));
					AITEST_TRUE(TEXT("NamespaceEnumByte"), Property != nullptr);
					CA_ASSUME(Property); // Suppress code analyzer warning C6011
					TEnumAsByte<ENamespaceEnumByte::Type> Value;
					Property->GetValue_InContainer(Memory, &Value);
					AITEST_TRUE(TEXT("Value"), Value == ENamespaceEnumByte::B);
				}
				{
					using namespace UE::StateTree::Test;
					const FProperty* Property = CastField<const FByteProperty>(Struct->FindPropertyByName("EnumByte"));
					AITEST_TRUE(TEXT("EnumByte"), Property != nullptr);
					CA_ASSUME(Property); // Suppress code analyzer warning C6011
					TEnumAsByte<EEnumByte> Value;
					Property->GetValue_InContainer(Memory, &Value);
					AITEST_TRUE(TEXT("Value"), Value == EEnumByte::EEnumByte_B);
				}
				{
					using namespace UE::StateTree::Test;
					const FStructProperty* Property = CastField<const FStructProperty>(Struct->FindPropertyByName("StructWithEnumRef1"));
					AITEST_TRUE(TEXT("StructWithEnumRef1 property"), Property != nullptr);
					CA_ASSUME(Property); // Suppress code analyzer warning C6011
					FStructWithEnumRef Value;
					Property->GetValue_InContainer(Memory, &Value);

					ECppClassEnumByte* CppClassEnumByte = UE::StateTree::PropertyRefHelpers::GetMutablePtrToProperty<ECppClassEnumByte>(
						Value.CppClassEnumByte.GetInternalPropertyRef(),
						InstanceData.GetMutableStorage(),
						nullptr,
						InstanceData.GetExecutionState()->ActiveFrames[0]);
					AITEST_TRUE(TEXT("CppClassEnumByte"), CppClassEnumByte != nullptr);
					CA_ASSUME(CppClassEnumByte); // Suppress code analyzer warning C6011
					AITEST_TRUE(TEXT("Value"), *CppClassEnumByte == ECppClassEnumByte::B);

					//@todo PropertyBag doesn't support enum > int8. It always generates an enum of int8. That can creates bad memory access.
					//ECppClassEnumInt* CppClassEnumInt = UE::StateTree::PropertyRefHelpers::GetMutablePtrToProperty<ECppClassEnumInt>(
					//	Value.CppClassEnumInt.GetInternalPropertyRef(),
					//	InstanceData.GetMutableStorage(),
					//	nullptr,
					//	InstanceData.GetExecutionState()->ActiveFrames[0]);
					//AITEST_TRUE(TEXT("CppClassEnumInt"), CppClassEnumInt != nullptr);
					//AITEST_TRUE(TEXT("Value"), *CppClassEnumInt == ECppClassEnumInt::B);

					TEnumAsByte<ENamespaceEnumByte::Type>* NamespaceEnumByte = UE::StateTree::PropertyRefHelpers::GetMutablePtrToProperty<TEnumAsByte<ENamespaceEnumByte::Type>>(
						Value.NamespaceEnumByte.GetInternalPropertyRef(),
						InstanceData.GetMutableStorage(),
						nullptr,
						InstanceData.GetExecutionState()->ActiveFrames[0]); 
					AITEST_TRUE(TEXT("NamespaceEnumByte"), NamespaceEnumByte != nullptr);
					CA_ASSUME(NamespaceEnumByte); // Suppress code analyzer warning C6011
					AITEST_TRUE(TEXT("Value"), *NamespaceEnumByte == ENamespaceEnumByte::B);

					TEnumAsByte<EEnumByte>* EnumByte = UE::StateTree::PropertyRefHelpers::GetMutablePtrToProperty<TEnumAsByte<EEnumByte>>(
						Value.EnumByte.GetInternalPropertyRef(),
						InstanceData.GetMutableStorage(),
						nullptr,
						InstanceData.GetExecutionState()->ActiveFrames[0]); 
					AITEST_TRUE(TEXT("EnumByte"), EnumByte != nullptr);
					CA_ASSUME(EnumByte); // Suppress code analyzer warning C6011
					AITEST_TRUE(TEXT("Value"), *EnumByte == EEnumByte::EEnumByte_B);
				}
				{
					using namespace UE::StateTree::Test;
					const FStructProperty* Property = CastField<const FStructProperty>(Struct->FindPropertyByName("StructWithEnumRef2"));
					AITEST_TRUE(TEXT("StructWithEnumRef2 property"), Property != nullptr);
					CA_ASSUME(Property); // Suppress code analyzer warning C6011
					FStructWithEnumRef Value;
					Property->GetValue_InContainer(Memory, &Value);

					ECppClassEnumByte* CppClassEnumByte = UE::StateTree::PropertyRefHelpers::GetMutablePtrToProperty<ECppClassEnumByte>(
						Value.CppClassEnumByte.GetInternalPropertyRef(),
						InstanceData.GetMutableStorage(),
						nullptr,
						InstanceData.GetExecutionState()->ActiveFrames[0]);
					AITEST_TRUE(TEXT("CppClassEnumByte"), CppClassEnumByte != nullptr);
					CA_ASSUME(CppClassEnumByte); // Suppress code analyzer warning C6011
					AITEST_TRUE(TEXT("Value"), *CppClassEnumByte == ECppClassEnumByte::B);

					ECppClassEnumInt* CppClassEnumInt = UE::StateTree::PropertyRefHelpers::GetMutablePtrToProperty<ECppClassEnumInt>(
						Value.CppClassEnumInt.GetInternalPropertyRef(),
						InstanceData.GetMutableStorage(),
						nullptr,
						InstanceData.GetExecutionState()->ActiveFrames[0]);
					AITEST_TRUE(TEXT("CppClassEnumInt"), CppClassEnumInt != nullptr);
					CA_ASSUME(CppClassEnumInt); // Suppress code analyzer warning C6011
					AITEST_TRUE(TEXT("Value"), *CppClassEnumInt == ECppClassEnumInt::B);

					TEnumAsByte<ENamespaceEnumByte::Type>* NamespaceEnumByte = UE::StateTree::PropertyRefHelpers::GetMutablePtrToProperty<TEnumAsByte<ENamespaceEnumByte::Type>>(
						Value.NamespaceEnumByte.GetInternalPropertyRef(),
						InstanceData.GetMutableStorage(),
						nullptr,
						InstanceData.GetExecutionState()->ActiveFrames[0]);
					AITEST_TRUE(TEXT("NamespaceEnumByte"), NamespaceEnumByte != nullptr);
					AITEST_TRUE(TEXT("Value"), *NamespaceEnumByte == ENamespaceEnumByte::B);

					TEnumAsByte<EEnumByte>* EnumByte = UE::StateTree::PropertyRefHelpers::GetMutablePtrToProperty<TEnumAsByte<EEnumByte>>(
						Value.EnumByte.GetInternalPropertyRef(),
						InstanceData.GetMutableStorage(),
						nullptr,
						InstanceData.GetExecutionState()->ActiveFrames[0]);
					AITEST_TRUE(TEXT("EnumByte"), EnumByte != nullptr);
					CA_ASSUME(EnumByte); // Suppress code analyzer warning C6011
					AITEST_TRUE(TEXT("Value"), *EnumByte == EEnumByte::EEnumByte_B);
				}
			}
			{
				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
				Exec.Stop();
			}
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_BindingEnum, "System.StateTree.Binding.Enum");

struct FStateTreeTest_OutputBinding : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		//Global: OutputTaskA
		//	Root: OutputTaskB

		FStateTreeCompilerLog Log;

		int32 Value = 0;
		UStateTree& StateTree = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
			{
				// Global
				FInstancedPropertyBag& GlobalPropertyBag = GetRootPropertyBag(EditorData);
				GlobalPropertyBag.AddProperty("Int1Param", EPropertyBagPropertyType::Int32);
				GlobalPropertyBag.AddProperty("Int2Param", EPropertyBagPropertyType::Int32);
				GlobalPropertyBag.AddProperty("IntWrapperParam", EPropertyBagPropertyType::Struct, FIntWrapper::StaticStruct());
				GlobalPropertyBag.AddContainerProperty("IntWrapperArrayParam", EPropertyBagContainerType::Array, EPropertyBagPropertyType::Struct, FIntWrapper::StaticStruct());

				TStateTreeEditorNode<FTestTask_OutputBindingsTask>& TaskA = EditorData.AddGlobalTask<FTestTask_OutputBindingsTask>("Tree1GlobalTaskA");
				TaskA.GetInstanceData().OutputBool = static_cast<bool>(++Value); // 1
				TaskA.GetInstanceData().OutputInt = { ++Value }; // 2
				TaskA.GetInstanceData().OutputIntWrapper = { ++Value }; // 3
				TaskA.GetInstanceData().OutputIntWrapperArray = TArray<FIntWrapper>();
				TaskA.GetInstanceData().OutputIntWrapperArray.Append({ {++Value}, {++Value}, {++Value} }); // 4, 5, 6

				EditorData.GetPropertyEditorBindings()->AddOutputBinding(
					FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("Int1Param")),
					FPropertyBindingPath(TaskA.ID, TEXT("OutputBool")));

				EditorData.GetPropertyEditorBindings()->AddOutputBinding(
					FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("Int2Param")),
					FPropertyBindingPath(TaskA.ID, TEXT("OutputInt")));

				EditorData.GetPropertyEditorBindings()->AddOutputBinding(
					FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("IntWrapperParam")),
					FPropertyBindingPath(TaskA.ID, TEXT("OutputIntWrapper")));

				EditorData.GetPropertyEditorBindings()->AddOutputBinding(
					FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("IntWrapperArrayParam")),
					FPropertyBindingPath(TaskA.ID, TEXT("OutputIntWrapperArray")));

				EditorData.GetPropertyEditorBindings()->AddBinding(
					FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("Int1Param")),
					FPropertyBindingPath(TaskA.ID, TEXT("InputIntA")));

				EditorData.GetPropertyEditorBindings()->AddBinding(
					FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("Int2Param")),
					FPropertyBindingPath(TaskA.ID, TEXT("InputIntB")));

				EditorData.GetPropertyEditorBindings()->AddBinding(
					FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("IntWrapperParam")),
					FPropertyBindingPath(TaskA.ID, TEXT("InputIntWrapper")));

				EditorData.GetPropertyEditorBindings()->AddBinding(
					FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("IntWrapperArrayParam")),
					FPropertyBindingPath(TaskA.ID, TEXT("InputIntWrapperArray")));

				// Root State
				auto& RootState = EditorData.AddSubTree(TEXT("Root"));
				auto& RootPropertyBag = const_cast<FInstancedPropertyBag&>(*RootState.GetDefaultParameters());
				RootState.Parameters.Parameters.AddProperty("IntParam", EPropertyBagPropertyType::Int32);

				TStateTreeEditorNode<FTestTask_OutputBindingsTask>& TaskB = RootState.AddTask<FTestTask_OutputBindingsTask>("Tree1RootTaskB");
				TaskB.GetInstanceData().OutputBool = static_cast<bool>(++Value); // 7
				TaskB.GetInstanceData().OutputInt = { ++Value }; // 8
				TaskB.GetInstanceData().OutputIntWrapper = { ++Value }; // 9
				TaskB.GetInstanceData().OutputIntWrapperArray = TArray<FIntWrapper>();
				TaskB.GetInstanceData().OutputIntWrapperArray.Append({ {++Value}, {++Value}, {++Value} }); // 10, 11, 12

				FPropertyBindingPath Path;
				Path.SetStructID(TaskB.ID);
				Path.FromString(TEXT("OutputIntWrapperArray[1].Value"));
				EditorData.GetPropertyEditorBindings()->AddOutputBinding(
					FPropertyBindingPath(RootState.Parameters.ID, TEXT("IntParam")),
					Path);

				EditorData.GetPropertyEditorBindings()->AddBinding(
					FPropertyBindingPath(RootState.Parameters.ID, TEXT("IntParam")),
					FPropertyBindingPath(TaskB.ID, TEXT("InputIntA")));

			}
		}
		{
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE(TEXT("StateTree2 should get compiled"), bResult);
		}

		{
			// Verify Copy Size and Type
			constexpr EPropertyCopyType OutputBoolToIntCopyType = EPropertyCopyType::PromoteBoolToInt32;
			constexpr int32 OutputBoolToIntCopySize = 0;
			constexpr EPropertyCopyType OutputIntToIntCopyType = EPropertyCopyType::CopyPlain;
			constexpr int32 OutputIntToIntCopySize = sizeof(int32);
			constexpr EPropertyCopyType OutputIntWrapperToIntWrapperCopyType = EPropertyCopyType::CopyStruct;
			constexpr int32 OutputIntWrapperToIntWrapperCopySize = 0;
			constexpr EPropertyCopyType OutputIntWrapperArrayToIntWrapperArrayCopyType = EPropertyCopyType::CopyComplex;
			constexpr int32 OutputIntWrapperArrayToIntWrapperArrayCopySize = 0;

			constexpr int32 TaskAOutputBindingBatchIndex = 1;
			TConstArrayView<FPropertyBindingCopyInfo> TaskAOutputBindingsCopyInfo = StateTree.GetPropertyBindings().Super::GetBatchCopies(FStateTreeIndex16(TaskAOutputBindingBatchIndex));
			AITEST_TRUE(TEXT("TaskA should have 4 output bindings"), TaskAOutputBindingsCopyInfo.Num() == 4);

			// Bindings are sorted by memory address
			constexpr int32 TaskAOutputBoolBindingIndex = 0;
			constexpr int32 TaskAOutputIntBindingIndex = 1;
			constexpr int32 TaskAOutputIntWrapperBindingIndex = 2;
			constexpr int32 TaskAOutputIntWrapperArrayBindingIndex = 3;

			AITEST_TRUE(TEXT("TaskA Copy Info should have been resolved correctly"), 
				TaskAOutputBindingsCopyInfo[TaskAOutputBoolBindingIndex].Type == OutputBoolToIntCopyType 
				&& TaskAOutputBindingsCopyInfo[TaskAOutputBoolBindingIndex].CopySize == OutputBoolToIntCopySize);
			AITEST_TRUE(TEXT("TaskA Copy Info should have been resolved correctly"), 
				TaskAOutputBindingsCopyInfo[TaskAOutputIntBindingIndex].Type == OutputIntToIntCopyType
				&& TaskAOutputBindingsCopyInfo[TaskAOutputIntBindingIndex].CopySize == OutputIntToIntCopySize);
			AITEST_TRUE(TEXT("TaskA Copy Info should have been resolved correctly"), 
				TaskAOutputBindingsCopyInfo[TaskAOutputIntWrapperBindingIndex].Type == OutputIntWrapperToIntWrapperCopyType
				&& TaskAOutputBindingsCopyInfo[TaskAOutputIntWrapperBindingIndex].CopySize == OutputIntWrapperToIntWrapperCopySize);
			AITEST_TRUE(TEXT("TaskA Copy Info should have been resolved correctly"), 
				TaskAOutputBindingsCopyInfo[TaskAOutputIntWrapperArrayBindingIndex].Type == OutputIntWrapperArrayToIntWrapperArrayCopyType
				&& TaskAOutputBindingsCopyInfo[TaskAOutputIntWrapperArrayBindingIndex].CopySize == OutputIntWrapperArrayToIntWrapperArrayCopySize);

			constexpr int32 TaskBOutputBindingBatchIndex = 3;
			TConstArrayView<FPropertyBindingCopyInfo> TaskBOutputBindingsCopyInfo = StateTree.GetPropertyBindings().Super::GetBatchCopies(FStateTreeIndex16(TaskBOutputBindingBatchIndex));
			AITEST_TRUE(TEXT("TaskB should have 1 output bindings"), TaskBOutputBindingsCopyInfo.Num() == 1);

			constexpr int32 TaskBOutputBindingIndex = 0;

			AITEST_TRUE(TEXT("TaskB Copy Info should have been resolved correctly"),
				TaskBOutputBindingsCopyInfo[TaskBOutputBindingIndex].Type == OutputIntToIntCopyType
				&& TaskBOutputBindingsCopyInfo[TaskBOutputBindingIndex].CopySize == OutputIntToIntCopySize);
		}
		{
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);

			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

			EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;

			Status = Exec.Start();

			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Root"));

			constexpr int32 TaskAInstanceDataIndex = 0;
			constexpr int32 TaskBInstanceDataIndex = 2;
			const auto& TaskAInstanceData = InstanceData.GetStruct(TaskAInstanceDataIndex).Get<const FTestTask_OutputBindingsTaskInstanceData>();
			const auto& TaskBInstanceData = InstanceData.GetStruct(TaskBInstanceDataIndex).Get<const FTestTask_OutputBindingsTaskInstanceData>();

			Exec.LogClear();

			Status = Exec.Tick(1.0f);
			AITEST_EQUAL(TEXT("2nd Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Root"));

			AITEST_TRUE(TEXT("Task A pushed output values to parameters successfully."),
				TaskAInstanceData.InputIntA == 1
				&& TaskAInstanceData.InputIntB == 2
				&& TaskAInstanceData.InputIntWrapper.Value == 3
				&& TaskAInstanceData.InputIntWrapperArray == TArray<FIntWrapper>({{4}, { 5 }, { 6 }}) 
			);

			AITEST_TRUE(TEXT("Task B pushed output values to parameters successfully."), TaskBInstanceData.InputIntA == 11);

			Exec.LogClear();

			Exec.Stop();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_OutputBinding, "System.StateTree.Binding.OutputBinding");

struct FStateTreeTest_SubtreeParameter : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		// Root
		//   LinkedState -> SubtreeA
		// SubtreeA (EnterConditionA, TaskA)

		auto AddIntPropertyToPropertyBag = [](FInstancedPropertyBag& PropertyBag, FName VarName, int32 DefaultValue)
			{
				PropertyBag.AddProperty(VarName, EPropertyBagPropertyType::Int32);
				PropertyBag.SetValueInt32(VarName, DefaultValue);
				return PropertyBag.FindPropertyDescByName(VarName)->ID;
			};

		UStateTree& StateTree = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
			{
				auto& RootState = EditorData.AddSubTree(TEXT("Root"));

				FGuid SubtreeAIntID;
				auto& SubtreeA = EditorData.AddSubTree(TEXT("SubtreeA"));
				{
					SubtreeA.Type = EStateTreeStateType::Subtree;
					SubtreeAIntID = AddIntPropertyToPropertyBag(SubtreeA.Parameters.Parameters, TEXT("SubtreeAInt"), 10);
					TStateTreeEditorNode<FTestCondition_PrintValue>& ConditionANode = SubtreeA.AddEnterCondition<FTestCondition_PrintValue>(TEXT("ConditionA"));
					EditorData.AddPropertyBinding(
						FPropertyBindingPath(SubtreeA.Parameters.ID, TEXT("SubtreeAInt")),
						FPropertyBindingPath(ConditionANode.ID, TEXT("Value"))
					);
					TStateTreeEditorNode<FTestTask_PrintValue>& TaskANode = SubtreeA.AddTask<FTestTask_PrintValue>(TEXT("TaskA"));
					EditorData.AddPropertyBinding(
						FPropertyBindingPath(SubtreeA.Parameters.ID, TEXT("SubtreeAInt")),
						FPropertyBindingPath(TaskANode.ID, TEXT("Value"))
					);
				}

				auto& LinkedState = RootState.AddChildState(TEXT("LinkedState"), EStateTreeStateType::Linked);
				LinkedState.SetLinkedState(SubtreeA.GetLinkToState());
				constexpr bool bIsOverridden = true;
				LinkedState.SetParametersPropertyOverridden(SubtreeAIntID, bIsOverridden);
				LinkedState.Parameters.Parameters.SetValueInt32(TEXT("SubtreeAInt"), 30);
			}

			{
				FStateTreeCompilerLog Log;
				FStateTreeCompiler Compiler(Log);
				const bool bResult = Compiler.Compile(StateTree);
				AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);
			}

			{
				FStateTreeInstanceData InstanceData;
				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);

				const bool bInitSucceeded = Exec.IsValid();
				AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

				EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;

				Status = Exec.Start();

				AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Root", "LinkedState", "SubtreeA"));

				AITEST_TRUE(TEXT("Conditions should correctly update bindings from temporary data"),
					Exec.Expect("ConditionA", TEXT("TestCondition30")));

				AITEST_TRUE(TEXT("Tasks should correctly update bindings from instance data"),
					Exec.Expect("TaskA", TEXT("EnterState30")));

				Exec.LogClear();

				Exec.Stop();
			}
		}
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_SubtreeParameter, "System.StateTree.Binding.SubtreeParameter");

struct FStateTreeTest_GlobalInstanceData : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		// Tree1(Global Task)
		//	Root(EnterCondition, Task)
		//    LinkedStateA -> SubtreeA
		//  SubtreeA(ConditionA, TaskA)
		//    LinkedStateB -> SubtreeB
		//  SubtreeB(ConditionB, TaskB)

		UStateTree& StateTree = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& GlobalTask = EditorData.AddGlobalTask<FTestTask_PrintValue>(TEXT("GlobalTask"));
				GlobalTask.GetInstanceData().Value = 10;

				auto AddStateConditionAndTask = [&EditorData, &GlobalTask](UStateTreeState& InState, const FName InConditionName, const FName InTaskName)
				{
					TStateTreeEditorNode<FTestCondition_PrintValue>& ConditionNode = InState.AddEnterCondition<FTestCondition_PrintValue>(InConditionName);
					EditorData.AddPropertyBinding(
						FPropertyBindingPath(GlobalTask.ID, TEXT("Value")),
						FPropertyBindingPath(ConditionNode.ID, TEXT("Value"))
					);
					TStateTreeEditorNode<FTestTask_PrintValue>& TaskNode = InState.AddTask<FTestTask_PrintValue>(InTaskName);
					EditorData.AddPropertyBinding(
						FPropertyBindingPath(GlobalTask.ID, TEXT("Value")),
						FPropertyBindingPath(TaskNode.ID, TEXT("Value"))
					);
				};

				auto& RootState = EditorData.AddSubTree(TEXT("Root"));
				{
					AddStateConditionAndTask(RootState, TEXT("RootCondition"), TEXT("RootTask"));
				}
				auto& SubtreeA = EditorData.AddSubTree(TEXT("SubtreeA"));
				{
					SubtreeA.Type = EStateTreeStateType::Subtree;
					AddStateConditionAndTask(SubtreeA, TEXT("ConditionA"), TEXT("TaskA"));
				}
				auto& SubtreeB = EditorData.AddSubTree(TEXT("SubtreeB"));
				{
					SubtreeB.Type = EStateTreeStateType::Subtree;
					AddStateConditionAndTask(SubtreeB, TEXT("ConditionB"), TEXT("TaskB"));
				}

				auto& LinkedStateA = RootState.AddChildState(TEXT("LinkedStateA"), EStateTreeStateType::Linked);
				LinkedStateA.SetLinkedState(SubtreeA.GetLinkToState());

				auto& LinkedStateB = SubtreeA.AddChildState(TEXT("LinkedStateB"), EStateTreeStateType::Linked);
				LinkedStateB.SetLinkedState(SubtreeB.GetLinkToState());
			}

			{
				FStateTreeCompilerLog Log;
				FStateTreeCompiler Compiler(Log);
				const bool bResult = Compiler.Compile(StateTree);
				AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);
			}

			{
				FStateTreeInstanceData InstanceData;
				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);

				const bool bInitSucceeded = Exec.IsValid();
				AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

				EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;

				Status = Exec.Start();

				AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Root", "LinkedStateA", "SubtreeA", "LinkedStateB", "SubtreeB"));

				AITEST_TRUE(TEXT("Conditions should correctly update bindings from temporary data"),
					Exec.Expect("RootCondition", TEXT("TestCondition10"))
					.Then("ConditionA", TEXT("TestCondition10"))
					.Then("ConditionB", TEXT("TestCondition10")));

				AITEST_TRUE(TEXT("Tasks should correctly update bindings from instance data"),
					Exec.Expect("RootTask", TEXT("EnterState10"))
					.Then("TaskA", TEXT("EnterState10"))
					.Then("TaskB", TEXT("EnterState10")));

				Exec.LogClear();

				Exec.Stop();
			}
		}
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_GlobalInstanceData, "System.StateTree.Binding.GlobalInstanceData");

struct FStateTreeTest_GlobalParameter : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		// TreeA(Global Parameter: 10)
		//	RootA(ConditionA, TaskA)
		//    LinkedStateB -> SubtreeB
		//  SubtreeB(ConditionB, TaskB)
		//    LinkedAssetStateC -> TreeC
		// TreeC(Global Parameter: 20)
		//  RootC(ConditionC, TaskC)
		//   LinkedStateD -> SubtreeD
		//  SubtreeD(ConditionD, TaskD)

		auto AddIntPropertyToPropertyBag = [](FInstancedPropertyBag& PropertyBag, FName VarName, int32 DefaultValue)
			{
				PropertyBag.AddProperty(VarName, EPropertyBagPropertyType::Int32);
				PropertyBag.SetValueInt32(VarName, DefaultValue);
				return PropertyBag.FindPropertyDescByName(VarName)->ID;
			};

		auto AddStateConditionAndTask = [](UStateTreeEditorData& InEditorData, UStateTreeState& InState, const FName InConditionName, const FName InTaskName, const FName InRootParameterName)
			{
				TStateTreeEditorNode<FTestCondition_PrintValue>& ConditionNode = InState.AddEnterCondition<FTestCondition_PrintValue>(InConditionName);
				InEditorData.AddPropertyBinding(
					FPropertyBindingPath(InEditorData.GetRootParametersGuid(), InRootParameterName),
					FPropertyBindingPath(ConditionNode.ID, TEXT("Value"))
				);
				TStateTreeEditorNode<FTestTask_PrintValue>& TaskNode = InState.AddTask<FTestTask_PrintValue>(InTaskName);
				InEditorData.AddPropertyBinding(
					FPropertyBindingPath(InEditorData.GetRootParametersGuid(), InRootParameterName),
					FPropertyBindingPath(TaskNode.ID, TEXT("Value"))
				);
			};

		UStateTree& StateTreeC = NewStateTree();
		{
			UStateTreeEditorData& EditorDataC = *Cast<UStateTreeEditorData>(StateTreeC.EditorData);
			{
				AddIntPropertyToPropertyBag(const_cast<FInstancedPropertyBag&>(EditorDataC.GetRootParametersPropertyBag()), TEXT("TreeCInt32"), 20);

				auto& RootCState = EditorDataC.AddSubTree(TEXT("RootC"));
				{
					AddStateConditionAndTask(EditorDataC, RootCState, TEXT("ConditionC"), TEXT("TaskC"), TEXT("TreeCInt32"));
				}

				auto& SubtreeDState = EditorDataC.AddSubTree(TEXT("SubtreeD"));
				{
					SubtreeDState.Type = EStateTreeStateType::Subtree;
					AddStateConditionAndTask(EditorDataC, SubtreeDState, TEXT("ConditionD"), TEXT("TaskD"), TEXT("TreeCInt32"));
				}

				auto& LinkedStateD = RootCState.AddChildState(TEXT("LinkedStateD"), EStateTreeStateType::Linked);
				LinkedStateD.SetLinkedState(SubtreeDState.GetLinkToState());
			}
		}

		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTreeC);
			AITEST_TRUE(TEXT("StateTreeC should get compiled"), bResult);
		}

		UStateTree& StateTreeA = NewStateTree();
		{
			UStateTreeEditorData& EditorDataA = *Cast<UStateTreeEditorData>(StateTreeA.EditorData);
			{
				AddIntPropertyToPropertyBag(const_cast<FInstancedPropertyBag&>(EditorDataA.GetRootParametersPropertyBag()), TEXT("TreeAInt32"), 10);

				auto& RootAState = EditorDataA.AddSubTree(TEXT("RootA"));
				{
					AddStateConditionAndTask(EditorDataA, RootAState, TEXT("ConditionA"), TEXT("TaskA"), TEXT("TreeAInt32"));
				}

				auto& SubtreeB = EditorDataA.AddSubTree(TEXT("SubtreeB"));
				{
					SubtreeB.Type = EStateTreeStateType::Subtree;
					AddStateConditionAndTask(EditorDataA, SubtreeB, TEXT("ConditionB"), TEXT("TaskB"), TEXT("TreeAInt32"));
				}

				auto& LinkedAssetStateC = SubtreeB.AddChildState(TEXT("LinkedAssetStateC"), EStateTreeStateType::LinkedAsset);
				{
					LinkedAssetStateC.SetLinkedStateAsset(&StateTreeC);
				}

				auto& LinkedStateB = RootAState.AddChildState(TEXT("LinkedStateB"), EStateTreeStateType::Linked);
				{
					LinkedStateB.SetLinkedState(SubtreeB.GetLinkToState());
				}
			}
		}
		
		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTreeA);
			AITEST_TRUE(TEXT("StateTreeA should get compiled"), bResult);
		}

		{
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTreeA, StateTreeA, InstanceData);

			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

			EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;

			Status = Exec.Start();

			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("RootA", "LinkedStateB", "SubtreeB", "LinkedAssetStateC", "RootC", "LinkedStateD", "SubtreeD"));

			AITEST_TRUE(TEXT("Conditions should correctly update bindings from temporary data"),
				Exec.Expect("ConditionA", TEXT("TestCondition10"))
				.Then("ConditionB", TEXT("TestCondition10"))
				.Then("ConditionC", TEXT("TestCondition20"))
				.Then("ConditionD", TEXT("TestCondition20")));

			AITEST_TRUE(TEXT("Tasks should correctly update bindings from instance data"),
				Exec.Expect("TaskA", TEXT("EnterState10"))
				.Then("TaskB", TEXT("EnterState10"))
				.Then("TaskC", TEXT("EnterState20"))
				.Then("TaskD", TEXT("EnterState20")));

			Exec.LogClear();

			Exec.Stop();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_GlobalParameter, "System.StateTree.Binding.GlobalParameter");

} // namespace UE::StateTree::Tests

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
