// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeTest.h"
#include "StateTreeTestBase.h"
#include "StateTreeTestTypes.h"

#include "StateTreeAsyncExecutionContext.h"
#include "StateTreeCompilerLog.h"
#include "StateTreeEditorData.h"
#include "StateTreeExecutionTypes.h"
#include "StateTreeCompiler.h"
#include "Conditions/StateTreeCommonConditions.h"

#define LOCTEXT_NAMESPACE "AITestSuite_StateTreeTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::StateTree::Tests
{
struct FStateTreeTest_Delegate_ConcurrentListeners : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		UStateTreeState& Root = EditorData.AddSubTree(FName("Root"));

		TStateTreeEditorNode<FTestTask_BroadcastDelegate>& DispatcherTask = Root.AddTask<FTestTask_BroadcastDelegate>(FName("DispatcherTask"));
		TStateTreeEditorNode<FTestTask_ListenDelegate>& ListenerTaskA = Root.AddTask<FTestTask_ListenDelegate>(FName("ListenerTaskA"));
		TStateTreeEditorNode<FTestTask_ListenDelegate>& ListenerTaskB = Root.AddTask<FTestTask_ListenDelegate>(FName("ListenerTaskB"));

		EditorData.AddPropertyBinding(DispatcherTask, GET_MEMBER_NAME_STRING_CHECKED(FTestTask_BroadcastDelegate_InstanceData, OnTickDelegate), ListenerTaskB, GET_MEMBER_NAME_STRING_CHECKED(FTestTask_ListenDelegate_InstanceData, Listener));
		EditorData.AddPropertyBinding(DispatcherTask, GET_MEMBER_NAME_STRING_CHECKED(FTestTask_BroadcastDelegate_InstanceData, OnTickDelegate), ListenerTaskA, GET_MEMBER_NAME_STRING_CHECKED(FTestTask_ListenDelegate_InstanceData, Listener));

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE("StateTree should get compiled", bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE("StateTree should init", bInitSucceeded);

		Exec.Start();
		AITEST_FALSE(TEXT("StateTree ListenerTaskA should not trigger."), Exec.Expect(ListenerTaskA.GetName(), *FString::Printf(TEXT("OnDelegate%d"), 1)));
		AITEST_FALSE(TEXT("StateTree ListenerTaskB should not trigger."), Exec.Expect(ListenerTaskB.GetName(), *FString::Printf(TEXT("OnDelegate%d"), 1)));
		Exec.LogClear();

		Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree ListenerTaskA should be triggered once."), Exec.Expect(ListenerTaskA.GetName(), *FString::Printf(TEXT("OnDelegate%d"), 1)));
		AITEST_TRUE(TEXT("StateTree ListenerTaskB should be triggered once."), Exec.Expect(ListenerTaskB.GetName(), *FString::Printf(TEXT("OnDelegate%d"), 1)));

		Exec.LogClear();

		Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree ListenerTaskA should be triggered twice."), Exec.Expect(ListenerTaskA.GetName(), *FString::Printf(TEXT("OnDelegate%d"), 2)));
		AITEST_TRUE(TEXT("StateTree ListenerTaskB should be triggered twice."), Exec.Expect(ListenerTaskB.GetName(), *FString::Printf(TEXT("OnDelegate%d"), 2)));
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Delegate_ConcurrentListeners, "System.StateTree.Delegate.ConcurrentListeners");

struct FStateTreeTest_Delegate_MutuallyExclusiveListeners : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));

		UStateTreeState& StateA = Root.AddChildState("A");
		UStateTreeState& StateB = Root.AddChildState("B");

		StateA.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::GotoState, &StateB);
		StateB.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::GotoState, &StateA);

		TStateTreeEditorNode<FTestTask_BroadcastDelegate>& DispatcherTask = Root.AddTask<FTestTask_BroadcastDelegate>(FName(TEXT("DispatcherTask")));
		TStateTreeEditorNode<FTestTask_ListenDelegate>& ListenerTaskA0 = StateA.AddTask<FTestTask_ListenDelegate>(FName(TEXT("ListenerTaskA0")));
		TStateTreeEditorNode<FTestTask_ListenDelegate>& ListenerTaskA1 = StateA.AddTask<FTestTask_ListenDelegate>(FName(TEXT("ListenerTaskA1")));
		TStateTreeEditorNode<FTestTask_ListenDelegate>& ListenerTaskB = StateB.AddTask<FTestTask_ListenDelegate>(FName(TEXT("ListenerTaskB")));

		EditorData.AddPropertyBinding(DispatcherTask, GET_MEMBER_NAME_STRING_CHECKED(FTestTask_BroadcastDelegate_InstanceData, OnTickDelegate), ListenerTaskA0, GET_MEMBER_NAME_STRING_CHECKED(FTestTask_ListenDelegate_InstanceData, Listener));
		EditorData.AddPropertyBinding(DispatcherTask, GET_MEMBER_NAME_STRING_CHECKED(FTestTask_BroadcastDelegate_InstanceData, OnTickDelegate), ListenerTaskA1, GET_MEMBER_NAME_STRING_CHECKED(FTestTask_ListenDelegate_InstanceData, Listener));
		EditorData.AddPropertyBinding(DispatcherTask, GET_MEMBER_NAME_STRING_CHECKED(FTestTask_BroadcastDelegate_InstanceData, OnTickDelegate), ListenerTaskB, GET_MEMBER_NAME_STRING_CHECKED(FTestTask_ListenDelegate_InstanceData, Listener));

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE("StateTree should get compiled", bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE("StateTree should init", bInitSucceeded);

		Exec.Start();
		AITEST_TRUE(TEXT("StateTree Active States should be in Root/A"), Exec.ExpectInActiveStates(Root.Name, StateA.Name));
		Exec.LogClear();

		Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree ListenerTaskA0 should be triggered once"), Exec.Expect(ListenerTaskA0.GetName(), *FString::Printf(TEXT("OnDelegate%d"), 1)));
		AITEST_TRUE(TEXT("StateTree ListenerTaskA1 should be triggered once"), Exec.Expect(ListenerTaskA1.GetName(), *FString::Printf(TEXT("OnDelegate%d"), 1)));
		AITEST_TRUE(TEXT("StateTree ListenerTaskB shouldn't be triggered."), !Exec.Expect(ListenerTaskB.GetName(), *FString::Printf(TEXT("OnDelegate%d"), 1)));
		AITEST_TRUE("StateTree Active States should be in Root/B", Exec.ExpectInActiveStates(Root.Name, StateB.Name));
		Exec.LogClear();

		Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree ListenerTaskB should be triggered once"), Exec.Expect(ListenerTaskB.GetName(), *FString::Printf(TEXT("OnDelegate%d"), 1)));
		AITEST_TRUE(TEXT("StateTree ListenerTaskA0 shouldn't be triggered."), !Exec.Expect(ListenerTaskA0.GetName(), *FString::Printf(TEXT("OnDelegate%d"), 1)));
		AITEST_TRUE(TEXT("StateTree ListenerTaskA1 shouldn't be triggered."), !Exec.Expect(ListenerTaskA1.GetName(), *FString::Printf(TEXT("OnDelegate%d"), 1)));
		AITEST_TRUE(TEXT("StateTree Active States should be in Root/A"), Exec.ExpectInActiveStates(Root.Name, StateA.Name));
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Delegate_MutuallyExclusiveListeners, "System.StateTree.Delegate.MutuallyExclusiveListeners");

struct FStateTreeTest_Delegate_Transitions : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));

		UStateTreeState& StateA = Root.AddChildState("A");
		UStateTreeState& StateB = Root.AddChildState("B");

		FStateTreeTransition& TransitionAToB = StateA.AddTransition(EStateTreeTransitionTrigger::OnDelegate, EStateTreeTransitionType::GotoState, &StateB);
		FStateTreeTransition& TransitionBToA = StateB.AddTransition(EStateTreeTransitionTrigger::OnDelegate, EStateTreeTransitionType::GotoState, &StateA);

		TStateTreeEditorNode<FTestTask_BroadcastDelegate>& DispatcherTask0 = Root.AddTask<FTestTask_BroadcastDelegate>(FName(TEXT("DispatcherTask0")));
		TStateTreeEditorNode<FTestTask_BroadcastDelegate>& DispatcherTask1 = Root.AddTask<FTestTask_BroadcastDelegate>(FName(TEXT("DispatcherTask1")));
		
		EditorData.AddPropertyBinding(FPropertyBindingPath(DispatcherTask0.ID, GET_MEMBER_NAME_CHECKED(FTestTask_BroadcastDelegate_InstanceData, OnTickDelegate)),  FPropertyBindingPath(TransitionAToB.ID, GET_MEMBER_NAME_CHECKED(FStateTreeTransition, DelegateListener)));
		EditorData.AddPropertyBinding(FPropertyBindingPath(DispatcherTask1.ID, GET_MEMBER_NAME_CHECKED(FTestTask_BroadcastDelegate_InstanceData, OnTickDelegate)), FPropertyBindingPath(TransitionBToA.ID, GET_MEMBER_NAME_CHECKED(FStateTreeTransition, DelegateListener)));

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE("StateTree should get compiled", bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE("StateTree should init", bInitSucceeded);

		Exec.Start();
		AITEST_TRUE(TEXT("StateTree Active States should be in Root/A"), Exec.ExpectInActiveStates(Root.Name, StateA.Name));
		Exec.LogClear();

		Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Active States should be in Root/B"), Exec.ExpectInActiveStates(Root.Name, StateB.Name));
		Exec.LogClear();

		Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Active States should be in Root/A"), Exec.ExpectInActiveStates(Root.Name, StateA.Name));
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Delegate_Transitions, "System.StateTree.Delegate.Transitions");

struct FStateTreeTest_Delegate_BroadcastingInTransitionsTick : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));

		UStateTreeState& StateA = Root.AddChildState("A");
		UStateTreeState& StateB = Root.AddChildState("B");

		FStateTreeTransition& TransitionAToA = StateA.AddTransition(EStateTreeTransitionTrigger::OnDelegate, EStateTreeTransitionType::GotoState, &StateA);
		FStateTreeTransition& TransitionRootToB = Root.AddTransition(EStateTreeTransitionTrigger::OnDelegate, EStateTreeTransitionType::GotoState, &StateB);

		TStateTreeEditorNode<FTestTask_BroadcastDelegate>& DispatcherTask0 = Root.AddTask<FTestTask_BroadcastDelegate>(FName(TEXT("DispatcherTask0")));
		TStateTreeEditorNode<FTestTask_BroadcastDelegate>& DispatcherTask1 = StateA.AddTask<FTestTask_BroadcastDelegate>(FName(TEXT("DispatcherTask1")));

		EditorData.AddPropertyBinding(FPropertyBindingPath(DispatcherTask0.ID, GET_MEMBER_NAME_CHECKED(FTestTask_BroadcastDelegate_InstanceData, OnTickDelegate)), FPropertyBindingPath(TransitionAToA.ID, GET_MEMBER_NAME_CHECKED(FStateTreeTransition, DelegateListener)));
		EditorData.AddPropertyBinding(FPropertyBindingPath(DispatcherTask1.ID, GET_MEMBER_NAME_CHECKED(FTestTask_BroadcastDelegate_InstanceData, OnExitDelegate)), FPropertyBindingPath(TransitionRootToB.ID, GET_MEMBER_NAME_CHECKED(FStateTreeTransition, DelegateListener)));

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE("StateTree should get compiled", bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE("StateTree should init", bInitSucceeded);

		Exec.Start();
		AITEST_TRUE(TEXT("StateTree Active States should be in Root/A"), Exec.ExpectInActiveStates(Root.Name, StateA.Name));
		Exec.LogClear();

		Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Active States should be in Root/B"), Exec.ExpectInActiveStates(Root.Name, StateB.Name));
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Delegate_BroadcastingInTransitionsTick, "System.StateTree.Delegate.BroadcastingInTransitionsTick");

struct FStateTreeTest_Delegate_Rebroadcasting : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));

		TStateTreeEditorNode<FTestTask_BroadcastDelegate>& DispatcherTask = Root.AddTask<FTestTask_BroadcastDelegate>(FName(TEXT("DispatcherTask")));
		TStateTreeEditorNode<FTestTask_RebroadcastDelegate>& RedispatcherTask = Root.AddTask<FTestTask_RebroadcastDelegate>(FName(TEXT("RedispatcherTask")));
		TStateTreeEditorNode<FTestTask_ListenDelegate>& ListenerTask = Root.AddTask<FTestTask_ListenDelegate>(FName(TEXT("ListenerTask")));

		EditorData.AddPropertyBinding(DispatcherTask, GET_MEMBER_NAME_STRING_CHECKED(FTestTask_BroadcastDelegate_InstanceData, OnTickDelegate), RedispatcherTask, TEXT("Listener"));
		EditorData.AddPropertyBinding(RedispatcherTask, TEXT("Dispatcher"), ListenerTask, TEXT("Listener"));

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		Exec.Start();
		Exec.LogClear();

		Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree ListenerTask should be triggered once."), Exec.Expect(ListenerTask.GetName(), *FString::Printf(TEXT("OnDelegate%d"), 1)));
		Exec.LogClear();

		Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree ListenerTask should be triggered twice."), Exec.Expect(ListenerTask.GetName(), *FString::Printf(TEXT("OnDelegate%d"), 2)));
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Delegate_Rebroadcasting, "System.StateTree.Delegate.Rebroadcasting");

struct FStateTreeTest_Delegate_SelfRemoval : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));

		TStateTreeEditorNode<FTestTask_BroadcastDelegate>& DispatcherTask = Root.AddTask<FTestTask_BroadcastDelegate>(FName(TEXT("DispatcherTask")));
		TStateTreeEditorNode<FTestTask_CustomFuncOnDelegate>& CustomFuncTask = Root.AddTask<FTestTask_CustomFuncOnDelegate>(FName(TEXT("CustomFuncTask")));

		uint32 TriggersCounter = 0;

		EditorData.AddPropertyBinding(DispatcherTask, GET_MEMBER_NAME_STRING_CHECKED(FTestTask_BroadcastDelegate_InstanceData, OnTickDelegate), CustomFuncTask, GET_MEMBER_NAME_STRING_CHECKED(FTestTask_CustomFuncOnDelegate_InstanceData, Listener));
		CustomFuncTask.GetNode().CustomFunc = [&TriggersCounter](const FStateTreeWeakExecutionContext& WeakContext, FStateTreeDelegateListener Listener)
			{
				++TriggersCounter;
				WeakContext.UnbindDelegate(Listener);
			};

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		Exec.Start();
		Exec.LogClear();

		Exec.Tick(0.1f);
		AITEST_EQUAL(TEXT("StateTree Delegate should be triggered once"), TriggersCounter, 1);
		Exec.LogClear();

		Exec.Tick(0.1f);
		AITEST_EQUAL(TEXT("StateTree Delegate should be triggered once"), TriggersCounter, 1);
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Delegate_SelfRemoval, "System.StateTree.Delegate.SelfRemoval");

struct FStateTreeTest_Delegate_WithoutRemoval : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));

		UStateTreeState& StateA = Root.AddChildState("A");
		UStateTreeState& StateB = Root.AddChildState("B");

		FStateTreeTransition& TransitionAToB = StateA.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::GotoState, &StateB);

		TStateTreeEditorNode<FTestTask_BroadcastDelegate>& DispatcherTask = Root.AddTask<FTestTask_BroadcastDelegate>(FName(TEXT("DispatcherTask")));
		TStateTreeEditorNode<FTestTask_ListenDelegate>& ListenerTask = StateA.AddTask<FTestTask_ListenDelegate>(FName(TEXT("ListenerTask")));
		EditorData.AddPropertyBinding(DispatcherTask, GET_MEMBER_NAME_STRING_CHECKED(FTestTask_BroadcastDelegate_InstanceData, OnTickDelegate), ListenerTask, GET_MEMBER_NAME_STRING_CHECKED(FTestTask_ListenDelegate_InstanceData, Listener));
		ListenerTask.GetNode().bRemoveOnExit = false;

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		Exec.Start();
		Exec.LogClear();
		AITEST_TRUE(TEXT("StateTree Active States should be in Root/A"), Exec.ExpectInActiveStates(Root.Name, StateA.Name));

		Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Delegate should be triggered once."), Exec.Expect(ListenerTask.GetName(), *FString::Printf(TEXT("OnDelegate%d"), 1)));
		AITEST_TRUE(TEXT("StateTree Active States should be in Root/B"), Exec.ExpectInActiveStates(Root.Name, StateB.Name));
		Exec.LogClear();

		Exec.Tick(0.1f);
		AITEST_FALSE(TEXT("StateTree Delegate shouldn't be triggered again."), Exec.Expect(ListenerTask.GetName()));
		AITEST_TRUE(TEXT("StateTree Active States should be in Root/B"), Exec.ExpectInActiveStates(Root.Name, StateB.Name));
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Delegate_WithoutRemoval, "System.StateTree.Delegate.WithoutRemoval");

struct FStateTreeTest_Delegate_GlobalDispatcherAndListener : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		TStateTreeEditorNode<FTestTask_Stand>& RootTask = Root.AddTask<FTestTask_Stand>();
		RootTask.GetNode().TicksToCompletion = 100;

		TStateTreeEditorNode<FTestTask_BroadcastDelegate>& DispatcherTask = EditorData.AddGlobalTask<FTestTask_BroadcastDelegate>(FName(TEXT("DispatcherTask")));
		TStateTreeEditorNode<FTestTask_ListenDelegate>& ListenerTask = EditorData.AddGlobalTask<FTestTask_ListenDelegate>(FName(TEXT("ListenerTask")));
		ListenerTask.GetNode().bRemoveOnExit = false;

		EditorData.AddPropertyBinding(DispatcherTask, GET_MEMBER_NAME_STRING_CHECKED(FTestTask_BroadcastDelegate_InstanceData, OnTickDelegate), ListenerTask, GET_MEMBER_NAME_STRING_CHECKED(FTestTask_CustomFuncOnDelegate_InstanceData, Listener));

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		Exec.Start();
		Exec.LogClear();

		Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Delegate should be triggered once."), Exec.Expect(ListenerTask.GetName(), *FString::Printf(TEXT("OnDelegate%d"), 1)));
		Exec.LogClear();

		Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Delegate should be triggered twice."), Exec.Expect(ListenerTask.GetName(), *FString::Printf(TEXT("OnDelegate%d"), 2)));
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Delegate_GlobalDispatcherAndListener, "System.StateTree.Delegate.GlobalDispatcherAndListener");

struct FStateTreeTest_Delegate_ListeningToDelegateOnExit : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));

		TStateTreeEditorNode<FTestTask_BroadcastDelegate>& DispatcherTask = Root.AddTask<FTestTask_BroadcastDelegate>(FName(TEXT("DispatcherTask")));
		TStateTreeEditorNode<FTestTask_ListenDelegate>& ListenerTask = Root.AddTask<FTestTask_ListenDelegate>(FName(TEXT("ListenerTask")));
		ListenerTask.GetNode().bRemoveOnExit = false;

		EditorData.AddPropertyBinding(DispatcherTask, GET_MEMBER_NAME_STRING_CHECKED(FTestTask_BroadcastDelegate_InstanceData, OnExitDelegate), ListenerTask, GET_MEMBER_NAME_STRING_CHECKED(FTestTask_CustomFuncOnDelegate_InstanceData, Listener));

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		Exec.Start();
		AITEST_TRUE("Expected Root to be active.", Exec.ExpectInActiveStates(Root.Name));
		Exec.LogClear();

		Exec.Stop();
		AITEST_FALSE(TEXT("StateTree Delegate shouldn't be triggered"), Exec.Expect(ListenerTask.GetName()));
		Exec.LogClear();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Delegate_ListeningToDelegateOnExit, "System.StateTree.Delegate.ListeningToDelegateOnExit");

/** Test same state and the state index < number of instance data. */
struct FStateTreeTest_Delegate_ListeningToDelegateOnExit2 : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		UStateTreeState& StateB = Root.AddChildState("StateA")
			.AddChildState("StateB");
		UStateTreeState& StateC= Root.AddChildState("StateC");
		{
			EditorData.AddGlobalTask<FTestTask_Stand>().GetNode().TicksToCompletion = 100;
			EditorData.AddGlobalTask<FTestTask_Stand>().GetNode().TicksToCompletion = 100;
			EditorData.AddGlobalTask<FTestTask_Stand>().GetNode().TicksToCompletion = 100;
			EditorData.AddGlobalTask<FTestTask_Stand>().GetNode().TicksToCompletion = 100;
			EditorData.AddGlobalTask<FTestTask_Stand>().GetNode().TicksToCompletion = 100;
			EditorData.AddGlobalTask<FTestTask_Stand>().GetNode().TicksToCompletion = 100;
			EditorData.AddGlobalTask<FTestTask_Stand>().GetNode().TicksToCompletion = 100;
			EditorData.AddGlobalTask<FTestTask_Stand>().GetNode().TicksToCompletion = 100;
			EditorData.AddGlobalTask<FTestTask_Stand>().GetNode().TicksToCompletion = 100;
			EditorData.AddGlobalTask<FTestTask_Stand>().GetNode().TicksToCompletion = 100;
		}
		{
			TStateTreeEditorNode<FTestTask_BroadcastDelegate>& DispatcherTask = StateB.AddTask<FTestTask_BroadcastDelegate>(FName(TEXT("DispatcherTask")));
			TStateTreeEditorNode<FTestTask_ListenDelegate>& ListenerTask = StateB.AddTask<FTestTask_ListenDelegate>(FName(TEXT("ListenerTask")));
			ListenerTask.GetNode().bRemoveOnExit = false;

			EditorData.AddPropertyBinding(DispatcherTask, GET_MEMBER_NAME_STRING_CHECKED(FTestTask_BroadcastDelegate_InstanceData, OnExitDelegate), ListenerTask, GET_MEMBER_NAME_STRING_CHECKED(FTestTask_CustomFuncOnDelegate_InstanceData, Listener));

			FStateTreeTransition& Transition = StateB.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::GotoState, &StateC);
			Transition.bDelayTransition = true;
			Transition.DelayDuration = 0.1f;
		}
		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE("StateTree should get compiled", bResult);
		}

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE("StateTree should init", bInitSucceeded);

		Exec.Start();
		AITEST_TRUE("Expected Root to be active.", Exec.ExpectInActiveStates("Root", "StateA", "StateB"));
		Exec.LogClear();

		Exec.Tick(1.0f);
		AITEST_TRUE("Expected Root to be active.", Exec.ExpectInActiveStates("Root", "StateA", "StateB"));
		AITEST_FALSE(TEXT("StateTree Delegate shouldn't be triggered"), Exec.Expect("ListenerTask"));
		Exec.LogClear();

		Exec.Tick(1.0f);
		AITEST_TRUE("Expected Root to be active.", Exec.ExpectInActiveStates("Root", "StateC"));
		AITEST_FALSE(TEXT("StateTree Delegate shouldn't be triggered"), Exec.Expect("ListenerTask"));
		Exec.LogClear();

		Exec.Stop();
		AITEST_FALSE(TEXT("StateTree Delegate shouldn't be triggered"), Exec.Expect("ListenerTask"));
		Exec.LogClear();

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FStateTreeTest_Delegate_ListeningToDelegateOnExit2, "System.StateTree.Delegate.ListeningToDelegateOnExit2");

struct FStateTreeTest_Delegate_ListenerDispatcherOnNode : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		// Root(Dispatcher, ListenerOnNode1 -> DispatcherOnNode, ListenerOnNode2 -> DispatcherOnInstance, ListenerOnInstance -> DispatcherOnNode)

		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));

		bool bListenerOnNode1Broadcasted = false;
		bool bListenerOnNode2Broadcasted = false;
		bool bListenerOnInstanceBroadcasted = false;

		{
			TStateTreeEditorNode<FTestTask_DispatcherOnNodeAndInstance>& DispatcherTaskNode = Root.AddTask<FTestTask_DispatcherOnNodeAndInstance>(FName(TEXT("Dispatcher")));

			TStateTreeEditorNode<FTestTask_ListenerOnNode>& ListenerOnNode1TaskNode = Root.AddTask<FTestTask_ListenerOnNode>(FName(TEXT("ListenerOnNode1")));
			ListenerOnNode1TaskNode.GetNode().CustomFunc = [&bListenerOnNode1Broadcasted](const FStateTreeWeakExecutionContext& WeakExecContext)
			{
				bListenerOnNode1Broadcasted = true;
			};

			TStateTreeEditorNode<FTestTask_ListenerOnNode>& ListenerOnNode2TaskNode = Root.AddTask<FTestTask_ListenerOnNode>(FName(TEXT("ListenerOnNode2")));
			ListenerOnNode2TaskNode.GetNode().CustomFunc = [&bListenerOnNode2Broadcasted](const FStateTreeWeakExecutionContext& WeakExecContext)
			{
				bListenerOnNode2Broadcasted = true;
			};

			TStateTreeEditorNode<FTestTask_ListenerOnInstance>& ListenerOnInstanceTaskNode = Root.AddTask<FTestTask_ListenerOnInstance>(FName(TEXT("ListenerOnInstance")));
			ListenerOnInstanceTaskNode.GetNode().CustomFunc = [&bListenerOnInstanceBroadcasted](const FStateTreeWeakExecutionContext& WeakExecContext)
			{
				bListenerOnInstanceBroadcasted = true;
			};

			EditorData.AddPropertyBinding(
				FPropertyBindingPath(DispatcherTaskNode.GetNodeID(), GET_MEMBER_NAME_CHECKED(FTestTask_DispatcherOnNodeAndInstance, DispatcherOnNode)),
				FPropertyBindingPath(ListenerOnNode1TaskNode.GetNodeID(), GET_MEMBER_NAME_CHECKED(FTestTask_ListenerOnNode, ListenerOnNode)));

			EditorData.AddPropertyBinding(
				FPropertyBindingPath(DispatcherTaskNode.ID, GET_MEMBER_NAME_CHECKED(FTestTask_DispatcherOnNodeAndInstance_InstanceData, DispatcherOnInstance)),
				FPropertyBindingPath(ListenerOnNode2TaskNode.GetNodeID(), GET_MEMBER_NAME_CHECKED(FTestTask_ListenerOnNode, ListenerOnNode)));

			EditorData.AddPropertyBinding(
				FPropertyBindingPath(DispatcherTaskNode.GetNodeID(), GET_MEMBER_NAME_CHECKED(FTestTask_DispatcherOnNodeAndInstance, DispatcherOnNode)),
				FPropertyBindingPath(ListenerOnInstanceTaskNode.ID, GET_MEMBER_NAME_CHECKED(FTestTask_ListenerOnInstance_InstanceData, ListenerOnInstance)));
		}

		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE("StateTree should get compiled", bResult);

			const FCompactStateTreeState& RootState = StateTree.GetStates()[0];
			int32 TaskNodeIndex = RootState.TasksBegin;

			FConstStructView DispatcherView = StateTree.GetNode(TaskNodeIndex++);
			FConstStructView DispatcherInstanceDataView = StateTree.GetDefaultInstanceData().GetStruct(DispatcherView.Get<const FStateTreeTaskBase>().InstanceTemplateIndex.Get());

			FConstStructView ListenerOnNode1View = StateTree.GetNode(TaskNodeIndex++);

			FConstStructView ListenerOnNode2View = StateTree.GetNode(TaskNodeIndex++);

			FConstStructView ListenerOnInstanceView = StateTree.GetNode(TaskNodeIndex++);
			FConstStructView ListenerOnInstanceInstanceDataView = StateTree.GetDefaultInstanceData().GetStruct(ListenerOnInstanceView.Get<const FStateTreeTaskBase>().InstanceTemplateIndex.Get());

			const FStateTreeDelegateDispatcher DispatcherOnNode = DispatcherView.Get<const FTestTask_DispatcherOnNodeAndInstance>().DispatcherOnNode;
			const FStateTreeDelegateDispatcher DispatcherOnInstance = DispatcherInstanceDataView.Get<const FTestTask_DispatcherOnNodeAndInstance_InstanceData>().DispatcherOnInstance;
			const FStateTreeDelegateListener ListenerOnNode1 = ListenerOnNode1View.Get<const FTestTask_ListenerOnNode>().ListenerOnNode;
			const FStateTreeDelegateListener ListenerOnNode2 = ListenerOnNode2View.Get<const FTestTask_ListenerOnNode>().ListenerOnNode;
			const FStateTreeDelegateListener ListenerOnInstance = ListenerOnInstanceInstanceDataView.Get<const FTestTask_ListenerOnInstance_InstanceData>().ListenerOnInstance;

			AITEST_TRUE("Dispatcher on Node should init", DispatcherOnNode.IsValid());
			AITEST_TRUE("Dispatcher on Instance should init", DispatcherOnInstance.IsValid());
			AITEST_TRUE("Listener on Node should init", ListenerOnNode1.IsValid() && ListenerOnNode2.IsValid());
			AITEST_TRUE("Listener on Instance should init", ListenerOnInstance.IsValid());

			AITEST_EQUAL("ListenerOnNode1 should be bound to Dispatcher on Node", DispatcherOnNode, ListenerOnNode1.GetDispatcher());
			AITEST_EQUAL("ListenerOnNode2 should be bound to Dispatcher on Instance", DispatcherOnInstance, ListenerOnNode2.GetDispatcher());
			AITEST_EQUAL("ListenerOnInstance should be bound to Dispatcher on Node", DispatcherOnNode, ListenerOnInstance.GetDispatcher());
		}

		{
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE("StateTree should init", bInitSucceeded);

			Exec.Start();
			AITEST_TRUE("Expected Root to be active.", Exec.ExpectInActiveStates("Root"));
			Exec.LogClear();

			Exec.Tick(0.1f);
			AITEST_TRUE("Expected Root to be active.", Exec.ExpectInActiveStates("Root"));
			AITEST_TRUE("ListenerOnNode1 should be broadcasted", bListenerOnNode1Broadcasted);
			AITEST_TRUE("ListenerOnNode2 should be broadcasted", bListenerOnNode2Broadcasted);
			AITEST_TRUE("ListenerOnInstance should be broadcasted", bListenerOnInstanceBroadcasted);

			Exec.Stop();
		}


		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FStateTreeTest_Delegate_ListenerDispatcherOnNode, "System.StateTree.Delegate.ListenerDispatcherOnNode");

struct FStateTreeTest_Delegate_DispatcherOnNodeListenerOnTransition : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		// Root(Dispatcher)
		//     State1(Transition on Delegate -> DispatcherOnNode) -> State2
		//	   State2(Transition on Delegate -> DispatcherOnInstance) -> Tree Succeeded

		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		TStateTreeEditorNode<FTestTask_DispatcherOnNodeAndInstance>& DispatcherTaskNode = Root.AddTask<FTestTask_DispatcherOnNodeAndInstance>(TEXT("Dispatcher"));
		UStateTreeState& State1 = Root.AddChildState(FName(TEXT("State1")));
		FStateTreeTransition& DelegateOnNodeTransition = State1.AddTransition(EStateTreeTransitionTrigger::OnDelegate, EStateTreeTransitionType::NextState);

		UStateTreeState& State2 = Root.AddChildState(FName(TEXT("State2")));
		FStateTreeTransition& DelegateOnInstanceTransition = State2.AddTransition(EStateTreeTransitionTrigger::OnDelegate, EStateTreeTransitionType::Succeeded);

		EditorData.AddPropertyBinding(
			FPropertyBindingPath(DispatcherTaskNode.GetNodeID(), GET_MEMBER_NAME_CHECKED(FTestTask_DispatcherOnNodeAndInstance, DispatcherOnNode)),
			FPropertyBindingPath(DelegateOnNodeTransition.ID, GET_MEMBER_NAME_CHECKED(FStateTreeTransition, DelegateListener)));

		EditorData.AddPropertyBinding(
			FPropertyBindingPath(DispatcherTaskNode.GetInstanceDataID(), GET_MEMBER_NAME_CHECKED(FTestTask_DispatcherOnNodeAndInstance_InstanceData, DispatcherOnInstance)),
			FPropertyBindingPath(DelegateOnInstanceTransition.ID, GET_MEMBER_NAME_CHECKED(FStateTreeTransition, DelegateListener)));

		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE("StateTree should get compiled", bResult);

			const FCompactStateTreeState& RootState = StateTree.GetStates()[0];
			const FCompactStateTreeState& State1State = StateTree.GetStates()[1];
			const FCompactStateTreeState& State2State = StateTree.GetStates()[2];

			const FTestTask_DispatcherOnNodeAndInstance& DispatcherTask = StateTree.GetNode(RootState.TasksBegin).Get<const FTestTask_DispatcherOnNodeAndInstance>();
			const FStateTreeDelegateDispatcher DispatcherOnNode = DispatcherTask.DispatcherOnNode;
			const FStateTreeDelegateDispatcher DispatcherOnInstance = StateTree.GetDefaultInstanceData().GetStruct(DispatcherTask.InstanceTemplateIndex.Get()).Get<const FTestTask_DispatcherOnNodeAndInstance_InstanceData>().DispatcherOnInstance;

			AITEST_TRUE("Dispatcher on node should be valid", DispatcherOnNode.IsValid());
			AITEST_TRUE("Dispatcher on instance should be valid", DispatcherOnInstance.IsValid());

			const FCompactStateTransition* CompactDelegateOnNodeTransition = StateTree.GetTransitionFromIndex(FStateTreeIndex16(State1State.TransitionsBegin));
			const FCompactStateTransition* CompactDelegateOnInstanceTransition = StateTree.GetTransitionFromIndex(FStateTreeIndex16(State2State.TransitionsBegin));

			AITEST_EQUAL("State1 Transition Delegate Listener should be bound to Dispatcher on Node", CompactDelegateOnNodeTransition->RequiredDelegateDispatcher, DispatcherOnNode);
			AITEST_EQUAL("State2 Transition Delegate Listener should be bound to Dispatcher on Instance", CompactDelegateOnInstanceTransition->RequiredDelegateDispatcher, DispatcherOnInstance);
		}

		{
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE("StateTree should init", bInitSucceeded);

			Exec.Start();
			AITEST_TRUE("Expected [Root, State1] to be active.", Exec.ExpectInActiveStates("Root", "State1"));
			Exec.LogClear();

			Exec.Tick(0.1f);
			AITEST_TRUE("Expected [Root, State2] to be active.", Exec.ExpectInActiveStates("Root", "State2"));
			Exec.LogClear();

			Exec.Tick(0.1f);
			AITEST_TRUE("Expected Tree to be succeeded.", Exec.GetStateTreeRunStatus() == EStateTreeRunStatus::Succeeded);
			Exec.LogClear();

			Exec.Stop();
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FStateTreeTest_Delegate_DispatcherOnNodeListenerOnTransition, "System.StateTree.Delegate.DispatcherOnNodeListenerOnTransition");


struct FStateTreeTest_Delegate_CompilerValidation_InvalidSourceType : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		UStateTreeState& Root = EditorData.AddSubTree(FName("Root"));

		TStateTreeEditorNode<FTestTask_BroadcastDelegate>& SourceTask = Root.AddTask<FTestTask_BroadcastDelegate>(FName("Source"));
		TStateTreeEditorNode<FTestTask_IntegersOutput>& TargetTask = Root.AddTask<FTestTask_IntegersOutput>(FName("Target"));

		EditorData.AddPropertyBinding(SourceTask, GET_MEMBER_NAME_STRING_CHECKED(FTestTask_BroadcastDelegate_InstanceData, OnTickDelegate), TargetTask, GET_MEMBER_NAME_STRING_CHECKED(FTestTask_IntegersOutput_InstanceData, IntA));

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);

		AITEST_FALSE("Compilation should fail for invalid binding type", bResult);
		AITEST_TRUE("Compiler log should contain errors", Log.ToTokenizedMessages().Num() > 0);

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Delegate_CompilerValidation_InvalidSourceType, "System.StateTree.Delegate.CompilerValidation.InvalidSourceType");

struct FStateTreeTest_Delegate_SelfReferential : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		UStateTreeState& Root = EditorData.AddSubTree(FName("Root"));

		TStateTreeEditorNode<FTestTask_BroadcastDelegate>& Source = Root.AddTask<FTestTask_BroadcastDelegate>(FName("Source"));
		TStateTreeEditorNode<FTestTask_RebroadcastDelegate>& Relay = Root.AddTask<FTestTask_RebroadcastDelegate>(FName("Relay"));
		TStateTreeEditorNode<FTestTask_ListenDelegate>& Listener = Root.AddTask<FTestTask_ListenDelegate>(FName("Listener"));

		EditorData.AddPropertyBinding(Source, GET_MEMBER_NAME_STRING_CHECKED(FTestTask_BroadcastDelegate_InstanceData, OnTickDelegate), Relay, GET_MEMBER_NAME_STRING_CHECKED(FTestTask_RebroadcastDelegate_InstanceData, Listener));
		EditorData.AddPropertyBinding(Relay, TEXT("Dispatcher"), Listener, GET_MEMBER_NAME_STRING_CHECKED(FTestTask_ListenDelegate_InstanceData, Listener));

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE("Self-referential relay should compile", bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE("Should initialize", Exec.IsValid());

		Exec.Start();
		Exec.Tick(0.1f);
		AITEST_TRUE("Listener should be triggered through relay", Exec.Expect(Listener.GetName()));
		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Delegate_SelfReferential, "System.StateTree.Delegate.SelfReferential");

struct FStateTreeTest_Delegate_CircularBinding : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		UStateTreeState& Root = EditorData.AddSubTree(FName("Root"));

		TStateTreeEditorNode<FTestTask_BroadcastDelegate>& TaskA = Root.AddTask<FTestTask_BroadcastDelegate>(FName("TaskA"));
		TStateTreeEditorNode<FTestTask_RebroadcastDelegate>& TaskB = Root.AddTask<FTestTask_RebroadcastDelegate>(FName("TaskB"));
		TStateTreeEditorNode<FTestTask_RebroadcastDelegate>& TaskC = Root.AddTask<FTestTask_RebroadcastDelegate>(FName("TaskC"));

		EditorData.AddPropertyBinding(TaskA, TEXT("OnTickDelegate"), TaskB, TEXT("Listener"));
		EditorData.AddPropertyBinding(TaskB, TEXT("Dispatcher"), TaskC, TEXT("Listener"));

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE("Circular binding should compile", bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE("Should execute without infinite loop", Exec.IsValid());

		Exec.Start();
		Exec.Tick(0.1f);
		Exec.Tick(0.1f);
		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Delegate_CircularBinding, "System.StateTree.Delegate.CircularBinding");

struct FStateTreeTest_Delegate_DanglingListener : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		UStateTreeState& Root = EditorData.AddSubTree(FName("Root"));

		TStateTreeEditorNode<FTestTask_ListenDelegate>& ListenerTask = Root.AddTask<FTestTask_ListenDelegate>(FName("Listener"));

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE("Dangling listener should compile (not an error)", bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE("Should initialize", Exec.IsValid());

		Exec.Start();
		Exec.Tick(0.1f);
		AITEST_FALSE("Dangling listener should not trigger", Exec.Expect(ListenerTask.GetName()));
		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Delegate_DanglingListener, "System.StateTree.Delegate.DanglingListener");

struct FStateTreeTest_Delegate_ManyToMany : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		UStateTreeState& Root = EditorData.AddSubTree(FName("Root"));

		constexpr int32 NumDispatcher = 5;
		TArray<FGuid> DispatcherIDs;
		DispatcherIDs.Reserve(NumDispatcher);

		constexpr int32 NumListener = 5;
		TArray<FGuid> ListenerIDs;
		ListenerIDs.Reserve(NumListener);
		for (int32 Index = 0; Index < NumDispatcher; ++Index)
		{
			DispatcherIDs.Add(Root.AddTask<FTestTask_BroadcastDelegate>(*FString::Printf(TEXT("Dispatcher%d"), Index)).ID);
		}

		for (int32 Index = 0; Index < NumListener; ++Index)
		{
			ListenerIDs.Add(Root.AddTask<FTestTask_ListenDelegate>(*FString::Printf(TEXT("Listener%d"), Index)).ID);
		}

		// There are NumDispatcher*NumListener bindings but Listener can only bind to one dispatcher. It should compile to NumListener bindings.
		for (const FGuid& DispatcherID : DispatcherIDs)
		{
			for (const FGuid& ListenerID : ListenerIDs)
			{
				FPropertyBindingPath SourcePath;
				FPropertyBindingPath TargetPath;
				SourcePath.SetStructID(DispatcherID);
				TargetPath.SetStructID(ListenerID);
				SourcePath.FromString(GET_MEMBER_NAME_STRING_CHECKED(FTestTask_BroadcastDelegate_InstanceData, OnTickDelegate));
				TargetPath.FromString(GET_MEMBER_NAME_STRING_CHECKED(FTestTask_ListenDelegate_InstanceData, Listener));
				EditorData.EditorBindings.AddBinding(SourcePath, TargetPath);
			}
		}

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE("Many-to-many binding should compile", bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE("Should initialize", Exec.IsValid());

		Exec.Start();
		Exec.LogClear();

		Exec.Tick(0.1f);

		for (int32 Index = 0; Index < NumListener; ++Index)
		{
			AITEST_TRUE(*FString::Printf(TEXT("Listener%d should be triggered"), Index), Exec.Expect(*FString::Printf(TEXT("Listener%d"), Index), TEXT("OnDelegate1")));
		}
		Exec.LogClear();

		Exec.Stop();
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Delegate_ManyToMany, "System.StateTree.Delegate.ManyToMany");

struct FStateTreeTest_Delegate_StressTest_ManyListeners : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		UStateTreeState& Root = EditorData.AddSubTree(FName("Root"));

		FGuid DispatcherID;
		{
			// Note AddTask increase the array size making the ref invalid in the following loop.
			TStateTreeEditorNode<FTestTask_BroadcastDelegate>& Dispatcher = Root.AddTask<FTestTask_BroadcastDelegate>(FName("Dispatcher"));
			DispatcherID = Dispatcher.ID;
		}

		constexpr int32 NumListeners = 31;
		for (int32 Index = 0; Index < NumListeners; ++Index)
		{
			const FName TaskName = *FString::Printf(TEXT("Listener%d"), Index);
			TStateTreeEditorNode<FTestTask_ListenDelegate>& Listener = Root.AddTask<FTestTask_ListenDelegate>(TaskName);

			FPropertyBindingPath SourcePath;
			FPropertyBindingPath TargetPath;
			SourcePath.SetStructID(DispatcherID);
			SourcePath.FromString(GET_MEMBER_NAME_STRING_CHECKED(FTestTask_BroadcastDelegate_InstanceData, OnTickDelegate));
			TargetPath.SetStructID(Listener.ID);
			TargetPath.FromString(GET_MEMBER_NAME_STRING_CHECKED(FTestTask_ListenDelegate_InstanceData, Listener));
			EditorData.AddPropertyBinding(SourcePath, TargetPath);
		}

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE("Stress test: compilation with 31 listeners should succeed", bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE("Should initialize with 31 listeners", Exec.IsValid());

		Exec.Start();
		Exec.Tick(0.1f);

		AITEST_TRUE("First listener should be triggered", Exec.Expect(FName("Listener0")));
		AITEST_TRUE("Last listener should be triggered", Exec.Expect(*FString::Printf(TEXT("Listener%d"), NumListeners - 1)));

		Exec.Stop();
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Delegate_StressTest_ManyListeners, "System.StateTree.Delegate.StressTest.ManyListeners");

struct FStateTreeTest_TaskCompletionDispatcher_Compilation : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		UStateTreeState& Root = EditorData.AddSubTree(FName("Root"));

		// Add two tasks: one broadcasts on completion, one listens for completion
		TStateTreeEditorNode<FTestTask_BroadcastDelegate>& ProducerTask = Root.AddTask<FTestTask_BroadcastDelegate>(FName("ProducerTask"));
		TStateTreeEditorNode<FTestTask_ListenDelegate>& ConsumerTask = Root.AddTask<FTestTask_ListenDelegate>(FName("ConsumerTask"));

		// Compile the state tree
		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE("StateTree should compile successfully", bResult);

			// Verify TaskCompletionDispatchers array was initially empty
			AITEST_TRUE("TaskCompletionDispatchers should initially be empty", StateTree.GetTaskCompletionDispatchers().Num() == 0);
		}

		// Add task completion binding from ProducerTask to ConsumerTask.Listener
		EditorData.EditorBindings.AddTaskCompletionBinding(ProducerTask.GetNodeID(), FPropertyBindingPath(ConsumerTask.GetNodeID(), GET_MEMBER_NAME_CHECKED(FTestTask_ListenDelegate_InstanceData, Listener)), UE::StateTree::ETaskCompletionCondition::Completes);

		// Compile again with the task completion binding
		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE("StateTree should compile successfully with task completion binding", bResult);

			// Test that there is exactly 1 task completion dispatcher
			AITEST_TRUE("TaskCompletionDispatchers should have exactly 1 dispatcher", StateTree.GetTaskCompletionDispatchers().Num() == 1);
		}

		// Add a delegate transition
		FStateTreeTransition& Transition = Root.AddTransition(EStateTreeTransitionTrigger::OnDelegate, EStateTreeTransitionType::GotoState, &Root);

		// Add task completion binding from ProducerTask to the transition's DelegateListener with Succeeds condition
		// Different condition means a different dispatcher entry
		EditorData.EditorBindings.AddTaskCompletionBinding(ProducerTask.GetNodeID(), FPropertyBindingPath(Transition.ID, GET_MEMBER_NAME_CHECKED(FStateTreeTransition, DelegateListener)), UE::StateTree::ETaskCompletionCondition::Succeeds);

		// Compile again with the additional transition delegate binding (different condition)
		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE("StateTree should compile successfully with transition delegate binding", bResult);

			// Test that there are now 2 task completion dispatchers (one for Completes, one for Succeeds)
			AITEST_TRUE("TaskCompletionDispatchers should have exactly 2 dispatchers (different conditions)", StateTree.GetTaskCompletionDispatchers().Num() == 2);
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_TaskCompletionDispatcher_Compilation, "System.StateTree.TaskCompletionDispatcher.Compilation");

struct FStateTreeTest_TaskCompletionDispatcher_DeterministicGUID : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		// Test that the same task + condition produces the same GUID across recompiles
		UStateTree& StateTree = NewStateTree();
		TArray<FStateTreeDelegateDispatcher> Dispatchers;
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
			UStateTreeState& Root = EditorData.AddSubTree(FName("Root"));
			UStateTreeState& State = Root.AddChildState(FName("State1"));
			TStateTreeEditorNode<FTestTask_BroadcastDelegate>& ProducerTask = State.AddTask<FTestTask_BroadcastDelegate>(FName("ProducerTask"));
			TStateTreeEditorNode<FTestTask_ListenDelegate>& ConsumerTask = State.AddTask<FTestTask_ListenDelegate>(FName("ConsumerTask"));

			// Add task completion binding from ProducerTask to ConsumerTask.Listener
			EditorData.EditorBindings.AddTaskCompletionBinding(ProducerTask.GetNodeID(), FPropertyBindingPath(ConsumerTask.GetNodeID(), GET_MEMBER_NAME_CHECKED(FTestTask_ListenDelegate_InstanceData, Listener)), UE::StateTree::ETaskCompletionCondition::Completes);

			// Add a delegate transition
			FStateTreeTransition& DelegateTransition = State.AddTransition(EStateTreeTransitionTrigger::OnDelegate, EStateTreeTransitionType::GotoState, &Root);

			// Add task completion binding from ProducerTask to the transition's DelegateListener
			EditorData.EditorBindings.AddTaskCompletionBinding(ProducerTask.GetNodeID(), FPropertyBindingPath(DelegateTransition.ID, GET_MEMBER_NAME_CHECKED(FStateTreeTransition, DelegateListener)), UE::StateTree::ETaskCompletionCondition::Completes);
		}

		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			Compiler.Compile(StateTree);

			// Fill DispatcherIDs from the first compile
			for (const UE::StateTree::FTaskCompletionDispatcher& TaskCompletionDispatcher : StateTree.GetTaskCompletionDispatchers())
			{
				Dispatchers.Add(TaskCompletionDispatcher.Dispatcher);
			}
		}
		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			Compiler.Compile(StateTree);

			// Test that DispatcherIDs match across recompiles (deterministic GUIDs)
			const TConstArrayView<UE::StateTree::FTaskCompletionDispatcher> RecompiledDispatchers = StateTree.GetTaskCompletionDispatchers();
			AITEST_TRUE("Dispatcher count should match", Dispatchers.Num() == RecompiledDispatchers.Num());
			for (int32 Index = 0; Index < Dispatchers.Num() && Index < RecompiledDispatchers.Num(); ++Index)
			{
				AITEST_TRUE(FString::Printf(TEXT("Dispatcher %d GUID should be deterministic"), Index), Dispatchers[Index] == RecompiledDispatchers[Index].Dispatcher);
			}
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_TaskCompletionDispatcher_DeterministicGUID, "System.StateTree.TaskCompletionDispatcher.DeterministicGUID");

struct FStateTreeTest_TaskCompletionDispatcher_MultipleListeners : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		UStateTreeState& Root = EditorData.AddSubTree(FName("Root"));

		// Add producer and multiple consumers
		TStateTreeEditorNode<FTestTask_BroadcastDelegate>& ProducerTask = Root.AddTask<FTestTask_BroadcastDelegate>(FName("ProducerTask"));
		TStateTreeEditorNode<FTestTask_ListenDelegate>& ConsumerTask1 = Root.AddTask<FTestTask_ListenDelegate>(FName("ConsumerTask1"));
		TStateTreeEditorNode<FTestTask_ListenDelegate>& ConsumerTask2 = Root.AddTask<FTestTask_ListenDelegate>(FName("ConsumerTask2"));
		TStateTreeEditorNode<FTestTask_ListenDelegate>& ConsumerTask3 = Root.AddTask<FTestTask_ListenDelegate>(FName("ConsumerTask3"));

		// Add task completion bindings from ProducerTask to consumers with same condition
		// This tests that multiple binding targets to the same task share the same dispatcher
		EditorData.EditorBindings.AddTaskCompletionBinding(ProducerTask.GetNodeID(), FPropertyBindingPath(ConsumerTask1.ID, GET_MEMBER_NAME_CHECKED(FTestTask_ListenDelegate_InstanceData, Listener)), UE::StateTree::ETaskCompletionCondition::Completes);
		EditorData.EditorBindings.AddTaskCompletionBinding(ProducerTask.GetNodeID(), FPropertyBindingPath(ConsumerTask2.ID, GET_MEMBER_NAME_CHECKED(FTestTask_ListenDelegate_InstanceData, Listener)), UE::StateTree::ETaskCompletionCondition::Completes);

		// Add binding with different condition
		EditorData.EditorBindings.AddTaskCompletionBinding(ProducerTask.GetNodeID(), FPropertyBindingPath(ConsumerTask3.ID, GET_MEMBER_NAME_CHECKED(FTestTask_ListenDelegate_InstanceData, Listener)), UE::StateTree::ETaskCompletionCondition::Fails);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE("StateTree should compile with multiple listeners", bResult);

		// Should have exactly 2 TaskCompletionDispatchers (one for Completes, one for Fails)
		AITEST_TRUE("TaskCompletionDispatchers should have exactly 2 dispatchers (different conditions)", StateTree.GetTaskCompletionDispatchers().Num() == 2);
		
		// Find the compiled FTestTask_ListenDelegate. There should be 3 of them, one for each consumer task.
		TArray<FStateTreeDelegateDispatcher, TInlineAllocator<3>> CompiledDispatchers;
		TArray<FStateTreeDelegateListener, TInlineAllocator<3>> CompiledListeners;
		for (const FConstStructView Node : StateTree.GetNodes())
		{
			if (const FTestTask_ListenDelegate* ListenDelegate = Node.GetPtr<const FTestTask_ListenDelegate>())
			{
				if (ListenDelegate->InstanceTemplateIndex.IsValid())
				{
					const FConstStructView InstanceDataView = StateTree.GetDefaultInstanceData().GetStruct(ListenDelegate->InstanceTemplateIndex.Get());
					if (const FTestTask_ListenDelegate_InstanceData* InstanceData = InstanceDataView.GetPtr<const FTestTask_ListenDelegate_InstanceData>())
					{
						CompiledListeners.Add(InstanceData->Listener);
						CompiledDispatchers.AddUnique(InstanceData->Listener.GetDispatcher());
					}
				}
			}
		}

		for (const FCompactStateTreeState& State : StateTree.GetStates())
		{
			const int32 TransitionEnd = State.TransitionsBegin + State.TransitionsNum;
			for (int32 TransitionIndex = State.TransitionsBegin; TransitionIndex < TransitionEnd; ++TransitionIndex)
			{
				if (const FCompactStateTransition* Transition = StateTree.GetTransitionFromIndex(FStateTreeIndex16(TransitionIndex)))
				{
					if (Transition->Trigger == EStateTreeTransitionTrigger::OnDelegate
						&& Transition->RequiredDelegateDispatcher.IsValid())
					{
						CompiledDispatchers.AddUnique(Transition->RequiredDelegateDispatcher);
					}
				}
			}
		}

		AITEST_EQUAL(TEXT("Incorrect number of compiled FStateTreeDelegateDispatcher."), CompiledDispatchers.Num(), 2);
		AITEST_EQUAL(TEXT("Incorrect number of compiled FStateTreeDelegateListener."), CompiledListeners.Num(), 3);

		for (const UE::StateTree::FTaskCompletionDispatcher& CompletionDispatcher : StateTree.GetTaskCompletionDispatchers())
		{
			CompiledDispatchers.RemoveSingleSwap(CompletionDispatcher.Dispatcher);
		}
		AITEST_EQUAL(TEXT("GetTaskCompletionDispatchers do not matches with the discovered FStateTreeDelegateDispatcher."), CompiledDispatchers.Num(), 0);

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_TaskCompletionDispatcher_MultipleListeners, "System.StateTree.TaskCompletionDispatcher.MultipleListeners");

/**
 * Test that listener bound to a producer task's completion dispatcher is correctly triggered when the producer task completes during execution.
 * and that the listener's completion callback is executed properly.
 */ 
struct FStateTreeTest_TaskCompletionDispatcher_ExecutionOnCompletion : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		// Integration test: Verify task completion dispatcher broadcasts listener when task completes
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		UStateTreeState& Root = EditorData.AddSubTree("Root");
		UStateTreeState& State1 = Root.AddChildState("State1");
		UStateTreeState& State2 = Root.AddChildState("State2");
		UStateTreeState& State3 = Root.AddChildState("State3");

		bool bState1ConsumerTriggered = false;

		// Add tasks
		TStateTreeEditorNode<FTestTask_Stand>& State1ProducerTask = State1.AddTask<FTestTask_Stand>("State1Producer");
		TStateTreeEditorNode<FTestTask_CustomFuncOnDelegate>& State1ConsumerTask = State1.AddTask<FTestTask_CustomFuncOnDelegate>("State1Consumer");
		State1ConsumerTask.GetNode().CustomFunc = [&bState1ConsumerTriggered](const FStateTreeWeakExecutionContext& WeakContext, FStateTreeDelegateListener Listener)
			{
				bState1ConsumerTriggered = true;
				WeakContext.FinishTask(EStateTreeFinishTaskType::Succeeded);
			};
		TStateTreeEditorNode<FTestTask_Stand>& State2ProducerTask = State2.AddTask<FTestTask_Stand>("State2Producer");
		TStateTreeEditorNode<FTestTask_PrintValue>& State3Task1 = State3.AddTask<FTestTask_PrintValue>("State3Task1");

		// Bind consumer listener to producer task's completion event via task completion dispatcher
		FStateTreeTransition& State1Transition = State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &State2);
		EditorData.EditorBindings.AddTaskCompletionBinding(State1ProducerTask.GetNodeID(), FPropertyBindingPath(State1ConsumerTask.ID, GET_MEMBER_NAME_CHECKED(FTestTask_ListenDelegate_InstanceData, Listener)), UE::StateTree::ETaskCompletionCondition::Completes);

		FStateTreeTransition& State2Transition = State2.AddTransition(EStateTreeTransitionTrigger::OnDelegate, EStateTreeTransitionType::GotoState, &State3);
		EditorData.EditorBindings.AddTaskCompletionBinding(State2ProducerTask.GetNodeID(), FPropertyBindingPath(State2Transition.ID, GET_MEMBER_NAME_CHECKED(FStateTreeTransition, DelegateListener)), UE::StateTree::ETaskCompletionCondition::Completes);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE("StateTree should compile successfully", bResult);

		// Verify task completion dispatcher was created
		AITEST_TRUE("Task completion dispatcher should be created", StateTree.GetTaskCompletionDispatchers().Num() == 2);

		// Execute the state tree
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE("StateTree should initialize", bInitSucceeded);

		// Start execution - should be in Root and State1
		Exec.Start();
		AITEST_TRUE("Expected [Root, State1] to be active", Exec.ExpectInActiveStates("Root", "State1"));
		AITEST_FALSE("bState1ConsumerTriggered should be false before tick", bState1ConsumerTriggered);
		Exec.LogClear();

		// Tick once - State1 producer completes, triggers consumer, should transition to State2
		Exec.Tick(0.1f);
		AITEST_TRUE("Expected [Root, State2] to be active", Exec.ExpectInActiveStates("Root", "State2"));
		AITEST_TRUE("bState1ConsumerTriggered should be true after tick", bState1ConsumerTriggered);
		bState1ConsumerTriggered = false;
		Exec.LogClear();

		// Tick again - should transition to State3
		Exec.Tick(0.1f);
		AITEST_TRUE("Expected [Root, State3] to be active", Exec.ExpectInActiveStates("Root", "State3"));
		AITEST_FALSE("bState1ConsumerTriggered should be false", bState1ConsumerTriggered);
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_TaskCompletionDispatcher_ExecutionOnCompletion, "System.StateTree.TaskCompletionDispatcher.ExecutionOnCompletion");

struct FStateTreeTest_TaskCompletionDispatcher_SucceedsCondition : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		UStateTreeState& Root = EditorData.AddSubTree(FName("Root"));

		TStateTreeEditorNode<FTestTask_Stand>& ProducerTask = Root.AddTask<FTestTask_Stand>(FName("ProducerTask"));
		ProducerTask.GetNode().TickCompletionResult = EStateTreeRunStatus::Succeeded;

		TStateTreeEditorNode<FTestTask_ListenDelegate>& ConsumerSucceeds = Root.AddTask<FTestTask_ListenDelegate>(FName("ConsumerSucceeds"));
		TStateTreeEditorNode<FTestTask_ListenDelegate>& ConsumerFails = Root.AddTask<FTestTask_ListenDelegate>(FName("ConsumerFails"));
		TStateTreeEditorNode<FTestTask_ListenDelegate>& ConsumerFailsSecond = Root.AddTask<FTestTask_ListenDelegate>(FName("ConsumerFailsSecond"));

		EditorData.EditorBindings.AddTaskCompletionBinding(ProducerTask.GetNodeID(), FPropertyBindingPath(ConsumerSucceeds.ID, GET_MEMBER_NAME_CHECKED(FTestTask_ListenDelegate_InstanceData, Listener)), UE::StateTree::ETaskCompletionCondition::Succeeds);
		EditorData.EditorBindings.AddTaskCompletionBinding(ProducerTask.GetNodeID(), FPropertyBindingPath(ConsumerFails.ID, GET_MEMBER_NAME_CHECKED(FTestTask_ListenDelegate_InstanceData, Listener)), UE::StateTree::ETaskCompletionCondition::Fails);
		EditorData.EditorBindings.AddTaskCompletionBinding(ProducerTask.GetNodeID(), FPropertyBindingPath(ConsumerFailsSecond.ID, GET_MEMBER_NAME_CHECKED(FTestTask_ListenDelegate_InstanceData, Listener)), UE::StateTree::ETaskCompletionCondition::Fails);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE("StateTree should compile with Succeeds and Fails conditions", bResult);

		AITEST_TRUE("TaskCompletionDispatchers should have exactly 2 dispatchers (Succeeds and Fails)", StateTree.GetTaskCompletionDispatchers().Num() == 2);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE("StateTree should initialize", Exec.IsValid());

		Exec.Start();
		Exec.LogClear();

		Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("ConsumerSucceeds should be triggered once."), Exec.Expect(ConsumerSucceeds.GetName(), *FString::Printf(TEXT("OnDelegate%d"), 1)));
		AITEST_FALSE(TEXT("ConsumerFails shouldn't be triggered."), Exec.Expect(ConsumerFails.GetName(), *FString::Printf(TEXT("OnDelegate%d"), 1)));
		AITEST_FALSE(TEXT("ConsumerFailsSecond shouldn't be triggered."), Exec.Expect(ConsumerFailsSecond.GetName(), *FString::Printf(TEXT("OnDelegate%d"), 1)));
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_TaskCompletionDispatcher_SucceedsCondition, "System.StateTree.TaskCompletionDispatcher.SucceedsCondition");

struct FStateTreeTest_TaskCompletionDispatcher_FailsCondition : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		UStateTreeState& Root = EditorData.AddSubTree(FName("Root"));

		TStateTreeEditorNode<FTestTask_Stand>& ProducerTask = Root.AddTask<FTestTask_Stand>(FName("ProducerTask"));
		ProducerTask.GetNode().TickCompletionResult = EStateTreeRunStatus::Failed;

		TStateTreeEditorNode<FTestTask_ListenDelegate>& ConsumerSucceeds = Root.AddTask<FTestTask_ListenDelegate>(FName("ConsumerSucceeds"));
		TStateTreeEditorNode<FTestTask_ListenDelegate>& ConsumerFails = Root.AddTask<FTestTask_ListenDelegate>(FName("ConsumerFails"));
		TStateTreeEditorNode<FTestTask_ListenDelegate>& ConsumerFailsSecond = Root.AddTask<FTestTask_ListenDelegate>(FName("ConsumerFailsSecond"));

		EditorData.EditorBindings.AddTaskCompletionBinding(ProducerTask.GetNodeID(), FPropertyBindingPath(ConsumerSucceeds.ID, GET_MEMBER_NAME_CHECKED(FTestTask_ListenDelegate_InstanceData, Listener)), UE::StateTree::ETaskCompletionCondition::Succeeds);
		EditorData.EditorBindings.AddTaskCompletionBinding(ProducerTask.GetNodeID(), FPropertyBindingPath(ConsumerFails.ID, GET_MEMBER_NAME_CHECKED(FTestTask_ListenDelegate_InstanceData, Listener)), UE::StateTree::ETaskCompletionCondition::Fails);
		EditorData.EditorBindings.AddTaskCompletionBinding(ProducerTask.GetNodeID(), FPropertyBindingPath(ConsumerFailsSecond.ID, GET_MEMBER_NAME_CHECKED(FTestTask_ListenDelegate_InstanceData, Listener)), UE::StateTree::ETaskCompletionCondition::Fails);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE("StateTree should compile with Succeeds and Fails conditions", bResult);

		AITEST_TRUE("TaskCompletionDispatchers should have exactly 2 dispatchers (Succeeds and Fails)", StateTree.GetTaskCompletionDispatchers().Num() == 2);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE("StateTree should initialize", Exec.IsValid());

		Exec.Start();
		Exec.LogClear();

		Exec.Tick(0.1f);
		AITEST_FALSE(TEXT("ConsumerSucceeds shouldn't be triggered."), Exec.Expect(ConsumerSucceeds.GetName(), *FString::Printf(TEXT("OnDelegate%d"), 1)));
		AITEST_TRUE(TEXT("ConsumerFails should be triggered once."), Exec.Expect(ConsumerFails.GetName(), *FString::Printf(TEXT("OnDelegate%d"), 1)));
		AITEST_TRUE(TEXT("ConsumerFailsSecond should be triggered once."), Exec.Expect(ConsumerFailsSecond.GetName(), *FString::Printf(TEXT("OnDelegate%d"), 1)));
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_TaskCompletionDispatcher_FailsCondition, "System.StateTree.TaskCompletionDispatcher.FailsCondition");

} // namespace UE::StateTree::Tests

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
