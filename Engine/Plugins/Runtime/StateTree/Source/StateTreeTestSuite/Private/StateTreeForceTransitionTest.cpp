// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeTest.h"
#include "StateTreeTestBase.h"
#include "StateTreeTestTypes.h"

#include "StateTreeCompilerLog.h"
#include "StateTreeEditorData.h"
#include "StateTreeCompiler.h"
#include "Conditions/StateTreeCommonConditions.h"

#define LOCTEXT_NAMESPACE "AITestSuite_StateTreeTask"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::StateTree::Tests
{

struct FStateTreeTest_ForceTransition_All : FStateTreeTestBase
{
	int32 AddStateTaskIndex = 0;
	int32 AddStateTreeIndex = 0;

	int32 TransitionIfTaskIndex = INDEX_NONE;
	int32 TransitionIfTreeIndex = INDEX_NONE;
	FStateTreeStateHandle TransitionTo;

	UStateTreeState& AddSubTree(UStateTreeEditorData& EditorData, FName StateName, EStateTreeStateType StateType)
	{
		UStateTreeState& State = EditorData.AddSubTree(StateName);
		State.Type = StateType;
		State.SelectionBehavior = EStateTreeStateSelectionBehavior::TryEnterState;
		TStateTreeEditorNode<FTestTask_PrintValue>& Task = State.AddTask<FTestTask_PrintValue>(StateName);
		Task.GetInstanceData().Value = AddStateTaskIndex + (AddStateTreeIndex*100);
		++AddStateTaskIndex;
		Task.GetNode().CustomTickFunc = [this](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
			{
				FTestTask_PrintValue::FInstanceDataType& InstanceData = Context.GetInstanceData(*Task);
				if ((InstanceData.Value % 100) == TransitionIfTaskIndex
					&& (InstanceData.Value / 100) == TransitionIfTreeIndex)
				{
					Context.RequestTransition(TransitionTo);
				}
			};
		return State;
	}

	UStateTreeState& AddChildState(UStateTreeState& ParentState, FName StateName, EStateTreeStateType StateType)
	{
		UStateTreeState& State = ParentState.AddChildState(StateName, StateType);
		State.SelectionBehavior = EStateTreeStateSelectionBehavior::TryEnterState;
		TStateTreeEditorNode<FTestTask_PrintValue>& Task = State.AddTask<FTestTask_PrintValue>(StateName);
		Task.GetInstanceData().Value = AddStateTaskIndex++;
		Task.GetNode().CustomTickFunc = [this](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
			{
				FTestTask_PrintValue::FInstanceDataType& InstanceData = Context.GetInstanceData(*Task);
				if ((InstanceData.Value % 100) == TransitionIfTaskIndex
					&& (InstanceData.Value / 100) == TransitionIfTreeIndex)
				{
					Context.RequestTransition(TransitionTo);
				}
			};
		return State;
	}

	UStateTree* BuildTree1(TNotNull<UStateTree*> Tree2)
	{
		//Tree1
		// StateA
		//  StateB
		//   StateC
		//  StateD
		//   StateLinkedE -> X
		//  StateF
		//   StateLinkedAssetG -> Tree2
		//  StateLinkedH -> X
		//  StateLinkedAssetI -> Tree2
		// StateQ (new root)
		//  StateR
		//   StateS
		//  StateLinkedT -> X
		// StateX
		//  StateY
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		AddStateTaskIndex = 0;
		AddStateTreeIndex = 0;
		UStateTreeState& StateA = AddSubTree(EditorData, "Tree1StateA", EStateTreeStateType::State);
		UStateTreeState& StateB = AddChildState(StateA, "Tree1StateB", EStateTreeStateType::State);
		UStateTreeState& StateC = AddChildState(StateB, "Tree1StateC", EStateTreeStateType::State);
		UStateTreeState& StateD = AddChildState(StateA, "Tree1StateD", EStateTreeStateType::State);
		UStateTreeState& StateE = AddChildState(StateD, "Tree1StateE", EStateTreeStateType::Linked);
		UStateTreeState& StateF = AddChildState(StateA, "Tree1StateF", EStateTreeStateType::State);
		UStateTreeState& StateG = AddChildState(StateF, "Tree1StateG", EStateTreeStateType::LinkedAsset);
		UStateTreeState& StateH = AddChildState(StateA, "Tree1StateH", EStateTreeStateType::Linked);
		UStateTreeState& StateI = AddChildState(StateA, "Tree1StateI", EStateTreeStateType::LinkedAsset);
		UStateTreeState& StateQ = AddSubTree(EditorData, "Tree1StateQ", EStateTreeStateType::State);
		UStateTreeState& StateR = AddChildState(StateQ, "Tree1StateR", EStateTreeStateType::State);
		UStateTreeState& StateS = AddChildState(StateR, "Tree1StateS", EStateTreeStateType::State);
		UStateTreeState& StateT = AddChildState(StateQ, "Tree1StateT", EStateTreeStateType::Linked);
		UStateTreeState& StateX = AddSubTree(EditorData, "Tree1StateX", EStateTreeStateType::Subtree);
		UStateTreeState& StateY = AddChildState(StateX, "Tree1StateY", EStateTreeStateType::State);

		StateE.SetLinkedState(StateX.GetLinkToState());
		StateG.SetLinkedStateAsset(Tree2);
		StateH.SetLinkedState(StateX.GetLinkToState());
		StateI.SetLinkedStateAsset(Tree2);
		StateT.SetLinkedState(StateX.GetLinkToState());

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		return Compiler.Compile(StateTree) ? &StateTree : nullptr;
	}

	UStateTree* BuildTree2(TNotNull<UStateTree*> Tree3)
	{
		//Tree2
		// StateA
		//  StateB
		//   StateC
		//   StateLinkedD -> X
		//   StateLinkedAssetE -> Tree3
		// StateQ (new root)
		//  StateR
		//   StateS
		//  StateLinkedT -> X
		// StateX
		//  StateY
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		AddStateTaskIndex = 0;
		AddStateTreeIndex = 1;
		UStateTreeState& StateA = AddSubTree(EditorData, "Tree2StateA", EStateTreeStateType::State);
		UStateTreeState& StateB = AddChildState(StateA, "Tree2StateB", EStateTreeStateType::State);
		UStateTreeState& StateC = AddChildState(StateB, "Tree2StateC", EStateTreeStateType::State);
		UStateTreeState& StateD = AddChildState(StateB, "Tree2StateD", EStateTreeStateType::Linked);
		UStateTreeState& StateE = AddChildState(StateB, "Tree2StateE", EStateTreeStateType::LinkedAsset);
		UStateTreeState& StateQ = AddSubTree(EditorData, "Tree2StateQ", EStateTreeStateType::State);
		UStateTreeState& StateR = AddChildState(StateQ, "Tree2StateR", EStateTreeStateType::State);
		UStateTreeState& StateS = AddChildState(StateR, "Tree2StateS", EStateTreeStateType::State);
		UStateTreeState& StateT = AddChildState(StateQ, "Tree2StateT", EStateTreeStateType::Linked);
		UStateTreeState& StateX = AddSubTree(EditorData, "Tree2StateX", EStateTreeStateType::Subtree);
		UStateTreeState& StateY = AddChildState(StateX, "Tree2StateY", EStateTreeStateType::State);

		StateD.SetLinkedState(StateX.GetLinkToState());
		StateE.SetLinkedStateAsset(Tree3);
		StateT.SetLinkedState(StateX.GetLinkToState());

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		return Compiler.Compile(StateTree) ? &StateTree : nullptr;

	}
	UStateTree* BuildTree3()
	{
		//Tree3
		// StateA
		//  StateB
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		AddStateTaskIndex = 0;
		AddStateTreeIndex = 2;
		UStateTreeState& StateA = AddSubTree(EditorData, "Tree3StateA", EStateTreeStateType::Subtree);
		UStateTreeState& StateB = AddChildState(StateA, "Tree3StateB", EStateTreeStateType::State);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		return Compiler.Compile(StateTree) ? &StateTree : nullptr;
	}

	virtual bool InstantTest() override
	{
		UStateTree* StateTree3 = BuildTree3();
		AITEST_TRUE(TEXT("StateTree3 should get compiled"), StateTree3 != nullptr);

		UStateTree* StateTree2 = BuildTree2(StateTree3);
		AITEST_TRUE(TEXT("StateTree2 should get compiled"), StateTree2 != nullptr);

		UStateTree* StateTree1 = BuildTree1(StateTree2);
		AITEST_TRUE(TEXT("StateTree2 should get compiled"), StateTree1 != nullptr);

		// Suppress code analyzer warning C6011
		CA_ASSUME(StateTree1);
		CA_ASSUME(StateTree2);
		CA_ASSUME(StateTree3);

		FStateTreeInstanceData InstanceData;
		FStateTreeInstanceData OldForceInstanceData;
		{
			FTestStateTreeExecutionContext Exec(*StateTree1, *StateTree1, InstanceData); //-C6011
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);
		}

		constexpr int32 Tree1StateAIndex = 0;
		constexpr int32 Tree1StateQIndex = 9;
		constexpr int32 Tree1StateXIndex = 13;
		AITEST_TRUE(TEXT("Invalid Tree1StateA index."), StateTree1->GetStates().IsValidIndex(Tree1StateAIndex) && StateTree1->GetStates()[Tree1StateAIndex].Name == "Tree1StateA");
		AITEST_TRUE(TEXT("Invalid Tree1StateQ index."), StateTree1->GetStates().IsValidIndex(Tree1StateQIndex) && StateTree1->GetStates()[Tree1StateQIndex].Name == "Tree1StateQ");
		AITEST_TRUE(TEXT("Invalid Tree1StateX index."), StateTree1->GetStates().IsValidIndex(Tree1StateXIndex) && StateTree1->GetStates()[Tree1StateXIndex].Name == "Tree1StateX");
		constexpr int32 Tree2StateAIndex = 0;
		constexpr int32 Tree2StateQIndex = 5;
		constexpr int32 Tree2StateXIndex = 9;
		AITEST_TRUE(TEXT("Invalid Tree2StateA index."), StateTree2->GetStates().IsValidIndex(Tree2StateAIndex) && StateTree2->GetStates()[Tree2StateAIndex].Name == "Tree2StateA");
		AITEST_TRUE(TEXT("Invalid Tree2StateQ index."), StateTree2->GetStates().IsValidIndex(Tree2StateQIndex) && StateTree2->GetStates()[Tree2StateQIndex].Name == "Tree2StateQ");
		AITEST_TRUE(TEXT("Invalid Tree2StateX index."), StateTree2->GetStates().IsValidIndex(Tree2StateXIndex) && StateTree2->GetStates()[Tree2StateXIndex].Name == "Tree2StateX");
		constexpr int32 Tree3StateAIndex = 0;
		AITEST_TRUE(TEXT("Invalid Tree3StateA index."), StateTree3->GetStates().IsValidIndex(Tree3StateAIndex) && StateTree3->GetStates()[Tree3StateAIndex].Name == "Tree3StateA");

		struct FTransition
		{
			FStateTreeStateHandle TargetState;
			int32 ActiveStateSourceIndex;
			int32 ActiveTreeSourceIndex;
			bool bTestNextTree;
			TArray<FName> ExpectedActiveStateNames;
		};

		const FTransition Tree1TransitionsToTest[] = {
			FTransition{FStateTreeStateHandle(Tree1StateAIndex + 0), Tree1StateAIndex, 0, false, TArray<FName>{"Tree1StateA"}},
			FTransition{FStateTreeStateHandle(Tree1StateAIndex + 1), Tree1StateAIndex, 0, false, TArray<FName>{"Tree1StateA", "Tree1StateB"}},
			FTransition{FStateTreeStateHandle(Tree1StateAIndex + 2), Tree1StateAIndex, 0, false, TArray<FName>{"Tree1StateA", "Tree1StateB", "Tree1StateC"}},
			FTransition{FStateTreeStateHandle(Tree1StateAIndex + 3), Tree1StateAIndex, 0, false, TArray<FName>{"Tree1StateA", "Tree1StateD"}},
			FTransition{FStateTreeStateHandle(Tree1StateAIndex + 4), Tree1StateAIndex, 0, false, TArray<FName>{"Tree1StateA", "Tree1StateD", "Tree1StateE", "Tree1StateX"}},
			FTransition{FStateTreeStateHandle(Tree1StateXIndex + 1), Tree1StateXIndex, 0, false, TArray<FName>{"Tree1StateA", "Tree1StateD", "Tree1StateE", "Tree1StateX", "Tree1StateY"}}, //5
			FTransition{FStateTreeStateHandle(Tree1StateAIndex + 5), Tree1StateAIndex, 0, false, TArray<FName>{"Tree1StateA", "Tree1StateF"}},
			FTransition{FStateTreeStateHandle(Tree1StateAIndex + 6), Tree1StateAIndex, 0, true,  TArray<FName>{"Tree1StateA", "Tree1StateF", "Tree1StateG", "Tree2StateA"}},
			FTransition{FStateTreeStateHandle(Tree1StateAIndex + 7), Tree1StateAIndex, 0, false, TArray<FName>{"Tree1StateA", "Tree1StateH", "Tree1StateX"}},
			FTransition{FStateTreeStateHandle(Tree1StateXIndex + 1), Tree1StateXIndex, 0, false, TArray<FName>{"Tree1StateA", "Tree1StateH", "Tree1StateX", "Tree1StateY"}},
			FTransition{FStateTreeStateHandle(Tree1StateAIndex + 8), Tree1StateAIndex, 0, true,  TArray<FName>{"Tree1StateA", "Tree1StateI", "Tree2StateA"}}, //10
			FTransition{FStateTreeStateHandle(Tree1StateQIndex + 0), Tree1StateAIndex, 0, false, TArray<FName>{"Tree1StateQ"}},
			FTransition{FStateTreeStateHandle(Tree1StateQIndex + 1), Tree1StateQIndex, 0, false, TArray<FName>{"Tree1StateQ", "Tree1StateR"}},
			FTransition{FStateTreeStateHandle(Tree1StateQIndex + 2), Tree1StateQIndex, 0, false, TArray<FName>{"Tree1StateQ", "Tree1StateR", "Tree1StateS"}},
			FTransition{FStateTreeStateHandle(Tree1StateQIndex + 3), Tree1StateQIndex, 0, false, TArray<FName>{"Tree1StateQ", "Tree1StateT", "Tree1StateX"}},
			FTransition{FStateTreeStateHandle(Tree1StateXIndex + 1), Tree1StateXIndex, 0, false, TArray<FName>{"Tree1StateQ", "Tree1StateT", "Tree1StateX", "Tree1StateY"}}, //15
		};

		const FTransition Tree2TransitionsToTest[] = {
			FTransition{FStateTreeStateHandle(Tree2StateAIndex + 0), Tree2StateAIndex, 1, false, TArray<FName>{"Tree2StateA"}},
			FTransition{FStateTreeStateHandle(Tree2StateAIndex + 1), Tree2StateAIndex, 1, false, TArray<FName>{"Tree2StateA", "Tree2StateB"}},
			FTransition{FStateTreeStateHandle(Tree2StateAIndex + 2), Tree2StateAIndex, 1, false, TArray<FName>{"Tree2StateA", "Tree2StateB", "Tree2StateC"}},
			FTransition{FStateTreeStateHandle(Tree2StateAIndex + 3), Tree2StateAIndex, 1, false, TArray<FName>{"Tree2StateA", "Tree2StateB", "Tree2StateD", "Tree2StateX"}},
			FTransition{FStateTreeStateHandle(Tree2StateXIndex + 1), Tree2StateXIndex, 1, false, TArray<FName>{"Tree2StateA", "Tree2StateB", "Tree2StateD", "Tree2StateX", "Tree2StateY"}},
			FTransition{FStateTreeStateHandle(Tree2StateAIndex + 4), Tree2StateAIndex, 1, true,  TArray<FName>{"Tree2StateA", "Tree2StateB", "Tree2StateE", "Tree3StateA"}}, // 5
			FTransition{FStateTreeStateHandle(Tree2StateQIndex + 0), Tree2StateAIndex, 1, false, TArray<FName>{"Tree2StateQ"}},
			FTransition{FStateTreeStateHandle(Tree2StateQIndex + 1), Tree2StateQIndex, 1, false, TArray<FName>{"Tree2StateQ", "Tree2StateR"}},
			FTransition{FStateTreeStateHandle(Tree2StateQIndex + 2), Tree2StateQIndex, 1, false, TArray<FName>{"Tree2StateQ", "Tree2StateR", "Tree2StateS"}},
			FTransition{FStateTreeStateHandle(Tree2StateQIndex + 3), Tree2StateQIndex, 1, false, TArray<FName>{"Tree2StateQ", "Tree2StateT", "Tree2StateX"}},
			FTransition{FStateTreeStateHandle(Tree2StateXIndex + 1), Tree2StateXIndex, 1, false, TArray<FName>{"Tree2StateQ", "Tree2StateT", "Tree2StateX", "Tree2StateY"}}, //10
		};

		const FTransition Tree3TransitionsToTest[] = {
			FTransition{FStateTreeStateHandle(Tree3StateAIndex + 0), Tree3StateAIndex, 2, false, TArray<FName>{"Tree3StateA"}},
			FTransition{FStateTreeStateHandle(Tree3StateAIndex + 1), Tree3StateAIndex, 2, false, TArray<FName>{"Tree3StateA", "Tree3StateB"}}
		};

		const TArrayView<const FTransition> AllTranstionsToTest[] = { MakeConstArrayView(Tree1TransitionsToTest), MakeConstArrayView(Tree2TransitionsToTest), MakeConstArrayView(Tree3TransitionsToTest) };
		const int32 AllRootStates[] = { Tree1StateAIndex, Tree2StateAIndex, Tree3StateAIndex };

		auto TestAllTransitions = [&](const auto& RecursiveLambda, int32 TreeIndex, const TArray<FName>& PreviousTreeActiveStateName, bool bUseRoot)
			{
				const auto& TreeTransitionsToTest = AllTranstionsToTest[TreeIndex];
				for (int32 TransitionIndex = 0; TransitionIndex < TreeTransitionsToTest.Num(); ++TransitionIndex)
				{
					TArray<FName> ActiveStateNames;
					TArray<FRecordedStateTreeTransitionResult> RecordedTransitions;
					TArray<UE::StateTree::ExecutionContext::FStateHandleContext> ActiveStateHandles;
					{
						FTestStateTreeExecutionContext Exec(*StateTree1, *StateTree1, InstanceData, {}, EStateTreeRecordTransitions::Yes);

						if (bUseRoot && TreeTransitionsToTest[TransitionIndex].ActiveStateSourceIndex == 0)
						{
							Exec.RequestTransition(TreeTransitionsToTest[TransitionIndex].TargetState);
						}
						else
						{
							TransitionIfTaskIndex = TreeTransitionsToTest[TransitionIndex].ActiveStateSourceIndex;
							TransitionIfTreeIndex = TreeTransitionsToTest[TransitionIndex].ActiveTreeSourceIndex;
							TransitionTo = TreeTransitionsToTest[TransitionIndex].TargetState;
						}

						Exec.Tick(0.01f);

						RecordedTransitions = Exec.GetRecordedTransitions();

						ActiveStateNames = Exec.GetActiveStateNames();
						TArray<FName> ExpectedActiveStateNames = PreviousTreeActiveStateName;
						ExpectedActiveStateNames.Append(TreeTransitionsToTest[TransitionIndex].ExpectedActiveStateNames);
						AITEST_TRUE(FString::Printf(TEXT("Normal transition is not in expected states %d:%d"), TreeIndex, TransitionIndex), ActiveStateNames == ExpectedActiveStateNames);

						// Build force transition
						ActiveStateHandles.Reserve(ActiveStateNames.Num());
						const FStateTreeExecutionState* ExecState = InstanceData.GetExecutionState();
						for (const FStateTreeExecutionFrame& CurrentFrame : ExecState->ActiveFrames)
						{
							const UStateTree* CurrentStateTree = CurrentFrame.StateTree;
							for (int32 Index = 0; Index < CurrentFrame.ActiveStates.Num(); Index++)
							{
								const FStateTreeStateHandle Handle = CurrentFrame.ActiveStates[Index];
								ActiveStateHandles.Emplace(CurrentStateTree, Handle);
							}
						}

						// Reset
						TransitionIfTaskIndex = INDEX_NONE;
						TransitionIfTreeIndex = INDEX_NONE;
						TransitionTo = FStateTreeStateHandle();
						Exec.LogClear();
					}
					{
						FTestStateTreeExecutionContext Exec(*StateTree1, *StateTree1, OldForceInstanceData);
						for(const FRecordedStateTreeTransitionResult& ForcedTransition : RecordedTransitions)
						{
							AITEST_TRUE(FString::Printf(TEXT("Old Force transition %d:%d"), TreeIndex, TransitionIndex), Exec.ForceTransition(ForcedTransition) != EStateTreeRunStatus::Unset);
						}
						const TArray<FName> NewActiveStateNames = Exec.GetActiveStateNames();
						AITEST_TRUE(FString::Printf(TEXT("Old force transition is not in expected states %d:%d"), TreeIndex, TransitionIndex), ActiveStateNames == NewActiveStateNames);
						Exec.LogClear();
					}

					if (TreeTransitionsToTest[TransitionIndex].bTestNextTree)
					{
						ActiveStateNames.Pop();
						if (!RecursiveLambda(RecursiveLambda, TreeIndex + 1, ActiveStateNames, false))
						{
							return false;
						}
					}
				}
				return true;
			};

		constexpr int32 MaxRules = 4;
		auto MakeStateSelectionRule = [](int32 Index)
			{
				EStateTreeStateSelectionRules Rule = EStateTreeStateSelectionRules::None;
				if ((Index % 2) == 1)
				{
					Rule |= EStateTreeStateSelectionRules::CompletedTransitionStatesCreateNewStates;
					Rule |= EStateTreeStateSelectionRules::CompletedStateBeforeTransitionSourceFailsTransition;
				}
				if (Index >= 2)
				{
					Rule |= EStateTreeStateSelectionRules::ReselectedStateCreatesNewStates;
				}
				return Rule;
			};


		for (int32 Index = 0; Index < MaxRules; ++Index)
		{
			const EStateTreeStateSelectionRules StateSelectionRules = MakeStateSelectionRule(Index);
			InstanceData = FStateTreeInstanceData();
			OldForceInstanceData = FStateTreeInstanceData();

			auto CompileTree = [StateSelectionRules](TNotNull<UStateTree*> StateTree)
				{
					StateTree->ResetCompiled();
					UStateTreeEditorData* EditorData = CastChecked<UStateTreeEditorData>(StateTree->EditorData);
					UStateTreeTestSchema* Schema = CastChecked<UStateTreeTestSchema>(EditorData->Schema);
					Schema->SetStateSelectionRules(StateSelectionRules);

					FStateTreeCompilerLog Log;
					FStateTreeCompiler Compiler(Log);
					return Compiler.Compile(StateTree);
				};

			AITEST_TRUE("StateTree3 should get compiled", CompileTree(StateTree3));
			AITEST_TRUE("StateTree2 should get compiled", CompileTree(StateTree2));
			AITEST_TRUE("StateTree1 should get compiled", CompileTree(StateTree1));

			{
				FTestStateTreeExecutionContext Exec(*StateTree1, *StateTree1, InstanceData);
				Exec.Start();
				AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateA"));
			}
			{
				FTestStateTreeExecutionContext Exec(*StateTree1, *StateTree1, OldForceInstanceData);
				Exec.Start();
				AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateA"));
			}
			bool bCurrentResult = TestAllTransitions(TestAllTransitions, 0, TArray<FName>(), true);
			{
				FTestStateTreeExecutionContext Exec(*StateTree1, *StateTree1, InstanceData);
				Exec.Stop();
			}
			{
				FTestStateTreeExecutionContext Exec(*StateTree1, *StateTree1, OldForceInstanceData);
				Exec.Stop();
			}

			AITEST_TRUE("Test failed", bCurrentResult);
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_ForceTransition_All, "System.StateTree.ForceTransition.All");

} // namespace UE::StateTree::Tests

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
