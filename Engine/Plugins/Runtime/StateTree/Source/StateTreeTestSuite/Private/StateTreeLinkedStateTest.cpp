// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeTest.h"
#include "StateTreeTestBase.h"
#include "StateTreeTestTypes.h"

#include "Conditions/StateTreeCommonConditions.h"
#include "Misc/ScopedCVar.h"
#include "StateTreeAsyncExecutionContext.h"
#include "StateTreeCompilerLog.h"
#include "StateTreeEditorData.h"
#include "StateTreeCompiler.h"

#define LOCTEXT_NAMESPACE "AITestSuite_StateTreeTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::StateTree::Tests
{

struct FStateTreeTest_FailEnterLinkedAsset : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		FStateTreeCompilerLog Log;

		// Asset 2
		UStateTree& StateTree2 = NewStateTree();
		UStateTreeEditorData& EditorData2 = *Cast<UStateTreeEditorData>(StateTree2.EditorData);
		UStateTreeState& Root2 = EditorData2.AddSubTree(FName(TEXT("Root2")));
		TStateTreeEditorNode<FTestTask_Stand>& Task2 = Root2.AddTask<FTestTask_Stand>(FName(TEXT("Task2")));
		TStateTreeEditorNode<FTestTask_Stand>& GlobalTask2 = EditorData2.AddGlobalTask<FTestTask_Stand>(FName(TEXT("GlobalTask2")));
		GlobalTask2.GetInstanceData().Value = 123;

		// Always failing enter condition
		TStateTreeEditorNode<FStateTreeCompareIntCondition>& IntCond2 = Root2.AddEnterCondition<FStateTreeCompareIntCondition>();
		EditorData2.AddPropertyBinding(GlobalTask2, TEXT("Value"), IntCond2, TEXT("Left"));
		IntCond2.GetInstanceData().Right = 0;

		FStateTreeCompiler Compiler2(Log);
		const bool bResult2 = Compiler2.Compile(StateTree2);
		AITEST_TRUE(TEXT("StateTree2 should get compiled"), bResult2);

		// Main asset
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root1")));
		UStateTreeState& A1 = Root.AddChildState(FName(TEXT("A1")), EStateTreeStateType::LinkedAsset);
		A1.SetLinkedStateAsset(&StateTree2);

		UStateTreeState& B1 = Root.AddChildState(FName(TEXT("B1")), EStateTreeStateType::State);
		TStateTreeEditorNode<FTestTask_Stand>& Task1 = B1.AddTask<FTestTask_Stand>(FName(TEXT("Task1")));

		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* ExitStateStr(TEXT("ExitState"));

		{
			EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

			Status = Exec.Start();
			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("StateTree should enter GlobalTask2"), Exec.Expect(GlobalTask2.GetName(), EnterStateStr));
			AITEST_TRUE(TEXT("StateTree should exit GlobalTask2"), Exec.Expect(GlobalTask2.GetName(), ExitStateStr));
			AITEST_FALSE(TEXT("StateTree should not enter Task2"), Exec.Expect(Task2.GetName(), EnterStateStr));
			AITEST_TRUE(TEXT("StateTree should enter Task1"), Exec.Expect(Task1.GetName(), EnterStateStr));

			Exec.Stop();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_FailEnterLinkedAsset, "System.StateTree.LinkedAsset.FailEnter");

struct FStateTreeTest_EnterAndExitLinkedAsset : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		FStateTreeCompilerLog Log;

		// Tree1
		//  Root1
		//   A1 (linked to Tree2) OnCompleted->B1
		//   B1 Task1
		// Tree2 GlobalTask2 => 2 ticks
		//  Root2 Task2 => 1 tick

		// Asset 2
		UStateTree& StateTree2 = NewStateTree();
		UStateTreeEditorData& EditorData2 = *Cast<UStateTreeEditorData>(StateTree2.EditorData);
		UStateTreeState& Root2 = EditorData2.AddSubTree(FName(TEXT("Root2")));
		TStateTreeEditorNode<FTestTask_Stand>& Task2 = Root2.AddTask<FTestTask_Stand>(FName(TEXT("Task2")));
		Task2.GetNode().TicksToCompletion = 1;
		TStateTreeEditorNode<FTestTask_Stand>& GlobalTask2 = EditorData2.AddGlobalTask<FTestTask_Stand>(FName(TEXT("GlobalTask2")));
		GlobalTask2.GetNode().TicksToCompletion = 2;
		{
			FStateTreeCompiler Compiler2(Log);
			const bool bResult2 = Compiler2.Compile(StateTree2);
			AITEST_TRUE(TEXT("StateTree2 should get compiled"), bResult2);
		}

		// Main asset
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root1")));
		UStateTreeState& A1 = Root.AddChildState(FName(TEXT("A1")), EStateTreeStateType::LinkedAsset);
		A1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::NextState);
		A1.SetLinkedStateAsset(&StateTree2);

		UStateTreeState& B1 = Root.AddChildState(FName(TEXT("B1")), EStateTreeStateType::State);
		TStateTreeEditorNode<FTestTask_Stand>& Task1 = B1.AddTask<FTestTask_Stand>(FName(TEXT("Task1")));
		Task1.GetNode().TicksToCompletion = 1;
		{
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);
		}

		const TCHAR* EnterStateStr = TEXT("EnterState");
		const TCHAR* SustainedEnterStateStr = TEXT("EnterState=Sustained");
		const TCHAR* ChangedEnterStateStr = TEXT("EnterState=Changed");
		const TCHAR* ExitStateStr = TEXT("ExitState");
		const TCHAR* SustainedExitStateStr = TEXT("ExitState=Sustained");
		const TCHAR* ChangedExitStateStr = TEXT("ExitState=Changed");
		const TCHAR* StateCompletedStateStr = TEXT("StateCompleted");
		const TCHAR* TickStr = TEXT("Tick");

		{
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);
			{
				EStateTreeRunStatus Status = Exec.Start();
				AITEST_EQUAL("Start should complete with Running", Status, EStateTreeRunStatus::Running);
				AITEST_TRUE("StateTree should enter GlobalTask2", Exec.Expect(GlobalTask2.GetName(), ChangedEnterStateStr));
				AITEST_FALSE("StateTree should not exit GlobalTask2", Exec.Expect(GlobalTask2.GetName(), ChangedExitStateStr));
				AITEST_FALSE("StateTree should not exit GlobalTask2", Exec.Expect(GlobalTask2.GetName(), SustainedExitStateStr));
				AITEST_TRUE("StateTree should enter Task2", Exec.Expect(Task2.GetName(), ChangedEnterStateStr));
				AITEST_FALSE("StateTree should not exit Task2", Exec.Expect(Task2.GetName(), ChangedExitStateStr));
				AITEST_FALSE("StateTree should not exit Task2", Exec.Expect(Task2.GetName(), SustainedExitStateStr));
				AITEST_FALSE("StateTree should not enter Task1", Exec.Expect(Task1.GetName(), EnterStateStr));
				Exec.LogClear();
			}
			{
				// Task2 completes.
				EStateTreeRunStatus Status = Exec.Tick(0.1f);
				AITEST_EQUAL("Tick should complete with Running", Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("StateTree should exit the root state but not the global"),
					Exec.Expect(GlobalTask2.GetName(), TickStr)
					.Then(Task2.GetName(), TickStr)
					.Then(Task2.GetName(), StateCompletedStateStr)
					.Then(Task2.GetName(), ChangedExitStateStr)
					.Then(GlobalTask2.GetName(), ChangedExitStateStr)
					.Then(Task1.GetName(), ChangedEnterStateStr)
				);
				AITEST_FALSE("StateTree should not tick Tasks1", Exec.Expect(Task1.GetName(), TickStr));
				AITEST_FALSE("StateTree should not exit Task1", Exec.Expect(Task1.GetName(), ExitStateStr));
				Exec.LogClear();
			}
			{
				Exec.Stop();
			}
			// Change the order, the global completes before task2. Task2 should not tick
			{
				Task2.GetNode().TicksToCompletion = 1;
				GlobalTask2.GetNode().TicksToCompletion = 1;
				FStateTreeCompiler Compiler2(Log);
				const bool bResult2 = Compiler2.Compile(StateTree2);
				AITEST_TRUE(TEXT("StateTree2 should get compiled"), bResult2);
			}
			{
				EStateTreeRunStatus Status = Exec.Start();
				AITEST_EQUAL("Start should complete with Running", Status, EStateTreeRunStatus::Running);
				Exec.LogClear();
			}
			{
				// GlobalTask2 completes Task2 won't tick and won't complete.
				EStateTreeRunStatus Status = Exec.Tick(0.1f);
				AITEST_EQUAL("Tick should complete with Running", Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("StateTree should exit the root state and the global"),
					Exec.Expect(GlobalTask2.GetName(), TickStr)
					.Then(Task2.GetName(), ChangedExitStateStr)
					.Then(GlobalTask2.GetName(), ChangedExitStateStr)
					.Then(Task1.GetName(), EnterStateStr)
				);
				AITEST_FALSE("StateTree should not tick Tasks2", Exec.Expect(Task2.GetName(), TickStr));
				AITEST_FALSE("StateTree should not tick Tasks1", Exec.Expect(Task1.GetName(), TickStr));
				AITEST_FALSE("StateTree should not exit Task1", Exec.Expect(Task1.GetName(), ExitStateStr));
				Exec.LogClear();
			}

			Exec.Stop();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_EnterAndExitLinkedAsset, "System.StateTree.LinkedAsset.EnterAndExit");

struct FStateTreeTest_EnterAndExitLinkedAsset2 : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		FStateTreeCompilerLog Log;

		// Tree1
		//  Root1 Task0
		//   A1 (linked to Tree2) no transition
		//   B1 Task1
		// Tree2 GlobalTask2 => 2 ticks
		//  Root2 Task2 => 1 tick

		// Asset 2
		UStateTree& StateTree2 = NewStateTree();
		UStateTreeEditorData& EditorData2 = *Cast<UStateTreeEditorData>(StateTree2.EditorData);
		UStateTreeState& Root2 = EditorData2.AddSubTree(FName(TEXT("Root2")));
		TStateTreeEditorNode<FTestTask_Stand>& Task2 = Root2.AddTask<FTestTask_Stand>(FName(TEXT("Task2")));
		Task2.GetNode().TicksToCompletion = 1;
		TStateTreeEditorNode<FTestTask_Stand>& GlobalTask2 = EditorData2.AddGlobalTask<FTestTask_Stand>(FName(TEXT("GlobalTask2")));
		GlobalTask2.GetNode().TicksToCompletion = 2;
		{
			FStateTreeCompiler Compiler2(Log);
			const bool bResult2 = Compiler2.Compile(StateTree2);
			AITEST_TRUE(TEXT("StateTree2 should get compiled"), bResult2);
		}

		// Main asset
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root1")));
		TStateTreeEditorNode<FTestTask_Stand>& Task0 = Root.AddTask<FTestTask_Stand>(FName(TEXT("Task0")));
		Task0.GetNode().TicksToCompletion = 999999;

		UStateTreeState& A1 = Root.AddChildState(FName(TEXT("A1")), EStateTreeStateType::LinkedAsset);
		A1.SetLinkedStateAsset(&StateTree2);

		UStateTreeState& B1 = Root.AddChildState(FName(TEXT("B1")), EStateTreeStateType::State);
		TStateTreeEditorNode<FTestTask_Stand>& Task1 = B1.AddTask<FTestTask_Stand>(FName(TEXT("Task1")));
		Task1.GetNode().TicksToCompletion = 1;
		{
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);
		}

		const TCHAR* EnterStateStr = TEXT("EnterState");
		const TCHAR* SustainedEnterStateStr = TEXT("EnterState=Sustained");
		const TCHAR* ChangedEnterStateStr = TEXT("EnterState=Changed");
		const TCHAR* ExitStateStr = TEXT("ExitState");
		const TCHAR* SustainedExitStateStr = TEXT("ExitState=Sustained");
		const TCHAR* ChangedExitStateStr = TEXT("ExitState=Changed");
		const TCHAR* StateCompletedStateStr = TEXT("StateCompleted");
		const TCHAR* TickStr = TEXT("Tick");

		{
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);
			{
				EStateTreeRunStatus Status = Exec.Start();
				AITEST_EQUAL("Start should complete with Running", Status, EStateTreeRunStatus::Running);
				AITEST_TRUE("StateTree should enter GlobalTask2", Exec.Expect(GlobalTask2.GetName(), ChangedEnterStateStr));
				AITEST_FALSE("StateTree should not exit GlobalTask2", Exec.Expect(GlobalTask2.GetName(), ChangedExitStateStr));
				AITEST_FALSE("StateTree should not exit GlobalTask2", Exec.Expect(GlobalTask2.GetName(), SustainedExitStateStr));
				AITEST_TRUE("StateTree should enter Task2", Exec.Expect(Task2.GetName(), ChangedEnterStateStr));
				AITEST_FALSE("StateTree should not exit Task2", Exec.Expect(Task2.GetName(), ChangedExitStateStr));
				AITEST_FALSE("StateTree should not exit Task2", Exec.Expect(Task2.GetName(), SustainedExitStateStr));
				AITEST_FALSE("StateTree should not enter Task1", Exec.Expect(Task1.GetName(), EnterStateStr));
				Exec.LogClear();
			}
			{
				// Task2 completes. The linked state didn't complete. Global is sustained.
				EStateTreeRunStatus Status = Exec.Tick(0.1f);
				AITEST_EQUAL("Tick should complete with Running", Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("StateTree should exit the root state but not the global"),
					Exec.Expect(Task0.GetName(), TickStr)
					.Then(GlobalTask2.GetName(), TickStr)
					.Then(Task2.GetName(), TickStr)
					.Then(Task2.GetName(), StateCompletedStateStr)
					.Then(Task0.GetName(), StateCompletedStateStr)
					.Then(Task2.GetName(), ChangedExitStateStr)
					.Then(GlobalTask2.GetName(), SustainedExitStateStr)
					.Then(Task0.GetName(), SustainedExitStateStr)
					.Then(Task0.GetName(), SustainedEnterStateStr)
					.Then(GlobalTask2.GetName(), SustainedEnterStateStr)
					.Then(Task2.GetName(), ChangedEnterStateStr)
				);
				AITEST_FALSE("StateTree should not tick Task1", Exec.Expect(Task1.GetName(), TickStr));
				AITEST_FALSE("StateTree should not exit Task1", Exec.Expect(Task1.GetName(), ExitStateStr));
				Exec.LogClear();
			}
			{
				Exec.Stop();
			}
			// Change the order, the global completes before task2. Task2 should not tick.
			{
				Task2.GetNode().TicksToCompletion = 1;
				GlobalTask2.GetNode().TicksToCompletion = 1;
				FStateTreeCompiler Compiler2(Log);
				const bool bResult2 = Compiler2.Compile(StateTree2);
				AITEST_TRUE(TEXT("StateTree2 should get compiled"), bResult2);
			}
			{
				EStateTreeRunStatus Status = Exec.Start();
				AITEST_EQUAL("Start should complete with Running", Status, EStateTreeRunStatus::Running);
				Exec.LogClear();
			}
			{
				// GlobalTask2 completes. The tree completed. Task2 won't tick and won't complete.
				EStateTreeRunStatus Status = Exec.Tick(0.1f);
				AITEST_EQUAL("Tick should complete with Running", Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("StateTree should exit the root state and the global"),
					Exec.Expect(GlobalTask2.GetName(), TickStr)
					.Then(Task2.GetName(), ChangedExitStateStr)
					.Then(GlobalTask2.GetName(), ChangedExitStateStr)
					.Then(Task0.GetName(), SustainedExitStateStr)
					.Then(Task0.GetName(), SustainedEnterStateStr)
					.Then(Task2.GetName(), EnterStateStr)
				);
				AITEST_TRUE(TEXT("StateTree should enter a new global task2"), Exec.Expect(GlobalTask2.GetName(), ChangedEnterStateStr));
				AITEST_FALSE("StateTree should not tick Tasks2", Exec.Expect(Task2.GetName(), TickStr));
				AITEST_FALSE("StateTree should not tick Tasks1", Exec.Expect(Task1.GetName(), TickStr));
				AITEST_FALSE("StateTree should not exit Task1", Exec.Expect(Task1.GetName(), EnterStateStr));
				AITEST_FALSE("StateTree should not exit Task1", Exec.Expect(Task1.GetName(), ExitStateStr));
				Exec.LogClear();
			}

			Exec.Stop();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_EnterAndExitLinkedAsset2, "System.StateTree.LinkedAsset.EnterAndExit2");

struct FStateTreeTest_MultipleSameLinkedAsset : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		//Tree 1
		//	Root
		//		StateA -> Next
		//		StateB -> Next
		//		StateLinkedTreeA (Tree2) -> Next
		//		StateLinkedTreeB (Tree2) -> Next
		//		StateLinkedTreeC (Tree2) -> Next
		//		StateC -> Succeeded
		//Tree 2
		//  Global task and parameter
		//	RootE
		//		StateA (with transition OnTick to succeeded)

		FStateTreeCompilerLog Log;

		// Asset 2
		UStateTree& StateTree2 = NewStateTree();
		FGuid RootParameter_ValueID;
		{
			UStateTreeEditorData& EditorData2 = *Cast<UStateTreeEditorData>(StateTree2.EditorData);
			{
				// Parameters
				FInstancedPropertyBag& RootPropertyBag = GetRootPropertyBag(EditorData2);
				RootPropertyBag.AddProperty("Value", EPropertyBagPropertyType::Int32);
				RootPropertyBag.SetValueInt32("Value", -111);
				RootParameter_ValueID = RootPropertyBag.FindPropertyDescByName("Value")->ID;

				TStateTreeEditorNode<FTestTask_PrintValue>& GlobalTask = EditorData2.AddGlobalTask<FTestTask_PrintValue>("Tree2GlobalTaskA");
				GlobalTask.GetInstanceData().Value = -1;
				EditorData2.AddPropertyBinding(FPropertyBindingPath(EditorData2.GetRootParametersGuid(), TEXT("Value")), FPropertyBindingPath(GlobalTask.ID, TEXT("Value")));
			}

			UStateTreeState& Root = EditorData2.AddSubTree("Tree2StateRoot");
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& Task1 = Root.AddTask<FTestTask_PrintValue>("Tree2StateRootTaskA");
				Task1.GetInstanceData().Value = -2;
				EditorData2.AddPropertyBinding(FPropertyBindingPath(EditorData2.GetRootParametersGuid(), TEXT("Value")), FPropertyBindingPath(Task1.ID, TEXT("Value")));
			}
			{
				UStateTreeState& State = Root.AddChildState("Tree2StateA", EStateTreeStateType::State);
				FStateTreeTransition& Transition = State.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::Succeeded);
				Transition.bDelayTransition = true;
				Transition.DelayDuration = 1.0;

				TStateTreeEditorNode<FTestTask_PrintValue>& Task1 = State.AddTask<FTestTask_PrintValue>("Tree2StateATaskA");
				Task1.GetInstanceData().Value = -2;
				EditorData2.AddPropertyBinding(FPropertyBindingPath(EditorData2.GetRootParametersGuid(), TEXT("Value")), FPropertyBindingPath(Task1.ID, TEXT("Value")));
			}

			FStateTreeCompiler Compiler2(Log);
			const bool bResult2 = Compiler2.Compile(StateTree2);
			AITEST_TRUE(TEXT("StateTree2 should get compiled"), bResult2);
		}

		// Main asset
		UStateTree& StateTree1 = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree1.EditorData);
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& GlobalTask = EditorData.AddGlobalTask<FTestTask_PrintValue>("Tree1GlobalTaskA");
				GlobalTask.GetInstanceData().Value = 99;
			}

			UStateTreeState& Root = EditorData.AddSubTree("Tree1StateRoot");
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& Task1 = Root.AddTask<FTestTask_PrintValue>("Tree1StateRootTaskA");
				Task1.GetInstanceData().Value = 88;
			}
			{
				UStateTreeState& StateB = Root.AddChildState("Tree1StateA", EStateTreeStateType::State);
				TStateTreeEditorNode<FTestTask_PrintValue>& Task = StateB.AddTask<FTestTask_PrintValue>("Tree1StateATaskA");
				Task.GetInstanceData().Value = 1;
				FStateTreeTransition& Transition = StateB.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::NextState);
				Transition.bDelayTransition = true;
				Transition.DelayDuration = 1.0;
			}
			{
				UStateTreeState& StateB = Root.AddChildState("Tree1StateB", EStateTreeStateType::State);
				TStateTreeEditorNode<FTestTask_PrintValue>& Task = StateB.AddTask<FTestTask_PrintValue>("Tree1StateBTaskA");
				Task.GetInstanceData().Value = 2;
				FStateTreeTransition& Transition = StateB.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::NextState);
				Transition.bDelayTransition = true;
				Transition.DelayDuration = 1.0;
			}
			{
				UStateTreeState& C1 = Root.AddChildState("Tree1StateLinkedTreeA", EStateTreeStateType::LinkedAsset);
				C1.SetLinkedStateAsset(&StateTree2);
				C1.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::NextState);
				C1.SetParametersPropertyOverridden(RootParameter_ValueID, true);
				C1.Parameters.Parameters.SetValueInt32("Value", 111);
			}
			{
				UStateTreeState& C2 = Root.AddChildState("Tree1StateLinkedTreeB", EStateTreeStateType::LinkedAsset);
				C2.SetLinkedStateAsset(&StateTree2);
				C2.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::NextState);
				C2.SetParametersPropertyOverridden(RootParameter_ValueID, true);
				C2.Parameters.Parameters.SetValueInt32("Value", 222);
			}
			{
				UStateTreeState& C3 = Root.AddChildState("Tree1StateLinkedTreeC", EStateTreeStateType::LinkedAsset);
				C3.SetLinkedStateAsset(&StateTree2);
				C3.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::NextState);
				C3.SetParametersPropertyOverridden(RootParameter_ValueID, true);
				C3.Parameters.Parameters.SetValueInt32("Value", 333);
			}
			{
				UStateTreeState& StateC = Root.AddChildState("Tree1StateC", EStateTreeStateType::State);
				TStateTreeEditorNode<FTestTask_PrintValue>& Task = StateC.AddTask<FTestTask_PrintValue>("Tree1StateCTaskA");
				Task.GetInstanceData().Value = 3;
				FStateTreeTransition& Transition = StateC.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::GotoState, &Root);
				Transition.bDelayTransition = true;
				Transition.DelayDuration = 1.0;
			}

			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree1);
			AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);
		}

		for (int32 Counter = 0; Counter < 2; ++Counter)
		{
			const bool bTickGlobalNodesWithHierarchy = Counter == 0;
			FScopedCVar CVarTickGlobalNodesWithHierarchy = FScopedCVar(TEXT("StateTree.TickGlobalNodesFollowingTreeHierarchy"), bTickGlobalNodesWithHierarchy);

			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
			{
				const bool bInitSucceeded = Exec.IsValid();
				AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);
			}
			{
				EStateTreeRunStatus Status = Exec.Start();
				AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("Start should be in the correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1StateA"));
				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("Tree1GlobalTaskA", TEXT("EnterState99"));
				AITEST_TRUE(TEXT("Start should enter Tree1GlobalTaskA"), LogOrder);
				LogOrder = LogOrder.Then("Tree1StateRootTaskA", TEXT("EnterState88"));
				AITEST_TRUE(TEXT("Start should enter Tree1StateRootTaskA"), LogOrder);
				LogOrder = LogOrder.Then("Tree1StateATaskA", TEXT("EnterState1"));
				AITEST_TRUE(TEXT("Start should enter Tree1StateATaskA"), LogOrder);
				Exec.LogClear();
			}
			{
				EStateTreeRunStatus Status = Exec.Tick(1.5f); // over tick, should trigger
				AITEST_EQUAL(TEXT("1st Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("1st Tick should be in the correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1StateA"));
				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("Tree1GlobalTaskA", TEXT("Tick99"));
				AITEST_TRUE(TEXT("1st should tick tasks Tree1GlobalTaskA"), LogOrder);
				LogOrder = LogOrder.Then("Tree1StateRootTaskA", TEXT("Tick88"));
				AITEST_TRUE(TEXT("1st should tick tasks Tree1StateRootTaskA"), LogOrder);
				LogOrder = LogOrder.Then("Tree1StateATaskA", TEXT("Tick1"));
				AITEST_TRUE(TEXT("1st should tick tasks Tree1StateATaskA"), LogOrder);
				Exec.LogClear();
			}
			{
				EStateTreeRunStatus Status = Exec.Tick(1.0f);
				AITEST_EQUAL(TEXT("2nd Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("2nd Tick should be in the correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1StateB"));
				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("Tree1GlobalTaskA", TEXT("Tick99"));
				AITEST_TRUE(TEXT("2nd Tick should tick Tree1GlobalTaskA"), LogOrder);
				LogOrder = LogOrder.Then("Tree1StateRootTaskA", TEXT("Tick88"));
				AITEST_TRUE(TEXT("2nd Tick should tick Tree1StateRootTaskA"), LogOrder);
				LogOrder = LogOrder.Then("Tree1StateATaskA", TEXT("Tick1"));
				AITEST_TRUE(TEXT("2nd Tick should tick Tree1StateATaskA"), LogOrder);
				LogOrder = LogOrder.Then("Tree1StateATaskA", TEXT("ExitState1"));
				AITEST_TRUE(TEXT("2nd Tick should exit Tree1StateATaskA"), LogOrder);
				LogOrder = LogOrder.Then("Tree1StateBTaskA", TEXT("EnterState2"));
				AITEST_TRUE(TEXT("2nd Tick should enter Tree1StateBTaskA"), LogOrder);
				Exec.LogClear();
			}
			{
				EStateTreeRunStatus Status = Exec.Tick(1.0f);
				AITEST_EQUAL(TEXT("3rd Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("3rd Tick should be in the correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1StateB"));
				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("Tree1GlobalTaskA", TEXT("Tick99"));
				AITEST_TRUE(TEXT("3rd should tick Tree1GlobalTaskA"), LogOrder);
				LogOrder = LogOrder.Then("Tree1StateRootTaskA", TEXT("Tick88"));
				AITEST_TRUE(TEXT("3rd should tick Tree1StateRootTaskA"), LogOrder);
				LogOrder = LogOrder.Then("Tree1StateBTaskA", TEXT("Tick2"));
				AITEST_TRUE(TEXT("3rd should tick Tree1StateBTaskA"), LogOrder);
				Exec.LogClear();
			}
			{
				EStateTreeRunStatus Status = Exec.Tick(1.0f);
				AITEST_EQUAL(TEXT("4th Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("4th Tick should be in the correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1StateLinkedTreeA", "Tree2StateRoot", "Tree2StateA"));
				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("Tree1GlobalTaskA", TEXT("Tick99"));
				AITEST_TRUE(TEXT("4th Tick should tick Tree1GlobalTaskA"), LogOrder);
				LogOrder = LogOrder.Then("Tree1StateRootTaskA", TEXT("Tick88"));
				AITEST_TRUE(TEXT("4th Tick should tick Tree1StateRootTaskA"), LogOrder);
				LogOrder = LogOrder.Then("Tree1StateBTaskA", TEXT("Tick2"));
				AITEST_TRUE(TEXT("4th Tick should tick Tree1StateBTaskA"), LogOrder);
				LogOrder = LogOrder.Then("Tree1StateBTaskA", TEXT("ExitState2"));
				AITEST_TRUE(TEXT("4th Tick should exit Tree1StateBTaskA"), LogOrder);
				LogOrder = LogOrder.Then("Tree2StateRootTaskA", TEXT("EnterState111"));
				AITEST_TRUE(TEXT("4th Tick should enter Tree2StateRootTaskA"), LogOrder);
				LogOrder = LogOrder.Then("Tree2StateATaskA", TEXT("EnterState111"));
				AITEST_TRUE(TEXT("4th Tick should enter Tree2StateATaskA"), LogOrder);
				AITEST_TRUE(TEXT("4th Tick should tick tasks"), Exec.Expect("Tree2GlobalTaskA", TEXT("EnterState111")));
				Exec.LogClear();
			}
			{
				EStateTreeRunStatus Status = Exec.Tick(0.001f);
				AITEST_EQUAL(TEXT("5th Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("5th Tick should be in the correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1StateLinkedTreeA", "Tree2StateRoot", "Tree2StateA"));
				if (bTickGlobalNodesWithHierarchy)
				{
					FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("Tree1GlobalTaskA", TEXT("Tick99"));
					AITEST_TRUE(TEXT("5h Tick should tick Tree1GlobalTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree1StateRootTaskA", TEXT("Tick88"));
					AITEST_TRUE(TEXT("5h Tick should tick Tree1StateRootTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2GlobalTaskA", TEXT("Tick111"));
					AITEST_TRUE(TEXT("5h Tick should tick Tree2GlobalTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2StateRootTaskA", TEXT("Tick111"));
					AITEST_TRUE(TEXT("5h Tick should tick Tree2StateRootTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2StateATaskA", TEXT("Tick111"));
					AITEST_TRUE(TEXT("5h Tick should tick Tree2StateATaskA"), LogOrder);
				}
				else
				{
					FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("Tree1GlobalTaskA", TEXT("Tick99"));
					AITEST_TRUE(TEXT("5h Tick should tick Tree1GlobalTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2GlobalTaskA", TEXT("Tick111"));
					AITEST_TRUE(TEXT("5h Tick should tick Tree2GlobalTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree1StateRootTaskA", TEXT("Tick88"));
					AITEST_TRUE(TEXT("5h Tick should tick Tree1StateRootTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2StateRootTaskA", TEXT("Tick111"));
					AITEST_TRUE(TEXT("5h Tick should tick Tree2StateRootTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2StateATaskA", TEXT("Tick111"));
					AITEST_TRUE(TEXT("5h Tick should tick Tree2StateATaskA"), LogOrder);
				}
				Exec.LogClear();
			}
			{
				EStateTreeRunStatus Status = Exec.Tick(1.0f);
				AITEST_EQUAL(TEXT("6th Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("6th Tick should be in the correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1StateLinkedTreeB", "Tree2StateRoot", "Tree2StateA"));
				if (bTickGlobalNodesWithHierarchy)
				{
					FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("Tree1GlobalTaskA", TEXT("Tick99"));
					AITEST_TRUE(TEXT("6th Tick should tick Tree1GlobalTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree1StateRootTaskA", TEXT("Tick88"));
					AITEST_TRUE(TEXT("6th Tick should tick Tree1StateRootTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2GlobalTaskA", TEXT("Tick111"));
					AITEST_TRUE(TEXT("6th Tick should tick Tree2GlobalTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2StateRootTaskA", TEXT("Tick111"));
					AITEST_TRUE(TEXT("6th Tick should tick Tree2StateRootTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2StateATaskA", TEXT("Tick111"));
					AITEST_TRUE(TEXT("6th Tick should tick Tree2StateATaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2StateATaskA", TEXT("ExitState111"));
					AITEST_TRUE(TEXT("6th Tick should exit Tree2StateATaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2StateRootTaskA", TEXT("ExitState111"));
					AITEST_TRUE(TEXT("6th Tick should exit Tree2StateRootTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2GlobalTaskA", TEXT("ExitState111"));
					AITEST_TRUE(TEXT("6th Tick should exit Tree2GlobalTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2StateRootTaskA", TEXT("EnterState222"));
					AITEST_TRUE(TEXT("6th Tick should enter Tree2StateRootTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2StateATaskA", TEXT("EnterState222"));
					AITEST_TRUE(TEXT("6th Tick should enter Tree2StateATaskA"), LogOrder);
					AITEST_TRUE(TEXT("6th Tick should enter Tree2GlobalTaskA"), Exec.Expect("Tree2GlobalTaskA", TEXT("EnterState222")));
				}
				else
				{
					FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("Tree1GlobalTaskA", TEXT("Tick99"));
					AITEST_TRUE(TEXT("6th Tick should tick Tree1GlobalTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2GlobalTaskA", TEXT("Tick111"));
					AITEST_TRUE(TEXT("6th Tick should tick Tree2GlobalTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree1StateRootTaskA", TEXT("Tick88"));
					AITEST_TRUE(TEXT("6th Tick should tick Tree1StateRootTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2StateRootTaskA", TEXT("Tick111"));
					AITEST_TRUE(TEXT("6th Tick should tick Tree2StateRootTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2StateATaskA", TEXT("Tick111"));
					AITEST_TRUE(TEXT("6th Tick should tick Tree2StateATaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2StateATaskA", TEXT("ExitState111"));
					AITEST_TRUE(TEXT("6th Tick should exit Tree2StateATaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2StateRootTaskA", TEXT("ExitState111"));
					AITEST_TRUE(TEXT("6th Tick should exit Tree2StateRootTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2GlobalTaskA", TEXT("ExitState111"));
					AITEST_TRUE(TEXT("6th Tick should exit Tree2GlobalTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2StateRootTaskA", TEXT("EnterState222"));
					AITEST_TRUE(TEXT("6th Tick should enter Tree2StateRootTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2StateATaskA", TEXT("EnterState222"));
					AITEST_TRUE(TEXT("6th Tick should enter Tree2StateATaskA"), LogOrder);
					AITEST_TRUE(TEXT("6th Tick should enter Tree2GlobalTaskA"), Exec.Expect("Tree2GlobalTaskA", TEXT("EnterState222")));
				}
				Exec.LogClear();
			}
			Exec.Stop();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_MultipleSameLinkedAsset, "System.StateTree.LinkedAsset.MultipleSameTree");

struct FStateTreeTest_EmptyStateWithTickTransitionLinkedAsset : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		//Tree 1
		//	Root
		//		StateA -> Next
		//		StateLinkedTree (Tree2) -> Next
		//		StateB -> Root
		//Tree 2
		//  Global task and parameter
		//	Root
		//		FailState (condition false)
		//		StateA (condition true and with transition OnTick to succeeded)

		FStateTreeCompilerLog Log;

		// Asset 2
		UStateTree& StateTree2 = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree2.EditorData);
			TStateTreeEditorNode<FTestTask_PrintValue>& GlobalTask = EditorData.AddGlobalTask<FTestTask_PrintValue>("Tree2GlobalTaskA");
			GlobalTask.GetInstanceData().Value = 21;

			UStateTreeState& Root = EditorData.AddSubTree("Tree2StateRoot");
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& Task = Root.AddTask<FTestTask_PrintValue>("Tree2StateRootTaskA");
				Task.GetInstanceData().Value = 22;
			}
			{
				UStateTreeState& State = Root.AddChildState("Tree2StateFail", EStateTreeStateType::State);
				// Add auto fails condition
				auto& Condition = State.AddEnterCondition<FStateTreeTestBooleanCondition>();
				Condition.GetInstanceData().bSuccess = false;

				// Should never see
				TStateTreeEditorNode<FTestTask_PrintValue>& Task = State.AddTask<FTestTask_PrintValue>("Tree2StateFailTaskA");
				Task.GetInstanceData().Value = 23;
			}
			{
				UStateTreeState& State = Root.AddChildState("Tree2StateB", EStateTreeStateType::State);

				// Add auto success condition
				auto& Condition = State.AddEnterCondition<FStateTreeTestBooleanCondition>();
				Condition.GetInstanceData().bSuccess = true;

				FStateTreeTransition& Transition = State.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::Succeeded);
				Transition.bDelayTransition = true;
				Transition.DelayDuration = 1.0;
			}


			FStateTreeCompiler Compiler2(Log);
			const bool bResult2 = Compiler2.Compile(StateTree2);
			AITEST_TRUE(TEXT("StateTree2 should get compiled"), bResult2);
		}

		// Main asset
		UStateTree& StateTree1 = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree1.EditorData);

			{
				TStateTreeEditorNode<FTestTask_PrintValue>& GlobalTask = EditorData.AddGlobalTask<FTestTask_PrintValue>("Tree1GlobalTaskA");
				GlobalTask.GetInstanceData().Value = 11;
			}
			UStateTreeState& Root = EditorData.AddSubTree("Tree1StateRoot");
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& Task = Root.AddTask<FTestTask_PrintValue>("Tree1StateRootTaskA");
				Task.GetInstanceData().Value = 12;
			}
			{
				UStateTreeState& State = Root.AddChildState("Tree1StateA", EStateTreeStateType::State);
				TStateTreeEditorNode<FTestTask_PrintValue>& Task = State.AddTask<FTestTask_PrintValue>("Tree1StateATaskA");
				Task.GetInstanceData().Value = 13;

				FStateTreeTransition& Transition = State.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::NextState);
				Transition.bDelayTransition = true;
				Transition.DelayDuration = 1.0;
			}
			{
				UStateTreeState& C1 = Root.AddChildState("Tree1StateLinkedTree", EStateTreeStateType::LinkedAsset);
				C1.SetLinkedStateAsset(&StateTree2);
				C1.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::NextState);
			}
			{
				UStateTreeState& State = Root.AddChildState("Tree1StateB", EStateTreeStateType::State);
				TStateTreeEditorNode<FTestTask_PrintValue>& Task = State.AddTask<FTestTask_PrintValue>("Tree1StateBTaskA");
				Task.GetInstanceData().Value = 14;
				FStateTreeTransition& Transition = State.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::GotoState, &Root);
				Transition.bDelayTransition = true;
				Transition.DelayDuration = 1.0;
			}

			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree1);
			AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);
		}

		for (int32 Counter = 0; Counter < 2; ++Counter)
		{
			const bool bTickGlobalNodesWithHierarchy = Counter == 0;
			FScopedCVar CVarTickGlobalNodesWithHierarchy = FScopedCVar(TEXT("StateTree.TickGlobalNodesFollowingTreeHierarchy"), bTickGlobalNodesWithHierarchy);

			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
			{
				const bool bInitSucceeded = Exec.IsValid();
				AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);
			}
			{
				EStateTreeRunStatus Status = Exec.Start();
				AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("Tree1GlobalTaskA", TEXT("EnterState11"));
				AITEST_TRUE(TEXT("Start enters in the correct order Tree1GlobalTaskA"), LogOrder);
				LogOrder = LogOrder.Then("Tree1StateRootTaskA", TEXT("EnterState12"));
				AITEST_TRUE(TEXT("Start enters in the correct order Tree1StateRootTaskA"), LogOrder);
				LogOrder = LogOrder.Then("Tree1StateATaskA", TEXT("EnterState13"));
				AITEST_TRUE(TEXT("Start enters in the correct order Tree1StateATaskA"), LogOrder);
				AITEST_TRUE(TEXT("Start should be in the correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1StateA"));
				Exec.LogClear();
			}
			{
				EStateTreeRunStatus Status = Exec.Tick(1.5f); // over tick, should trigger
				AITEST_EQUAL(TEXT("1st Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("Tree1GlobalTaskA", TEXT("Tick11"));
				AITEST_TRUE(TEXT("1st Tick should tick Tree1GlobalTaskA"), LogOrder);
				LogOrder = LogOrder.Then("Tree1StateRootTaskA", TEXT("Tick12"));
				AITEST_TRUE(TEXT("1st Tick should tick Tree1StateRootTaskA"), LogOrder);
				LogOrder = LogOrder.Then("Tree1StateATaskA", TEXT("Tick13"));
				AITEST_TRUE(TEXT("1st Tick should tick Tree1StateATaskA"), LogOrder);
				AITEST_TRUE(TEXT("Start should be in the correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1StateA"));
				Exec.LogClear();
			}
			{
				EStateTreeRunStatus Status = Exec.Tick(1.0f);
				AITEST_EQUAL(TEXT("2nd Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("Tree1GlobalTaskA", TEXT("Tick11"));
				AITEST_TRUE(TEXT("2nd Tick should tick Tree1GlobalTaskA"), LogOrder);
				LogOrder = LogOrder.Then("Tree1StateRootTaskA", TEXT("Tick12"));
				AITEST_TRUE(TEXT("2nd Tick should tick Tree1StateRootTaskA"), LogOrder);
				LogOrder = LogOrder.Then("Tree1StateATaskA", TEXT("Tick13"));
				AITEST_TRUE(TEXT("2nd Tick should tick Tree1StateATaskA"), LogOrder);
				LogOrder = LogOrder.Then("Tree2GlobalTaskA", TEXT("EnterState21"));
				AITEST_TRUE(TEXT("2nd Tick should enter Tree2GlobalTaskA"), LogOrder);
				LogOrder = LogOrder.Then("State Tree Test Boolean Condition", TEXT("TestCondition=0"));
				AITEST_TRUE(TEXT("2nd Tick should test Bool"), LogOrder);
				LogOrder = LogOrder.Then("State Tree Test Boolean Condition", TEXT("TestCondition=1"));
				AITEST_TRUE(TEXT("2nd Tick should test Bool"), LogOrder);
				LogOrder = LogOrder.Then("Tree1StateATaskA", TEXT("ExitState13"));
				AITEST_TRUE(TEXT("2nd Tick should exit Tree1StateATaskA"), LogOrder);
				LogOrder = LogOrder.Then("Tree2StateRootTaskA", TEXT("EnterState22"));
				AITEST_TRUE(TEXT("2nd Tick should enter Tree2StateRootTaskA"), LogOrder);
				AITEST_FALSE(TEXT("2nd Tick should not enter the fail state."), Exec.Expect("Tree2StateFailTaskA"));
				AITEST_TRUE(TEXT("2nd Tick should be in the correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1StateLinkedTree", "Tree2StateRoot", "Tree2StateB"));
				Exec.LogClear();
			}
			{
				EStateTreeRunStatus Status = Exec.Tick(1.0f);
				AITEST_EQUAL(TEXT("3rd Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("3rd Tick should be in the correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1StateLinkedTree", "Tree2StateRoot", "Tree2StateB"));
				if (bTickGlobalNodesWithHierarchy)
				{
					FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("Tree1GlobalTaskA", TEXT("Tick11"));
					AITEST_TRUE(TEXT("3rd Tick should tick Tree1GlobalTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree1StateRootTaskA", TEXT("Tick12"));
					AITEST_TRUE(TEXT("3rd Tick should tick Tree1StateRootTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2GlobalTaskA", TEXT("Tick21"));
					AITEST_TRUE(TEXT("3rd Tick should tick Tree2GlobalTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2StateRootTaskA", TEXT("Tick22"));
					AITEST_TRUE(TEXT("3rd Tick should tick Tree2StateRootTaskA"), LogOrder);
				}
				else
				{
					FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("Tree1GlobalTaskA", TEXT("Tick11"));
					AITEST_TRUE(TEXT("3rd Tick should tick Tree1GlobalTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2GlobalTaskA", TEXT("Tick21"));
					AITEST_TRUE(TEXT("3rd Tick should tick Tree2GlobalTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree1StateRootTaskA", TEXT("Tick12"));
					AITEST_TRUE(TEXT("3rd Tick should tick Tree1StateRootTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2StateRootTaskA", TEXT("Tick22"));
					AITEST_TRUE(TEXT("3rd Tick should tick Tree2StateRootTaskA"), LogOrder);
				}
				Exec.LogClear();
			}
			{
				EStateTreeRunStatus Status = Exec.Tick(1.0f);
				AITEST_EQUAL(TEXT("4th Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				if (bTickGlobalNodesWithHierarchy)
				{
					FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("Tree1GlobalTaskA", TEXT("Tick11"));
					AITEST_TRUE(TEXT("4th Tick should tick Tree1GlobalTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree1StateRootTaskA", TEXT("Tick12"));
					AITEST_TRUE(TEXT("4th Tick should tick Tree1StateRootTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2GlobalTaskA", TEXT("Tick21"));
					AITEST_TRUE(TEXT("4th Tick should tick Tree2GlobalTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2StateRootTaskA", TEXT("Tick22"));
					AITEST_TRUE(TEXT("4th Tick should tick Tree2StateRootTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2StateRootTaskA", TEXT("ExitState22"));
					AITEST_TRUE(TEXT("4th Tick should exit Tree2StateRootTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2GlobalTaskA", TEXT("ExitState21"));
					AITEST_TRUE(TEXT("4th Tick should exit Tree2GlobalTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree1StateBTaskA", TEXT("EnterState14"));
					AITEST_TRUE(TEXT("4th Tick should enter Tree1StateBTaskA"), LogOrder);
				}
				else
				{
					FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("Tree1GlobalTaskA", TEXT("Tick11"));
					AITEST_TRUE(TEXT("4th Tick should tick Tree1GlobalTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2GlobalTaskA", TEXT("Tick21"));
					AITEST_TRUE(TEXT("4th Tick should tick Tree2GlobalTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree1StateRootTaskA", TEXT("Tick12"));
					AITEST_TRUE(TEXT("4th Tick should tick Tree1StateRootTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2StateRootTaskA", TEXT("Tick22"));
					AITEST_TRUE(TEXT("4th Tick should tick Tree2StateRootTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2StateRootTaskA", TEXT("ExitState22"));
					AITEST_TRUE(TEXT("4th Tick should exit Tree2StateRootTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree2GlobalTaskA", TEXT("ExitState21"));
					AITEST_TRUE(TEXT("4th Tick should exit Tree2GlobalTaskA"), LogOrder);
					LogOrder = LogOrder.Then("Tree1StateBTaskA", TEXT("EnterState14"));
					AITEST_TRUE(TEXT("4th Tick should enter Tree1StateBTaskA"), LogOrder);
				}
				AITEST_TRUE(TEXT("4th Tick should be in the correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1StateB"));
				Exec.LogClear();
			}
			{
				EStateTreeRunStatus Status = Exec.Tick(0.001f);
				AITEST_EQUAL(TEXT("5th Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("Tree1GlobalTaskA", TEXT("Tick11"));
				AITEST_TRUE(TEXT("5th Tick should tick Tree1GlobalTaskA"), LogOrder);
				LogOrder = LogOrder.Then("Tree1StateRootTaskA", TEXT("Tick12"));
				AITEST_TRUE(TEXT("5th Tick should tick Tree1StateRootTaskA"), LogOrder);
				LogOrder = LogOrder.Then("Tree1StateBTaskA", TEXT("Tick14"));
				AITEST_TRUE(TEXT("5th Tick should tick Tree1StateBTaskA"), LogOrder);
				AITEST_TRUE(TEXT("5th Tick should be in the correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1StateB"));
				Exec.LogClear();
			}
			Exec.Stop();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_EmptyStateWithTickTransitionLinkedAsset, "System.StateTree.LinkedAsset.EmptyStateWithTickTransition");

struct FStateTreeTest_RecursiveLinkedAsset : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		//Tree 1
		//	Root
		//		StateLinkedTree1 (Tree2) -> Next
		//		StateA -> Succeeded
		//Tree 2
		//	Root
		//		StateLinkedTreeA (Tree1) -> Next
		//		StateA -> Succeeded

		UStateTree& StateTree1 = NewStateTree();
		UStateTreeState* Root1 = nullptr;
		// Asset 1 definition
		{
			UStateTreeEditorData& EditorData1 = *Cast<UStateTreeEditorData>(StateTree1.EditorData);
			Root1 = &EditorData1.AddSubTree("Tree1StateRoot");

			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree1);
			AITEST_TRUE(TEXT("StateTree 1 should get compiled"), bResult);
		}

		UStateTree& StateTree2 = NewStateTree();
		UStateTreeState* Root2 = nullptr;
		// Asset 2 definition
		{
			UStateTreeEditorData& EditorData2 = *Cast<UStateTreeEditorData>(StateTree2.EditorData);
			Root2 = &EditorData2.AddSubTree("Tree2StateRoot");

			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree2);
			AITEST_TRUE(TEXT("StateTree 2 should get compiled"), bResult);
		}
		// Asset 1 implementation
		{
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& Task1 = Root1->AddTask<FTestTask_PrintValue>("Tree1StateRootTaskA");
				Task1.GetInstanceData().Value = 101;
			}
			{
				UStateTreeState& C1 = Root1->AddChildState("Tree1StateLinkedTree1", EStateTreeStateType::LinkedAsset);
				C1.Tag = GetTestTag1();
				C1.SetLinkedStateAsset(&StateTree2);
				C1.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::NextState);
			}
			{
				UStateTreeState& StateA = Root1->AddChildState("Tree1StateA", EStateTreeStateType::State);
				TStateTreeEditorNode<FTestTask_PrintValue>& Task = StateA.AddTask<FTestTask_PrintValue>("Tree1StateA");
				Task.GetInstanceData().Value = 102;
				FStateTreeTransition& Transition = StateA.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::GotoState, Root1);
				Transition.bDelayTransition = true;
				Transition.DelayDuration = 1.0f;
			}

			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree1);
			AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);
		}
		// Asset 2 implementation
		UStateTreeState* Tree2StateLinkedTree1 = nullptr;
		{
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& Task1 = Root2->AddTask<FTestTask_PrintValue>("Tree2StateRootTaskA");
				Task1.GetInstanceData().Value = 201;
			}
			{
				UStateTreeState& C1 = Root2->AddChildState("Tree2StateLinkedTree1", EStateTreeStateType::LinkedAsset);
				C1.Tag = GetTestTag2();
				C1.SetLinkedStateAsset(&StateTree2);
				C1.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::NextState);
				Tree2StateLinkedTree1 = &C1;
			}
			{
				UStateTreeState& StateD = Root2->AddChildState("Tree2StateA", EStateTreeStateType::State);
				TStateTreeEditorNode<FTestTask_PrintValue>& Task = StateD.AddTask<FTestTask_PrintValue>("Tree2StateA");
				Task.GetInstanceData().Value = 202;
				FStateTreeTransition& Transition = StateD.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::GotoState, Root2);
				Transition.bDelayTransition = true;
				Transition.DelayDuration = 1.0f;
			}

			// circular dependency detected
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree2);
			AITEST_FALSE(TEXT("StateTree should not compiled"), bResult);
		}
		// Fix circular dependency
		{
			Tree2StateLinkedTree1->SetLinkedStateAsset(&StateTree1);
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree2);
			AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);
		}
		// Run test
		{
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
			{
				const bool bInitSucceeded = Exec.IsValid();
				AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);
			}

			{
				GetTestRunner().AddExpectedErrorPlain(TEXT("Trying to recursively enter subtree"), EAutomationExpectedErrorFlags::Contains, 1);

				EStateTreeRunStatus Status = Exec.Start();
				AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("Tree1StateRootTaskA", TEXT("EnterState101"));
				AITEST_TRUE(TEXT("Start should enter Tree1StateRootTaskA"), LogOrder);
				LogOrder = LogOrder.Then("Tree2StateRootTaskA", TEXT("EnterState201"));
				AITEST_TRUE(TEXT("Start should enter Tree2StateRootTaskA"), LogOrder);
				LogOrder = LogOrder.Then("Tree2StateA", TEXT("EnterState202"));
				AITEST_TRUE(TEXT("Start should enter Tree2StateA"), LogOrder);
				Exec.LogClear();
				AITEST_TRUE(TEXT("Doesn't have the expected error message."), GetTestRunner().HasMetExpectedMessages());
			}
			Exec.Stop();
		}

		return true;
	}
};
// This test invokes editor-only methods which aren't AutoRTFM-safe, so we skip it when AutoRTFM is on.
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_RecursiveLinkedAsset, "System.StateTree.LinkedAsset.RecursiveLinkedAsset");

struct FStateTreeTest_LinkedAssetTransitionSameTick : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		//Tree 1
		//	Root
		//		State1 -> Delay 1 -> StateLinkedTree1
		//		LinkState2 (Tree2) -> Next
		//		State3 -> Root
		//Tree 2
		//	Root
		//		State1 -> Succeeded

		UStateTree& StateTree2 = NewStateTree();
		UStateTreeState* Root2 = nullptr;
		// Asset 2
		{
			UStateTreeEditorData& EditorData2 = *Cast<UStateTreeEditorData>(StateTree2.EditorData);
			Root2 = &EditorData2.AddSubTree("Tree2StateRoot");

			{
				UStateTreeState& C1 = Root2->AddChildState("Tree2State1", EStateTreeStateType::State);
				C1.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::Succeeded);
			}
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree2);
			AITEST_TRUE(TEXT("StateTree should not compiled"), bResult);
		}
		// Asset 1
		UStateTree& StateTree1 = NewStateTree();
		{
			UStateTreeState* Root1 = nullptr;
			{
				UStateTreeEditorData& EditorData1 = *Cast<UStateTreeEditorData>(StateTree1.EditorData);
				Root1 = &EditorData1.AddSubTree("Tree1StateRoot");

				FStateTreeCompilerLog Log;
				FStateTreeCompiler Compiler(Log);
				const bool bResult = Compiler.Compile(StateTree1);
				AITEST_TRUE(TEXT("StateTree 1 should get compiled"), bResult);
			}
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& Task1 = Root1->AddTask<FTestTask_PrintValue>("Tree1StateRootTask1");
				Task1.GetInstanceData().Value = 100;
			}
			{
				UStateTreeState& State1 = Root1->AddChildState("Tree1State1", EStateTreeStateType::State);
				TStateTreeEditorNode<FTestTask_PrintValue>& Task = State1.AddTask<FTestTask_PrintValue>("Tree1State1Task1");
				Task.GetInstanceData().Value = 101;
				Task.GetInstanceData().TickRunStatus = EStateTreeRunStatus::Succeeded;
				State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::NextState);
			}
			{
				UStateTreeState& LinkState2 = Root1->AddChildState("Tree1State2LinkedTree2", EStateTreeStateType::LinkedAsset);
				LinkState2.SetLinkedStateAsset(&StateTree2);
				LinkState2.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::NextState);
			}
			{
				UStateTreeState& State3 = Root1->AddChildState("Tree1State3", EStateTreeStateType::State);
				TStateTreeEditorNode<FTestTask_PrintValue>& Task = State3.AddTask<FTestTask_PrintValue>("Tree1State3Task1");
				Task.GetInstanceData().Value = 103;
				FStateTreeTransition& Transition = State3.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::GotoState, Root1);
				Transition.bDelayTransition = true;
				Transition.DelayDuration = 1.0f;
			}

			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree1);
			AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);
		}

		// Run test
		{
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
			{
				const bool bInitSucceeded = Exec.IsValid();
				AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);
			}

			{
				EStateTreeRunStatus Status = Exec.Start();
				AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1State1"));
				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("Tree1StateRootTask1", TEXT("EnterState100"));
				AITEST_TRUE(TEXT("Start should enter Tree1StateRootTask1"), LogOrder);
				LogOrder = LogOrder.Then("Tree1State1Task1", TEXT("EnterState101"));
				AITEST_TRUE(TEXT("Start should enter Tree1State1"), LogOrder);
				Exec.LogClear();
			}
			{
				EStateTreeRunStatus Status = Exec.Tick(1.01f);
				AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1State2LinkedTree2", "Tree2StateRoot", "Tree2State1"));
				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("Tree1StateRootTask1", TEXT("Tick100"));
				AITEST_TRUE(TEXT("Start should tick Tree1StateRootTask1"), LogOrder);
				LogOrder = LogOrder.Then("Tree1State1Task1", TEXT("Tick101"));
				AITEST_TRUE(TEXT("Start should tick Tree1State1"), LogOrder);
				LogOrder = LogOrder.Then("Tree1State1Task1", TEXT("ExitState101"));
				AITEST_TRUE(TEXT("Start should exit Tree1State1"), LogOrder);
				Exec.LogClear();
			}
			//Tree2State1 -> Succeeded should transition to Tree1State3
			{
				EStateTreeRunStatus Status = Exec.Tick(1.01f);
				AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1State3"));
				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("Tree1StateRootTask1", TEXT("Tick100"));
				AITEST_TRUE(TEXT("Start should tick Tree1StateRootTask1"), LogOrder);
				LogOrder = LogOrder.Then("Tree1State3Task1", TEXT("EnterState103"));
				AITEST_TRUE(TEXT("Start should enter Tree1State3"), LogOrder);
				Exec.LogClear();
			}
			{
				EStateTreeRunStatus Status = Exec.Tick(1.01f);
				AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1State3"));
				Exec.LogClear();
			}
			{
				EStateTreeRunStatus Status = Exec.Tick(1.01f);
				AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1State1"));
				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("Tree1StateRootTask1", TEXT("Tick100"));
				AITEST_TRUE(TEXT("Start should tick Tree1StateRootTask1"), LogOrder);
				LogOrder = LogOrder.Then("Tree1State3Task1", TEXT("ExitState103"));
				AITEST_TRUE(TEXT("Start should exit Tree1State3"), LogOrder);
				LogOrder = LogOrder.Then("Tree1State1Task1", TEXT("EnterState101"));
				AITEST_TRUE(TEXT("Start should enter Tree1State1"), LogOrder);
				Exec.LogClear();
			}
			Exec.Stop();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_LinkedAssetTransitionSameTick, "System.StateTree.LinkedAsset.TransitionSameTick");

struct FStateTreeTest_Linked_GlobalParameter : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		//Tree 1
		//  Global task and parameter
		//	Root
		//		State1 LinkedAsset(Tree2) [OnSucceeded goto Next]
		//		State2 Linked(StateSub1) [OnSucceeded goto succeeded]
		//	StateSub1
		//		State3 [OnTick goto succeeded]
		//Tree 2
		//  Global task and parameter
		//	Root
		//		State1 [OnTick goto succeeded]

		auto AddInt = [](FInstancedPropertyBag& PropertyBag, FName VarName)
			{
				PropertyBag.AddProperty(VarName, EPropertyBagPropertyType::Int32);
				PropertyBag.SetValueInt32(VarName, -99);
				return PropertyBag.FindPropertyDescByName(VarName)->ID;
			};
		auto AddDouble = [](FInstancedPropertyBag& PropertyBag, FName VarName)
			{
				PropertyBag.AddProperty(VarName, EPropertyBagPropertyType::Double);
				PropertyBag.SetValueDouble(VarName, -99.0);
				return PropertyBag.FindPropertyDescByName(VarName)->ID;
			};

		FGuid Tree2GlobalParameter_ValueID_Int;

		// Tree 2
		UStateTree& StateTree2 = NewStateTree();
		{
			UStateTreeEditorData& EditorData2 = *Cast<UStateTreeEditorData>(StateTree2.EditorData);
			
			// note double before int
			AddDouble(GetRootPropertyBag(EditorData2), "Tree2GlobalDouble");
			Tree2GlobalParameter_ValueID_Int = AddInt(GetRootPropertyBag(EditorData2), "Tree2GlobalInt");

			// Global tasks
			TStateTreeEditorNode<FTestTask_PrintValue>& GlobalTask1 = EditorData2.AddGlobalTask<FTestTask_PrintValue>("Tree2GlobalTask1");
			{
				GlobalTask1.GetInstanceData().Value = -1;
				EditorData2.AddPropertyBinding(FPropertyBindingPath(EditorData2.GetRootParametersGuid(), "Tree2GlobalInt"), FPropertyBindingPath(GlobalTask1.ID, "Value"));
			}
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& GlobalTask2 = EditorData2.AddGlobalTask<FTestTask_PrintValue>("Tree2GlobalTask2");
				GlobalTask2.GetInstanceData().Value = -2;
				EditorData2.AddPropertyBinding(FPropertyBindingPath(GlobalTask1.ID, "Value"), FPropertyBindingPath(GlobalTask2.ID, "Value"));
			}

			UStateTreeState& Root = EditorData2.AddSubTree("Tree2StateRoot");
			{
				AddInt(Root.Parameters.Parameters, "Tree2StateRootParametersInt");
				AddDouble(Root.Parameters.Parameters, "Tree2StateRootParametersDouble");
				Root.bCopyParameterBindingsOnTick = true;
				EditorData2.AddPropertyBinding(FPropertyBindingPath(EditorData2.GetRootParametersGuid(), "Tree2GlobalInt"), FPropertyBindingPath(Root.Parameters.ID, "Tree2StateRootParametersInt"));
			}
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& Task1 = Root.AddTask<FTestTask_PrintValue>("Tree2StateRootTask1");
				Task1.GetInstanceData().Value = -1;
				EditorData2.AddPropertyBinding(FPropertyBindingPath(EditorData2.GetRootParametersGuid(), "Tree2GlobalInt"), FPropertyBindingPath(Task1.ID, "Value"));

				TStateTreeEditorNode<FTestTask_PrintValue>& Task2 = Root.AddTask<FTestTask_PrintValue>("Tree2StateRootTask2");
				Task2.GetInstanceData().Value = -2;
				EditorData2.AddPropertyBinding(FPropertyBindingPath(Root.Parameters.ID, "Tree2StateRootParametersInt"), FPropertyBindingPath(Task2.ID, "Value"));
				
				TStateTreeEditorNode<FTestTask_PrintValue>& Task3 = Root.AddTask<FTestTask_PrintValue>("Tree2StateRootTask3");
				Task3.GetInstanceData().Value = -3;
				EditorData2.AddPropertyBinding(FPropertyBindingPath(GlobalTask1.ID, "Value"), FPropertyBindingPath(Task3.ID, "Value"));
			}
			{
				UStateTreeState& State1 = Root.AddChildState("Tree2State1", EStateTreeStateType::State);
				{
					AddDouble(State1.Parameters.Parameters, "Tree2State1ParametersDouble");
					AddInt(State1.Parameters.Parameters, "Tree2State1ParametersInt");
					State1.bCopyParameterBindingsOnTick = true;
					EditorData2.AddPropertyBinding(FPropertyBindingPath(Root.Parameters.ID, "Tree2StateRootParametersInt"), FPropertyBindingPath(State1.Parameters.ID, "Tree2State1ParametersInt"));
				}

				TStateTreeEditorNode<FTestTask_PrintValue>& Task1 = State1.AddTask<FTestTask_PrintValue>("Tree2State1Task1");
				Task1.GetInstanceData().Value = -1;
				EditorData2.AddPropertyBinding(FPropertyBindingPath(EditorData2.GetRootParametersGuid(), "Tree2GlobalInt"), FPropertyBindingPath(Task1.ID, TEXT("Value")));

				TStateTreeEditorNode<FTestTask_PrintValue>& Task2 = State1.AddTask<FTestTask_PrintValue>("Tree2State1Task2");
				Task2.GetInstanceData().Value = -2;
				EditorData2.AddPropertyBinding(FPropertyBindingPath(Root.Parameters.ID, "Tree2StateRootParametersInt"), FPropertyBindingPath(Task2.ID, TEXT("Value")));

				TStateTreeEditorNode<FTestTask_PrintValue>& Task3 = State1.AddTask<FTestTask_PrintValue>("Tree2State1Task3");
				Task3.GetInstanceData().Value = -3;
				EditorData2.AddPropertyBinding(FPropertyBindingPath(GlobalTask1.ID, "Value"), FPropertyBindingPath(Task3.ID, "Value"));

				TStateTreeEditorNode<FTestTask_PrintValue>& Task4 = State1.AddTask<FTestTask_PrintValue>("Tree2State1Task4");
				Task3.GetInstanceData().Value = -4;
				EditorData2.AddPropertyBinding(FPropertyBindingPath(State1.Parameters.ID, "Tree2State1ParametersInt"), FPropertyBindingPath(Task4.ID, "Value"));

				State1.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::Succeeded);
			}

			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler2(Log);
			const bool bResult2 = Compiler2.Compile(StateTree2);
			AITEST_TRUE(TEXT("StateTree2 should get compiled"), bResult2);
		}

		// Tree 1
		UStateTree& StateTree1 = NewStateTree();
		FGuid Tree1GlobalParameter_ValueID_Int;
		{
			UStateTreeEditorData& EditorData1 = *Cast<UStateTreeEditorData>(StateTree1.EditorData);

			Tree1GlobalParameter_ValueID_Int = AddInt(GetRootPropertyBag(EditorData1), "Tree1GlobalInt");
			AddDouble(GetRootPropertyBag(EditorData1), "Tree1GlobalDouble");

			// Global tasks
			TStateTreeEditorNode<FTestTask_PrintValue>& GlobalTask1 = EditorData1.AddGlobalTask<FTestTask_PrintValue>("Tree1GlobalTask1");
			{
				GlobalTask1.GetInstanceData().Value = -1;
				EditorData1.AddPropertyBinding(FPropertyBindingPath(EditorData1.GetRootParametersGuid(), "Tree1GlobalInt"), FPropertyBindingPath(GlobalTask1.ID, "Value"));
			}
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& GlobalTask2 = EditorData1.AddGlobalTask<FTestTask_PrintValue>("Tree1GlobalTask2");
				GlobalTask2.GetInstanceData().Value = -2;
				EditorData1.AddPropertyBinding(FPropertyBindingPath(GlobalTask1.ID, "Value"), FPropertyBindingPath(GlobalTask2.ID, "Value"));
			}

			UStateTreeState& Root = EditorData1.AddSubTree("Tree1StateRoot");
			{
				AddDouble(Root.Parameters.Parameters, "Tree1StateRootParametersDouble");
				AddInt(Root.Parameters.Parameters, "Tree1StateRootParametersInt");
				Root.bCopyParameterBindingsOnTick = true;
				EditorData1.AddPropertyBinding(FPropertyBindingPath(EditorData1.GetRootParametersGuid(), "Tree1GlobalInt"), FPropertyBindingPath(Root.Parameters.ID, "Tree1StateRootParametersInt"));
			}
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& Task1 = Root.AddTask<FTestTask_PrintValue>("Tree1StateRootTask1");
				Task1.GetInstanceData().Value = -1;
				EditorData1.AddPropertyBinding(FPropertyBindingPath(EditorData1.GetRootParametersGuid(), "Tree1GlobalInt"), FPropertyBindingPath(Task1.ID, "Value"));

				TStateTreeEditorNode<FTestTask_PrintValue>& Task2 = Root.AddTask<FTestTask_PrintValue>("Tree1StateRootTask2");
				Task2.GetInstanceData().Value = -2;
				EditorData1.AddPropertyBinding(FPropertyBindingPath(Root.Parameters.ID, "Tree1StateRootParametersInt"), FPropertyBindingPath(Task2.ID, "Value"));

				TStateTreeEditorNode<FTestTask_PrintValue>& Task3 = Root.AddTask<FTestTask_PrintValue>("Tree1StateRootTask3");
				Task3.GetInstanceData().Value = -3;
				EditorData1.AddPropertyBinding(FPropertyBindingPath(GlobalTask1.ID, "Value"), FPropertyBindingPath(Task3.ID, "Value"));
			}
			{
				UStateTreeState& State1 = Root.AddChildState("Tree1State1", EStateTreeStateType::LinkedAsset);
				State1.SetLinkedStateAsset(&StateTree2);
				State1.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::NextState);

				State1.SetParametersPropertyOverridden(Tree2GlobalParameter_ValueID_Int, true);
				State1.bCopyParameterBindingsOnTick = true;
				const FInstancedPropertyBag* Parameters = State1.GetDefaultParameters();
				AITEST_TRUE(TEXT("Parameter is invalid"), Parameters != nullptr);
				EditorData1.AddPropertyBinding(FPropertyBindingPath(GlobalTask1.ID, "Value"), FPropertyBindingPath(State1.Parameters.ID, "Tree2GlobalInt"));
			}

			FGuid Tree1Sub1Parameter_ValueID_Int;
			UStateTreeState& Sub1 = EditorData1.AddSubTree("Tree1StateSub1");
			{
				Sub1.Type = EStateTreeStateType::Subtree;
				Sub1.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::Succeeded);

				AddDouble(Sub1.Parameters.Parameters, "Tree1StateSub1ParametersDouble");
				Tree1Sub1Parameter_ValueID_Int = AddInt(Sub1.Parameters.Parameters, "Tree1StateSub1ParametersInt");

				TStateTreeEditorNode<FTestTask_PrintValue>& Sub1Task1 = Sub1.AddTask<FTestTask_PrintValue>("Tree1StateSub1Task1");
				Sub1Task1.GetInstanceData().Value = -1;
				EditorData1.AddPropertyBinding(FPropertyBindingPath(Sub1.Parameters.ID, "Tree1StateSub1ParametersInt"), FPropertyBindingPath(Sub1Task1.ID, "Value"));

				{
					UStateTreeState& State3 = Sub1.AddChildState("Tree1State3", EStateTreeStateType::State);
					{
						AddDouble(State3.Parameters.Parameters, "Tree1State3ParametersDouble1");
						AddDouble(State3.Parameters.Parameters, "Tree1State3ParametersDouble2");
						AddInt(State3.Parameters.Parameters, "Tree1State3ParametersInt");
						State3.bCopyParameterBindingsOnTick = true;
						EditorData1.AddPropertyBinding(FPropertyBindingPath(Sub1.Parameters.ID, "Tree1StateSub1ParametersInt"), FPropertyBindingPath(State3.Parameters.ID, "Tree1State3ParametersInt"));
					}

					TStateTreeEditorNode<FTestTask_PrintValue>& Task1 = State3.AddTask<FTestTask_PrintValue>("Tree1State3Task1");
					Task1.GetInstanceData().Value = -1;
					EditorData1.AddPropertyBinding(FPropertyBindingPath(EditorData1.GetRootParametersGuid(), "Tree1GlobalInt"), FPropertyBindingPath(Task1.ID, TEXT("Value")));

					TStateTreeEditorNode<FTestTask_PrintValue>& Task2 = State3.AddTask<FTestTask_PrintValue>("Tree1State3Task2");
					Task2.GetInstanceData().Value = -2;
					EditorData1.AddPropertyBinding(FPropertyBindingPath(Sub1.Parameters.ID, "Tree1StateSub1ParametersInt"), FPropertyBindingPath(Task2.ID, TEXT("Value")));

					TStateTreeEditorNode<FTestTask_PrintValue>& Task3 = State3.AddTask<FTestTask_PrintValue>("Tree1State3Task3");
					Task3.GetInstanceData().Value = -3;
					EditorData1.AddPropertyBinding(FPropertyBindingPath(GlobalTask1.ID, "Value"), FPropertyBindingPath(Task3.ID, "Value"));

					TStateTreeEditorNode<FTestTask_PrintValue>& Task4 = State3.AddTask<FTestTask_PrintValue>("Tree1State3Task4");
					Task3.GetInstanceData().Value = -4;
					EditorData1.AddPropertyBinding(FPropertyBindingPath(Task1.ID, "Value"), FPropertyBindingPath(Task4.ID, "Value"));

					State3.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::Succeeded);
				}
			}
			{
				UStateTreeState& State2 = Root.AddChildState("Tree1State2", EStateTreeStateType::Linked);
				State2.SetLinkedState(Sub1.GetLinkToState());
				State2.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::Succeeded);

				State2.SetParametersPropertyOverridden(Tree1Sub1Parameter_ValueID_Int, true);
				State2.bCopyParameterBindingsOnTick = true;
				const FInstancedPropertyBag* Parameters = State2.GetDefaultParameters();
				AITEST_TRUE(TEXT("Parameter is invalid"), Parameters != nullptr);
				EditorData1.AddPropertyBinding(FPropertyBindingPath(GlobalTask1.ID, "Value"), FPropertyBindingPath(State2.Parameters.ID, "Tree1StateSub1ParametersInt"));
			}

			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler1(Log);
			const bool bResult1 = Compiler1.Compile(StateTree1);
			AITEST_TRUE(TEXT("StateTree1 should get compiled"), bResult1);
		}

		FStateTreeInstanceData InstanceData;
		{
			FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);
		}
		{
			FStateTreeReference StateTreeRef;
			StateTreeRef.SetStateTree(&StateTree1);
			StateTreeRef.SetPropertyOverridden(Tree1GlobalParameter_ValueID_Int, true);
			StateTreeRef.GetMutableParameters().SetValueInt32("Tree1GlobalInt", 5);
			FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
			const EStateTreeRunStatus Status = Exec.Start(FStateTreeExecutionContext::FStartParameters
				{
					.InitialGlobalParameters = StateTreeRef.GetGlobalParameters(),
				});
			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1State1", "Tree2StateRoot", "Tree2State1"));
			AITEST_TRUE(TEXT("Start should enter Tree1GlobalTask1"), Exec.Expect("Tree1GlobalTask1", TEXT("EnterState5")));
			AITEST_TRUE(TEXT("Start should enter Tree1GlobalTask2"), Exec.Expect("Tree1GlobalTask2", TEXT("EnterState5")));
			AITEST_TRUE(TEXT("Start should enter Tree1StateRootTask1"), Exec.Expect("Tree1StateRootTask1", TEXT("EnterState5")));
			AITEST_TRUE(TEXT("Start should enter Tree1StateRootTask2"), Exec.Expect("Tree1StateRootTask2", TEXT("EnterState5")));
			AITEST_TRUE(TEXT("Start should enter Tree1StateRootTask3"), Exec.Expect("Tree1StateRootTask3", TEXT("EnterState5")));
			AITEST_TRUE(TEXT("Start should enter Tree2GlobalTask1"), Exec.Expect("Tree2GlobalTask1", TEXT("EnterState5")));
			AITEST_TRUE(TEXT("Start should enter Tree2GlobalTask2"), Exec.Expect("Tree2GlobalTask2", TEXT("EnterState5")));
			AITEST_TRUE(TEXT("Start should enter Tree2StateRootTask1"), Exec.Expect("Tree2StateRootTask1", TEXT("EnterState5")));
			AITEST_TRUE(TEXT("Start should enter Tree2StateRootTask2"), Exec.Expect("Tree2StateRootTask2", TEXT("EnterState5")));
			AITEST_TRUE(TEXT("Start should enter Tree2StateRootTask3"), Exec.Expect("Tree2StateRootTask3", TEXT("EnterState5")));
			AITEST_TRUE(TEXT("Start should enter Tree2State1Task1"), Exec.Expect("Tree2State1Task1", TEXT("EnterState5")));
			AITEST_TRUE(TEXT("Start should enter Tree2State1Task2"), Exec.Expect("Tree2State1Task2", TEXT("EnterState5")));
			AITEST_TRUE(TEXT("Start should enter Tree2State1Task3"), Exec.Expect("Tree2State1Task3", TEXT("EnterState5")));
			AITEST_TRUE(TEXT("Start should enter Tree2State1Task4"), Exec.Expect("Tree2State1Task4", TEXT("EnterState5")));
			Exec.LogClear();
		}
		{
			FStateTreeReference StateTreeRef;
			StateTreeRef.SetStateTree(&StateTree1);
			StateTreeRef.SetPropertyOverridden(Tree1GlobalParameter_ValueID_Int, true);
			StateTreeRef.GetMutableParameters().SetValueInt32("Tree1GlobalInt", 6);

			FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
			InstanceData.GetMutableStorage().SetGlobalParameters(StateTreeRef.GetGlobalParameters());

			const EStateTreeRunStatus Status = Exec.Tick(1.0f);
			AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);

			AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1State2", "Tree1StateSub1", "Tree1State3"));
			AITEST_TRUE(TEXT("Tick should tick Tree1GlobalTask1"), Exec.Expect("Tree1GlobalTask1", TEXT("Tick6")));
			AITEST_TRUE(TEXT("Tick should tick Tree1GlobalTask2"), Exec.Expect("Tree1GlobalTask2", TEXT("Tick6")));
			AITEST_TRUE(TEXT("Tick should tick Tree1StateRootTask1"), Exec.Expect("Tree1StateRootTask1", TEXT("Tick6")));
			AITEST_TRUE(TEXT("Tick should tick Tree1StateRootTask2"), Exec.Expect("Tree1StateRootTask2", TEXT("Tick6")));
			AITEST_TRUE(TEXT("Tick should tick Tree1StateRootTask3"), Exec.Expect("Tree1StateRootTask3", TEXT("Tick6")));
			AITEST_TRUE(TEXT("Tick should tick Tree2GlobalTask1"), Exec.Expect("Tree2GlobalTask1", TEXT("Tick6")));
			AITEST_TRUE(TEXT("Tick should tick Tree2GlobalTask2"), Exec.Expect("Tree2GlobalTask2", TEXT("Tick6")));
			AITEST_TRUE(TEXT("Tick should tick Tree2StateRootTask1"), Exec.Expect("Tree2StateRootTask1", TEXT("Tick6")));
			AITEST_TRUE(TEXT("Tick should tick Tree2StateRootTask2"), Exec.Expect("Tree2StateRootTask2", TEXT("Tick6")));
			AITEST_TRUE(TEXT("Tick should tick Tree2StateRootTask3"), Exec.Expect("Tree2StateRootTask3", TEXT("Tick6")));
			AITEST_TRUE(TEXT("Tick should tick Tree2State1Task1"), Exec.Expect("Tree2State1Task1", TEXT("Tick6")));
			AITEST_TRUE(TEXT("Tick should tick Tree2State1Task2"), Exec.Expect("Tree2State1Task2", TEXT("Tick6")));
			AITEST_TRUE(TEXT("Tick should tick Tree2State1Task3"), Exec.Expect("Tree2State1Task3", TEXT("Tick6")));
			AITEST_TRUE(TEXT("Tick should tick Tree2State1Task4"), Exec.Expect("Tree2State1Task4", TEXT("Tick6")));
			AITEST_TRUE(TEXT("Tick should enter Tree1StateSub1Task1"), Exec.Expect("Tree1StateSub1Task1", TEXT("EnterState6")));
			AITEST_TRUE(TEXT("Tick should enter Tree1State3Task1"), Exec.Expect("Tree1State3Task1", TEXT("EnterState6")));
			AITEST_TRUE(TEXT("Tick should enter Tree1State3Task2"), Exec.Expect("Tree1State3Task2", TEXT("EnterState6")));
			AITEST_TRUE(TEXT("Tick should enter Tree1State3Task3"), Exec.Expect("Tree1State3Task3", TEXT("EnterState6")));
			AITEST_TRUE(TEXT("Tick should enter Tree1State3Task4"), Exec.Expect("Tree1State3Task4", TEXT("EnterState6")));
		}
		{
			FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
			Exec.Stop();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Linked_GlobalParameter, "System.StateTree.LinkedAsset.GlobalParameter");

struct FStateTreeTest_Linked_AccessGlobalInstanceData : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		//Tree 1
		//	Root
		//		State1
		//			State2 (linked Tree2)
		//Tree 2 (with global tasks)
		//	Root

		FSimpleDelegate State1OnEnter;
		bool bDelegateCalled = false;

		// Tree 2
		UStateTree& StateTree2 = NewStateTree();
		{
			UStateTreeEditorData& EditorData2 = *Cast<UStateTreeEditorData>(StateTree2.EditorData);

			// Global tasks
			TStateTreeEditorNode<FTestTask_PrintValue>& GlobalTask1 = EditorData2.AddGlobalTask<FTestTask_PrintValue>("Tree2GlobalTask1");
			{
				GlobalTask1.GetInstanceData().Value = 77;
			}
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& GlobalTask2 = EditorData2.AddGlobalTask<FTestTask_PrintValue>("Tree2GlobalTask2");
				GlobalTask2.GetInstanceData().Value = 88;
				GlobalTask2.GetNode().CustomEnterStateFunc = [this, &State1OnEnter, &bDelegateCalled](FStateTreeExecutionContext& ExecContext, const FTestTask_PrintValue* Node)
					{
						FTestTask_PrintValue::FInstanceDataType& InstanceData = ExecContext.GetInstanceData<FTestTask_PrintValue::FInstanceDataType>(*Node);
						if (!GetTestRunner().TestTrue(TEXT("Tree2GlobalTask2 instance data is not 88"), InstanceData.Value == 88))
						{
							return;
						}
						State1OnEnter.BindLambda([this, &bDelegateCalled, WeakContext = ExecContext.MakeWeakExecutionContext()]()
							{
								FStateTreeStrongReadOnlyExecutionContext StrongContext = WeakContext.MakeStrongReadOnlyExecutionContext();
								const FTestTask_PrintValue::FInstanceDataType* InstanceData = StrongContext.GetInstanceDataPtr<const FTestTask_PrintValue::FInstanceDataType>();
								if (!GetTestRunner().TestTrue(TEXT("Tree2GlobalTask2 instance data is nullptr"), InstanceData != nullptr))
								{
									return;
								}
								if (!GetTestRunner().TestTrue(TEXT("Tree2GlobalTask2 instance data is not 88"), InstanceData && InstanceData->Value == 88))
								{
									return;
								}
								bDelegateCalled = true;
							});

					};
			}

			UStateTreeState& Root = EditorData2.AddSubTree("Tree2StateRoot");
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& Task1 = Root.AddTask<FTestTask_PrintValue>("Tree2StateRootTask1");
				Task1.GetInstanceData().Value = 99;
			}

			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler2(Log);
			const bool bResult2 = Compiler2.Compile(StateTree2);
			AITEST_TRUE(TEXT("StateTree2 should get compiled"), bResult2);
		}

		// Tree 1
		UStateTree& StateTree1 = NewStateTree();
		FGuid Tree1GlobalParameter_ValueID_Int;
		{
			UStateTreeEditorData& EditorData1 = *Cast<UStateTreeEditorData>(StateTree1.EditorData);

			// Global tasks
			TStateTreeEditorNode<FTestTask_PrintValue>& GlobalTask1 = EditorData1.AddGlobalTask<FTestTask_PrintValue>("Tree1GlobalTask1");
			{
				GlobalTask1.GetInstanceData().Value = 11;
			}

			UStateTreeState& Root = EditorData1.AddSubTree("Tree1StateRoot");
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& Task1 = Root.AddTask<FTestTask_PrintValue>("Tree1StateRootTask1");
				Task1.GetInstanceData().Value = 22;
			}
			{
				UStateTreeState& State1 = Root.AddChildState("Tree1State1");
				TStateTreeEditorNode<FTestTask_PrintValue>& Task1 = State1.AddTask<FTestTask_PrintValue>("Tree1State1Task1");
				Task1.GetInstanceData().Value = 33;
				Task1.GetNode().CustomEnterStateFunc = [this, &State1OnEnter](FStateTreeExecutionContext& ExecContext, const FTestTask_PrintValue* Node)
					{
						FTestTask_PrintValue::FInstanceDataType& InstanceData = ExecContext.GetInstanceData<FTestTask_PrintValue::FInstanceDataType>(*Node);
						if (!GetTestRunner().TestTrue(TEXT("Tree1State1Task1 instance data is not 33"), InstanceData.Value == 33))
						{
							return;
						}
						State1OnEnter.ExecuteIfBound();
					};
				{
					UStateTreeState& State2 = State1.AddChildState("Tree1State2", EStateTreeStateType::LinkedAsset);
					State2.SetLinkedStateAsset(&StateTree2);
				}
			}

			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler1(Log);
			const bool bResult1 = Compiler1.Compile(StateTree1);
			AITEST_TRUE(TEXT("StateTree1 should get compiled"), bResult1);
		}

		FStateTreeInstanceData InstanceData;
		{
			FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);
		}
		{
			FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
			const EStateTreeRunStatus Status = Exec.Start();
			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1State1", "Tree1State2", "Tree2StateRoot"));
			AITEST_TRUE(TEXT("Start should enter Tree1GlobalTask1"), Exec.Expect("Tree1GlobalTask1", TEXT("EnterState11")));
			AITEST_TRUE(TEXT("Start should enter Tree1StateRootTask1"), Exec.Expect("Tree1StateRootTask1", TEXT("EnterState22")));
			AITEST_TRUE(TEXT("Start should enter Tree1State1Task1"), Exec.Expect("Tree1State1Task1", TEXT("EnterState33")));
			AITEST_TRUE(TEXT("Start should enter Tree2GlobalTask1"), Exec.Expect("Tree2GlobalTask1", TEXT("EnterState77")));
			AITEST_TRUE(TEXT("Start should enter Tree2GlobalTask2"), Exec.Expect("Tree2GlobalTask2", TEXT("EnterState88")));
			AITEST_TRUE(TEXT("Start should enter Tree2StateRootTask1"), Exec.Expect("Tree2StateRootTask1", TEXT("EnterState99")));

			AITEST_TRUE(TEXT("Start should enter bDelegateCalled"), bDelegateCalled);
			Exec.LogClear();
		}
		{
			FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
			Exec.Stop();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Linked_AccessGlobalInstanceData, "System.StateTree.LinkedAsset.AccessGlobalInstanceData");

struct FStateTreeTest_Linked_FinishGlobalTasks : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		//Tree 1
		//  Global task and parameter
		//	Root
		//		StateLinkedTree1 (Tree2) -> Next
		//		SubTree2
		//	SubTree
		//		State3
		//Tree 2
		//  Global task and parameter
		//	Root
		//		State1


		// Tree 2
		UStateTree& StateTree2 = NewStateTree();
		{
			UStateTreeEditorData& EditorData2 = *Cast<UStateTreeEditorData>(StateTree2.EditorData);

			// Global tasks
			{
				EditorData2.GlobalTasksCompletion = EStateTreeTaskCompletionType::Any;

				TStateTreeEditorNode<FTestTask_PrintValue>& GlobalTask1 = EditorData2.AddGlobalTask<FTestTask_PrintValue>("Tree2GlobalTask1");
				GlobalTask1.GetInstanceData().Value = 1;

				EditorData2.AddGlobalTask<FTestTask_Stand>("Tree2GlobalTask2").GetNode().TicksToCompletion = 99;
				EditorData2.AddGlobalTask<FTestTask_Stand>("Tree2GlobalTask3").GetNode().TicksToCompletion = 99;
				EditorData2.AddGlobalTask<FTestTask_Stand>("Tree2GlobalTask4").GetNode().TicksToCompletion = 99;
				EditorData2.AddGlobalTask<FTestTask_Stand>("Tree2GlobalTask5").GetNode().TicksToCompletion = 99;
				EditorData2.AddGlobalTask<FTestTask_Stand>("Tree2GlobalTask6").GetNode().TicksToCompletion = 99;

				TStateTreeEditorNode<FTestTask_Stand>& GlobalTask2 = EditorData2.AddGlobalTask<FTestTask_Stand>("Tree2GlobalTask7");
				GlobalTask2.GetNode().TicksToCompletion = 2;
				GlobalTask2.GetNode().TickCompletionResult = EStateTreeRunStatus::Succeeded;
			}

			UStateTreeState& Root = EditorData2.AddSubTree("Tree2StateRoot");
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& Task1 = Root.AddTask<FTestTask_PrintValue>("Tree2StateRootTask1");
				Task1.GetInstanceData().Value = 1;
			}

			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler2(Log);
			const bool bResult2 = Compiler2.Compile(StateTree2);
			AITEST_TRUE(TEXT("StateTree2 should get compiled"), bResult2);
		}

		// Tree 1
		UStateTree& StateTree1 = NewStateTree();
		{
			UStateTreeEditorData& EditorData1 = *Cast<UStateTreeEditorData>(StateTree1.EditorData);

			// Global tasks
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& GlobalTask1 = EditorData1.AddGlobalTask<FTestTask_PrintValue>("Tree1GlobalTask1");
				GlobalTask1.GetInstanceData().Value = 1;

				TStateTreeEditorNode<FTestTask_Stand>& GlobalTask2 = EditorData1.AddGlobalTask<FTestTask_Stand>("Tree1GlobalTask2");
				GlobalTask2.GetNode().TicksToCompletion = 4;
				GlobalTask2.GetNode().TickCompletionResult = EStateTreeRunStatus::Succeeded;
			}

			UStateTreeState& Root = EditorData1.AddSubTree("Tree1StateRoot");
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& Task1 = Root.AddTask<FTestTask_PrintValue>("Tree1StateRootTask1");
				Task1.GetInstanceData().Value = 1;
			}
			{
				UStateTreeState& State1 = Root.AddChildState("Tree1State1", EStateTreeStateType::LinkedAsset);
				State1.SetLinkedStateAsset(&StateTree2);
				State1.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::NextState);
			}
			{
				UStateTreeState& State2 = Root.AddChildState("Tree1State2", EStateTreeStateType::State);
				State2.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::Succeeded);

				TStateTreeEditorNode<FTestTask_Stand>& Task1 = State2.AddTask<FTestTask_Stand>("Tree1State2Task1");
				Task1.GetNode().TicksToCompletion = 10;
				Task1.GetNode().TickCompletionResult = EStateTreeRunStatus::Succeeded;
			}

			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler1(Log);
			const bool bResult1 = Compiler1.Compile(StateTree1);
			AITEST_TRUE(TEXT("StateTree1 should get compiled"), bResult1);
		}

		for (int32 Index = 0; Index < 4; ++Index)
		{
			const bool bGlobalTasksCompleteOwningFrame = (Index % 2) == 0;
			FScopedCVar CVarGGlobalTasksCompleteOwningFrame = FScopedCVar(TEXT("StateTree.GlobalTasksCompleteOwningFrame"), bGlobalTasksCompleteOwningFrame);

			const bool bTickGlobalNodesWithHierarchy = Index >= 2;
			FScopedCVar CVarTickGlobalNodesWithHierarchy = FScopedCVar(TEXT("StateTree.TickGlobalNodesFollowingTreeHierarchy"), bTickGlobalNodesWithHierarchy);

			FStateTreeInstanceData InstanceData;
			{
				FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
				const bool bInitSucceeded = Exec.IsValid();
				AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);
			}
			{
				FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
				const EStateTreeRunStatus Status = Exec.Start();
				AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1State1", "Tree2StateRoot"));
				AITEST_TRUE(TEXT("Start should enter Tree1GlobalTask1"), Exec.Expect("Tree1GlobalTask1", TEXT("EnterState1")));
				AITEST_TRUE(TEXT("Start should enter Tree1GlobalTask2"), Exec.Expect("Tree1GlobalTask2", TEXT("EnterState")));
				AITEST_TRUE(TEXT("Start should enter Tree1StateRootTask1"), Exec.Expect("Tree1StateRootTask1", TEXT("EnterState1")));
				AITEST_TRUE(TEXT("Start should enter Tree2GlobalTask1"), Exec.Expect("Tree2GlobalTask1", TEXT("EnterState1")));
				AITEST_TRUE(TEXT("Start should enter Tree2GlobalTask7"), Exec.Expect("Tree2GlobalTask7", TEXT("EnterState")));
				AITEST_TRUE(TEXT("Start should enter Tree2StateRootTask1"), Exec.Expect("Tree2StateRootTask1", TEXT("EnterState1")));
				Exec.LogClear();
			}
			{
				FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
				const EStateTreeRunStatus Status = Exec.Tick(1.0f);
				AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1State1", "Tree2StateRoot"));
				AITEST_TRUE(TEXT("Tick should tick Tree1GlobalTask1"), Exec.Expect("Tree1GlobalTask1", TEXT("Tick1")));
				AITEST_TRUE(TEXT("Tick should tick Tree1GlobalTask2"), Exec.Expect("Tree1GlobalTask2", TEXT("Tick")));
				AITEST_TRUE(TEXT("Tick should tick Tree1StateRootTask1"), Exec.Expect("Tree1StateRootTask1", TEXT("Tick1")));
				AITEST_TRUE(TEXT("Tick should tick Tree2GlobalTask1"), Exec.Expect("Tree2GlobalTask1", TEXT("Tick1")));
				AITEST_TRUE(TEXT("Tick should tick Tree2GlobalTask7"), Exec.Expect("Tree2GlobalTask7", TEXT("Tick")));
				AITEST_TRUE(TEXT("Tick should tick Tree2StateRootTask1"), Exec.Expect("Tree2StateRootTask1", TEXT("Tick1")));
				Exec.LogClear();
			}
			if (bGlobalTasksCompleteOwningFrame)
			{
				{
					FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
					const EStateTreeRunStatus Status = Exec.Tick(1.0f);
					AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
					AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1State2"));
					AITEST_TRUE(TEXT("Tick should tick Tree1GlobalTask1"), Exec.Expect("Tree1GlobalTask1", TEXT("Tick1")));
					AITEST_TRUE(TEXT("Tick should tick Tree1GlobalTask2"), Exec.Expect("Tree1GlobalTask2", TEXT("Tick")));
					if (bTickGlobalNodesWithHierarchy)
					{
						AITEST_TRUE(TEXT("Tick should tick Tree1StateRootTask1"), Exec.Expect("Tree1StateRootTask1", TEXT("Tick1")));
					}
					AITEST_TRUE(TEXT("Tick should tick Tree2GlobalTask1"), Exec.Expect("Tree2GlobalTask1", TEXT("Tick1")));
					AITEST_TRUE(TEXT("Tick should tick Tree2GlobalTask7"), Exec.Expect("Tree2GlobalTask7", TEXT("Tick")));
					if (bTickGlobalNodesWithHierarchy)
					{
						AITEST_FALSE(TEXT("Tick not should tick Tree2StateRootTask1"), Exec.Expect("Tree2StateRootTask1", TEXT("Tick1")));
					}

					AITEST_TRUE(TEXT("Tick should ExitState Tree2StateRootTask1"), Exec.Expect("Tree2StateRootTask1", TEXT("ExitState1")));
					AITEST_TRUE(TEXT("Tick should ExitState Tree2GlobalTask7"), Exec.Expect("Tree2GlobalTask7", TEXT("ExitState")));
					AITEST_TRUE(TEXT("Tick should ExitState Tree2GlobalTask1"), Exec.Expect("Tree2GlobalTask1", TEXT("ExitState1")));
					AITEST_TRUE(TEXT("Start should enter Tree2State2Task1"), Exec.Expect("Tree1State2Task1", TEXT("EnterState")));
					Exec.LogClear();
				}
				{
					FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
					const EStateTreeRunStatus Status = Exec.Tick(1.0f);
					AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
					Exec.LogClear();
				}
				{
					FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
					const EStateTreeRunStatus Status = Exec.Tick(1.0f);
					AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Succeeded);
					Exec.LogClear();
				}
			}
			else
			{
				FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
				const EStateTreeRunStatus Status = Exec.Tick(1.0f);
				AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Succeeded);
				Exec.LogClear();
			}
			{
				FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
				Exec.Stop();
			}
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Linked_FinishGlobalTasks, "System.StateTree.LinkedAsset.FinishGlobalTasks");

struct FStateTreeTest_OverrideLinkedAsset : FStateTreeTestBase
{
	// test override while inside Tree1.Root.State1.Tree2.Root.State1, the override changes.
	virtual bool InstantTest() override
	{
		FStateTreeCompilerLog Log;

		// Tree1
		//  Root1 (tick => override then request new transition to A1)
		//   State1 (linked to Tree2)
		// Tree2 GlobalTask2 => 2 ticks
		//  Root
			// State1 => 1 tick to completion
			// State2 => print
		// Tree3 GlobalTask2 => 2 ticks
		//  Root Task => 1 tick

		// Asset 3
		UStateTree& StateTree3 = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree3.EditorData);
			GetRootPropertyBag(EditorData).AddProperty("B", EPropertyBagPropertyType::Int32);

			UStateTreeState& Root = EditorData.AddSubTree(FName("Tree3Root"));
			TStateTreeEditorNode<FTestTask_PrintValue>& RootTask1 = Root.AddTask<FTestTask_PrintValue>(FName("Tree3RootTask1"));
			RootTask1.GetInstanceData().Value = -3;

			EditorData.AddPropertyBinding(FPropertyBindingPath(EditorData.GetRootParametersGuid(), "B"), FPropertyBindingPath(RootTask1.ID, TEXT("Value")));

			{
				FStateTreeCompiler Compiler(Log);
				const bool bResult = Compiler.Compile(StateTree3);
				AITEST_TRUE(TEXT("StateTree3 should get compiled"), bResult);
			}
		}

		// Asset 2
		UStateTree& StateTree2 = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree2.EditorData);
			GetRootPropertyBag(EditorData).AddProperty("A", EPropertyBagPropertyType::Int32);

			UStateTreeState& Root2 = EditorData.AddSubTree(FName("Tree2Root"));

			UStateTreeState& State1 = Root2.AddChildState("Tree2State1");
			TStateTreeEditorNode<FTestTask_Stand>& State1Task1 = State1.AddTask<FTestTask_Stand>(FName("Tree2State1Task1"));
			TStateTreeEditorNode<FTestTask_PrintValue>& State1Task2 = State1.AddTask<FTestTask_PrintValue>(FName("Tree2State1Task2"));
			State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::NextState);

			EditorData.AddPropertyBinding(FPropertyBindingPath(EditorData.GetRootParametersGuid(), "A"), FPropertyBindingPath(State1Task2.ID, "Value"));

			UStateTreeState& State2 = Root2.AddChildState("Tree2State2");
			TStateTreeEditorNode<FTestTask_PrintValue>& State2Task1 = State2.AddTask<FTestTask_PrintValue>(FName("Tree2State2Task1"));

			EditorData.AddPropertyBinding(FPropertyBindingPath(EditorData.GetRootParametersGuid(), "A"), FPropertyBindingPath(State2Task1.ID, "Value"));

			{
				FStateTreeCompiler Compiler(Log);
				const bool bResult = Compiler.Compile(StateTree2);
				AITEST_TRUE(TEXT("StateTree2 should get compiled"), bResult);
			}
		}

		// Main asset
		UStateTree& StateTree1 = NewStateTree();
		UStateTreeEditorData& EditorData1 = *Cast<UStateTreeEditorData>(StateTree1.EditorData);
		FPropertyBindingPath Tree1LinkedStateParameterBindingPath;
		FPropertyBindingPath Tree1LinkedStateCondition1BindingPath;
		FPropertyBindingPath Tree1LinkedStateCondition2BindingPath;
		{
			GetRootPropertyBag(EditorData1).AddProperty("Int1Param", EPropertyBagPropertyType::Int32);

			UStateTreeState& Root = EditorData1.AddSubTree(FName("Tree1Root1"));
			UStateTreeState& State1 = Root.AddChildState(FName("Tree1State1"), EStateTreeStateType::LinkedAsset);
			State1.Tag = GetTestTag1();
			State1.SetLinkedStateAsset(&StateTree2);

			FStateTreeEditorNode& Cond = State1.AddEnterCondition<FTestCondition_PrintValue>(FName("Tree1State2Condition1"));
			Tree1LinkedStateCondition1BindingPath = FPropertyBindingPath(Cond.ID, "Value");
			
			FStateTreeEditorNode& PropertyRefCond = State1.AddEnterCondition<FTestCondition_PropertyRefOnNodeAndInstance>(FName("Tree1State2Condition2"));
			Tree1LinkedStateCondition2BindingPath = FPropertyBindingPath(PropertyRefCond.GetNodeID(), "RefOnNode");

			Tree1LinkedStateParameterBindingPath = FPropertyBindingPath(State1.Parameters.ID, "A");
			EditorData1.AddPropertyBinding(FPropertyBindingPath(EditorData1.GetRootParametersGuid(), "Int1Param"), Tree1LinkedStateParameterBindingPath);
			
			{
				FStateTreeCompiler Compiler(Log);
				const bool bResult = Compiler.Compile(StateTree1);
				AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);
			}
		}

		{
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

			FInstancedPropertyBag GlobalParameters = StateTree1.GetDefaultParameters();
			GlobalParameters.SetValueInt32("Int1Param", 11);
			{
				EStateTreeRunStatus Status = Exec.Start(GlobalParameters.GetValue());
				AITEST_EQUAL("Start should complete with Running", Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("StateTree should tick Tree2State1Task1"), Exec.Expect("Tree2State1Task1", TEXT("EnterState")));
				AITEST_TRUE(TEXT("StateTree should tick Tree2State1Task2"), Exec.Expect("Tree2State1Task2", TEXT("EnterState11")));
				Exec.LogClear();
			}
			// Set the overrides
			{
				FStateTreeReference StateTreeReference;
				StateTreeReference.SetStateTree(&StateTree3);
				FStateTreeReferenceOverrides Overrides;
				Overrides.AddOverride(GetTestTag1(), StateTreeReference);
				Exec.SetLinkedStateTreeOverrides(Overrides);
			}
			{
				// Tree2State1Task1 completes and transition to Tree2State2Task1
				//The Tree1State1 didn't complete.
				EStateTreeRunStatus Status = Exec.Tick(0.1f);
				AITEST_EQUAL("Tick should complete with Running", Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("StateTree should tick Tree2State2Task1"), Exec.Expect("Tree2State2Task1", TEXT("EnterState11")));
				Exec.LogClear();
			}
			{
				Exec.Stop();
				Exec.LogClear();
			}
			// Change the override. The override should not work because there bindings.
			{
				GlobalParameters.SetValueInt32("Int1Param", 33);
				EStateTreeRunStatus Status = Exec.Start(GlobalParameters.GetValue());
				AITEST_EQUAL("Start should complete with Running", Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("StateTree should tick Tree2State1Task2"), Exec.Expect("Tree2State1Task2", TEXT("EnterState33")));
				Exec.LogClear();
			}
			Exec.Stop();
		}

		auto RecompileAndStartTree = [this, &StateTree1, &Log, &StateTree2, &StateTree3](TNotNull<const UStateTree*> TargetLinkedAsset)
			{
				{
					FStateTreeCompiler Compiler(Log);
					const bool bResult = Compiler.Compile(StateTree1);
					AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);
				}

				FStateTreeInstanceData InstanceData;
				FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
				const bool bInitSucceeded = Exec.IsValid();
				AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

				// Set the overrides
				{
					FStateTreeReference StateTreeReference;
					StateTreeReference.SetStateTree(&StateTree3);
					FStateTreeReferenceOverrides Overrides;
					Overrides.AddOverride(GetTestTag1(), StateTreeReference);
					Exec.SetLinkedStateTreeOverrides(Overrides);
				}

				{
					EStateTreeRunStatus Status = Exec.Start();
					AITEST_EQUAL("Start should complete with Running", Status, EStateTreeRunStatus::Running);

					if (TargetLinkedAsset == &StateTree2)
					{
						AITEST_TRUE("StateTree should be in Tree2's active frame because binding from condition to linked state parameter prevents override",
							Exec.ExpectInActiveStates("Tree1Root1", "Tree1State1", "Tree2Root", "Tree2State1"));
					}
					else
					{
						AITEST_TRUE("StateTree should be in Tree3's active frame because of linked tree override",
							Exec.ExpectInActiveStates("Tree1Root1", "Tree1State1", "Tree3Root"));
					}
					
				}

				{
					Exec.Stop();
					Exec.LogClear();
				}

				return true;
			};


		// Reset binding from global parameter to linked state parameter and add binding from state parameter to enter condition
		// Linked asset override should be disabled
		{
			EditorData1.RemovePropertyBinding(Tree1LinkedStateParameterBindingPath);
			EditorData1.AddPropertyBinding(Tree1LinkedStateParameterBindingPath, Tree1LinkedStateCondition1BindingPath);

			if (!RecompileAndStartTree(&StateTree2))
			{
				return false;
			}
		}

		// Reset binding from Linked state parameter to enter condition
		// Linked asset override should be enabled
		{
			EditorData1.RemovePropertyBinding(Tree1LinkedStateCondition1BindingPath);
			if (!RecompileAndStartTree(&StateTree3))
			{
				return false;
			}
		}

		// Add binding from enter condition property ref to linked state parameter
		// Linked asset override should be disabled
		{
			EditorData1.AddPropertyBinding(Tree1LinkedStateParameterBindingPath, Tree1LinkedStateCondition2BindingPath);
			
			if (!RecompileAndStartTree(&StateTree2))
			{
				return false;
			}
		}
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_OverrideLinkedAsset, "System.StateTree.LinkedAsset.Override");

struct FStateTreeTest_ReenterLinkedAssetWithDifferentAsset : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		//Main Tree
		//	Root
		//		StateMain [LinkedAsset with tag] -> StateMain (OnTick)
		//Tree A
		//	Root (TaskA)
		//Tree B
		//	Root (TaskB)
		// StateMain starts with Tree A set and gets entered.
		// StateMain's Linked Asset is overridden to Tree B and transition on StateMain to self is attempted.
		// StateMain should be reentered with the Tree A asset or Tree B depending on the selection rules.

		// Asset A
		UStateTree& StateTreeA = NewStateTree();
		{
			UStateTreeEditorData& EditorDataA = *Cast<UStateTreeEditorData>(StateTreeA.EditorData);
			GetRootPropertyBag(EditorDataA).AddProperty("A", EPropertyBagPropertyType::Bool);
			UStateTreeState& RootA = EditorDataA.AddSubTree(FName(TEXT("RootA")));
			RootA.AddTask<FTestTask_Stand>(FName(TEXT("TaskA")));
		}
		// Asset B
		UStateTree& StateTreeB = NewStateTree();
		{
			UStateTreeEditorData& EditorDataB = *Cast<UStateTreeEditorData>(StateTreeB.EditorData);
			GetRootPropertyBag(EditorDataB).AddProperty("B", EPropertyBagPropertyType::Int32);
			UStateTreeState& RootB = EditorDataB.AddSubTree(FName(TEXT("RootB")));
			RootB.AddTask<FTestTask_Stand>(FName(TEXT("TaskB")));
		}
		// Main asset
		const FGameplayTag StateTag = GetTestTag1();
		UStateTree& StateTree = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
			UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
			UStateTreeState& StateMain = Root.AddChildState(FName(TEXT("StateMain")), EStateTreeStateType::LinkedAsset);
			StateMain.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::GotoState, &StateMain);
			StateMain.Tag = StateTag;
		}
		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* ExitStateStr(TEXT("ExitState"));

		constexpr int32 MaxRules = 4;
		auto MakeStateSelectionRule = [](int32 Index)
			{
				EStateTreeStateSelectionRules Rule = EStateTreeStateSelectionRules::None;
				if ((Index % 2) == 1)
				{
					Rule |= EStateTreeStateSelectionRules::CompletedTransitionStatesCreateNewStates;
					Rule |= EStateTreeStateSelectionRules::CompletedStateBeforeTransitionSourceFailsTransition;
				}
				if (Index >= 2)
				{
					Rule |= EStateTreeStateSelectionRules::ReselectedStateCreatesNewStates;
				}
				return Rule;
			};


		for (int32 Index = 0; Index < MaxRules; ++Index)
		{
			const EStateTreeStateSelectionRules StateSelectionRules = MakeStateSelectionRule(Index);
			auto CompileTree = [StateSelectionRules](UStateTree& StateTree)
				{
					StateTree.ResetCompiled();
					UStateTreeEditorData* EditorData = CastChecked<UStateTreeEditorData>(StateTree.EditorData);
					UStateTreeTestSchema* Schema = CastChecked<UStateTreeTestSchema>(EditorData->Schema);
					Schema->SetStateSelectionRules(StateSelectionRules);

					FStateTreeCompilerLog Log;
					FStateTreeCompiler Compiler(Log);
					return Compiler.Compile(&StateTree);
				};

			AITEST_TRUE("StateTree2 should get compiled", CompileTree(StateTreeA));
			AITEST_TRUE("StateTree1 should get compiled", CompileTree(StateTreeB));
			AITEST_TRUE("StateTree3 should get compiled", CompileTree(StateTree));

			{
				EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
				FStateTreeInstanceData InstanceData;
				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
				{
					FStateTreeReferenceOverrides Overrides;
					FStateTreeReference Ref;
					Ref.SetStateTree(&StateTreeA);
					Overrides.AddOverride(StateTag, Ref);
					Exec.SetLinkedStateTreeOverrides(Overrides);
				}
				const bool bInitSucceeded = Exec.IsValid();
				AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);
				Status = Exec.Start();
				AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("StateTree should enter TaskA"), Exec.Expect("TaskA", EnterStateStr));
				Exec.LogClear();
				Exec.Tick(0.1f);
				AITEST_TRUE(TEXT("StateTree should enter TaskA"), Exec.Expect("TaskA", EnterStateStr));
				Exec.LogClear();
				{
					FStateTreeReferenceOverrides Overrides;
					FStateTreeReference Ref;
					Ref.SetStateTree(&StateTreeB);
					Overrides.AddOverride(StateTag, Ref);
					Exec.SetLinkedStateTreeOverrides(Overrides);
				}
				Exec.Tick(0.1f);
				if (EnumHasAllFlags(StateSelectionRules, EStateTreeStateSelectionRules::ReselectedStateCreatesNewStates))
				{
					AITEST_TRUE(TEXT("StateTree should enter TaskB"), Exec.Expect("TaskB", EnterStateStr));
				}
				else
				{
					AITEST_TRUE(TEXT("StateTree should enter TaskA"), Exec.Expect("TaskA", EnterStateStr));
				}
				Exec.Stop();
			}
		}
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_ReenterLinkedAssetWithDifferentAsset, "System.StateTree.LinkedAsset.ReenterWithDifferentAsset");


struct FStateTreeTest_AutoCompletedLinkedAsset : FStateTreeTestBase
{
	// Test going back to the same linked state after it completed via another transition.
	virtual bool InstantTest() override
	{
		FStateTreeCompilerLog Log;

		// Tree1
		//  Root
		//	  State1 (linked to Tree2) [OnCompleted go to State2]
		//    State2 (auto complete on enter)[OnCompleted go to State1]
		// Tree2
			// State1 (linked to Tree3)
		// Tree3
			// State1 => 1 tick to completion

		// Tree 3
		UStateTree& StateTree3 = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree3.EditorData);

			UStateTreeState& State1 = EditorData.AddSubTree("Tree3State1");
			TStateTreeEditorNode<FTestTask_Stand>& State1Task1 = State1.AddTask<FTestTask_Stand>("Tree3State1Task1");
			State1Task1.GetNode().TicksToCompletion = 1;
			State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

			{
				FStateTreeCompiler Compiler(Log);
				const bool bResult = Compiler.Compile(StateTree3);
				AITEST_TRUE(TEXT("StateTree3 should get compiled"), bResult);
			}
		}

		// Tree 2
		UStateTree& StateTree2 = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree2.EditorData);

			UStateTreeState& State1 = EditorData.AddSubTree("Tree2State1", EStateTreeStateType::LinkedAsset);
			State1.SetLinkedStateAsset(&StateTree3);

			{
				FStateTreeCompiler Compiler(Log);
				const bool bResult = Compiler.Compile(StateTree2);
				AITEST_TRUE(TEXT("StateTree2 should get compiled"), bResult);
			}
		}

		// Tree 1
		UStateTree& StateTree = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

			UStateTreeState& Root = EditorData.AddSubTree("Tree1StateRoot");
			UStateTreeState& State1 = Root.AddChildState("Tree1State1", EStateTreeStateType::LinkedAsset);
			State1.SetLinkedStateAsset(&StateTree2);

			UStateTreeState& State2 = Root.AddChildState("Tree1State2");
			TStateTreeEditorNode<FTestTask_Stand>& State2Task1 = State2.AddTask<FTestTask_Stand>("Tree1State2Task1");
			State2Task1.GetNode().EnterStateResult = EStateTreeRunStatus::Succeeded;

			State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &State2);
			State2.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &State1);
			{
				FStateTreeCompiler Compiler(Log);
				const bool bResult = Compiler.Compile(StateTree);
				AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);
			}
		}

		{
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);
			{
				EStateTreeRunStatus Status = Exec.Start();
				AITEST_EQUAL("Start should complete with Running", Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("StateTree should tick Tree2State1Task1"), Exec.Expect("Tree3State1Task1", TEXT("EnterState")));
				AITEST_TRUE(TEXT("StateTree should tick Tree2State1Task1"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1State1", "Tree2State1", "Tree3State1"));
				Exec.LogClear();
			}
			{
				EStateTreeRunStatus Status = Exec.Tick(0.1f);
				AITEST_EQUAL("Tick should complete with Running", Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("StateTree should tick Tree2State1Task1"), Exec.Expect("Tree3State1Task1", TEXT("ExitState")));
				AITEST_TRUE(TEXT("StateTree should tick Tree2State1Task1"), Exec.Expect("Tree1State2Task1", TEXT("EnterState")));
				AITEST_TRUE(TEXT("StateTree should tick Tree2State1Task1"), Exec.Expect("Tree1State2Task1", TEXT("ExitState")));
				AITEST_TRUE(TEXT("StateTree should tick Tree2State1Task1"), Exec.Expect("Tree3State1Task1", TEXT("EnterState")));
				AITEST_TRUE(TEXT("StateTree should tick Tree2State1Task1"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1State1", "Tree2State1", "Tree3State1"));
				Exec.LogClear();
			}
			Exec.Stop();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_AutoCompletedLinkedAsset, "System.StateTree.LinkedAsset.AutoCompleted");

/*
 * Test for when a subtree completes, events for current transition processing phase are correctly retained.
 */
struct FStateTreeTest_EventTransitionInParentTree : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		// MainTree
		//   Root
		//     State1 (LinkedAsset -> LATree) OnEvent(Tag1) -> State2
		//     State2
		//
		// LATree
		//   LARoot OnTick(Critical) -> Succeeded

		FStateTreeCompilerLog Log;

		UStateTree& LATree = NewStateTree();
		{
			UStateTreeEditorData& LAEditorData = *Cast<UStateTreeEditorData>(LATree.EditorData);

			UStateTreeState& LARoot = LAEditorData.AddSubTree("LARoot");
			FStateTreeTransition& LATransition = LARoot.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::Succeeded);
			LATransition.Priority = EStateTreeTransitionPriority::Critical;

			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(LATree);
			AITEST_TRUE(TEXT("LATree should get compiled"), bResult);
		}

		UStateTree& MainTree = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(MainTree.EditorData);

			UStateTreeState& Root = EditorData.AddSubTree("Root");

			UStateTreeState& State1 = Root.AddChildState("State1", EStateTreeStateType::LinkedAsset);
			UStateTreeState& State2 = Root.AddChildState("State2");

			State1.SetLinkedStateAsset(&LATree);
			State1.AddTransition(EStateTreeTransitionTrigger::OnEvent, GetTestTag1(), EStateTreeTransitionType::GotoState, &State2);

			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(MainTree);
			AITEST_TRUE(TEXT("MainTree should get compiled"), bResult);
		}

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(MainTree, MainTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		{
			EStateTreeRunStatus Status = Exec.Start();
			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Active states should be Root/State1/LARoot"), Exec.ExpectInActiveStates("Root", "State1", "LARoot"));
			Exec.LogClear();
		}

		{
			Exec.SendEvent(GetTestTag1());

			EStateTreeRunStatus Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Active states should be Root/State2"), Exec.ExpectInActiveStates("Root", "State2"));
			Exec.LogClear();
		}

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_EventTransitionInParentTree, "System.StateTree.LinkedAsset.EventTransitionInParentTree");

/*
 * Test for when a subtree completes, broadcasted delegates in parent frame are correctly retained.
 */
struct FStateTreeTest_DelegateTransitionInParentTree : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		// MainTree
		//   Root [BroadcastDelegateTask] OnDelegate -> State2
		//     State1 (LinkedAsset -> LATree)
		//     State2
		//
		// LATree
		//   LARoot OnTick -> Succeeded (Critical)

		FStateTreeCompilerLog Log;

		UStateTree& LATree = NewStateTree();
		{
			UStateTreeEditorData& LAEditorData = *Cast<UStateTreeEditorData>(LATree.EditorData);

			UStateTreeState& LARoot = LAEditorData.AddSubTree("LARoot");
			FStateTreeTransition& LATransition = LARoot.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::Succeeded);
			LATransition.Priority = EStateTreeTransitionPriority::Critical;

			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(LATree);
			AITEST_TRUE(TEXT("LATree should get compiled"), bResult);
		}

		UStateTree& MainTree = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(MainTree.EditorData);

			UStateTreeState& Root = EditorData.AddSubTree("Root");
			TStateTreeEditorNode<FTestTask_BroadcastDelegate>& BroadcastTask = Root.AddTask<FTestTask_BroadcastDelegate>("BroadcastTask");

			UStateTreeState& State1 = Root.AddChildState("State1", EStateTreeStateType::LinkedAsset);
			State1.SetLinkedStateAsset(&LATree);

			UStateTreeState& State2 = Root.AddChildState("State2");

			FStateTreeTransition& DelegateTransition = Root.AddTransition(EStateTreeTransitionTrigger::OnDelegate, EStateTreeTransitionType::GotoState, &State2);

			EditorData.AddPropertyBinding(
				FPropertyBindingPath(BroadcastTask.ID, GET_MEMBER_NAME_CHECKED(FTestTask_BroadcastDelegate_InstanceData, OnTickDelegate)),
				FPropertyBindingPath(DelegateTransition.ID, GET_MEMBER_NAME_CHECKED(FStateTreeTransition, DelegateListener)));

			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(MainTree);
			AITEST_TRUE(TEXT("MainTree should get compiled"), bResult);
		}

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(MainTree, MainTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		{
			EStateTreeRunStatus Status = Exec.Start();
			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Active states should be Root/State1/LARoot"), Exec.ExpectInActiveStates("Root", "State1", "LARoot"));
			Exec.LogClear();
		}

		{
			EStateTreeRunStatus Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Active states should be Root/State2"), Exec.ExpectInActiveStates("Root", "State2"));
			Exec.LogClear();
		}

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_DelegateTransitionInParentTree, "System.StateTree.LinkedAsset.DelegateTransitionInParentTree");

/*
 * Test for subtrees of deep hierarchy complete one after another.
 */
struct FStateTreeTest_DeepSubtreeChain : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		// MainTree
		//   Root
		//     State1 (LinkedAsset -> LA1) OnCompleted -> GotoState State2
		//     State2
		//
		// LA1
		//   LA1Root (LinkedAsset -> LA2) OnCompleted -> Succeeded
		//
		// LA2
		//   LA2Root (LinkedAsset -> LA3) OnTick -> Succeeded
		//
		// LA3
		//   LA3Root (LinkedAsset -> LA4) OnCompleted -> Succeeded
		//
		// LA4
		//   LA4Root (LinkedAsset -> LA5) OnTick -> Succeeded
		//
		// LA5
		//   LA5Root (LinkedAsset -> LA6) OnCompleted -> Succeeded
		//
		// LA6
		//   LA6Root (LinkedAsset -> LA7) OnTick -> Succeeded
		//
		// LA7
		//   LA7Root (LinkedAsset -> LA8) OnCompleted -> Succeeded
		//
		// LA8
		//   LA8Root OnTick -> Succeeded
		//
		// LA9
		//   LA9Root (LinkedAsset -> LA10) OnCompleted -> Succeeded
		//
		// LA10
		//   LA10Root OnTick -> Succeeded

		FStateTreeCompilerLog Log;

		bool bLinkedAssetCompilationSucceeded = false;

		auto CreateLinkedAssetTree = [this, &Log, &bLinkedAssetCompilationSucceeded](const auto* RootName, const bool bUseOnTick, UStateTree* NextTree = nullptr) -> UStateTree&
		{
			UStateTree& Tree = NewStateTree();
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(Tree.EditorData);
				
			UStateTreeState& Root = EditorData.AddSubTree(RootName, EStateTreeStateType::State);
			if (NextTree)
			{
				Root.Type = EStateTreeStateType::LinkedAsset;
				Root.SetLinkedStateAsset(NextTree);
			}

			const EStateTreeTransitionTrigger Trigger = bUseOnTick ? EStateTreeTransitionTrigger::OnTick : EStateTreeTransitionTrigger::OnStateCompleted;
			FStateTreeTransition& Transition = Root.AddTransition(Trigger, EStateTreeTransitionType::Succeeded);

			FStateTreeCompiler Compiler(Log);
			bLinkedAssetCompilationSucceeded = Compiler.Compile(Tree);

			return Tree;
		};

		UStateTree& LA10Tree = CreateLinkedAssetTree("LA10Root", true);
		AITEST_TRUE(TEXT("LinkedAsset Tree should get compiled"), bLinkedAssetCompilationSucceeded);

		UStateTree& LA9Tree = CreateLinkedAssetTree("LA9Root", false, &LA10Tree);
		AITEST_TRUE(TEXT("LinkedAsset Tree should get compiled"), bLinkedAssetCompilationSucceeded);

		UStateTree& LA8Tree = CreateLinkedAssetTree("LA8Root", true, &LA9Tree);
		AITEST_TRUE(TEXT("LinkedAsset Tree should get compiled"), bLinkedAssetCompilationSucceeded);

		UStateTree& LA7Tree = CreateLinkedAssetTree("LA7Root", false, &LA8Tree);
		AITEST_TRUE(TEXT("LinkedAsset Tree should get compiled"), bLinkedAssetCompilationSucceeded);

		UStateTree& LA6Tree = CreateLinkedAssetTree("LA6Root", true, &LA7Tree);
		AITEST_TRUE(TEXT("LinkedAsset Tree should get compiled"), bLinkedAssetCompilationSucceeded);

		UStateTree& LA5Tree = CreateLinkedAssetTree("LA5Root", false, &LA6Tree);
		AITEST_TRUE(TEXT("LinkedAsset Tree should get compiled"), bLinkedAssetCompilationSucceeded);

		UStateTree& LA4Tree = CreateLinkedAssetTree("LA4Root", true, &LA5Tree);
		AITEST_TRUE(TEXT("LinkedAsset Tree should get compiled"), bLinkedAssetCompilationSucceeded);

		UStateTree& LA3Tree = CreateLinkedAssetTree("LA3Root", false, &LA4Tree);
		AITEST_TRUE(TEXT("LinkedAsset Tree should get compiled"), bLinkedAssetCompilationSucceeded);

		UStateTree& LA2Tree = CreateLinkedAssetTree("LA2Root", true, &LA3Tree);
		AITEST_TRUE(TEXT("LinkedAsset Tree should get compiled"), bLinkedAssetCompilationSucceeded);

		UStateTree& LA1Tree = CreateLinkedAssetTree("LA1Root", false, &LA2Tree);
		AITEST_TRUE(TEXT("LinkedAsset Tree should get compiled"), bLinkedAssetCompilationSucceeded);

		UStateTree& MainTree = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(MainTree.EditorData);
			UStateTreeState& Root = EditorData.AddSubTree("Root");

			UStateTreeState& State1 = Root.AddChildState("State1", EStateTreeStateType::LinkedAsset);
			UStateTreeState& State2 = Root.AddChildState("State2");

			State1.SetLinkedStateAsset(&LA1Tree);
			State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &State2);

			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(MainTree);
			AITEST_TRUE(TEXT("MainTree should get compiled"), bResult);
		}

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(MainTree, MainTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		{
			EStateTreeRunStatus Status = Exec.Start();
			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Active states should be Root/State1/LA1Root/.../LA10Root"),
				Exec.ExpectInActiveStates("Root", "State1", "LA1Root", "LA2Root", "LA3Root", "LA4Root", "LA5Root", "LA6Root", "LA7Root", "LA8Root", "LA9Root", "LA10Root"));
			Exec.LogClear();
		}

		{
			EStateTreeRunStatus Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Active states should be Root/State2"),
				Exec.ExpectInActiveStates("Root", "State2"));
			Exec.LogClear();
		}

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_DeepSubtreeChain, "System.StateTree.LinkedAsset.DeepSubtreeChain");


/*
 * Test that a LinkedAsset state can reselect itself when the linked tree completes.
 * The OnStateCompleted transition goes back to itself, causing the linked tree to re-enter each cycle.
 */
struct FStateTreeTest_LinkedAssetReselectOnCompletion : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		// MainTree
		//   Root
		//     LinkedState (LinkedAsset -> LATree) [OnCompleted -> GotoState LinkedState (reselect)]
		//
		// LATree
		//   LARoot [OnCompleted -> Succeeded]

		FStateTreeCompilerLog Log;

		// LATree
		UStateTree& LATree = NewStateTree();
		{
			UStateTreeEditorData& LAEditorData = *Cast<UStateTreeEditorData>(LATree.EditorData);

			UStateTreeState& LARoot = LAEditorData.AddSubTree("LARoot");
			LARoot.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(LATree);
			AITEST_TRUE(TEXT("LATree should get compiled"), bResult);
		}

		// MainTree
		UStateTree& MainTree = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(MainTree.EditorData);

			UStateTreeState& Root = EditorData.AddSubTree("Root");
			UStateTreeState& LinkedState = Root.AddChildState("LinkedState", EStateTreeStateType::LinkedAsset);
			LinkedState.SetLinkedStateAsset(&LATree);
			LinkedState.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &LinkedState);

			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(MainTree);
			AITEST_TRUE(TEXT("MainTree should get compiled"), bResult);
		}

		const TCHAR* EnterStateStr = TEXT("EnterState");

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(MainTree, MainTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		// Start: enter Root/LinkedState/LARoot
		{
			const EStateTreeRunStatus Status = Exec.Start();
			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Active states should be Root/LinkedState/LARoot"), Exec.ExpectInActiveStates("Root", "LinkedState", "LARoot"));
			Exec.LogClear();
		}

		// Tick 1: LATask completes -> LATree succeeds -> LinkedState reselects itself -> LATask re-enters
		{
			const EStateTreeRunStatus Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick 1 should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Active states should be Root/LinkedState/LARoot after reselect"), Exec.ExpectInActiveStates("Root", "LinkedState", "LARoot"));
			Exec.LogClear();
		}

		// Tick 2: Same cycle again
		{
			const EStateTreeRunStatus Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick 2 should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Active states should still be Root/LinkedState/LARoot"), Exec.ExpectInActiveStates("Root", "LinkedState", "LARoot"));
			Exec.LogClear();
		}

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_LinkedAssetReselectOnCompletion, "System.StateTree.LinkedAsset.ReselectOnCompletion");

} // namespace UE::StateTree::Tests

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
