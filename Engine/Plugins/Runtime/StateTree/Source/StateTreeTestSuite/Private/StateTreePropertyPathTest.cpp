// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeTest.h"
#include "StateTreeTestBase.h"
#include "StateTreeTestTypes.h"

#include "StateTreeEditorData.h"
#include "Conditions/StateTreeCommonConditions.h"

#define LOCTEXT_NAMESPACE "AITestSuite_StateTreeTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::StateTree::Tests
{

struct FStateTreeTest_PropertyPathOffset : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		FPropertyBindingPath Path;
		const bool bParseResult = Path.FromString(TEXT("StructB.B"));

		AITEST_TRUE(TEXT("Parsing path should succeeed"), bParseResult);
		AITEST_EQUAL(TEXT("Should have 2 path segments"), Path.NumSegments(), 2);

		FString ResolveErrors;
		TArray<FPropertyBindingPathIndirection> Indirections;
		const bool bResolveResult = Path.ResolveIndirections(FStateTreeTest_PropertyStruct::StaticStruct(), Indirections, &ResolveErrors);

		AITEST_TRUE(TEXT("Resolve path should succeeed"), bResolveResult);
		AITEST_EQUAL(TEXT("Should have no resolve errors"), ResolveErrors.Len(), 0);
		
		AITEST_EQUAL(TEXT("Should have 2 indirections"), Indirections.Num(), 2);
		AITEST_EQUAL(TEXT("Indirection 0 should be Offset type"), Indirections[0].GetAccessType(), EPropertyBindingPropertyAccessType::Offset);
		AITEST_EQUAL(TEXT("Indirection 1 should be Offset type"), Indirections[1].GetAccessType(), EPropertyBindingPropertyAccessType::Offset);

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_PropertyPathOffset, "System.StateTree.PropertyPath.Offset");

struct FStateTreeTest_PropertyPathParseFail : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		{
			FPropertyBindingPath Path;
			const bool bParseResult = Path.FromString(TEXT("")); // empty is valid.
			AITEST_TRUE(TEXT("Parsing path should succeed"), bParseResult);
		}

		{
			FPropertyBindingPath Path;
			const bool bParseResult = Path.FromString(TEXT("StructB.[0]B"));
			AITEST_FALSE(TEXT("Parsing path should fail"), bParseResult);
		}

		{
			FPropertyBindingPath Path;
			const bool bParseResult = Path.FromString(TEXT("StructB..NoThere"));
			AITEST_FALSE(TEXT("Parsing path should fail"), bParseResult);
		}

		{
			FPropertyBindingPath Path;
			const bool bParseResult = Path.FromString(TEXT("."));
			AITEST_FALSE(TEXT("Parsing path should fail"), bParseResult);
		}

		{
			FPropertyBindingPath Path;
			const bool bParseResult = Path.FromString(TEXT("StructB..B"));
			AITEST_FALSE(TEXT("Parsing path should fail"), bParseResult);
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_PropertyPathParseFail, "System.StateTree.PropertyPath.ParseFail");

struct FStateTreeTest_PropertyPathOffsetFail : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		FPropertyBindingPath Path;
		const bool bParseResult = Path.FromString(TEXT("StructB.Q"));

		AITEST_TRUE(TEXT("Parsing path should succeeed"), bParseResult);
		AITEST_EQUAL(TEXT("Should have 2 path segments"), Path.NumSegments(), 2);

		FString ResolveErrors;
		TArray<FPropertyBindingPathIndirection> Indirections;
		const bool bResolveResult = Path.ResolveIndirections(FStateTreeTest_PropertyStruct::StaticStruct(), Indirections, &ResolveErrors);

		AITEST_FALSE(TEXT("Resolve path should not succeeed"), bResolveResult);
		AITEST_NOT_EQUAL(TEXT("Should have errors"), ResolveErrors.Len(), 0);
		
		AITEST_EQUAL(TEXT("Should have 0 indirections"), Indirections.Num(), 0);

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_PropertyPathOffsetFail, "System.StateTree.PropertyPath.OffsetFail");

struct FStateTreeTest_PropertyPathObject : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		FPropertyBindingPath Path;
		const bool bParseResult = Path.FromString(TEXT("InstancedObject.A"));

		AITEST_TRUE(TEXT("Parsing path should succeeed"), bParseResult);
		AITEST_EQUAL(TEXT("Should have 2 path segments"), Path.NumSegments(), 2);

		UStateTreeTest_PropertyObject* Object = NewObject<UStateTreeTest_PropertyObject>();
		Object->InstancedObject = NewObject<UStateTreeTest_PropertyObjectInstanced>();
		
		const bool bUpdateResult = Path.UpdateSegmentsFromValue(FStateTreeDataView(Object));

		AITEST_TRUE(TEXT("Update instance types should succeeed"), bUpdateResult);
		AITEST_TRUE(TEXT("Path segment 0 instance type should be UStateTreeTest_PropertyObjectInstanced"), Path.GetSegment(0).GetInstanceStruct() == UStateTreeTest_PropertyObjectInstanced::StaticClass());
		AITEST_TRUE(TEXT("Path segment 1 instance type should be nullptr"), Path.GetSegment(1).GetInstanceStruct() == nullptr);

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_PropertyPathObject, "System.StateTree.PropertyPath.Object");

struct FStateTreeTest_PropertyPathWrongObject : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		FPropertyBindingPath Path;
		const bool bParseResult = Path.FromString(TEXT("InstancedObject.B"));

		AITEST_TRUE(TEXT("Parsing path should succeeed"), bParseResult);
		AITEST_EQUAL(TEXT("Should have 2 path segments"), Path.NumSegments(), 2);

		UStateTreeTest_PropertyObject* Object = NewObject<UStateTreeTest_PropertyObject>();

		Object->InstancedObject = NewObject<UStateTreeTest_PropertyObjectInstancedWithB>();
		{
			FString ResolveErrors;
			TArray<FPropertyBindingPathIndirection> Indirections;
			const bool bResolveResult = Path.ResolveIndirectionsWithValue(FStateTreeDataView(Object), Indirections, &ResolveErrors);

			AITEST_TRUE(TEXT("Resolve path should succeeed"), bResolveResult);
			AITEST_EQUAL(TEXT("Should have 2 indirections"), Indirections.Num(), 2);
			AITEST_TRUE(TEXT("Object "), Indirections[0].GetAccessType() == EPropertyBindingPropertyAccessType::ObjectInstance);
			AITEST_TRUE(TEXT("Object "), Indirections[0].GetContainerStruct() == Object->GetClass());
			AITEST_TRUE(TEXT("Object "), Indirections[0].GetInstanceStruct() == UStateTreeTest_PropertyObjectInstancedWithB::StaticClass());
			AITEST_EQUAL(TEXT("Should not have error"), ResolveErrors.Len(), 0);
		}

		Object->InstancedObject = NewObject<UStateTreeTest_PropertyObjectInstanced>();
		{
			FString ResolveErrors;
			TArray<FPropertyBindingPathIndirection> Indirections;
			const bool bResolveResult = Path.ResolveIndirectionsWithValue(FStateTreeDataView(Object), Indirections, &ResolveErrors);

			AITEST_FALSE(TEXT("Resolve path should fail"), bResolveResult);
			AITEST_EQUAL(TEXT("Should have 0 indirections"), Indirections.Num(), 0);
			AITEST_NOT_EQUAL(TEXT("Should have error"), ResolveErrors.Len(), 0);
		}
		
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_PropertyPathWrongObject, "System.StateTree.PropertyPath.WrongObject");

struct FStateTreeTest_PropertyPathArray : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		FPropertyBindingPath Path;
		const bool bParseResult = Path.FromString(TEXT("ArrayOfInts[1]"));

		AITEST_TRUE(TEXT("Parsing path should succeeed"), bParseResult);
		AITEST_EQUAL(TEXT("Should have 1 path segments"), Path.NumSegments(), 1);

		UStateTreeTest_PropertyObject* Object = NewObject<UStateTreeTest_PropertyObject>();
		Object->ArrayOfInts.Add(42);
		Object->ArrayOfInts.Add(123);

		FString ResolveErrors;
		TArray<FPropertyBindingPathIndirection> Indirections;
		const bool bResolveResult = Path.ResolveIndirectionsWithValue(FStateTreeDataView(Object), Indirections, &ResolveErrors);

		AITEST_TRUE(TEXT("Resolve path should succeeed"), bResolveResult);
		AITEST_EQUAL(TEXT("Should have no resolve errors"), ResolveErrors.Len(), 0);
		AITEST_EQUAL(TEXT("Should have 2 indirections"), Indirections.Num(), 2);
		AITEST_EQUAL(TEXT("Indirection 0 should be IndexArray type"), Indirections[0].GetAccessType(), EPropertyBindingPropertyAccessType::IndexArray);
		AITEST_EQUAL(TEXT("Indirection 1 should be Offset type"), Indirections[1].GetAccessType(), EPropertyBindingPropertyAccessType::Offset);

		const int32 Value = *reinterpret_cast<const int32*>(Indirections[1].GetPropertyAddress());
		AITEST_EQUAL(TEXT("Value should be 123"), Value, 123);
		
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_PropertyPathArray, "System.StateTree.PropertyPath.Array");

struct FStateTreeTest_PropertyPathArrayInvalidIndex : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		FPropertyBindingPath Path;
		const bool bParseResult = Path.FromString(TEXT("ArrayOfInts[123]"));

		AITEST_TRUE(TEXT("Parsing path should succeeed"), bParseResult);
		AITEST_EQUAL(TEXT("Should have 1 path segments"), Path.NumSegments(), 1);

		UStateTreeTest_PropertyObject* Object = NewObject<UStateTreeTest_PropertyObject>();
		Object->ArrayOfInts.Add(42);
		Object->ArrayOfInts.Add(123);

		FString ResolveErrors;
		TArray<FPropertyBindingPathIndirection> Indirections;
		const bool bResolveResult = Path.ResolveIndirectionsWithValue(FStateTreeDataView(Object), Indirections, &ResolveErrors);

		AITEST_FALSE(TEXT("Resolve path should fail"), bResolveResult);
		
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_PropertyPathArrayInvalidIndex, "System.StateTree.PropertyPath.ArrayInvalidIndex");

struct FStateTreeTest_PropertyPathArrayOfStructs : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		FPropertyBindingPath Path1;
		Path1.FromString(TEXT("ArrayOfStruct[0].B"));

		FPropertyBindingPath Path2;
		Path2.FromString(TEXT("ArrayOfStruct[2].StructB.B"));

		UStateTreeTest_PropertyObject* Object = NewObject<UStateTreeTest_PropertyObject>();
		Object->ArrayOfStruct.AddDefaulted_GetRef().B = 3;
		Object->ArrayOfStruct.AddDefaulted();
		Object->ArrayOfStruct.AddDefaulted_GetRef().StructB.B = 42;

		{
			FString ResolveErrors;
			TArray<FPropertyBindingPathIndirection> Indirections;
			const bool bResolveResult = Path1.ResolveIndirectionsWithValue(FStateTreeDataView(Object), Indirections, &ResolveErrors);

			AITEST_TRUE(TEXT("Resolve path1 should succeeed"), bResolveResult);
			AITEST_EQUAL(TEXT("Should have no resolve errors"), ResolveErrors.Len(), 0);
			AITEST_EQUAL(TEXT("Should have 3 indirections"), Indirections.Num(), 3);
			AITEST_EQUAL(TEXT("Indirection 0 should be ArrayIndex type"), Indirections[0].GetAccessType(), EPropertyBindingPropertyAccessType::IndexArray);
			AITEST_EQUAL(TEXT("Indirection 1 should be Offset type"), Indirections[1].GetAccessType(), EPropertyBindingPropertyAccessType::Offset);
			AITEST_EQUAL(TEXT("Indirection 2 should be Offset type"), Indirections[2].GetAccessType(), EPropertyBindingPropertyAccessType::Offset);

			const int32 Value = *reinterpret_cast<const int32*>(Indirections[2].GetPropertyAddress());
			AITEST_EQUAL(TEXT("Value should be 3"), Value, 3);
		}

		{
			FString ResolveErrors;
			TArray<FPropertyBindingPathIndirection> Indirections;
			const bool bResolveResult = Path2.ResolveIndirectionsWithValue(FStateTreeDataView(Object), Indirections, &ResolveErrors);

			AITEST_TRUE(TEXT("Resolve path2 should succeeed"), bResolveResult);
			AITEST_EQUAL(TEXT("Should have no resolve errors"), ResolveErrors.Len(), 0);
			AITEST_EQUAL(TEXT("Should have 4 indirections"), Indirections.Num(), 4);
			AITEST_EQUAL(TEXT("Indirection 0 should be ArrayIndex type"), Indirections[0].GetAccessType(), EPropertyBindingPropertyAccessType::IndexArray);
			AITEST_EQUAL(TEXT("Indirection 1 should be Offset type"), Indirections[1].GetAccessType(), EPropertyBindingPropertyAccessType::Offset);
			AITEST_EQUAL(TEXT("Indirection 2 should be Offset type"), Indirections[2].GetAccessType(), EPropertyBindingPropertyAccessType::Offset);
			AITEST_EQUAL(TEXT("Indirection 3 should be Offset type"), Indirections[3].GetAccessType(), EPropertyBindingPropertyAccessType::Offset);

			const int32 Value = *reinterpret_cast<const int32*>(Indirections[3].GetPropertyAddress());
			AITEST_EQUAL(TEXT("Value should be 42"), Value, 42);
		}
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_PropertyPathArrayOfStructs, "System.StateTree.PropertyPath.ArrayOfStructs");

struct FStateTreeTest_PropertyPathArrayOfInstancedObjects : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		FPropertyBindingPath Path;
		Path.FromString(TEXT("ArrayOfInstancedStructs[0].B"));

		FStateTreeTest_PropertyStruct Struct;
		Struct.B = 123;
		
		UStateTreeTest_PropertyObject* Object = NewObject<UStateTreeTest_PropertyObject>();
		Object->ArrayOfInstancedStructs.Emplace(FConstStructView::Make(Struct));

		const bool bUpdateResult = Path.UpdateSegmentsFromValue(FStateTreeDataView(Object));
		AITEST_TRUE(TEXT("Update instance types should succeeed"), bUpdateResult);
		AITEST_EQUAL(TEXT("Should have 2 path segments"), Path.NumSegments(), 2);
		AITEST_TRUE(TEXT("Path segment 0 instance type should be FStateTreeTest_PropertyStruct"), Path.GetSegment(0).GetInstanceStruct() == FStateTreeTest_PropertyStruct::StaticStruct());
		AITEST_TRUE(TEXT("Path segment 1 instance type should be nullptr"), Path.GetSegment(1).GetInstanceStruct() == nullptr);

		{
			FString ResolveErrors;
			TArray<FPropertyBindingPathIndirection> Indirections;
			const bool bResolveResult = Path.ResolveIndirections(UStateTreeTest_PropertyObject::StaticClass(), Indirections, &ResolveErrors);

			AITEST_TRUE(TEXT("Resolve path should succeeed"), bResolveResult);
			AITEST_EQUAL(TEXT("Should have no resolve errors"), ResolveErrors.Len(), 0);
			AITEST_EQUAL(TEXT("Should have 3 indirections"), Indirections.Num(), 3);
			AITEST_EQUAL(TEXT("Indirection 0 should be ArrayIndex type"), Indirections[0].GetAccessType(), EPropertyBindingPropertyAccessType::IndexArray);
			AITEST_EQUAL(TEXT("Indirection 1 should be StructInstance type"), Indirections[1].GetAccessType(), EPropertyBindingPropertyAccessType::StructInstance);
			AITEST_EQUAL(TEXT("Indirection 2 should be Offset type"), Indirections[2].GetAccessType(), EPropertyBindingPropertyAccessType::Offset);
		}

		{
			FString ResolveErrors;
			TArray<FPropertyBindingPathIndirection> Indirections;
			const bool bResolveResult = Path.ResolveIndirectionsWithValue(FStateTreeDataView(Object), Indirections, &ResolveErrors);

			AITEST_TRUE(TEXT("Resolve path should succeeed"), bResolveResult);
			AITEST_EQUAL(TEXT("Should have no resolve errors"), ResolveErrors.Len(), 0);
			AITEST_EQUAL(TEXT("Should have 3 indirections"), Indirections.Num(), 3);
			AITEST_EQUAL(TEXT("Indirection 0 should be ArrayIndex type"), Indirections[0].GetAccessType(), EPropertyBindingPropertyAccessType::IndexArray);
			AITEST_EQUAL(TEXT("Indirection 1 should be StructInstance type"), Indirections[1].GetAccessType(), EPropertyBindingPropertyAccessType::StructInstance);
			AITEST_EQUAL(TEXT("Indirection 2 should be Offset type"), Indirections[2].GetAccessType(), EPropertyBindingPropertyAccessType::Offset);

			const int32 Value = *reinterpret_cast<const int32*>(Indirections[2].GetPropertyAddress());
			AITEST_EQUAL(TEXT("Value should be 123"), Value, 123);
		}
		
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_PropertyPathArrayOfInstancedObjects, "System.StateTree.PropertyPath.ArrayOfInstancedObjects");

} // namespace UE::StateTree::Tests

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
