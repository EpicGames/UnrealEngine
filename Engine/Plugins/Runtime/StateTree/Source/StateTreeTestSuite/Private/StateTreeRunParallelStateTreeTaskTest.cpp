// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeTest.h"
#include "StateTreeTestBase.h"
#include "StateTreeTestTypes.h"

#include "StateTreeCompilerLog.h"
#include "StateTreeEditorData.h"
#include "StateTreeCompiler.h"
#include "Tasks/StateTreeRunParallelStateTreeTask.h"
#include "Conditions/StateTreeCommonConditions.h"

#define LOCTEXT_NAMESPACE "AITestSuite_StateTreeRunParallelTask"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::StateTree::Tests
{

/**
 * Test: Basic parallel tree execution - parallel tree runs while main tree continues
 */
struct FStateTreeTest_RunParallelTask_BasicExecution : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		// Create parallel state tree
		UStateTree& ParallelStateTree = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(ParallelStateTree.EditorData);

			UStateTreeState& Root = EditorData.AddSubTree("ParallelRoot");
			{
				TStateTreeEditorNode<FTestTask_Stand>& StandTask = Root.AddTask<FTestTask_Stand>("ParallelTask");
				StandTask.GetNode().TicksToCompletion = 3;
				StandTask.GetNode().TickCompletionResult = EStateTreeRunStatus::Succeeded;

				Root.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);
			}

			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(ParallelStateTree);
			AITEST_TRUE(TEXT("Parallel StateTree should compile"), bResult);
		}

		// Create main state tree
		UStateTree& MainStateTree = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(MainStateTree.EditorData);

			UStateTreeState& Root = EditorData.AddSubTree("MainRoot");

			// Add a task that runs a parallel tree
			TStateTreeEditorNode<FStateTreeRunParallelStateTreeTask>& ParallelTask = Root.AddTask<FStateTreeRunParallelStateTreeTask>();
			ParallelTask.GetInstanceData().StateTree.SetStateTree(&ParallelStateTree);

			TStateTreeEditorNode<FTestTask_Stand>& StandTask = Root.AddTask<FTestTask_Stand>("MainTask");
			StandTask.GetNode().TicksToCompletion = 100;
			StandTask.GetNode().bConsideredForCompletion = false;

			Root.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(MainStateTree);
			AITEST_TRUE(TEXT("Main StateTree should compile"), bResult);
		}

		// Execute main state tree
		{
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(MainStateTree, MainStateTree, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("MainStateTree should initialize"), bInitSucceeded);

			EStateTreeRunStatus Status = Exec.Start();
			AITEST_EQUAL(TEXT("Start should return Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("ParallelTask should have entered"), Exec.Expect("ParallelTask", TEXT("EnterState")));
			AITEST_TRUE(TEXT("MainTask should have entered"), Exec.Expect("MainTask", TEXT("EnterState")));
			Exec.LogClear();

			// Tick multiple times - parallel task completes after 2 ticks
			for (int32 TickIndex = 0; TickIndex < 2; ++TickIndex)
			{
				Status = Exec.Tick(0.1f);
				AITEST_TRUE(TEXT("ParallelTask should have ticked"), Exec.Expect("ParallelTask", TEXT("Tick")));
				AITEST_TRUE(TEXT("Tick should return Running"), Status == EStateTreeRunStatus::Running);
				Exec.LogClear();
			}

			// Completes on 3rd tick (MainTask completes after 3 ticks)
			Status = Exec.Tick(0.1f);
			AITEST_TRUE(TEXT("ParallelTask should have Tick"), Exec.Expect("ParallelTask", TEXT("Tick")));
			AITEST_TRUE(TEXT("Tick should return Succeeded"), Status == EStateTreeRunStatus::Succeeded);
			AITEST_TRUE(TEXT("MainTask should have exited"), Exec.Expect("MainTask", TEXT("ExitState")));
			Exec.LogClear();

			Exec.Stop();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_RunParallelTask_BasicExecution, "System.StateTree.ParallelTask.BasicExecution");

/**
 * Test: Invalid parallel tree reference returns Failed
 */
struct FStateTreeTest_RunParallelTask_InvalidReference : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& MainStateTree = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(MainStateTree.EditorData);

			UStateTreeState& Root = EditorData.AddSubTree("Root");

			// Add parallel task without setting a valid state tree reference
			TStateTreeEditorNode<FStateTreeRunParallelStateTreeTask>& ParallelTask = Root.AddTask<FStateTreeRunParallelStateTreeTask>();
			// Note: ParallelTask.GetNode().InstanceData.StateTree is not set

			UStateTreeState& State1 = Root.AddChildState("State1");
			{
				TStateTreeEditorNode<FTestTask_Stand>& StandTask = State1.AddTask<FTestTask_Stand>("Task1");
				StandTask.GetNode().TicksToCompletion = 1;
			}

			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			AITEST_TRUE(TEXT("StateTree should compile"), Compiler.Compile(MainStateTree));
		}

		// Execute with invalid reference - should fail on enter
		{
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(MainStateTree, MainStateTree, InstanceData);

			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should initialize"), bInitSucceeded);

			GetTestRunner().AddExpectedErrorPlain(TEXT("Trying to start an invalid parallel tree from the tree"), EAutomationExpectedErrorFlags::Contains, 1);

			EStateTreeRunStatus Status = Exec.Start();

			AITEST_TRUE(TEXT("Invalid error should have been logged"), GetTestRunner().HasMetExpectedMessages());

			Exec.Stop();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_RunParallelTask_InvalidReference, "System.StateTree.ParallelTask.InvalidReference");

/**
 * Test: Recursive/circular reference detection
 */
struct FStateTreeTest_RunParallelTask_RecursionDetection : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		// Create tree that references itself
		UStateTree& SelfRefStateTree = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(SelfRefStateTree.EditorData);

			UStateTreeState& Root = EditorData.AddSubTree("Root");

			TStateTreeEditorNode<FStateTreeRunParallelStateTreeTask>& ParallelTask = Root.AddTask<FStateTreeRunParallelStateTreeTask>();
			ParallelTask.GetInstanceData().StateTree.SetStateTree(&SelfRefStateTree);

			UStateTreeState& State1 = Root.AddChildState("State1");
			{
				TStateTreeEditorNode<FTestTask_Stand>& StandTask = State1.AddTask<FTestTask_Stand>("Task1");
				StandTask.GetNode().TicksToCompletion = 1;
			}

			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			AITEST_TRUE(TEXT("StateTree should compile"), Compiler.Compile(SelfRefStateTree));
		}

		// Try to execute - recursion detection should prevent stack overflow
		{
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(SelfRefStateTree, SelfRefStateTree, InstanceData);

			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should initialize"), bInitSucceeded);

			// This should detect recursion and fail gracefully
			GetTestRunner().AddExpectedErrorPlain(TEXT("Trying to start a new parallel tree from the same tree"), EAutomationExpectedErrorFlags::Contains, 1);

			EStateTreeRunStatus Status = Exec.Start();

			AITEST_TRUE(TEXT("Trying to start a new parallel tree"), GetTestRunner().HasMetExpectedMessages());

			Exec.Stop();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_RunParallelTask_RecursionDetection, "System.StateTree.ParallelTask.RecursionDetection");

/**
 * Test: Copy instance data on tick
 */
struct FStateTreeTest_RunParallelTask_CopyInstanceDataOnTick : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& ParallelStateTree = NewStateTree();
		FGuid EvalIndexID;
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(ParallelStateTree.EditorData);

			// Add root parameters to ParallelStateTree
			FInstancedPropertyBag& RootPropertyBag = GetRootPropertyBag(EditorData);
			RootPropertyBag.AddProperty("EvalIndex", EPropertyBagPropertyType::Int32);
			RootPropertyBag.SetValueInt32("EvalIndex", -1);
			EvalIndexID = RootPropertyBag.FindPropertyDescByName("EvalIndex")->ID;

			// Add global parameter to track updated values
			TStateTreeEditorNode<FTestTask_PrintValue>& GlobalTask = EditorData.AddGlobalTask<FTestTask_PrintValue>("ParallelGlobalTask");
			GlobalTask.GetInstanceData().Value = 0;
			EditorData.AddPropertyBinding(FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("EvalIndex")), FPropertyBindingPath(GlobalTask.ID, TEXT("Value")));

			UStateTreeState& Root = EditorData.AddSubTree("ParallelRoot");
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& ParallelTask = Root.AddTask<FTestTask_PrintValue>("ParallelTask");
				ParallelTask.GetInstanceData().Value = 0;
				EditorData.AddPropertyBinding(FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("EvalIndex")), FPropertyBindingPath(ParallelTask.ID, TEXT("Value")));
			}

			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			AITEST_TRUE(TEXT("Parallel StateTree should compile"), Compiler.Compile(ParallelStateTree));
		}

		UStateTree& MainStateTree = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(MainStateTree.EditorData);

			TStateTreeEditorNode<FTestEval_Custom>& CustomEval = EditorData.AddEvaluator<FTestEval_Custom>();
			CustomEval.GetInstanceData().IntA = 0;

			// Simulate updating a value on each tick that should propagate to parallel tree
			CustomEval.GetNode().CustomTickFunc = [](FStateTreeExecutionContext& Context, const FTestEval_Custom* Task)
					{
						++Context.GetInstanceData(*Task).IntA;
					};

			UStateTreeState& Root = EditorData.AddSubTree("Root");

			TStateTreeEditorNode<FStateTreeRunParallelStateTreeTask>& ParallelTask = Root.AddTask<FStateTreeRunParallelStateTreeTask>();
			ParallelTask.GetInstanceData().StateTree.SetStateTree(&ParallelStateTree);
			ParallelTask.GetInstanceData().StateTree.SetPropertyOverridden(EvalIndexID, true);
			ParallelTask.GetNode().bShouldCopyParametersOnTick = true;
			ParallelTask.GetNode().bShouldCopyParametersOnExitState = false;

			// Bind CustomEval output to ParallelStateTree's root parameter
			FPropertyBindingPath TargetBindingPath = FPropertyBindingPath(ParallelTask.ID);
			TargetBindingPath.FromString(TEXT("StateTree.Parameters.Value.EvalIndex"));
			EditorData.AddPropertyBinding(FPropertyBindingPath(CustomEval.ID, "IntA"), TargetBindingPath);

			UStateTreeState& MainState = Root.AddChildState("MainState");
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& PrintTask = MainState.AddTask<FTestTask_PrintValue>("MainTask");
				PrintTask.GetInstanceData().Value = 0;
				EditorData.AddPropertyBinding(FPropertyBindingPath(CustomEval.ID, TEXT("IntA")), FPropertyBindingPath(PrintTask.ID, TEXT("Value")));
			}

			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			bool bCompileResult = Compiler.Compile(MainStateTree);
			AITEST_TRUE(TEXT("StateTree should compile"), bCompileResult);
		}

		// Execute with data copy enabled
		{
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(MainStateTree, MainStateTree, InstanceData);

			AITEST_TRUE(TEXT("StateTree should initialize"), Exec.IsValid());

			EStateTreeRunStatus Status = Exec.Start();
			AITEST_EQUAL(TEXT("Start should return Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("ParallelGlobalTask should have entered"), Exec.Expect("ParallelGlobalTask", TEXT("EnterState1")));
			AITEST_TRUE(TEXT("ParallelTask should have entered"), Exec.Expect("ParallelTask", TEXT("EnterState1")));
			AITEST_TRUE(TEXT("MainTask should have entered"), Exec.Expect("MainTask", TEXT("EnterState1")));
			Exec.LogClear();

			Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick should return Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("MainTask should have ticked"), Exec.Expect("MainTask", TEXT("Tick2")));
			AITEST_TRUE(TEXT("ParallelTask should have ticked"), Exec.Expect("ParallelTask", TEXT("Tick2")));
			AITEST_TRUE(TEXT("ParallelGlobalTask should have ticked"), Exec.Expect("ParallelGlobalTask", TEXT("Tick2")));
			Exec.LogClear();

			Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick should return Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("MainTask should have ticked"), Exec.Expect("MainTask", TEXT("Tick3")));
			AITEST_TRUE(TEXT("ParallelTask should have ticked"), Exec.Expect("ParallelTask", TEXT("Tick3")));
			AITEST_TRUE(TEXT("ParallelGlobalTask should have ticked"), Exec.Expect("ParallelGlobalTask", TEXT("Tick3")));
			Exec.LogClear();

			Exec.Stop();
			AITEST_TRUE(TEXT("ParallelGlobalTask should have exited"), Exec.Expect("ParallelGlobalTask", TEXT("ExitState3")));
			AITEST_TRUE(TEXT("ParallelTask should have exited"), Exec.Expect("ParallelTask", TEXT("ExitState3")));
			AITEST_TRUE(TEXT("MainTask should have exited"), Exec.Expect("MainTask", TEXT("ExitState3")));
			Exec.LogClear();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_RunParallelTask_CopyInstanceDataOnTick, "System.StateTree.ParallelTask.CopyInstanceDataOnTick");

/**
 * Test: Copy instance data on exit state
 */
struct FStateTreeTest_RunParallelTask_CopyInstanceDataOnExitState : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& ParallelStateTree = NewStateTree();
		FGuid EvalIndexID;
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(ParallelStateTree.EditorData);

			// Add root parameters to ParallelStateTree
			FInstancedPropertyBag& RootPropertyBag = GetRootPropertyBag(EditorData);
			RootPropertyBag.AddProperty("EvalIndex", EPropertyBagPropertyType::Int32);
			RootPropertyBag.SetValueInt32("EvalIndex", -1);
			EvalIndexID = RootPropertyBag.FindPropertyDescByName("EvalIndex")->ID;

			// Add global parameter to track updated values
			TStateTreeEditorNode<FTestTask_PrintValue>& GlobalTask = EditorData.AddGlobalTask<FTestTask_PrintValue>("ParallelGlobalTask");
			GlobalTask.GetInstanceData().Value = 0;
			EditorData.AddPropertyBinding(FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("EvalIndex")), FPropertyBindingPath(GlobalTask.ID, TEXT("Value")));

			UStateTreeState& Root = EditorData.AddSubTree("ParallelRoot");
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& ParallelTask = Root.AddTask<FTestTask_PrintValue>("ParallelTask");
				ParallelTask.GetInstanceData().Value = 0;
				EditorData.AddPropertyBinding(FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("EvalIndex")), FPropertyBindingPath(ParallelTask.ID, TEXT("Value")));
			}

			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			AITEST_TRUE(TEXT("Parallel StateTree should compile"), Compiler.Compile(ParallelStateTree));
		}

		UStateTree& MainStateTree = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(MainStateTree.EditorData);

			TStateTreeEditorNode<FTestEval_Custom>& CustomEval = EditorData.AddEvaluator<FTestEval_Custom>();
			CustomEval.GetInstanceData().IntA = 0;

			// Simulate updating a value on each tick that should propagate to parallel tree
			CustomEval.GetNode().CustomTickFunc = [](FStateTreeExecutionContext& Context, const FTestEval_Custom* Task)
				{
					++Context.GetInstanceData(*Task).IntA;
				};

			UStateTreeState& Root = EditorData.AddSubTree("Root");

			TStateTreeEditorNode<FStateTreeRunParallelStateTreeTask>& ParallelTask = Root.AddTask<FStateTreeRunParallelStateTreeTask>();
			ParallelTask.GetInstanceData().StateTree.SetStateTree(&ParallelStateTree);
			ParallelTask.GetInstanceData().StateTree.SetPropertyOverridden(EvalIndexID, true);
			ParallelTask.GetNode().bShouldCopyParametersOnTick = false;
			ParallelTask.GetNode().bShouldCopyParametersOnExitState = true;

			// Bind CustomEval output to ParallelStateTree's root parameter
			FPropertyBindingPath TargetBindingPath = FPropertyBindingPath(ParallelTask.ID);
			TargetBindingPath.FromString(TEXT("StateTree.Parameters.Value.EvalIndex"));
			EditorData.AddPropertyBinding(FPropertyBindingPath(CustomEval.ID, "IntA"), TargetBindingPath);

			UStateTreeState& MainState = Root.AddChildState("MainState");
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& PrintTask = MainState.AddTask<FTestTask_PrintValue>("MainTask");
				PrintTask.GetInstanceData().Value = 0;
				EditorData.AddPropertyBinding(FPropertyBindingPath(CustomEval.ID, TEXT("IntA")), FPropertyBindingPath(PrintTask.ID, TEXT("Value")));
			}

			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			bool bCompileResult = Compiler.Compile(MainStateTree);
			AITEST_TRUE(TEXT("StateTree should compile"), bCompileResult);
		}

		// Execute with data copy enabled
		{
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(MainStateTree, MainStateTree, InstanceData);

			AITEST_TRUE(TEXT("StateTree should initialize"), Exec.IsValid());

			EStateTreeRunStatus Status = Exec.Start();
			AITEST_EQUAL(TEXT("Start should return Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("ParallelGlobalTask should have entered"), Exec.Expect("ParallelGlobalTask", TEXT("EnterState1")));
			AITEST_TRUE(TEXT("ParallelTask should have entered"), Exec.Expect("ParallelTask", TEXT("EnterState1")));
			AITEST_TRUE(TEXT("MainTask should have entered"), Exec.Expect("MainTask", TEXT("EnterState1")));
			Exec.LogClear();

			// is inc to 2 but not copied
			Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick should return Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("MainTask should have ticked"), Exec.Expect("MainTask", TEXT("Tick2")));
			AITEST_TRUE(TEXT("ParallelTask should have ticked"), Exec.Expect("ParallelTask", TEXT("Tick1")));
			AITEST_TRUE(TEXT("ParallelGlobalTask should have ticked"), Exec.Expect("ParallelGlobalTask", TEXT("Tick1")));
			Exec.LogClear();

			Exec.Stop();
			AITEST_TRUE(TEXT("ParallelGlobalTask should have exited"), Exec.Expect("ParallelGlobalTask", TEXT("ExitState2")));
			AITEST_TRUE(TEXT("ParallelTask should have exited"), Exec.Expect("ParallelTask", TEXT("ExitState2")));
			AITEST_TRUE(TEXT("MainTask should have exited"), Exec.Expect("MainTask", TEXT("ExitState2")));
			Exec.LogClear();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_RunParallelTask_CopyInstanceDataOnExitState, "System.StateTree.ParallelTask.CopyInstanceDataOnExitState");


// Test parallel tree event priority handling.
struct FStateTreeTest_RecursiveParallelTask : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		//Tree 1
		//	Root (with task that runs Tree 1)

		UStateTree& StateTree1 = NewStateTree();
		{
			UStateTreeEditorData& EditorData1 = *Cast<UStateTreeEditorData>(StateTree1.EditorData);
			UStateTreeState* Root1 = &EditorData1.AddSubTree("Tree1StateRoot");

			TStateTreeEditorNode<FStateTreeRunParallelStateTreeTask>& GlobalTask = EditorData1.AddGlobalTask<FStateTreeRunParallelStateTreeTask>();
			GlobalTask.GetInstanceData().StateTree.SetStateTree(&StateTree1);

			TStateTreeEditorNode<FTestTask_PrintValue>& RootTask = Root1->AddTask<FTestTask_PrintValue>();
			RootTask.GetInstanceData().Value = 101;
		}
		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree1);
			AITEST_TRUE(TEXT("StateTreePar should get compiled"), bResult);
		}
		{
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

			{
				GetTestRunner().AddExpectedErrorPlain(TEXT("Trying to start a new parallel tree from the same tree"), EAutomationExpectedErrorFlags::Contains, 1);

				EStateTreeRunStatus Status = Exec.Start();
				AITEST_EQUAL(TEXT("Start should complete with failed"), Status, EStateTreeRunStatus::Failed);
				AITEST_TRUE(TEXT(""), GetTestRunner().HasMetExpectedMessages());

				Exec.Stop();
			}
		}
		return true;
	}
};

IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_RecursiveParallelTask, "System.StateTree.RecursiveParallelTask");

struct FStateTreeTest_ParallelTreeSendEvents : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		// MainTree
		//  Root (Run ParallelTree1, Run ParallelTree2)

		// ParallelTree1
		// PT1Root On Event 4 -> PT1Child4 Critical
		//  PT1Child1 On Tick -> PT1Child2
		//  PT1Child2 SendEvent 5 on Enter, On Event 5 -> PT1Child3
		//  PT1Child3 -> Linked Asset
		//  PT1Child4

		// Linked Asset global : Send Event 1 on Enter
		// LARoot
		//  LAChild1 On Event 1 -> LAChild2
		//  LAChild2 Send Event 2 on enter

		// ParallelTree2
		// PT2Root
		//  PT2Child1 On Event 2 -> PT2Child 2
		//  PT2Child2 SendEvent 4 on Enter 
		auto MakeSendEventOnEnterTask = []<typename T, typename... TArgs>(TNotNull<T*> StateOrTree, const FGameplayTag EventTag, TArgs&&... InArgs)->TStateTreeEditorNode<FTestTask_PrintValue>&
			requires TIsDerivedFrom<T, UStateTreeState>::IsDerived || TIsDerivedFrom<T, UStateTreeEditorData>::IsDerived 
			{
				TStateTreeEditorNode<FTestTask_PrintValue>* PrintValueTaskPtr = nullptr;
				
				if constexpr (TIsDerivedFrom<T, UStateTreeState>::IsDerived)
				{
					PrintValueTaskPtr = &StateOrTree->template AddTask<FTestTask_PrintValue>(InArgs...);
					
				}
				else if constexpr (TIsDerivedFrom<T, UStateTreeEditorData>::IsDerived)
				{
					PrintValueTaskPtr = &StateOrTree->template AddGlobalTask<FTestTask_PrintValue>(InArgs...);

				}

				PrintValueTaskPtr->GetNode().CustomEnterStateFunc = [EventTag](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* PrintTask)
					{
						Context.SendEvent(EventTag);
					};

				return *PrintValueTaskPtr;
			};

		auto HasEvent = [](const FStateTreeEventQueue& EventQueue, const FGameplayTag EventTag)
			{
				TConstArrayView<FStateTreeSharedEvent> EventsView = EventQueue.GetEventsView();

				for (const FStateTreeSharedEvent& Event : EventsView)
				{
					if (Event->Tag == EventTag)
					{
						return true;
					}
				}

				return false;
			};
			
		// Linked Asset
		UStateTree& LinkedAsset = NewStateTree();
		{
			UStateTreeEditorData& LAEditorData = *Cast<UStateTreeEditorData>(LinkedAsset.EditorData);

			MakeSendEventOnEnterTask(TNotNull<UStateTreeEditorData*>(&LAEditorData), GetTestTag1(), "LAGlobalTask1");

			UStateTreeState& LARoot = LAEditorData.AddSubTree("LARoot");

			UStateTreeState& LAChild1 = LARoot.AddChildState("LAChild1");
			TStateTreeEditorNode<FTestTask_PrintValue>& LAChild1Task1 = LAChild1.AddTask<FTestTask_PrintValue>("LAChild1Task1");

			UStateTreeState& LAChild2 = LARoot.AddChildState("LAChild2");
			TStateTreeEditorNode<FTestTask_PrintValue>& LAChild2Task1 = MakeSendEventOnEnterTask(TNotNull<UStateTreeState*>(&LAChild2), GetTestTag2(), "LAChild2Task1");

			LAChild1.AddTransition(EStateTreeTransitionTrigger::OnEvent, GetTestTag1(), EStateTreeTransitionType::GotoState, &LAChild2);

			{
				FStateTreeCompilerLog Log;
				FStateTreeCompiler Compiler(Log);
				const bool bResult = Compiler.Compile(LinkedAsset);
				AITEST_TRUE("LinkedAsset should get compiled", bResult);
			}
		}

		// Parallel Tree 1
		UStateTree& ParallelTree1 = NewStateTree();
		{
			UStateTreeEditorData& PT1EditorData = *Cast<UStateTreeEditorData>(ParallelTree1.EditorData);

			UStateTreeState& PT1Root = PT1EditorData.AddSubTree("PT1Root");

			UStateTreeState& PT1Child1 = PT1Root.AddChildState("PT1Child1");
			TStateTreeEditorNode<FTestTask_PrintValue>& PT1Child1Task1 = PT1Child1.AddTask<FTestTask_PrintValue>("PT1Child1Task1");

			UStateTreeState& PT1Child2 = PT1Root.AddChildState("PT1Child2");
			TStateTreeEditorNode<FTestTask_PrintValue>& PT1Child2Task1 = MakeSendEventOnEnterTask(TNotNull<UStateTreeState*>(&PT1Child2), GetTestTag5(), "PT1Child2Task1");

			UStateTreeState& PT1Child3 = PT1Root.AddChildState("PT1Child3", EStateTreeStateType::LinkedAsset);
			PT1Child3.SetLinkedStateAsset(&LinkedAsset);

			UStateTreeState& PT1Child4 = PT1Root.AddChildState("PT1Child4");
			TStateTreeEditorNode<FTestTask_PrintValue>& PT1Child4Task1 = PT1Child4.AddTask<FTestTask_PrintValue>("PT1Child4Task1");

			{
				FStateTreeTransition& PT1RootTrans = PT1Root.AddTransition(EStateTreeTransitionTrigger::OnEvent, GetTestTag4(), EStateTreeTransitionType::GotoState, &PT1Child4);
				PT1RootTrans.Priority = EStateTreeTransitionPriority::Critical;

				PT1Child1.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::GotoState, &PT1Child2);
				PT1Child2.AddTransition(EStateTreeTransitionTrigger::OnEvent, GetTestTag5(), EStateTreeTransitionType::GotoState, &PT1Child3);
			}

			{
				FStateTreeCompilerLog Log;
				FStateTreeCompiler Compiler(Log);
				const bool bResult = Compiler.Compile(ParallelTree1);
				AITEST_TRUE("ParallelTree1 should get compiled", bResult);
			}
		}

		// Parallel Tree2
		UStateTree& ParallelTree2 = NewStateTree();
		{
			UStateTreeEditorData& PT2EditorData = *Cast<UStateTreeEditorData>(ParallelTree2.EditorData);

			UStateTreeState& PT2Root = PT2EditorData.AddSubTree("PT2Root");

			UStateTreeState& PT2Child1 = PT2Root.AddChildState("PT2Child1");
			TStateTreeEditorNode<FTestTask_PrintValue>& PT2Child1Task1 = PT2Child1.AddTask<FTestTask_PrintValue>("PT2Child1Task1");

			UStateTreeState& PT2Child2 = PT2Root.AddChildState("PT2Child2");
			TStateTreeEditorNode<FTestTask_PrintValue>& PT2Child2Task1 = MakeSendEventOnEnterTask(TNotNull<UStateTreeState*>(&PT2Child2), GetTestTag4(), "PT2Child2Task1");

			PT2Child1.AddTransition(EStateTreeTransitionTrigger::OnEvent, GetTestTag2(), EStateTreeTransitionType::GotoState, &PT2Child2);

			{
				FStateTreeCompilerLog Log;
				FStateTreeCompiler Compiler(Log);
				const bool bResult = Compiler.Compile(ParallelTree2);
				AITEST_TRUE("ParallelTree2 should get compiled", bResult);
			}
		}

		// Main Tree
		UStateTree& MainTree = NewStateTree();
		{
			UStateTreeEditorData& MainEditorData = *Cast<UStateTreeEditorData>(MainTree.EditorData);

			UStateTreeState& Root = MainEditorData.AddSubTree("Root");

			TStateTreeEditorNode<FStateTreeRunParallelStateTreeTask>& PT1Task = Root.AddTask<FStateTreeRunParallelStateTreeTask>();
			PT1Task.GetInstanceData().StateTree.SetStateTree(&ParallelTree1);

			TStateTreeEditorNode<FStateTreeRunParallelStateTreeTask>& PT2Task = Root.AddTask<FStateTreeRunParallelStateTreeTask>();
			PT2Task.GetInstanceData().StateTree.SetStateTree(&ParallelTree2);

			{
				FStateTreeCompilerLog Log;
				FStateTreeCompiler Compiler(Log);
				const bool bResult = Compiler.Compile(MainTree);
				AITEST_TRUE("MainTree should get compiled", bResult);
			}
		}

		constexpr TCHAR TickStr[] = TEXT("Tick0");
		constexpr TCHAR EnterStateStr[] = TEXT("EnterState0");
		constexpr TCHAR ExitStateStr[] = TEXT("ExitState0");

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;
		const FStateTreeEventQueue& EventQueue = InstanceData.GetEventQueue();
		FTestStateTreeExecutionContext Exec(MainTree, MainTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		{
			Status = Exec.Start();
			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("StateTree should enter PT2 Child1"), Exec.Expect("PT2Child1Task1", EnterStateStr));
			AITEST_TRUE(TEXT("StateTree should enter PT1 Child1"), Exec.Expect("PT1Child1Task1", EnterStateStr));
			AITEST_TRUE(TEXT("Event queue should be empty"), !EventQueue.HasEvents());
			Exec.LogClear();
		}

		{
			Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_FALSE(TEXT("StateTree should not enter PT2 Child2"), Exec.Expect("PT2Child2Task1", EnterStateStr));
			AITEST_TRUE(TEXT("StateTree should enter PT1 Child2"), Exec.Expect("PT1Child2Task1", EnterStateStr));
			AITEST_TRUE(TEXT("Event queue should have correct events"), HasEvent(EventQueue, GetTestTag5()));
			Exec.LogClear();
		}

		{
			Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_FALSE(TEXT("StateTree should not enter PT2 Child2"), Exec.Expect("PT2Child2Task1", EnterStateStr));
			AITEST_TRUE(TEXT("StateTree should enter PT1 Child3, LA Child1"), Exec.Expect("LAGlobalTask1", EnterStateStr).Then("LAChild1Task1", EnterStateStr));
			AITEST_TRUE(TEXT("Event queue should have correct events"), HasEvent(EventQueue, GetTestTag1()));
			Exec.LogClear();
		}

		{
			Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_FALSE(TEXT("StateTree should not enter PT2 Child2"), Exec.Expect("PT2Child2Task1", EnterStateStr));
			AITEST_TRUE(TEXT("StateTree should enter LA Child2"), Exec.Expect("LAChild2Task1", EnterStateStr));
			AITEST_TRUE(TEXT("Event queue should have correct events"), HasEvent(EventQueue, GetTestTag2()));
			Exec.LogClear();
		}

		{
			Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("StateTree should enter PT2 Child2"), Exec.Expect("PT2Child2Task1", EnterStateStr));
			AITEST_TRUE(TEXT("StateTree should enter PT1 Child4"), Exec.Expect("PT1Child4Task1", EnterStateStr));
			AITEST_TRUE(TEXT("Event queue should be empty"), !EventQueue.HasEvents());
			Exec.LogClear();
		}

		Status = Exec.Stop();
		Exec.LogClear();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_ParallelTreeSendEvents, "System.StateTree.ParallelTask.ParallelTreeSendEvents");

// Test parallel tree event priority handling.
struct FStateTreeTest_ParallelEventPriority : FStateTreeTestBase
{
	EStateTreeTransitionPriority ParallelTreePriority = EStateTreeTransitionPriority::Normal;

	virtual bool InstantTest() override
	{
		FStateTreeCompilerLog Log;

		const FGameplayTag EventTag = GetTestTag1();

		// Parallel tree
		// - Root
		//   - State1 ?-> State2
		//   - State2
		UStateTree& StateTreePar = NewStateTree();
		UStateTreeEditorData& EditorDataPar = *Cast<UStateTreeEditorData>(StateTreePar.EditorData);

		UStateTreeState& RootPar = EditorDataPar.AddSubTree(FName("Root"));
		UStateTreeState& State1 = RootPar.AddChildState(FName("State1"));
		UStateTreeState& State2 = RootPar.AddChildState(FName("State2"));

		TStateTreeEditorNode<FTestTask_Stand>& Task1 = State1.AddTask<FTestTask_Stand>(FName("Task1"));
		Task1.GetNode().TicksToCompletion = 100;
		State1.AddTransition(EStateTreeTransitionTrigger::OnEvent, EventTag, EStateTreeTransitionType::NextState);

		TStateTreeEditorNode<FTestTask_Stand>& Task2 = State2.AddTask<FTestTask_Stand>(FName("Task2"));
		Task2.GetNode().TicksToCompletion = 100;

		{
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTreePar);
			AITEST_TRUE(TEXT("StateTreePar should get compiled"), bResult);
		}

		// Main asset
		// - Root [StateTreePar]
		//   - State3 ?-> State4
		//   - State4
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree(FName("Root"));
		UStateTreeState& State3 = Root.AddChildState(FName("State3"));
		UStateTreeState& State4 = Root.AddChildState(FName("State4"));

		TStateTreeEditorNode<FStateTreeRunParallelStateTreeTask>& TaskPar = Root.AddTask<FStateTreeRunParallelStateTreeTask>();
		TaskPar.GetNode().SetEventHandlingPriority(ParallelTreePriority);

		TaskPar.GetInstanceData().StateTree.SetStateTree(&StateTreePar);

		TStateTreeEditorNode<FTestTask_Stand>& Task3 = State3.AddTask<FTestTask_Stand>(FName("Task3"));
		Task3.GetNode().TicksToCompletion = 100;
		State3.AddTransition(EStateTreeTransitionTrigger::OnEvent, EventTag, EStateTreeTransitionType::NextState);

		TStateTreeEditorNode<FTestTask_Stand>& Task4 = State4.AddTask<FTestTask_Stand>(FName("Task4"));
		Task4.GetNode().TicksToCompletion = 100;

		{
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);
		}

		const TCHAR* TickStr(TEXT("Tick"));
		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* ExitStateStr(TEXT("ExitState"));

		// Run StateTreePar in parallel with the main tree.
		// Both trees have a transition on same event.
		// Setting the priority to Low, should make the main tree to take the transition.
		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		Status = Exec.Start();
		AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
		AITEST_TRUE(TEXT("StateTree should enter Task1, Task3"), Exec.Expect(Task1.GetName(), EnterStateStr).Then(Task3.GetName(), EnterStateStr));
		Exec.LogClear();

		Status = Exec.Tick(0.1f);
		AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
		AITEST_TRUE(TEXT("StateTree should tick Task1, Task3"), Exec.Expect(Task1.GetName(), TickStr).Then(Task3.GetName(), TickStr));
		Exec.LogClear();

		Exec.SendEvent(EventTag);

		// If the parallel tree priority is < Normal, then it should always be handled after the main tree.
		// If the parallel tree priority is Normal, then the state order decides (leaf to root)
		// If the parallel tree priority is > Normal, then it should always be handled before the main tree.
		if (ParallelTreePriority <= EStateTreeTransitionPriority::Normal)
		{
			// Main tree should do the transition.
			Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("StateTree should enter Task4"), Exec.Expect(Task4.GetName(), EnterStateStr));
			Exec.LogClear();

			Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("StateTree should tick Task1, Task4"), Exec.Expect(Task1.GetName(), TickStr).Then(Task4.GetName(), TickStr));
			Exec.LogClear();
		}
		else
		{
			// Parallel tree should do the transition.
			Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("StateTree should enter Task2"), Exec.Expect(Task2.GetName(), EnterStateStr));
			Exec.LogClear();

			Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("StateTree should tick Task2, Task3"), Exec.Expect(Task2.GetName(), TickStr).Then(Task3.GetName(), TickStr));
			Exec.LogClear();
		}

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_ParallelEventPriority, "System.StateTree.ParallelTask.ParallelEventPriority");


struct FStateTreeTest_ParallelEventPriority_Low : FStateTreeTest_ParallelEventPriority
{
	FStateTreeTest_ParallelEventPriority_Low()
	{
		ParallelTreePriority = EStateTreeTransitionPriority::Low;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_ParallelEventPriority_Low, "System.StateTree.ParallelTask.ParallelEventPriority.Low");

struct FStateTreeTest_ParallelEventPriority_High : FStateTreeTest_ParallelEventPriority
{
	FStateTreeTest_ParallelEventPriority_High()
	{
		ParallelTreePriority = EStateTreeTransitionPriority::High;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_ParallelEventPriority_High, "System.StateTree.ParallelTask.ParallelEventPriority.High");

// Test 2 parallel tree inside the same state with an All completion.
struct FStateTreeTest_ParallelTask_TaskCompletion : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		//Tree 1
		//	Root (with 2 tasks that runs Tree 2)
		//Tree 2/3
		//	Root wait x frames to completes.

		UStateTree& StateTree1 = NewStateTree();
		UStateTree& StateTree2 = NewStateTree();
		UStateTree& StateTree3 = NewStateTree();
		auto Compile = [](TNotNull<UStateTree*> StateTree)
			{
				FStateTreeCompilerLog Log;
				FStateTreeCompiler Compiler(Log);
				return Compiler.Compile(StateTree);
			};

		auto BuildLinkedTree = [](TNotNull<UStateTree*> StateTree, FName TaskName, int32 TickToCompletion)
			{
				{
					UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree->EditorData);
					UStateTreeState& Root = EditorData.AddSubTree("Root");

					TStateTreeEditorNode<FTestTask_Stand>& TaskA = Root.AddTask<FTestTask_Stand>(TaskName);
					TaskA.GetNode().TicksToCompletion = TickToCompletion;

					Root.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);
				}
			};

		{
			UStateTreeEditorData& EditorData1 = *Cast<UStateTreeEditorData>(StateTree1.EditorData);
			UStateTreeState& Root1 = EditorData1.AddSubTree("Tree1StateRoot");

			Root1.TasksCompletion = EStateTreeTaskCompletionType::All;
			TStateTreeEditorNode<FStateTreeRunParallelStateTreeTask>& Task1 = Root1.AddTask<FStateTreeRunParallelStateTreeTask>();
			Task1.GetInstanceData().StateTree.SetStateTree(&StateTree2);
			TStateTreeEditorNode<FStateTreeRunParallelStateTreeTask>& Task2 = Root1.AddTask<FStateTreeRunParallelStateTreeTask>();
			Task2.GetInstanceData().StateTree.SetStateTree(&StateTree3);
			TStateTreeEditorNode<FTestTask_Stand>& Task3 = Root1.AddTask<FTestTask_Stand>(FName("Tree1Root1Task3"));
			Task3.GetNode().TicksToCompletion = 0;
			Root1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);
		}
		BuildLinkedTree(&StateTree2, "Tree2Root1Task1", 1);
		BuildLinkedTree(&StateTree3, "Tree3Root1Task1", 2);

		AITEST_TRUE(TEXT("StateTree3 should get compiled"), Compile(&StateTree3));
		AITEST_TRUE(TEXT("StateTree2 should get compiled"), Compile(&StateTree2));
		AITEST_TRUE(TEXT("StateTree1 should get compiled"), Compile(&StateTree1));

		{
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

			{
				const TCHAR* EnterStateStr(TEXT("EnterState"));
				const TCHAR* ExitStateStr(TEXT("ExitState"));

				EStateTreeRunStatus Status = Exec.Start();
				AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("StateTree Active States"), Exec.ExpectInActiveStates("Tree1StateRoot"));
				AITEST_TRUE(TEXT("Tree2Root1Task1 enter state"), Exec.Expect("Tree2Root1Task1", EnterStateStr));
				AITEST_TRUE(TEXT("Tree3Root1Task1 enter state"), Exec.Expect("Tree3Root1Task1", EnterStateStr));
				AITEST_TRUE(TEXT("Tree1Root1Task3 enter state"), Exec.Expect("Tree1Root1Task3", EnterStateStr));
				Exec.LogClear();

				Status = Exec.Tick(0.1f);
				AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("StateTree Active States"), Exec.ExpectInActiveStates("Tree1StateRoot"));
				AITEST_TRUE(TEXT("Tree2Root1Task1 exit state"), Exec.Expect("Tree2Root1Task1", ExitStateStr));
				AITEST_FALSE(TEXT("Tree3Root1Task1 exit state"), Exec.Expect("Tree3Root1Task1", ExitStateStr));
				AITEST_FALSE(TEXT("Tree1Root1Task3 exit state"), Exec.Expect("Tree1Root1Task3", ExitStateStr));
				Exec.LogClear();

				Status = Exec.Tick(2.0f);
				AITEST_EQUAL(TEXT("Tick should complete with Succeeded"), Status, EStateTreeRunStatus::Succeeded);
				AITEST_FALSE(TEXT("Tree2Root1Task1 exit state"), Exec.Expect("Tree2Root1Task1", ExitStateStr));
				AITEST_TRUE(TEXT("Tree3Root1Task1 exit state"), Exec.Expect("Tree3Root1Task1", ExitStateStr));
				AITEST_TRUE(TEXT("Tree1Root1Task3 exit state"), Exec.Expect("Tree1Root1Task3", ExitStateStr));
				Exec.LogClear();

				Exec.Stop();
			}
		}
		return true;
	}
};

IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_ParallelTask_TaskCompletion, "System.StateTree.ParallelTask.Completion");


} // namespace UE::StateTree::Tests

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
