// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeTest.h"
#include "StateTreeTestBase.h"
#include "StateTreeTestTypes.h"

#include "StateTreeCompilerLog.h"
#include "StateTreeEditorData.h"
#include "StateTreeCompiler.h"
#include "Conditions/StateTreeCommonConditions.h"

#define LOCTEXT_NAMESPACE "AITestSuite_StateTreeTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::StateTree::Tests
{
struct FStateTreeTest_StatePath_LinkStates : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		//Tree 1
		//	Root
		//		RootState1 -> Next
		//			StateLinkedTree (Sub1) -> Next
		//			StateLinkedTree (Sub1) -> Next
		//		RootStateLinkedTree (Sub2) -> Next
		//		RootStateLinkedTree (Sub2) -> Next
		//		RootStateLinkedTree (Tree2) -> Next
		//		RootStateLinkedTree (Tree3) -> Next			# Tree3 fails
		//		RootStateLinkedTree (Tree2) -> Next
		//		RootState2 -> Root
		//	Sub1
		//		StateLinkedTree (Tree2) -> Next
		//		StateLinkedTree (Tree2) -> Next
		//		StateLinkedTree (Sub2) -> Next
		//		StateLinkedTree (Sub2) -> Next
		//		Sub1State1 -> Success
		//	Sub2
		//		Sub2State1 -> Next
		//		Sub2State2 -> Success
		//Tree 2
		//	Root
		//		RootStateLinkedTree (Sub1) -> Next
		//		RootStateLinkedTree (Sub1) -> Next
		//		RootState1 -> Success
		//	Sub1
		//		Sub1State1 -> Next
		//		Sub1State2 -> Success
		//Tree 3
		//	Root
		//		RootStateLinkedTree (Sub1) -> Success
		//	Sub1
		//		Sub1State1 -> cond fail -> success
		UStateTree& StateTree1 = NewStateTree();
		UStateTree& StateTree2 = NewStateTree();
		UStateTree& StateTree3 = NewStateTree();
		// Tree1
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree1.EditorData);
			UStateTreeState& Root = EditorData.AddSubTree("Tree1Root");
			UStateTreeState& Sub1 = EditorData.AddSubTree("Tree1Sub1");
			Sub1.Type = EStateTreeStateType::Subtree;
			UStateTreeState& Sub2 = EditorData.AddSubTree("Tree1Sub2");
			Sub2.Type = EStateTreeStateType::Subtree;
			// Root
			{
				UStateTreeState& Tree1RootState1 = Root.AddChildState("Tree1RootState1", EStateTreeStateType::State);
				UStateTreeState& Tree1RootStateSub2A = Root.AddChildState("Tree1RootStateSub2A", EStateTreeStateType::Linked);
				//Tree1RootState1
				{
					{
						UStateTreeState& ChildState = Tree1RootState1.AddChildState("Tree1State1StateSub1A", EStateTreeStateType::Linked);
						ChildState.SetLinkedState(Sub1.GetLinkToState());
						ChildState.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::NextState);
					}
					{
						UStateTreeState& ChildState = Tree1RootState1.AddChildState("Tree1State1StateSub1B", EStateTreeStateType::Linked);
						ChildState.SetLinkedState(Sub1.GetLinkToState());
						FStateTreeTransition& Transition = ChildState.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::GotoState);
						Transition.State = Tree1RootStateSub2A.GetLinkToState();
					}
				}
				//Tree1RootStateSub2A
				{
					Tree1RootStateSub2A.SetLinkedState(Sub2.GetLinkToState());
					Tree1RootStateSub2A.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::NextState);
				}
				{
					UStateTreeState& State = Root.AddChildState("Tree1RootStateSub2B", EStateTreeStateType::Linked);
					State.SetLinkedState(Sub2.GetLinkToState());
					State.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::NextState);
				}
				{
					UStateTreeState& State = Root.AddChildState("Tree1RootStateLinkTree2A", EStateTreeStateType::LinkedAsset);
					State.SetLinkedStateAsset(&StateTree2);
					State.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::NextState);
				}
				{
					UStateTreeState& State = Root.AddChildState("Tree1RootStateLinkTree3", EStateTreeStateType::LinkedAsset);
					State.SetLinkedStateAsset(&StateTree3);
					State.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::NextState);
				}
				{
					UStateTreeState& State = Root.AddChildState("Tree1RootStateLinkTree2B", EStateTreeStateType::LinkedAsset);
					State.SetLinkedStateAsset(&StateTree2);
					State.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::NextState);
				}
				{
					UStateTreeState& State = Root.AddChildState("Tree1RootState1", EStateTreeStateType::State);
					FStateTreeTransition& Transition = State.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::GotoState);
					Transition.State = Root.GetLinkToState();
					Transition.bDelayTransition = true;
					Transition.DelayDuration = 0.999f;
				}
			}
			//Tree1Sub1
			{
				{
					UStateTreeState& State = Sub1.AddChildState("Tree1Sub1StateLinkTree2A", EStateTreeStateType::LinkedAsset);
					State.SetLinkedStateAsset(&StateTree2);
					State.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::NextState);
				}
				{
					UStateTreeState& State = Sub1.AddChildState("Tree1Sub1StateLinkTree2B", EStateTreeStateType::LinkedAsset);
					State.SetLinkedStateAsset(&StateTree2);
					State.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::NextState);
				}
				{
					UStateTreeState& State = Sub1.AddChildState("Tree1Sub1StateSub2A", EStateTreeStateType::Linked);
					State.SetLinkedState(Sub2.GetLinkToState());
					State.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::NextState);
				}
				{
					UStateTreeState& State = Sub1.AddChildState("Tree1Sub1StateSub2B", EStateTreeStateType::Linked);
					State.SetLinkedState(Sub2.GetLinkToState());
					State.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::NextState);
				}
				{
					UStateTreeState& State = Sub1.AddChildState("TreeASub1State1", EStateTreeStateType::State);
					FStateTreeTransition& Transition = State.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::Succeeded);
					Transition.bDelayTransition = true;
					Transition.DelayDuration = 0.999f;
				}
			}
			//Tree1Sub2
			{
				{
					UStateTreeState& State = Sub2.AddChildState("Tree1Sub2State1", EStateTreeStateType::State);
					State.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::NextState);
				}
				{
					UStateTreeState& State = Sub2.AddChildState("Tree1Sub2State2", EStateTreeStateType::State);
					FStateTreeTransition& Transition = State.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::Succeeded);
					Transition.bDelayTransition = true;
					Transition.DelayDuration = 0.999f;
				}
			}
		}
		//Tree 2
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree2.EditorData);
			UStateTreeState& Root = EditorData.AddSubTree("Tree2StateRoot");
			UStateTreeState& Sub1 = EditorData.AddSubTree("Tree2Sub1");
			Sub1.Type = EStateTreeStateType::Subtree;
			//Root
			{
				{
					UStateTreeState& State = Root.AddChildState("Tree2RootStateSub1A", EStateTreeStateType::Linked);
					State.SetLinkedState(Sub1.GetLinkToState());
					State.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::NextState);
				}
				{
					UStateTreeState& State = Root.AddChildState("Tree2RootStateSub1B", EStateTreeStateType::Linked);
					State.SetLinkedState(Sub1.GetLinkToState());
					State.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::NextState);
				}
				{
					UStateTreeState& State = Root.AddChildState("Tree2RootState1", EStateTreeStateType::State);
					FStateTreeTransition& Transition = State.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::Succeeded);
					Transition.bDelayTransition = true;
					Transition.DelayDuration = 0.999f;
				}
			}
			//Tree2Sub1
			{
				{
					UStateTreeState& State = Sub1.AddChildState("Tree2Sub1State1", EStateTreeStateType::State);
					State.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::NextState);
				}
				{
					UStateTreeState& State = Sub1.AddChildState("Tree2Sub1State2", EStateTreeStateType::State);
					FStateTreeTransition& Transition = State.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::Succeeded);
					Transition.bDelayTransition = true;
					Transition.DelayDuration = 0.999f;
				}
			}
		}
		//Tree 3
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree3.EditorData);
			UStateTreeState& Root = EditorData.AddSubTree("Tree3StateRoot");
			UStateTreeState& Sub1 = EditorData.AddSubTree("Tree3Sub1");
			Sub1.Type = EStateTreeStateType::Subtree;
			//Root
			{
				UStateTreeState& State = Root.AddChildState("Tree3RootStateSub1A", EStateTreeStateType::Linked);
				State.SetLinkedState(Sub1.GetLinkToState());
				State.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::Succeeded);
			}
			//Tree3Sub1
			{
				UStateTreeState& State = Sub1.AddChildState("Tree3Sub1State1", EStateTreeStateType::State);
				FStateTreeTransition& Transition = State.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::Succeeded);
				Transition.bDelayTransition = true;
				Transition.DelayDuration = 0.999f;

				TStateTreeEditorNode<FStateTreeRandomCondition>& Cond = State.AddEnterCondition<FStateTreeRandomCondition>();
				Cond.GetNode().EvaluationMode = EStateTreeConditionEvaluationMode::ForcedFalse;
			}
		}

		// Compile tree
		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree3);
			AITEST_TRUE(TEXT("StateTree3 should get compiled"), bResult);
		}
		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree2);
			AITEST_TRUE(TEXT("StateTree2 should get compiled"), bResult);
		}
		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree1);
			AITEST_TRUE(TEXT("StateTree1 should get compiled"), bResult);
		}

		// Create context
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
		{
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);
		}
		{
			UE::StateTree::FActiveStatePath ExecPath = InstanceData.GetExecutionState()->GetActiveStatePath();
			AITEST_TRUE(TEXT("ExecPath should be empty."), ExecPath.Num() == 0);
		}

		// Test variable and helper functions
		int32 ActiveCounter = 0;
		TArray<UE::StateTree::FActiveFrameID> PreviousFrameIDs;
		auto PreTestFrameId = [&PreviousFrameIDs, &InstanceData]()
			{
				PreviousFrameIDs.Reset();
				for (int32 Index = 0; Index < InstanceData.GetExecutionState()->ActiveFrames.Num(); ++Index)
				{
					PreviousFrameIDs.Add(InstanceData.GetExecutionState()->ActiveFrames[Index].FrameID);
				}
			};
		auto TestFrameId = [&PreviousFrameIDs, &InstanceData](int32 CorrectAmount)
			{
				int32 Index = 0;
				for (; Index < PreviousFrameIDs.Num() && Index < CorrectAmount; ++Index)
				{
					if (PreviousFrameIDs[Index] != InstanceData.GetExecutionState()->ActiveFrames[Index].FrameID)
					{
						return false;
					}
				}
				for (; Index < InstanceData.GetExecutionState()->ActiveFrames.Num() && Index < PreviousFrameIDs.Num(); ++Index)
				{
					if (PreviousFrameIDs[Index] == InstanceData.GetExecutionState()->ActiveFrames[Index].FrameID)
					{
						return false;
					}
				}
				return true;
			};

		// Start tests
		{
			using namespace UE::StateTree;

			const EStateTreeRunStatus Status = Exec.Start();
			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Should be in the correct state"), Exec.ExpectInActiveStates("Tree1Root", "Tree1RootState1", "Tree1State1StateSub1A", "Tree1Sub1", "Tree1Sub1StateLinkTree2A", "Tree2StateRoot", "Tree2RootStateSub1A", "Tree2Sub1", "Tree2Sub1State1"));

			const FActiveStatePath ExecPath = InstanceData.GetExecutionState()->GetActiveStatePath();
			AITEST_TRUE(TEXT("Has the correct number of path elements"), ExecPath.Num() == 9);
			AITEST_TRUE(TEXT("Has the correct number of active states"), InstanceData.GetExecutionState()->ActiveFrames.Num() == 4);
			{
				const FActiveFrameID FirstFrameID = InstanceData.GetExecutionState()->ActiveFrames[0].FrameID;
				AITEST_TRUE(TEXT("Frame for Tree1Root is active"), FirstFrameID == FActiveFrameID(++ActiveCounter));
				AITEST_TRUE(TEXT("State Tree1Root is active"), ExecPath.Contains(FActiveStateID(++ActiveCounter)));
				AITEST_TRUE(TEXT("State Tree1Root is active"), ExecPath.Contains(FActiveState(FirstFrameID, FActiveStateID(ActiveCounter), FStateTreeStateHandle(0))));
				AITEST_TRUE(TEXT("State Tree1RootState1 is active"), ExecPath.Contains(FActiveStateID(++ActiveCounter)));
				AITEST_TRUE(TEXT("State Tree1RootState1 is active"), ExecPath.Contains(FActiveState(FirstFrameID, FActiveStateID(ActiveCounter), FStateTreeStateHandle(1))));
				AITEST_TRUE(TEXT("State Tree1State1StateSub1A is active"), ExecPath.Contains(FActiveStateID(++ActiveCounter)));
				AITEST_TRUE(TEXT("State Tree1State1StateSub1A is active"), ExecPath.Contains(FActiveState(FirstFrameID, FActiveStateID(4), FStateTreeStateHandle(2))));
			}
			{
				AITEST_TRUE(TEXT("Frame for Tree1Sub1 is active"), InstanceData.GetExecutionState()->ActiveFrames[1].FrameID == FActiveFrameID(++ActiveCounter));
				AITEST_TRUE(TEXT("State Tree1Sub1 is active"), ExecPath.Contains(FActiveStateID(++ActiveCounter)));
				AITEST_TRUE(TEXT("State Tree1Sub1StateLinkTree2A is active"), ExecPath.Contains(FActiveStateID(++ActiveCounter)));
			}
			{
				AITEST_TRUE(TEXT("Frame for Tree2StateRoot is active"), InstanceData.GetExecutionState()->ActiveFrames[2].FrameID == FActiveFrameID(++ActiveCounter));
				AITEST_TRUE(TEXT("State Tree2StateRoot is active"), ExecPath.Contains(FActiveStateID(++ActiveCounter)));
				AITEST_TRUE(TEXT("State Tree2RootStateSub1A is active"), ExecPath.Contains(FActiveStateID(++ActiveCounter)));
			}
			{
				AITEST_TRUE(TEXT("Frame for Tree2Sub1 is active"), InstanceData.GetExecutionState()->ActiveFrames[3].FrameID == FActiveFrameID(++ActiveCounter));
				AITEST_TRUE(TEXT("State Tree2Sub1 is active"), ExecPath.Contains(FActiveStateID(++ActiveCounter)));
				AITEST_TRUE(TEXT("State Tree2Sub1State1 is active"), ExecPath.Contains(FActiveStateID(++ActiveCounter)));
			}
			{
				AITEST_FALSE(TEXT("No accidental increment"), ExecPath.Contains(FActiveStateID(ActiveCounter+1)));
			}
			Exec.LogClear();
		}
		{
			using namespace UE::StateTree;

			PreTestFrameId();
			EStateTreeRunStatus Status = Exec.Tick(1.0f);
			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Should be in the correct state"), Exec.ExpectInActiveStates("Tree1Root", "Tree1RootState1", "Tree1State1StateSub1A", "Tree1Sub1", "Tree1Sub1StateLinkTree2A", "Tree2StateRoot", "Tree2RootStateSub1A", "Tree2Sub1", "Tree2Sub1State2"));

			FActiveStatePath ExecPath = InstanceData.GetExecutionState()->GetActiveStatePath();
			AITEST_TRUE(TEXT("Has the correct number of path elements"), ExecPath.Num() == 9);
			AITEST_TRUE(TEXT("Has the correct number of active states"), InstanceData.GetExecutionState()->ActiveFrames.Num() == 4);
			AITEST_TRUE(TEXT("State Tree2Sub1State2 is active"), ExecPath.Contains(FActiveStateID(++ActiveCounter)));
			AITEST_FALSE(TEXT("No accidental increment"), ExecPath.Contains(FActiveStateID(ActiveCounter + 1)));
			AITEST_TRUE(TEXT("All frames are the same"), TestFrameId(4));
			Exec.LogClear();
		}
		{
			PreTestFrameId();
			EStateTreeRunStatus Status = Exec.Tick(1.0f);
			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Should be in the correct state"), Exec.ExpectInActiveStates("Tree1Root", "Tree1RootState1", "Tree1State1StateSub1A", "Tree1Sub1", "Tree1Sub1StateLinkTree2A", "Tree2StateRoot", "Tree2RootStateSub1B", "Tree2Sub1", "Tree2Sub1State1"));

			const UE::StateTree::FActiveStatePath ExecPath = InstanceData.GetExecutionState()->GetActiveStatePath();
			AITEST_TRUE(TEXT("Has the correct number of path elements"), ExecPath.Num() == 9);
			AITEST_TRUE(TEXT("Has the correct number of active states"), InstanceData.GetExecutionState()->ActiveFrames.Num() == 4);
			AITEST_TRUE(TEXT("The last frame changed"), TestFrameId(3));
			Exec.LogClear();
		}
		{
			PreTestFrameId();
			EStateTreeRunStatus Status = Exec.Tick(1.0f);
			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Should be in the correct state"), Exec.ExpectInActiveStates("Tree1Root", "Tree1RootState1", "Tree1State1StateSub1A", "Tree1Sub1", "Tree1Sub1StateLinkTree2A", "Tree2StateRoot", "Tree2RootStateSub1B", "Tree2Sub1", "Tree2Sub1State2"));

			UE::StateTree::FActiveStatePath ExecPath = InstanceData.GetExecutionState()->GetActiveStatePath();
			AITEST_TRUE(TEXT("Has the correct number of path elements"), ExecPath.Num() == 9);
			AITEST_TRUE(TEXT("Has the correct number of active states"), InstanceData.GetExecutionState()->ActiveFrames.Num() == 4);
			AITEST_TRUE(TEXT("All frames are the same"), TestFrameId(4));
			Exec.LogClear();
		}
		{
			PreTestFrameId();
			EStateTreeRunStatus Status = Exec.Tick(1.0f);
			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Should be in the correct state"), Exec.ExpectInActiveStates("Tree1Root", "Tree1RootState1", "Tree1State1StateSub1A", "Tree1Sub1", "Tree1Sub1StateLinkTree2A", "Tree2StateRoot", "Tree2RootState1"));

			UE::StateTree::FActiveStatePath ExecPath = InstanceData.GetExecutionState()->GetActiveStatePath();
			AITEST_TRUE(TEXT("Has the correct number of path elements"), ExecPath.Num() == 7);
			AITEST_TRUE(TEXT("Has the correct number of active states"), InstanceData.GetExecutionState()->ActiveFrames.Num() == 3);
			AITEST_TRUE(TEXT("All frames are the same"), TestFrameId(3));
			Exec.LogClear();
		}
		{
			PreTestFrameId();
			EStateTreeRunStatus Status = Exec.Tick(1.0f);
			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Should be in the correct state"), Exec.ExpectInActiveStates("Tree1Root", "Tree1RootState1", "Tree1State1StateSub1A", "Tree1Sub1", "Tree1Sub1StateLinkTree2B", "Tree2StateRoot", "Tree2RootStateSub1A", "Tree2Sub1", "Tree2Sub1State1"));

			UE::StateTree::FActiveStatePath ExecPath = InstanceData.GetExecutionState()->GetActiveStatePath();
			AITEST_TRUE(TEXT("Has the correct number of path elements"), ExecPath.Num() == 9);
			AITEST_TRUE(TEXT("Has the correct number of active states"), InstanceData.GetExecutionState()->ActiveFrames.Num() == 4);
			AITEST_TRUE(TEXT("All frames are the same"), TestFrameId(2));
			Exec.LogClear();
		}
		{
			PreTestFrameId();
			EStateTreeRunStatus Status = Exec.Tick(1.0f);
			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Should be in the correct state"), Exec.ExpectInActiveStates("Tree1Root", "Tree1RootState1", "Tree1State1StateSub1A", "Tree1Sub1", "Tree1Sub1StateLinkTree2B", "Tree2StateRoot", "Tree2RootStateSub1A", "Tree2Sub1", "Tree2Sub1State2"));
			AITEST_TRUE(TEXT("All frames are the same"), TestFrameId(4));
			Exec.LogClear();

			PreTestFrameId();
			Status = Exec.Tick(1.0f);
			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Should be in the correct state"), Exec.ExpectInActiveStates("Tree1Root", "Tree1RootState1", "Tree1State1StateSub1A", "Tree1Sub1", "Tree1Sub1StateLinkTree2B", "Tree2StateRoot", "Tree2RootStateSub1B", "Tree2Sub1", "Tree2Sub1State1"));
			AITEST_TRUE(TEXT("All frames are the same"), TestFrameId(3));
			Exec.LogClear();

			PreTestFrameId();
			Status = Exec.Tick(1.0f);
			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Should be in the correct state"), Exec.ExpectInActiveStates("Tree1Root", "Tree1RootState1", "Tree1State1StateSub1A", "Tree1Sub1", "Tree1Sub1StateLinkTree2B", "Tree2StateRoot", "Tree2RootStateSub1B", "Tree2Sub1", "Tree2Sub1State2"));
			AITEST_TRUE(TEXT("Has the correct number of active states"), InstanceData.GetExecutionState()->ActiveFrames.Num() == 4);
			AITEST_TRUE(TEXT("All frames are the same"), TestFrameId(4));
			Exec.LogClear();

			PreTestFrameId();
			Status = Exec.Tick(1.0f);
			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Should be in the correct state"), Exec.ExpectInActiveStates("Tree1Root", "Tree1RootState1", "Tree1State1StateSub1A", "Tree1Sub1", "Tree1Sub1StateLinkTree2B", "Tree2StateRoot", "Tree2RootState1"));
			AITEST_TRUE(TEXT("Has the correct number of active states"), InstanceData.GetExecutionState()->ActiveFrames.Num() == 3);
			AITEST_TRUE(TEXT("All frames are the same"), TestFrameId(3));
			Exec.LogClear();
		}
		{
			PreTestFrameId();
			EStateTreeRunStatus Status = Exec.Tick(1.0f);
			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Should be in the correct state"), Exec.ExpectInActiveStates("Tree1Root", "Tree1RootState1", "Tree1State1StateSub1A", "Tree1Sub1", "Tree1Sub1StateSub2A", "Tree1Sub2", "Tree1Sub2State1"));
			AITEST_TRUE(TEXT("Has the correct number of active states"), InstanceData.GetExecutionState()->ActiveFrames.Num() == 3);
			AITEST_TRUE(TEXT("All frames are the same"), TestFrameId(2));
			Exec.LogClear();

			PreTestFrameId();
			Status = Exec.Tick(1.0f);
			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Should be in the correct state"), Exec.ExpectInActiveStates("Tree1Root", "Tree1RootState1", "Tree1State1StateSub1A", "Tree1Sub1", "Tree1Sub1StateSub2A", "Tree1Sub2", "Tree1Sub2State2"));
			AITEST_TRUE(TEXT("Has the correct number of active states"), InstanceData.GetExecutionState()->ActiveFrames.Num() == 3);
			AITEST_TRUE(TEXT("All frames are the same"), TestFrameId(3));
			Exec.LogClear();
		}

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_StatePath_LinkStates, "System.StateTree.StatePath.LinkStates");

} // namespace UE::StateTree::Tests

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
