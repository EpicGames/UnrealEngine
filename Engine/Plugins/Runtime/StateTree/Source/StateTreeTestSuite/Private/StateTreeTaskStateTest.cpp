// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeTest.h"
#include "StateTreeTestBase.h"
#include "StateTreeTestTypes.h"

#include "StateTreeCompilerLog.h"
#include "StateTreeEditorData.h"
#include "StateTreeCompiler.h"
#include "Conditions/StateTreeCommonConditions.h"

#define LOCTEXT_NAMESPACE "AITestSuite_StateTreeTask"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::StateTree::Tests
{

namespace Private
{
} // namespace Private

struct FStateTreeTest_TasksCompletion_AllAny : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		// Main asset
		UStateTree& StateTree = NewStateTree();
		UStateTreeState* RootState = nullptr;
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

			UStateTreeState& Root = EditorData.AddSubTree("Tree1StateRoot");
			RootState = &Root;
			Root.TasksCompletion = EStateTreeTaskCompletionType::All;
			for (int32 TaskIndex = 0; TaskIndex < FStateTreeTasksCompletionStatus::MaxNumberOfTasksPerGroup; ++TaskIndex)
			{
				TStateTreeEditorNode<FTestTask_Stand>& Task = Root.AddTask<FTestTask_Stand>(FName(*FString::Printf(TEXT("Tree1StateRootTask_%d"), TaskIndex)));
				Task.GetNode().TicksToCompletion = TaskIndex + 1;
				if (TaskIndex == 10)
				{
					Task.GetNode().bConsideredForCompletion = false;
					Task.GetNode().TickCompletionResult = EStateTreeRunStatus::Failed;
				}
			}
			UStateTreeState& Tree1StateA = Root.AddChildState("Tree1StateA");
			Tree1StateA.TasksCompletion = EStateTreeTaskCompletionType::All;
			for (int32 TaskIndex = 0; TaskIndex < FStateTreeTasksCompletionStatus::MaxNumberOfTasksPerGroup; ++TaskIndex)
			{
				TStateTreeEditorNode<FTestTask_Stand>& Task = Tree1StateA.AddTask<FTestTask_Stand>(FName(*FString::Printf(TEXT("Tree1StateATask_%d"), TaskIndex)));
				Task.GetNode().TicksToCompletion = TaskIndex + 1;

				if (TaskIndex == 22)
				{
					Task.GetNode().bConsideredForCompletion = false;
					Task.GetNode().EnterStateResult = EStateTreeRunStatus::Failed;
				}
			}

			{
				FStateTreeCompilerLog Log;
				FStateTreeCompiler Compiler(Log);
				const bool bResult = Compiler.Compile(StateTree);
				AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);
			}
		}

		// Test All
		{
			EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

			{
				Status = Exec.Start();
				AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
				Exec.LogClear();
			}

			for (int32 TickIndex = 0; TickIndex < FStateTreeTasksCompletionStatus::MaxNumberOfTasksPerGroup; ++TickIndex)
			{
				Status = Exec.Tick(0.1f);
				AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1StateA"));

				for (int32 TaskIndex = 0; TaskIndex < FStateTreeTasksCompletionStatus::MaxNumberOfTasksPerGroup; ++TaskIndex)
				{
					bool bTickedRoot = Exec.Expect(*FString::Printf(TEXT("Tree1StateRootTask_%d"), TaskIndex), TEXT("Tick"));
					if (TaskIndex < TickIndex)
					{
						AITEST_FALSE(*FString::Printf(TEXT("Should not tick Task %d, %d"), TickIndex, TaskIndex), bTickedRoot);
					}
					else
					{
						AITEST_TRUE(*FString::Printf(TEXT("Should tick Task %d, %d"), TickIndex, TaskIndex), bTickedRoot);
					}
					bool bTickedA = Exec.Expect(*FString::Printf(TEXT("Tree1StateATask_%d"), TaskIndex), TEXT("Tick"));
					if (TaskIndex < TickIndex || TaskIndex == 22) // task 22 fails on enter
					{
						AITEST_FALSE(*FString::Printf(TEXT("Should not tick Task A %d, %d"), TickIndex, TaskIndex), bTickedA);
					}
					else
					{
						AITEST_TRUE(*FString::Printf(TEXT("Should tick Task A %d, %d"), TickIndex, TaskIndex), bTickedA);
					}
				}
				const bool bStateSucceeded = Exec.Expect("Tree1StateRootTask_0", TEXT("StateCompleted"));
				const bool bLastTick = TickIndex == FStateTreeTasksCompletionStatus::MaxNumberOfTasksPerGroup - 1;
				AITEST_EQUAL(TEXT("Tick should not complete the task."), bStateSucceeded, bLastTick);
				Exec.LogClear();
			}
			{
				Status = Exec.Tick(0.1f);
				AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1StateA"));
				bool bTicked = Exec.Expect("Tree1StateRootTask_0", TEXT("Tick"));
				AITEST_TRUE(TEXT("Reset should allow new tick."), bTicked);
			}

			Exec.Stop();
		}

		// Test any
		RootState->TasksCompletion = EStateTreeTaskCompletionType::Any;
		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);
		}
		{
			EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

			{
				Status = Exec.Start();
				AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
				Exec.LogClear();
			}

			{
				Status = Exec.Tick(0.1f);
				AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1StateA"));
				for (int32 TaskIndex = 0; TaskIndex < FStateTreeTasksCompletionStatus::MaxNumberOfTasksPerGroup; ++TaskIndex)
				{
					bool bTicked = Exec.Expect(*FString::Printf(TEXT("Tree1StateRootTask_%d"), TaskIndex), TEXT("Tick"));
					AITEST_TRUE(*FString::Printf(TEXT("Should tick Task %d"), TaskIndex), bTicked);
				}
				const bool bStateSucceeded = Exec.Expect("Tree1StateRootTask_0", TEXT("StateCompleted"));
				AITEST_TRUE(TEXT("Tick should not complete the task."), bStateSucceeded);
				Exec.LogClear();
			}
			{
				Status = Exec.Tick(0.1f);
				AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1StateA"));
				bool bTicked = Exec.Expect("Tree1StateRootTask_0", TEXT("Tick"));
				AITEST_TRUE(TEXT("Reset should allow new tick."), bTicked);
			}

			Exec.Stop();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_TasksCompletion_AllAny, "System.StateTree.TasksCompletion.AllAny");

struct FStateTreeTest_TasksCompletion_FailureTasks : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 BadTask = 2;
		// Main asset
		UStateTree& StateTree = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

			UStateTreeState& Root = EditorData.AddSubTree("Tree1StateRoot");
			Root.TasksCompletion = EStateTreeTaskCompletionType::All;
			for (int32 TaskIndex = 0; TaskIndex < FStateTreeTasksCompletionStatus::MaxNumberOfTasksPerGroup; ++TaskIndex)
			{
				TStateTreeEditorNode<FTestTask_Stand>& Task = Root.AddTask<FTestTask_Stand>(FName(*FString::Printf(TEXT("Tree1StateRootTask_%d"), TaskIndex)));
				Task.GetNode().TicksToCompletion = TaskIndex + 1;
				if (TaskIndex == BadTask)
				{
					Task.GetNode().TickCompletionResult = EStateTreeRunStatus::Failed;
				}
			}

			{
				FStateTreeCompilerLog Log;
				FStateTreeCompiler Compiler(Log);
				const bool bResult = Compiler.Compile(StateTree);
				AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);
			}
		}

		{
			EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

			{
				Status = Exec.Start();
				AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
				Exec.LogClear();
			}

			for (int32 TickIndex = 0; TickIndex <= BadTask; ++TickIndex)
			{
				Status = Exec.Tick(0.1f);
				AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateRoot"));
				const bool bLastTick = TickIndex == BadTask;
				for (int32 TaskIndex = 0; TaskIndex < FStateTreeTasksCompletionStatus::MaxNumberOfTasksPerGroup; ++TaskIndex)
				{
					bool bTicked = Exec.Expect(*FString::Printf(TEXT("Tree1StateRootTask_%d"), TaskIndex), TEXT("Tick"));
					if (TaskIndex < TickIndex || (bLastTick && TaskIndex > BadTask))
					{
						AITEST_FALSE(*FString::Printf(TEXT("Should not tick Task %d, %d"), TickIndex, TaskIndex), bTicked);
					}
					else
					{
						AITEST_TRUE(*FString::Printf(TEXT("Should tick Task %d, %d"), TickIndex, TaskIndex), bTicked);
					}
				}
				const bool bStateSucceeded = Exec.Expect("Tree1StateRootTask_0", TEXT("StateCompleted"));
				AITEST_EQUAL(TEXT("Tick should not complete the task."), bStateSucceeded, bLastTick);
				Exec.LogClear();
			}
			{
				Status = Exec.Tick(0.1f);
				AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateRoot"));
				bool bTicked = Exec.Expect("Tree1StateRootTask_0", TEXT("Tick"));
				AITEST_TRUE(TEXT("Reset should allow new tick."), bTicked);
			}

			Exec.Stop();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_TasksCompletion_FailureTasks, "System.StateTree.TasksCompletion.Failure");

struct FStateTreeTest_TasksCompletion_StartGlobalFailureTasks : FStateTreeTestBase
{
	// Test when global tasks fails when the tree starts.
	virtual bool InstantTest() override
	{
		// Tree1 (3 global tasks)
		//		Root
		UStateTree& StateTree = NewStateTree();
		TStateTreeEditorNode<FTestTask_Stand>* GlobalTask1Ptr = nullptr;
		TStateTreeEditorNode<FTestTask_Stand>* GlobalTask2Ptr = nullptr;
		TStateTreeEditorNode<FTestTask_Stand>* GlobalTask3Ptr = nullptr;

		auto CompileTree = [&StateTree, this]()
			{
				FStateTreeCompilerLog Log;
				FStateTreeCompiler Compiler(Log);
				const bool bResult = Compiler.Compile(StateTree);
				AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);
				return bResult;
			};

		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

			TStateTreeEditorNode<FTestTask_Stand>& GlobalTask1 = EditorData.AddGlobalTask<FTestTask_Stand>("Tree1GlobalTask1");
			TStateTreeEditorNode<FTestTask_Stand>& GlobalTask2 = EditorData.AddGlobalTask<FTestTask_Stand>("Tree1GlobalTask2");
			TStateTreeEditorNode<FTestTask_Stand>& GlobalTask3 = EditorData.AddGlobalTask<FTestTask_Stand>("Tree1GlobalTask3");
			GlobalTask1Ptr = &GlobalTask1;
			GlobalTask2Ptr = &GlobalTask2;
			GlobalTask3Ptr = &GlobalTask3;

			UStateTreeState& Root = EditorData.AddSubTree("Tree1StateRoot");
			Root.AddTask<FTestTask_Stand>("Tree1StateRootTask1");
		}

		// test global task 1 failing
		GlobalTask1Ptr->GetNode().EnterStateResult = EStateTreeRunStatus::Failed;
		GlobalTask2Ptr->GetNode().EnterStateResult = EStateTreeRunStatus::Succeeded;
		if (!CompileTree())
		{
			return false;
		}
		{
			EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

			{
				Status = Exec.Start();
				AITEST_EQUAL(TEXT("Start should complete with failed"), Status, EStateTreeRunStatus::Failed);
				AITEST_TRUE(TEXT("Task1 should enter but fail."),
					Exec.Expect("Tree1GlobalTask1", TEXT("EnterState"))
					.Then("Tree1GlobalTask1", TEXT("ExitFailed"))
					.Then("Tree1GlobalTask1", TEXT("ExitState"))
				);
				AITEST_FALSE(TEXT("Task2 should not enter."), Exec.Expect("Tree1GlobalTask2", TEXT("EnterState")));
				AITEST_FALSE(TEXT("Task3 should not enter."), Exec.Expect("Tree1GlobalTask3", TEXT("EnterState")));
				AITEST_FALSE(TEXT("Task2 should not exit."), Exec.Expect("Tree1GlobalTask2", TEXT("ExitState")));
				AITEST_FALSE(TEXT("Task3 should not exit."), Exec.Expect("Tree1GlobalTask3", TEXT("ExitState")));
				Exec.LogClear();
				Exec.Stop();
			}
		}

		// test global task 2 failing
		GlobalTask1Ptr->GetNode().EnterStateResult = EStateTreeRunStatus::Succeeded;
		GlobalTask2Ptr->GetNode().EnterStateResult = EStateTreeRunStatus::Failed;
		if (!CompileTree())
		{
			return false;
		}
		{
			EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

			{
				Status = Exec.Start();
				AITEST_EQUAL(TEXT("Start should complete with failed"), Status, EStateTreeRunStatus::Failed);
				AITEST_TRUE(TEXT("Task2 should enter."),
					Exec.Expect("Tree1GlobalTask1", TEXT("EnterState"))
					.Then("Tree1GlobalTask2", TEXT("EnterState"))
					.Then("Tree1GlobalTask2", TEXT("ExitFailed"))
					.Then("Tree1GlobalTask2", TEXT("ExitState"))
					.Then("Tree1GlobalTask1", TEXT("ExitState"))
				);
				AITEST_FALSE(TEXT("Task3 should not enter."), Exec.Expect("Tree1GlobalTask3", TEXT("EnterState")));
				AITEST_FALSE(TEXT("Task3 should not exit."), Exec.Expect("Tree1GlobalTask3", TEXT("ExitState")));
				Exec.LogClear();
				Exec.Stop();
			}
		}

		// test global task 3 failing
		GlobalTask2Ptr->GetNode().EnterStateResult = EStateTreeRunStatus::Succeeded;
		GlobalTask3Ptr->GetNode().EnterStateResult = EStateTreeRunStatus::Failed;
		if (!CompileTree())
		{
			return false;
		}
		{
			EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

			{
				Status = Exec.Start();
				AITEST_EQUAL(TEXT("Start should complete with failed"), Status, EStateTreeRunStatus::Failed);
				AITEST_TRUE(TEXT("Task3 should enter."),
					Exec.Expect("Tree1GlobalTask1", TEXT("EnterState"))
					.Then("Tree1GlobalTask2", TEXT("EnterState"))
					.Then("Tree1GlobalTask3", TEXT("EnterState"))
					.Then("Tree1GlobalTask3", TEXT("ExitFailed"))
					.Then("Tree1GlobalTask3", TEXT("ExitState"))
					.Then("Tree1GlobalTask2", TEXT("ExitState"))
					.Then("Tree1GlobalTask1", TEXT("ExitState"))
				);
				Exec.LogClear();
				Exec.Stop();
			}
		}

		// test global task 3 succeeded
		GlobalTask3Ptr->GetNode().EnterStateResult = EStateTreeRunStatus::Succeeded;
		if (!CompileTree())
		{
			return false;
		}
		{
			EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

			{
				Status = Exec.Start();
				AITEST_EQUAL(TEXT("Start should complete with succeeded."), Status, EStateTreeRunStatus::Succeeded);
				AITEST_TRUE(TEXT("Task3 should enter."),
					Exec.Expect("Tree1GlobalTask1", TEXT("EnterState"))
					.Then("Tree1GlobalTask2", TEXT("EnterState"))
					.Then("Tree1GlobalTask3", TEXT("EnterState"))
					.Then("Tree1GlobalTask3", TEXT("ExitSucceeded"))
					.Then("Tree1GlobalTask2", TEXT("ExitSucceeded"))
					.Then("Tree1GlobalTask1", TEXT("ExitSucceeded"))
				);
				Exec.LogClear();
				Exec.Stop();
			}
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_TasksCompletion_StartGlobalFailureTasks, "System.StateTree.TasksCompletion.StartGlobalTaskFailure");

// test set status with priority
struct FStateTreeTest_TasksCompletion_Status : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		FCompactStateTreeFrame Frame;
		Frame.NumberOfTasksStatusMasks = 1;
		FStateTreeTasksCompletionStatus Status = FStateTreeTasksCompletionStatus(Frame);

		constexpr int32 NumberOfTasks = 4;
		FCompactStateTreeState State;
		State.CompletionTasksControl = EStateTreeTaskCompletionType::All;
		State.CompletionTasksMaskBitsOffset = 3;
		State.CompletionTasksMaskBufferIndex = 0;
		State.CompletionTasksMask = 1 << NumberOfTasks;
		State.CompletionTasksMask -= 1;
		State.CompletionTasksMask <<= State.CompletionTasksMaskBitsOffset;

		UE::StateTree::FTasksCompletionStatus TestingStatus = Status.GetStatus(State);

		auto TestEmpty = [this, &TestingStatus]()
			{
				AITEST_FALSE(TEXT("Empty is not completed."), TestingStatus.IsCompleted());
				AITEST_FALSE(TEXT("Empty has not failure."), TestingStatus.HasAnyFailed());
				AITEST_FALSE(TEXT("Empty is not completed."), TestingStatus.HasAnyCompleted());
				AITEST_FALSE(TEXT("Empty is not completed."), TestingStatus.HasAllCompleted());
				AITEST_EQUAL(TEXT("The completion status is running."), TestingStatus.GetCompletionStatus(), UE::StateTree::ETaskCompletionStatus::Running);
				for (int32 Index = 0; Index < NumberOfTasks; ++Index)
				{
					AITEST_FALSE(TEXT("Empty is not completed."), TestingStatus.HasFailed(Index));
					AITEST_TRUE(TEXT("Empty is running."), TestingStatus.IsRunning(Index));
					AITEST_EQUAL(TEXT("Empty is running."), TestingStatus.GetStatus(Index), UE::StateTree::ETaskCompletionStatus::Running);
				}
				return true;
			};

		// Test new/empty completion status
		{
			if (!TestEmpty())
			{
				return false;
			}
		}

		// Set tasks (1) to running. Does nothing.
		{
			TestingStatus.SetStatus(1, UE::StateTree::ETaskCompletionStatus::Running);
			if (!TestEmpty())
			{
				return false;
			}
		}

		auto Test1Expected = [this, &TestingStatus](UE::StateTree::ETaskCompletionStatus Expected)
			{
				AITEST_FALSE(TEXT("1 doesn't completed the state."), TestingStatus.IsCompleted());
				AITEST_FALSE(TEXT("1 doesn't completed the failed."), TestingStatus.HasAnyFailed());
				AITEST_TRUE(TEXT("1 does completed any state."), TestingStatus.HasAnyCompleted());
				AITEST_FALSE(TEXT("1 doesn't completed all state."), TestingStatus.HasAllCompleted());
				AITEST_EQUAL(TEXT("States running."), TestingStatus.GetCompletionStatus(), UE::StateTree::ETaskCompletionStatus::Running);
				for (int32 Index = 0; Index < NumberOfTasks; ++Index)
				{
					AITEST_FALSE(TEXT("1 is not completed."), TestingStatus.HasFailed(Index));
					if (1 == Index)
					{
						AITEST_FALSE(TEXT("1 is Stopped."), TestingStatus.IsRunning(Index));
						AITEST_EQUAL(TEXT("1 is Stopped."), TestingStatus.GetStatus(Index), Expected);
					}
					else
					{
						AITEST_TRUE(TEXT("1 is Stopped."), TestingStatus.IsRunning(Index));
						AITEST_EQUAL(TEXT("1 is Stopped."), TestingStatus.GetStatus(Index), UE::StateTree::ETaskCompletionStatus::Running);
					}
				}
				return true;
			};
		// Set tasks (1) to stop.
		{
			TestingStatus.SetStatus(1, UE::StateTree::ETaskCompletionStatus::Stopped);
			if (!Test1Expected(UE::StateTree::ETaskCompletionStatus::Stopped))
			{
				return false;
			}
		}
		// Set tasks (1) to running. Does nothing.
		{
			ETaskCompletionStatus NewStatus = TestingStatus.SetStatusWithPriority(1, UE::StateTree::ETaskCompletionStatus::Running);
			AITEST_EQUAL(TEXT("1 make the state Stopped."), NewStatus, UE::StateTree::ETaskCompletionStatus::Stopped);
			if (!Test1Expected(UE::StateTree::ETaskCompletionStatus::Stopped))
			{
				return false;
			}
		}
		// Set tasks (1) to Succeeded. Does nothing.
		{
			ETaskCompletionStatus NewStatus = TestingStatus.SetStatusWithPriority(1, UE::StateTree::ETaskCompletionStatus::Succeeded);
			AITEST_EQUAL(TEXT("1 make the state Succeed."), NewStatus, UE::StateTree::ETaskCompletionStatus::Succeeded);
			if (!Test1Expected(UE::StateTree::ETaskCompletionStatus::Succeeded))
			{
				return false;
			}
		}
		// Set tasks (1) to stop. Does nothing.
		{
			ETaskCompletionStatus NewStatus = TestingStatus.SetStatusWithPriority(1, UE::StateTree::ETaskCompletionStatus::Stopped);
			AITEST_EQUAL(TEXT("1 make the state Succeed."), NewStatus, UE::StateTree::ETaskCompletionStatus::Succeeded);
			if (!Test1Expected(UE::StateTree::ETaskCompletionStatus::Succeeded))
			{
				return false;
			}
		}
		// Set tasks (1) to running. Does nothing.
		{
			ETaskCompletionStatus NewStatus = TestingStatus.SetStatusWithPriority(1, UE::StateTree::ETaskCompletionStatus::Stopped);
			AITEST_EQUAL(TEXT("1 make the state Succeed."), NewStatus, UE::StateTree::ETaskCompletionStatus::Succeeded);
			if (!Test1Expected(UE::StateTree::ETaskCompletionStatus::Succeeded))
			{
				return false;
			}
		}
		// Set tasks (1) to fail. Does nothing.
		{
			ETaskCompletionStatus NewStatus = TestingStatus.SetStatusWithPriority(1, UE::StateTree::ETaskCompletionStatus::Failed);
			AITEST_EQUAL(TEXT("1 make the state Succeed."), NewStatus, UE::StateTree::ETaskCompletionStatus::Failed);
			AITEST_TRUE(TEXT("1 completes the state."), TestingStatus.IsCompleted());
			AITEST_TRUE(TEXT("1 completes the failed."), TestingStatus.HasAnyFailed());
			AITEST_TRUE(TEXT("1 completes any state."), TestingStatus.HasAnyCompleted());
			AITEST_TRUE(TEXT("1 completes all state."), TestingStatus.HasAllCompleted());
			AITEST_EQUAL(TEXT("States failed."), TestingStatus.GetCompletionStatus(), UE::StateTree::ETaskCompletionStatus::Failed);
			for (int32 Index = 0; Index < NumberOfTasks; ++Index)
			{
				if (1 == Index)
				{
					AITEST_TRUE(TEXT("1 is not completed."), TestingStatus.HasFailed(Index));
					AITEST_FALSE(TEXT("1 is failed."), TestingStatus.IsRunning(Index));
					AITEST_EQUAL(TEXT("1 is failed."), TestingStatus.GetStatus(Index), UE::StateTree::ETaskCompletionStatus::Failed);
				}
				else
				{
					AITEST_FALSE(TEXT("1 is not completed."), TestingStatus.HasFailed(Index));
					AITEST_TRUE(TEXT("1 is running."), TestingStatus.IsRunning(Index));
					AITEST_EQUAL(TEXT("1 is running."), TestingStatus.GetStatus(Index), UE::StateTree::ETaskCompletionStatus::Running);
				}
			}
		}
		// Set completion status
		{
			TestingStatus.SetCompletionStatus(UE::StateTree::ETaskCompletionStatus::Stopped);
			AITEST_TRUE(TEXT("All complete the state."), TestingStatus.IsCompleted());
			AITEST_FALSE(TEXT("All complete the failed."), TestingStatus.HasAnyFailed());
			AITEST_TRUE(TEXT("All complete any state."), TestingStatus.HasAnyCompleted());
			AITEST_TRUE(TEXT("All complete all state."), TestingStatus.HasAllCompleted());
			AITEST_EQUAL(TEXT("States stopped."), TestingStatus.GetCompletionStatus(), UE::StateTree::ETaskCompletionStatus::Stopped);
			for (int32 Index = 0; Index < NumberOfTasks; ++Index)
			{
				AITEST_FALSE(TEXT("1 is not completed."), TestingStatus.HasFailed(Index));
				AITEST_FALSE(TEXT("1 is stopped."), TestingStatus.IsRunning(Index));
				AITEST_EQUAL(TEXT("1 is stopped."), TestingStatus.GetStatus(Index), UE::StateTree::ETaskCompletionStatus::Stopped);
			}
		}
		// Test GetCompletionStatus
		{
			TestingStatus.SetCompletionStatus(UE::StateTree::ETaskCompletionStatus::Running);
			AITEST_EQUAL(TEXT("States stopped."), TestingStatus.GetCompletionStatus(), UE::StateTree::ETaskCompletionStatus::Running);
		}
		// Test GetCompletionStatus
		{
			TestingStatus.SetCompletionStatus(UE::StateTree::ETaskCompletionStatus::Succeeded);
			AITEST_EQUAL(TEXT("States succeeded."), TestingStatus.GetCompletionStatus(), UE::StateTree::ETaskCompletionStatus::Succeeded);
		}
		// Test GetCompletionStatus
		{
			TestingStatus.SetCompletionStatus(UE::StateTree::ETaskCompletionStatus::Failed);
			AITEST_EQUAL(TEXT("States failed."), TestingStatus.GetCompletionStatus(), UE::StateTree::ETaskCompletionStatus::Failed);
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_TasksCompletion_Status, "System.StateTree.TasksCompletion.Status");

// The base class for testing marked bConsideredForCompletion=false
struct FStateTreeTest_TasksCompletion_ChildStateNotBlockedByNonCompletionTask_Base : FStateTreeTestBase
{
	bool RunTest(EStateTreeTaskCompletionType BCompletionType, int32 NumBTasksNotConsidered, bool bUseSingleTaskProperty = false)
	{
		check(!bUseSingleTaskProperty || NumBTasksNotConsidered == 1);

		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root    = EditorData.AddSubTree("Root");
		UStateTreeState& StateA  = Root.AddChildState("A");
		UStateTreeState& StateB  = Root.AddChildState("B");
		UStateTreeState& StateB1 = StateB.AddChildState("B1");
		UStateTreeState& StateB2 = StateB1.AddChildState("B2");

		// A: single task completes in 1 tick, then the tree transitions to B.
		{
			TStateTreeEditorNode<FTestTask_Stand>& TaskA = StateA.AddTask<FTestTask_Stand>("TaskA");
			TaskA.GetNode().TicksToCompletion = 1;
			StateA.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &StateB);
		}

		// B: one or more tasks that complete on the first tick but are NOT considered for completion.
		StateB.TasksCompletion = BCompletionType;
		if (bUseSingleTaskProperty)
		{
			// Set up B's task via the SingleTask field (single-task schema path).
			StateB.SingleTask.InitializeAs<FTestTask_Stand>(&StateB, "TaskB_0");
			TStateTreeEditorNode<FTestTask_Stand>& TaskB = static_cast<TStateTreeEditorNode<FTestTask_Stand>&>(StateB.SingleTask);
			TaskB.GetNode().TicksToCompletion = 1;
			TaskB.GetNode().bConsideredForCompletion = false;
		}
		else
		{
			for (int32 i = 0; i < NumBTasksNotConsidered; ++i)
			{
				TStateTreeEditorNode<FTestTask_Stand>& TaskB = StateB.AddTask<FTestTask_Stand>(
					FName(*FString::Printf(TEXT("TaskB_%d"), i)));
				TaskB.GetNode().TicksToCompletion = 1;
				TaskB.GetNode().bConsideredForCompletion = false;
			}
		}

		// B2: task that requires 2 ticks to complete.
		{
			TStateTreeEditorNode<FTestTask_Stand>& TaskB2 = StateB2.AddTask<FTestTask_Stand>("TaskB2");
			TaskB2.GetNode().TicksToCompletion = 2;
		}

		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE(TEXT("StateTree should compile"), bResult);
			if (!bResult)
			{
				return false;
			}
		}

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE(TEXT("StateTree should init"), Exec.IsValid());

		// Start: A is the initial state.
		{
			const EStateTreeRunStatus Status = Exec.Start();
			AITEST_EQUAL(TEXT("Start: should be Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Start: should be in A"), Exec.ExpectInActiveStates("Root", "A"));
			Exec.LogClear();
		}

		// Tick 1: TaskA completes (1 tick), A transitions to B. B/B1/B2 enter but their tasks
		// are not ticked yet in the same tick as the transition.
		{
			const EStateTreeRunStatus Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick 1: should be Running after transition to B"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Tick 1: should be in B/B1/B2"), Exec.ExpectInActiveStates("Root", "B", "B1", "B2"));
			AITEST_FALSE(TEXT("Tick 1: TaskB2 should not have ticked yet"), Exec.Expect("TaskB2", TEXT("Tick")));
			Exec.LogClear();
		}

		// Tick 2: First tick in B. B's task(s) run and return Succeeded, but they are not
		// considered for completion so B must NOT auto-complete. B2's task runs its first tick
		// (CurrentTick=1 < TicksToCompletion=2) and returns Running.
		{
			const EStateTreeRunStatus Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick 2: should be Running (B must not auto-complete)"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Tick 2: B must not auto-complete; still in B/B1/B2"), Exec.ExpectInActiveStates("Root", "B", "B1", "B2"));
			AITEST_TRUE(TEXT("Tick 2: TaskB2 should have ticked"), Exec.Expect("TaskB2", TEXT("Tick")));
			Exec.LogClear();
		}

		// Tick 3: TaskB2 second tick, completes. B2 runs to full completion.
		{
			Exec.Tick(0.1f);
			AITEST_TRUE(TEXT("Tick 3: TaskB2 should have completed"), Exec.Expect("TaskB2", TEXT("StateCompleted")));
			Exec.LogClear();
		}

		Exec.Stop();
		return true;
	}
};

// --- 1 task via AddTask ---

struct FStateTreeTest_TasksCompletion_ChildStateNotBlockedByNonCompletionTask_1Task_AddTask_All
	: FStateTreeTest_TasksCompletion_ChildStateNotBlockedByNonCompletionTask_Base
{
	virtual bool InstantTest() override
	{
		return RunTest(EStateTreeTaskCompletionType::All, 1, /*bUseSingleTaskProperty=*/false);
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(
	FStateTreeTest_TasksCompletion_ChildStateNotBlockedByNonCompletionTask_1Task_AddTask_All,
	"System.StateTree.TasksCompletion.ChildStateNotBlockedByNonCompletionTask.1Task.AddTask.All");

struct FStateTreeTest_TasksCompletion_ChildStateNotBlockedByNonCompletionTask_1Task_AddTask_Any
	: FStateTreeTest_TasksCompletion_ChildStateNotBlockedByNonCompletionTask_Base
{
	virtual bool InstantTest() override
	{
		return RunTest(EStateTreeTaskCompletionType::Any, 1, /*bUseSingleTaskProperty=*/false);
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(
	FStateTreeTest_TasksCompletion_ChildStateNotBlockedByNonCompletionTask_1Task_AddTask_Any,
	"System.StateTree.TasksCompletion.ChildStateNotBlockedByNonCompletionTask.1Task.AddTask.Any");

// --- 1 task via SingleTask property ---

struct FStateTreeTest_TasksCompletion_ChildStateNotBlockedByNonCompletionTask_1Task_SingleTask_All
	: FStateTreeTest_TasksCompletion_ChildStateNotBlockedByNonCompletionTask_Base
{
	virtual bool InstantTest() override
	{
		return RunTest(EStateTreeTaskCompletionType::All, 1, /*bUseSingleTaskProperty=*/true);
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(
	FStateTreeTest_TasksCompletion_ChildStateNotBlockedByNonCompletionTask_1Task_SingleTask_All,
	"System.StateTree.TasksCompletion.ChildStateNotBlockedByNonCompletionTask.1Task.SingleTask.All");

struct FStateTreeTest_TasksCompletion_ChildStateNotBlockedByNonCompletionTask_1Task_SingleTask_Any
	: FStateTreeTest_TasksCompletion_ChildStateNotBlockedByNonCompletionTask_Base
{
	virtual bool InstantTest() override
	{
		return RunTest(EStateTreeTaskCompletionType::Any, 1, /*bUseSingleTaskProperty=*/true);
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(
	FStateTreeTest_TasksCompletion_ChildStateNotBlockedByNonCompletionTask_1Task_SingleTask_Any,
	"System.StateTree.TasksCompletion.ChildStateNotBlockedByNonCompletionTask.1Task.SingleTask.Any");

// --- Multiple tasks via AddTask ---

struct FStateTreeTest_TasksCompletion_ChildStateNotBlockedByNonCompletionTask_MultiTask_All
	: FStateTreeTest_TasksCompletion_ChildStateNotBlockedByNonCompletionTask_Base
{
	virtual bool InstantTest() override
	{
		return RunTest(EStateTreeTaskCompletionType::All, 2, /*bUseSingleTaskProperty=*/false);
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(
	FStateTreeTest_TasksCompletion_ChildStateNotBlockedByNonCompletionTask_MultiTask_All,
	"System.StateTree.TasksCompletion.ChildStateNotBlockedByNonCompletionTask.MultiTask.All");

// MultiTask via AddTask, Any mode (complements MultiTask_All).
struct FStateTreeTest_TasksCompletion_ChildStateNotBlockedByNonCompletionTask_MultiTask_Any
	: FStateTreeTest_TasksCompletion_ChildStateNotBlockedByNonCompletionTask_Base
{
	virtual bool InstantTest() override
	{
		return RunTest(EStateTreeTaskCompletionType::Any, 2, /*bUseSingleTaskProperty=*/false);
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(
	FStateTreeTest_TasksCompletion_ChildStateNotBlockedByNonCompletionTask_MultiTask_Any,
	"System.StateTree.TasksCompletion.ChildStateNotBlockedByNonCompletionTask.MultiTask.Any");

// State re-entry resets the task completion bitmask so that B does not auto-complete on the second visit.
struct FStateTreeTest_TasksCompletion_NonCompletionTaskReEntryResetsStatus : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root    = EditorData.AddSubTree("Root");
		UStateTreeState& StateB  = Root.AddChildState("B");
		UStateTreeState& StateB1 = StateB.AddChildState("B1");
		UStateTreeState& StateB2 = StateB1.AddChildState("B2");

		// B: ineligible task, loops back to itself on completion.
		StateB.TasksCompletion = EStateTreeTaskCompletionType::All;
		{
			TStateTreeEditorNode<FTestTask_Stand>& TaskB = StateB.AddTask<FTestTask_Stand>("TaskB");
			TaskB.GetNode().TicksToCompletion = 1;
			TaskB.GetNode().bConsideredForCompletion = false;
			StateB.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &StateB);
		}

		// B2: task that needs 2 ticks to complete.
		{
			TStateTreeEditorNode<FTestTask_Stand>& TaskB2 = StateB2.AddTask<FTestTask_Stand>("TaskB2");
			TaskB2.GetNode().TicksToCompletion = 2;
		}

		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE(TEXT("StateTree should compile"), bResult);
			if (!bResult)
			{
				return false;
			}
		}

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE(TEXT("StateTree should init"), Exec.IsValid());

		// Start: B/B1/B2 entered, tasks not yet ticked.
		{
			const EStateTreeRunStatus Status = Exec.Start();
			AITEST_EQUAL(TEXT("Start: should be Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Start: should be in B/B1/B2"), Exec.ExpectInActiveStates("Root", "B", "B1", "B2"));
			Exec.LogClear();
		}

		// Tick 1: First tick in B. B's ineligible task returns Succeeded (not counted).
		// B2's task first tick, still running. B must NOT auto-complete.
		{
			const EStateTreeRunStatus Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick 1: should be Running (no auto-complete on first entry)"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Tick 1: still in B/B1/B2"), Exec.ExpectInActiveStates("Root", "B", "B1", "B2"));
			AITEST_TRUE(TEXT("Tick 1: TaskB2 ticked"), Exec.Expect("TaskB2", TEXT("Tick")));
			Exec.LogClear();
		}

		// Tick 2: B2's task second tick, completes. B completes and transitions back to B (re-entry).
		// B/B1/B2 are re-entered in the same tick but their tasks are not ticked yet.
		{
			const EStateTreeRunStatus Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick 2: should be Running after B re-enters itself"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Tick 2: TaskB2 completed on first visit"), Exec.Expect("TaskB2", TEXT("StateCompleted")));
			AITEST_TRUE(TEXT("Tick 2: still in B/B1/B2 (re-entered)"), Exec.ExpectInActiveStates("Root", "B", "B1", "B2"));
			Exec.LogClear();
		}

		// Tick 3: Second visit to B. Bitmask must have been reset. B must NOT auto-complete again.
		// B2's task ticks for the first time on the second visit (CurrentTick reset to 0 on re-entry).
		{
			const EStateTreeRunStatus Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick 3: should be Running (no auto-complete on re-entry)"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Tick 3: still in B/B1/B2 — B2 is running on second visit"), Exec.ExpectInActiveStates("Root", "B", "B1", "B2"));
			AITEST_TRUE(TEXT("Tick 3: TaskB2 ticked again on second visit"), Exec.Expect("TaskB2", TEXT("Tick")));
			Exec.LogClear();
		}

		Exec.Stop();
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(
	FStateTreeTest_TasksCompletion_NonCompletionTaskReEntryResetsStatus,
	"System.StateTree.TasksCompletion.NonCompletionTaskReEntryResetsStatus");

// The state must not complete when only the ineligible task finishes; it must wait for the eligible task to complete.
struct FStateTreeTest_TasksCompletion_MixedEligibleTasks_All : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		// Tree:
		//   Root
		//     B  (TasksCompletion=All)
		//       TaskB_Ineligible: bConsideredForCompletion=false, TicksToCompletion=1
		//       TaskB_Eligible:   bConsideredForCompletion=true,  TicksToCompletion=3
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root   = EditorData.AddSubTree("Root");
		UStateTreeState& StateB = Root.AddChildState("B");

		StateB.TasksCompletion = EStateTreeTaskCompletionType::All;
		{
			TStateTreeEditorNode<FTestTask_Stand>& TaskIneligible = StateB.AddTask<FTestTask_Stand>("TaskB_Ineligible");
			TaskIneligible.GetNode().TicksToCompletion = 1;
			TaskIneligible.GetNode().bConsideredForCompletion = false;
		}
		{
			TStateTreeEditorNode<FTestTask_Stand>& TaskEligible = StateB.AddTask<FTestTask_Stand>("TaskB_Eligible");
			TaskEligible.GetNode().TicksToCompletion = 3;
		}

		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE(TEXT("StateTree should compile"), bResult);
			if (!bResult)
			{
				return false;
			}
		}

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE(TEXT("StateTree should init"), Exec.IsValid());

		{
			const EStateTreeRunStatus Status = Exec.Start();
			AITEST_EQUAL(TEXT("Start: should be Running"), Status, EStateTreeRunStatus::Running);
			Exec.LogClear();
		}

		// Tick 1: TaskB_Ineligible completes (returns Succeeded, not counted).
		// TaskB_Eligible first tick, still running. B must NOT complete yet.
		{
			const EStateTreeRunStatus Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick 1: should be Running (ineligible done, eligible still running)"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Tick 1: still in B"), Exec.ExpectInActiveStates("Root", "B"));
			AITEST_FALSE(TEXT("Tick 1: TaskB_Eligible must not have completed"), Exec.Expect("TaskB_Eligible", TEXT("StateCompleted")));
			Exec.LogClear();
		}

		// Tick 2: TaskB_Eligible second tick, still running. B still must not complete.
		{
			const EStateTreeRunStatus Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick 2: should be Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Tick 2: still in B"), Exec.ExpectInActiveStates("Root", "B"));
			Exec.LogClear();
		}

		// Tick 3: TaskB_Eligible third tick, completes. B completes.
		{
			Exec.Tick(0.1f);
			AITEST_TRUE(TEXT("Tick 3: TaskB_Eligible completed"), Exec.Expect("TaskB_Eligible", TEXT("StateCompleted")));
			Exec.LogClear();
		}

		Exec.Stop();
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(
	FStateTreeTest_TasksCompletion_MixedEligibleTasks_All,
	"System.StateTree.TasksCompletion.MixedEligibleTasks.All");

// An ineligible task returning Failed from EnterState must not prevent the state from
// entering, and eligible tasks must still drive completion normally.
struct FStateTreeTest_TasksCompletion_IneligibleTaskEnterStateFail : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		//Tree:
		//   Root
		//     B  (TasksCompletion=All)
		//       TaskB_Ineligible: bConsideredForCompletion=false, EnterStateResult=Failed
		//       TaskB_Eligible:   bConsideredForCompletion=true,  TicksToCompletion=2
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root   = EditorData.AddSubTree("Root");
		UStateTreeState& StateB = Root.AddChildState("B");

		StateB.TasksCompletion = EStateTreeTaskCompletionType::All;
		{
			TStateTreeEditorNode<FTestTask_Stand>& TaskIneligible = StateB.AddTask<FTestTask_Stand>("TaskB_Ineligible");
			TaskIneligible.GetNode().EnterStateResult = EStateTreeRunStatus::Failed;
			TaskIneligible.GetNode().bConsideredForCompletion = false;
		}
		{
			TStateTreeEditorNode<FTestTask_Stand>& TaskEligible = StateB.AddTask<FTestTask_Stand>("TaskB_Eligible");
			TaskEligible.GetNode().TicksToCompletion = 2;
		}

		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE(TEXT("StateTree should compile"), bResult);
			if (!bResult)
			{
				return false;
			}
		}

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE(TEXT("StateTree should init"), Exec.IsValid());

		// Start: B entered. TaskB_Ineligible fails EnterState (does not tick).
		// TaskB_Eligible enters and ticks normally.
		{
			const EStateTreeRunStatus Status = Exec.Start();
			AITEST_EQUAL(TEXT("Start: should be Running (ineligible EnterState failure is ignored)"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Start: should be in B"), Exec.ExpectInActiveStates("Root", "B"));
			AITEST_TRUE(TEXT("Start: TaskB_Ineligible entered"), Exec.Expect("TaskB_Ineligible", TEXT("EnterState")));
			AITEST_TRUE(TEXT("Start: TaskB_Eligible entered"), Exec.Expect("TaskB_Eligible", TEXT("EnterState")));
			Exec.LogClear();
		}

		// Tick 1: TaskB_Ineligible does not tick (failed EnterState). TaskB_Eligible first tick.
		{
			const EStateTreeRunStatus Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick 1: should be Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Tick 1: still in B"), Exec.ExpectInActiveStates("Root", "B"));
			AITEST_FALSE(TEXT("Tick 1: TaskB_Ineligible must not tick after failed EnterState"), Exec.Expect("TaskB_Ineligible", TEXT("Tick")));
			AITEST_TRUE(TEXT("Tick 1: TaskB_Eligible ticked"), Exec.Expect("TaskB_Eligible", TEXT("Tick")));
			Exec.LogClear();
		}

		// Tick 2: TaskB_Eligible second tick, completes. B completes.
		{
			Exec.Tick(0.1f);
			AITEST_TRUE(TEXT("Tick 2: TaskB_Eligible completed"), Exec.Expect("TaskB_Eligible", TEXT("StateCompleted")));
			Exec.LogClear();
		}

		Exec.Stop();
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(
	FStateTreeTest_TasksCompletion_IneligibleTaskEnterStateFail,
	"System.StateTree.TasksCompletion.IneligibleTaskEnterStateFail");

// With TasksCompletion=Any, when two tasks both complete in the same tick — one with
// Succeeded and one with Failed — Failed must win (priority: Failed > Succeeded).
struct FStateTreeTest_TasksCompletion_Any_FailPriorityOverSucceeded : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		//Tree:
		//   Root
		//     B  (TasksCompletion=Any)
		//       TaskB_Succeed: TicksToCompletion=2, TickCompletionResult=Succeeded
		//       TaskB_Fail:    TicksToCompletion=2, TickCompletionResult=Failed

		// Both tasks complete on tick 2 in the same pass. Because all tasks are still processed after
		// the first Any-completion (see AllAny test), TaskB_Fail gets to update the status to Failed
		// which has higher priority than the already-set Succeeded.
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root   = EditorData.AddSubTree("Root");
		UStateTreeState& StateB = Root.AddChildState("B");

		StateB.TasksCompletion = EStateTreeTaskCompletionType::Any;
		{
			TStateTreeEditorNode<FTestTask_Stand>& TaskSucceed = StateB.AddTask<FTestTask_Stand>("TaskB_Succeed");
			TaskSucceed.GetNode().TicksToCompletion = 2;
			TaskSucceed.GetNode().TickCompletionResult = EStateTreeRunStatus::Succeeded;
		}
		{
			TStateTreeEditorNode<FTestTask_Stand>& TaskFail = StateB.AddTask<FTestTask_Stand>("TaskB_Fail");
			TaskFail.GetNode().TicksToCompletion = 2;
			TaskFail.GetNode().TickCompletionResult = EStateTreeRunStatus::Failed;
		}

		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE(TEXT("StateTree should compile"), bResult);
			if (!bResult)
			{
				return false;
			}
		}

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE(TEXT("StateTree should init"), Exec.IsValid());

		{
			const EStateTreeRunStatus Status = Exec.Start();
			AITEST_EQUAL(TEXT("Start: should be Running"), Status, EStateTreeRunStatus::Running);
			Exec.LogClear();
		}

		// Tick 1: Both tasks running (CurrentTick=1, < TicksToCompletion=2).
		{
			const EStateTreeRunStatus Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick 1: should be Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Tick 1: still in B"), Exec.ExpectInActiveStates("Root", "B"));
			Exec.LogClear();
		}

		// Tick 2: Both tasks complete in the same tick.
		// TaskB_Succeed (index 0) fires first with Succeeded; TaskB_Fail (index 1) fires with Failed.
		// Failed has higher priority than Succeeded, so the state must complete with Failed.
		{
			Exec.Tick(0.1f);
			AITEST_TRUE(TEXT("Tick 2: TaskB_Succeed ticked"), Exec.Expect("TaskB_Succeed", TEXT("Tick")));
			AITEST_TRUE(TEXT("Tick 2: TaskB_Fail ticked"), Exec.Expect("TaskB_Fail", TEXT("Tick")));
			// Both tasks received StateCompleted because they share the same completion round.
			AITEST_TRUE(TEXT("Tick 2: TaskB_Succeed received StateCompleted"), Exec.Expect("TaskB_Succeed", TEXT("StateCompleted")));
			AITEST_TRUE(TEXT("Tick 2: TaskB_Fail received StateCompleted"), Exec.Expect("TaskB_Fail", TEXT("StateCompleted")));
			// The state exited with Failed status because Failed > Succeeded in priority.
			AITEST_TRUE(TEXT("Tick 2: B exited as Failed"), Exec.Expect("TaskB_Succeed", TEXT("ExitFailed")));
			Exec.LogClear();
		}

		Exec.Stop();
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(
	FStateTreeTest_TasksCompletion_Any_FailPriorityOverSucceeded,
	"System.StateTree.TasksCompletion.Any.FailPriorityOverSucceeded");

// Disabled task in the middle of the task list
// The compiler excludes disabled tasks from CompletionMask but still allocates a bit slot
// for them. CompletionMask = 0b101 (bits 0 and 2 only; bit 1 for disabled TaskB_1 is 0).
// StateB must complete only when BOTH TaskB_0 (tick 2) and TaskB_2 (tick 3) complete.
struct FStateTreeTest_TasksCompletion_DisabledTaskInMiddle : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		//Tree: Root → StateB(3 tasks, All)
		//   TaskB_0: bTaskEnabled=true,  TicksToCompletion=2
		//   TaskB_1: bTaskEnabled=false  — never entered, never ticked
		//   TaskB_2: bTaskEnabled=true,  TicksToCompletion=3
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root   = EditorData.AddSubTree("Root");
		UStateTreeState& StateB = Root.AddChildState("B");

		StateB.TasksCompletion = EStateTreeTaskCompletionType::All;

		// Bit 0: enabled, completes in 2 ticks — set in CompletionMask.
		{
			TStateTreeEditorNode<FTestTask_Stand>& Task = StateB.AddTask<FTestTask_Stand>("TaskB_0");
			Task.GetNode().TicksToCompletion = 2;
		}
		// Bit 1: disabled — occupies the bit slot but excluded from CompletionMask.
		{
			TStateTreeEditorNode<FTestTask_Stand>& Task = StateB.AddTask<FTestTask_Stand>("TaskB_1");
			Task.GetNode().bTaskEnabled = false;
		}
		// Bit 2: enabled, completes in 3 ticks — set in CompletionMask.
		{
			TStateTreeEditorNode<FTestTask_Stand>& Task = StateB.AddTask<FTestTask_Stand>("TaskB_2");
			Task.GetNode().TicksToCompletion = 3;
		}

		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE(TEXT("StateTree should compile"), bResult);
			if (!bResult)
			{
				return false;
			}
		}

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE(TEXT("StateTree should init"), Exec.IsValid());

		// Start: TaskB_0 and TaskB_2 are entered; disabled TaskB_1 is skipped entirely.
		{
			const EStateTreeRunStatus Status = Exec.Start();
			AITEST_EQUAL(TEXT("Start: should be Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Start: TaskB_0 should enter"), Exec.Expect("TaskB_0", TEXT("EnterState")));
			AITEST_FALSE(TEXT("Start: TaskB_1 must not enter (disabled)"), Exec.Expect("TaskB_1", TEXT("EnterState")));
			AITEST_TRUE(TEXT("Start: TaskB_2 should enter"), Exec.Expect("TaskB_2", TEXT("EnterState")));
			Exec.LogClear();
		}

		// Tick 1: TaskB_0 and TaskB_2 tick (1/2 and 1/3). TaskB_1 is never ticked. B still running.
		{
			const EStateTreeRunStatus Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick 1: should be Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Tick 1: TaskB_0 ticked"), Exec.Expect("TaskB_0", TEXT("Tick")));
			AITEST_FALSE(TEXT("Tick 1: TaskB_1 must not tick (disabled)"), Exec.Expect("TaskB_1", TEXT("Tick")));
			AITEST_TRUE(TEXT("Tick 1: TaskB_2 ticked"), Exec.Expect("TaskB_2", TEXT("Tick")));
			Exec.LogClear();
		}

		// Tick 2: TaskB_0 completes (2/2). TaskB_2 still at 2/3. TaskB_1 still skipped.
		// Only bit 0 of CompletionMask (0b101) is set; bit 2 is not → B must NOT complete yet.
		{
			const EStateTreeRunStatus Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick 2: should be Running (TaskB_2 not yet done)"), Status, EStateTreeRunStatus::Running);
			AITEST_FALSE(TEXT("Tick 2: TaskB_1 must not tick (disabled)"), Exec.Expect("TaskB_1", TEXT("Tick")));
			AITEST_TRUE(TEXT("Tick 2: still in B"), Exec.ExpectInActiveStates("Root", "B"));
			Exec.LogClear();
		}

		// Tick 3: TaskB_2 completes (3/3). Both bits 0 and 2 of CompletionMask are set → B completes.
		{
			Exec.Tick(0.1f);
			AITEST_FALSE(TEXT("Tick 3: TaskB_1 must not tick (disabled)"), Exec.Expect("TaskB_1", TEXT("Tick")));
			AITEST_TRUE(TEXT("Tick 3: TaskB_2 completed"), Exec.Expect("TaskB_2", TEXT("StateCompleted")));
			Exec.LogClear();
		}

		Exec.Stop();
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(
	FStateTreeTest_TasksCompletion_DisabledTaskInMiddle,
	"System.StateTree.TasksCompletion.DisabledTaskInMiddle");

// Empty state (no tasks at all)
// When all active states have zero enabled tasks, the runtime forces the bottom active state
// (StateB) to complete with Succeeded on the first tick.
struct FStateTreeTest_TasksCompletion_EmptyState : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		//Tree: Root → StateB(no tasks)
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree("Root");
		UStateTreeState& StateB = Root.AddChildState("B");
		// No tasks added — StateB is an empty leaf state.
		StateB.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE(TEXT("StateTree should compile"), bResult);
			if (!bResult)
			{
				return false;
			}
		}

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE(TEXT("StateTree should init"), Exec.IsValid());

		{
			const EStateTreeRunStatus Status = Exec.Start();
			AITEST_EQUAL(TEXT("Start: should be Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Start: should be in B"), Exec.ExpectInActiveStates("Root", "B"));
			Exec.LogClear();
		}

		// Tick 1: No enabled tasks anywhere (NumTotalEnabledTasks == 0).
		// The runtime forces the bottom active state (B) to Succeeded.
		{
			const EStateTreeRunStatus Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick 1: empty state must complete with Succeeded"), Status, EStateTreeRunStatus::Succeeded);
			Exec.LogClear();
		}

		Exec.Stop();
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(
	FStateTreeTest_TasksCompletion_EmptyState,
	"System.StateTree.TasksCompletion.EmptyState");

// Completion mask buffer boundary crossing.
// Bit layout at compile time (no global tasks; global-frame synthetic bit = bit 0):
//   GlobalTaskEndBit = 1  (global-frame synthetic bit)
//   Root (no tasks, no parent) starts at bit 1 → synthetic bit at 1, NextBitIndex = 2
//   ParentState starts at bit 2 → 29 tasks occupy bits 2–30 (buffer 0), NextBitIndex = 31
//   ChildState starts at bit 31 → needs bits 31–32:
//     (31+2-1)/32 = 1  ≠  31/32 = 0 → boundary crossing detected
//     → compiler places ChildState entirely in buffer 1 (bits 0–1)
//
// Expected: ChildState's two tasks complete at tick 2, correctly evaluated in buffer 1.
struct FStateTreeTest_TasksCompletion_MaskBufferBoundary : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		//Tree:
		//	Root
		//		ParentState(29 eligible tasks, All, TicksToCompletion = 3)
		//			ChildState  (2 eligible tasks, All, TicksToCompletion=2)
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root        = EditorData.AddSubTree("Root");
		UStateTreeState& ParentState = Root.AddChildState("Parent");
		UStateTreeState& ChildState  = ParentState.AddChildState("Child");

		ParentState.TasksCompletion = EStateTreeTaskCompletionType::All;

		// 29 eligible tasks occupy bits 2–30 in buffer 0, pushing ChildState's first bit
		// to position 31 so its two tasks (31–32) cross the 32-bit buffer boundary.
		for (int32 i = 0; i < 29; ++i)
		{
			TStateTreeEditorNode<FTestTask_Stand>& Task = ParentState.AddTask<FTestTask_Stand>(
				FName(*FString::Printf(TEXT("TaskParent_%d"), i)));
			Task.GetNode().TicksToCompletion = 3;
		}

		ChildState.TasksCompletion = EStateTreeTaskCompletionType::All;

		{
			TStateTreeEditorNode<FTestTask_Stand>& Task = ChildState.AddTask<FTestTask_Stand>("TaskChild_0");
			Task.GetNode().TicksToCompletion = 2;
		}
		{
			TStateTreeEditorNode<FTestTask_Stand>& Task = ChildState.AddTask<FTestTask_Stand>("TaskChild_1");
			Task.GetNode().TicksToCompletion = 2;
		}

		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE(TEXT("StateTree should compile despite buffer boundary crossing"), bResult);
			if (!bResult)
			{
				return false;
			}
		}

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE(TEXT("StateTree should init"), Exec.IsValid());

		{
			const EStateTreeRunStatus Status = Exec.Start();
			AITEST_EQUAL(TEXT("Start: should be Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Start: should be in Parent/Child"), Exec.ExpectInActiveStates("Root", "Parent", "Child"));
			Exec.LogClear();
		}

		// Tick 1: All tasks ticking. ChildState tasks at 1/2 — still Running.
		{
			const EStateTreeRunStatus Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Tick 1: should be Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Tick 1: TaskChild_0 ticked"), Exec.Expect("TaskChild_0", TEXT("Tick")));
			AITEST_TRUE(TEXT("Tick 1: TaskChild_1 ticked"), Exec.Expect("TaskChild_1", TEXT("Tick")));
			AITEST_TRUE(TEXT("Tick 1: still in Parent/Child"), Exec.ExpectInActiveStates("Root", "Parent", "Child"));
			Exec.LogClear();
		}

		// Tick 2: ChildState tasks complete (2/2). Both tasks reside in buffer 1; the runtime
		// correctly reads their status from buffer 1 and exits ChildState with Succeeded.
		{
			Exec.Tick(0.1f);
			AITEST_TRUE(TEXT("Tick 2: TaskChild_0 completed"), Exec.Expect("TaskChild_0", TEXT("StateCompleted")));
			AITEST_TRUE(TEXT("Tick 2: TaskChild_1 completed"), Exec.Expect("TaskChild_1", TEXT("StateCompleted")));
			Exec.LogClear();
		}

		Exec.Stop();
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(
	FStateTreeTest_TasksCompletion_MaskBufferBoundary,
	"System.StateTree.TasksCompletion.MaskBufferBoundary");

} // namespace UE::StateTree::Tests

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
