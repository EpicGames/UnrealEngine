// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeTest.h"
#include "StateTreeTestBase.h"
#include "AITestsCommon.h"
#include "StateTreeCompilerLog.h"
#include "StateTreeEditorData.h"
#include "StateTreeCompiler.h"
#include "Conditions/StateTreeCommonConditions.h"
#include "Tasks/StateTreeRunParallelStateTreeTask.h"
#include "StateTreeTestTypes.h"
#include "AutoRTFM.h"
#include "Engine/World.h"
#include "Async/ParallelFor.h"
#include "Considerations/StateTreeCommonConsiderations.h"
#include "GameplayTagsManager.h"
#include "StateTreeReference.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeTest)

#define LOCTEXT_NAMESPACE "AITestSuite_StateTreeTest"

UE_DISABLE_OPTIMIZATION_SHIP

std::atomic<int32> FStateTreeTestConditionInstanceData::GlobalCounter = 0;

namespace UE::StateTree::Tests
{


struct FStateTreeTest_MakeAndBakeStateTree : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		
		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		UStateTreeState& StateA = Root.AddChildState(FName(TEXT("A")));
		UStateTreeState& StateB = Root.AddChildState(FName(TEXT("B")));

		// Root
		auto& EvalA = EditorData.AddEvaluator<FTestEval_A>();
		
		// State A
		auto& TaskB1 = StateA.AddTask<FTestTask_B>();
		EditorData.AddPropertyBinding(EvalA, TEXT("IntA"), TaskB1, TEXT("IntB"));

		auto& IntCond = StateA.AddEnterCondition<FStateTreeCompareIntCondition>(UE::StateTree::EComparisonOperator::Less);
		IntCond.GetInstanceData().Right = 2;

		EditorData.AddPropertyBinding(EvalA, TEXT("IntA"), IntCond, TEXT("Left"));

		StateA.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &StateB);

		// State B
		auto& TaskB2 = StateB.AddTask<FTestTask_B>();
		EditorData.AddPropertyBinding(EvalA, TEXT("bBoolA"), TaskB2, TEXT("bBoolB"));

		FStateTreeTransition& Trans = StateB.AddTransition({}, EStateTreeTransitionType::GotoState, &Root);
		auto& TransFloatCond = Trans.AddConditionWithOuter<FStateTreeCompareFloatCondition>(&StateB, UE::StateTree::EComparisonOperator::Less);
		TransFloatCond.GetInstanceData().Right = 13.0f;
		EditorData.AddPropertyBinding(EvalA, TEXT("FloatA"), TransFloatCond, TEXT("Left"));

		StateB.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);

		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);
		AITEST_TRUE(TEXT("StateTree should be ready to run"), StateTree.IsReadyToRun());

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_MakeAndBakeStateTree, "System.StateTree.MakeAndBakeStateTree");


struct FStateTreeTest_EmptyStateTree : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		
		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		Root.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);

		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		Status = Exec.Start();
		AITEST_TRUE(TEXT("StateTree should be running"), Status == EStateTreeRunStatus::Running);
		Exec.LogClear();

		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree should be completed"), Status == EStateTreeRunStatus::Succeeded);
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_EmptyStateTree, "System.StateTree.Empty");

struct FStateTreeTest_Sequence : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		
		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		UStateTreeState& State1 = Root.AddChildState(FName(TEXT("State1")));
		UStateTreeState& State2 = Root.AddChildState(FName(TEXT("State2")));

		auto& Task1 = State1.AddTask<FTestTask_Stand>(FName(TEXT("Task1")));
		State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::NextState);

		auto& Task2 = State2.AddTask<FTestTask_Stand>(FName(TEXT("Task2")));
		State2.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		const TCHAR* TickStr(TEXT("Tick"));
		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* ExitStateStr(TEXT("ExitState"));
		
		Status = Exec.Start();
		AITEST_TRUE(TEXT("StateTree Task1 should enter state"), Exec.Expect(Task1.GetName(), EnterStateStr));
		AITEST_FALSE(TEXT("StateTree Task1 should not tick"), Exec.Expect(Task1.GetName(), TickStr));
		Exec.LogClear();

		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Task1 should tick, and exit state"), Exec.Expect(Task1.GetName(), TickStr).Then(Task1.GetName(), ExitStateStr));
		AITEST_TRUE(TEXT("StateTree Task2 should enter state"), Exec.Expect(Task2.GetName(), EnterStateStr));
		AITEST_FALSE(TEXT("StateTree Task2 should not tick"), Exec.Expect(Task2.GetName(), TickStr));
		AITEST_TRUE(TEXT("StateTree should be running"), Status == EStateTreeRunStatus::Running);
		Exec.LogClear();
		
		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Task2 should tick, and exit state"), Exec.Expect(Task2.GetName(), TickStr).Then(Task2.GetName(), ExitStateStr));
		AITEST_FALSE(TEXT("StateTree Task1 should not tick"), Exec.Expect(Task1.GetName(), TickStr));
		AITEST_TRUE(TEXT("StateTree should be completed"), Status == EStateTreeRunStatus::Succeeded);
		Exec.LogClear();

		Status = Exec.Tick(0.1f);
		AITEST_FALSE(TEXT("StateTree Task1 should not tick"), Exec.Expect(Task1.GetName(), TickStr));
		AITEST_FALSE(TEXT("StateTree Task2 should not tick"), Exec.Expect(Task2.GetName(), TickStr));
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Sequence, "System.StateTree.Sequence");

struct FStateTreeTest_Select : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		
		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		UStateTreeState& State1 = Root.AddChildState(FName(TEXT("State1")));
		UStateTreeState& State1A = State1.AddChildState(FName(TEXT("State1A")));

		auto& TaskRoot = Root.AddTask<FTestTask_Stand>(FName(TEXT("TaskRoot")));
		TaskRoot.GetNode().TicksToCompletion = 3;  // let Task1A to complete first

		auto& Task1 = State1.AddTask<FTestTask_Stand>(FName(TEXT("Task1")));
		Task1.GetNode().TicksToCompletion = 3; // let Task1A to complete first

		auto& Task1A = State1A.AddTask<FTestTask_Stand>(FName(TEXT("Task1A")));
		Task1A.GetNode().TicksToCompletion = 2;
		State1A.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &State1);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		const TCHAR* TickStr(TEXT("Tick"));
		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* ExitStateStr(TEXT("ExitState"));

		// Start and enter state
		Status = Exec.Start();
		AITEST_TRUE(TEXT("StateTree TaskRoot should enter state"), Exec.Expect(TaskRoot.GetName(), EnterStateStr));
		AITEST_TRUE(TEXT("StateTree Task1 should enter state"), Exec.Expect(Task1.GetName(), EnterStateStr));
		AITEST_TRUE(TEXT("StateTree Task1A should enter state"), Exec.Expect(Task1A.GetName(), EnterStateStr));
		AITEST_FALSE(TEXT("StateTree TaskRoot should not tick"), Exec.Expect(TaskRoot.GetName(), TickStr));
		AITEST_FALSE(TEXT("StateTree Task1 should not tick"), Exec.Expect(Task1.GetName(), TickStr));
		AITEST_FALSE(TEXT("StateTree Task1A should not tick"), Exec.Expect(Task1A.GetName(), TickStr));
		AITEST_TRUE(TEXT("StateTree should be running"), Status == EStateTreeRunStatus::Running);
		Exec.LogClear();

		// Regular tick, no state selection at all.
		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree tasks should update in order"), Exec.Expect(TaskRoot.GetName(), TickStr).Then(Task1.GetName(), TickStr).Then(Task1A.GetName(), TickStr));
		AITEST_FALSE(TEXT("StateTree TaskRoot should not EnterState"), Exec.Expect(TaskRoot.GetName(), EnterStateStr));
		AITEST_FALSE(TEXT("StateTree Task1 should not EnterState"), Exec.Expect(Task1.GetName(), EnterStateStr));
		AITEST_FALSE(TEXT("StateTree Task1A should not EnterState"), Exec.Expect(Task1A.GetName(), EnterStateStr));
		AITEST_FALSE(TEXT("StateTree TaskRoot should not ExitState"), Exec.Expect(TaskRoot.GetName(), ExitStateStr));
		AITEST_FALSE(TEXT("StateTree Task1 should not ExitState"), Exec.Expect(Task1.GetName(), ExitStateStr));
		AITEST_FALSE(TEXT("StateTree Task1A should not ExitState"), Exec.Expect(Task1A.GetName(), ExitStateStr));
		AITEST_TRUE(TEXT("StateTree should be running"), Status == EStateTreeRunStatus::Running);
		Exec.LogClear();

		// Partial reselect, Root should not get EnterState
		Status = Exec.Tick(0.1f);
		AITEST_FALSE(TEXT("StateTree TaskRoot should not enter state"), Exec.Expect(TaskRoot.GetName(), EnterStateStr));
		AITEST_TRUE(TEXT("StateTree Task1 should tick, exit state, and enter state"), Exec.Expect(Task1.GetName(), TickStr).Then(Task1.GetName(), ExitStateStr).Then(Task1.GetName(), EnterStateStr));
		AITEST_TRUE(TEXT("StateTree Task1A should tick, exit state, and enter state"), Exec.Expect(Task1A.GetName(), TickStr).Then(Task1A.GetName(), ExitStateStr).Then(Task1A.GetName(), EnterStateStr));
		AITEST_TRUE(TEXT("StateTree should be running"), Status == EStateTreeRunStatus::Running);
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Select, "System.StateTree.Select");


struct FStateTreeTest_FailEnterState : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		
		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		UStateTreeState& State1 = Root.AddChildState(FName(TEXT("State1")));
		UStateTreeState& State1A = State1.AddChildState(FName(TEXT("State1A")));

		auto& TaskRoot = Root.AddTask<FTestTask_Stand>(FName(TEXT("TaskRoot")));

		auto& Task1 = State1.AddTask<FTestTask_Stand>(FName(TEXT("Task1")));
		auto& Task2 = State1.AddTask<FTestTask_Stand>(FName(TEXT("Task2")));
		Task2.GetNode().EnterStateResult = EStateTreeRunStatus::Failed;
		auto& Task3 = State1.AddTask<FTestTask_Stand>(FName(TEXT("Task3")));

		auto& Task1A = State1A.AddTask<FTestTask_Stand>(FName(TEXT("Task1A")));
		State1A.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &State1);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		const TCHAR* TickStr(TEXT("Tick"));
		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* ExitStateStr(TEXT("ExitState"));
		const TCHAR* StateCompletedStr(TEXT("StateCompleted"));

		// Start and enter state
		Status = Exec.Start();
		AITEST_TRUE(TEXT("StateTree TaskRoot should enter state"), Exec.Expect(TaskRoot.GetName(), EnterStateStr));
		AITEST_TRUE(TEXT("StateTree Task1 should enter state"), Exec.Expect(Task1.GetName(), EnterStateStr));
		AITEST_TRUE(TEXT("StateTree Task2 should enter state"), Exec.Expect(Task2.GetName(), EnterStateStr));
		AITEST_FALSE(TEXT("StateTree Task3 should not enter state"), Exec.Expect(Task3.GetName(), EnterStateStr));
		AITEST_TRUE(TEXT("StateTree Should execute StateCompleted in reverse order"), Exec.Expect(Task2.GetName(), StateCompletedStr).Then(Task1.GetName(), StateCompletedStr).Then(TaskRoot.GetName(), StateCompletedStr));
		AITEST_FALSE(TEXT("StateTree Task3 should not state complete"), Exec.Expect(Task3.GetName(), StateCompletedStr));
		AITEST_TRUE(TEXT("StateTree exec status should be failed"), Exec.GetLastTickStatus() == EStateTreeRunStatus::Failed);
		Exec.LogClear();

		// It will try 5 times to reenter the same states.
		Exec.Tick(0.01f);
		AITEST_TRUE(TEXT("StateTree TaskRoot should tick"), Exec.Expect(TaskRoot.GetName(), TickStr));
		AITEST_TRUE(TEXT("StateTree Task1 should tick"), Exec.Expect(Task1.GetName(), TickStr));
		AITEST_FALSE(TEXT("StateTree Task2 should not tick because it completed"), Exec.Expect(Task2.GetName(), TickStr));
		AITEST_FALSE(TEXT("StateTree Task3 should not tick because it didn't enter state"), Exec.Expect(Task3.GetName(), TickStr));
		AITEST_FALSE(TEXT("StateTree Task3 should not exit state"), Exec.Expect(Task3.GetName(), ExitStateStr));
		Exec.LogClear();

		// Stop and exit state
		Status = Exec.Stop();
		AITEST_FALSE(TEXT("StateTree Task1 should not state complete"), Exec.Expect(Task1.GetName(), StateCompletedStr));
		AITEST_FALSE(TEXT("StateTree Task2 should not state complete"), Exec.Expect(Task2.GetName(), StateCompletedStr));
		AITEST_FALSE(TEXT("StateTree Task3 should not state complete"), Exec.Expect(Task3.GetName(), StateCompletedStr));
		AITEST_FALSE(TEXT("StateTree TaskRoot should not state complete"), Exec.Expect(TaskRoot.GetName(), StateCompletedStr));
		AITEST_TRUE(TEXT("StateTree TaskRoot should exit state"), Exec.Expect(TaskRoot.GetName(), ExitStateStr));
		AITEST_TRUE(TEXT("StateTree Task1 should exit state"), Exec.Expect(Task1.GetName(), ExitStateStr));
		AITEST_TRUE(TEXT("StateTree Task2 should exit state"), Exec.Expect(Task2.GetName(), ExitStateStr));
		AITEST_FALSE(TEXT("StateTree Task3 should not exit state"), Exec.Expect(Task3.GetName(), ExitStateStr));
		AITEST_TRUE(TEXT("StateTree status should be stopped"), Status == EStateTreeRunStatus::Stopped);
		Exec.LogClear();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_FailEnterState, "System.StateTree.FailEnterState");


struct FStateTreeTest_Restart : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		
		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		UStateTreeState& State1 = Root.AddChildState(FName(TEXT("State1")));

		auto& Task1 = State1.AddTask<FTestTask_Stand>(FName(TEXT("Task1")));
		Task1.GetNode().TicksToCompletion = 2;

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE("StateTree should get compiled", bResult);

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE("StateTree should init", bInitSucceeded);

		const TCHAR* TickStr(TEXT("Tick"));
		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* ExitStateStr(TEXT("ExitState"));
		const TCHAR* StateCompletedStr(TEXT("StateCompleted"));

		// Start and enter state
		Status = Exec.Start();
		AITEST_TRUE(TEXT("StateTree Task1 should enter state"), Exec.Expect(Task1.GetName(), EnterStateStr));
		AITEST_TRUE(TEXT("StateTree exec status should be running"), Exec.GetLastTickStatus() == EStateTreeRunStatus::Running);
		Exec.LogClear();

		// Tick
		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree exec status should be running"), Exec.GetLastTickStatus() == EStateTreeRunStatus::Running);
		Exec.LogClear();

		// Call Start again, should stop and start the tree.
		Status = Exec.Start();
		AITEST_TRUE(TEXT("StateTree Task1 should exit state"), Exec.Expect(Task1.GetName(), ExitStateStr));
		AITEST_TRUE(TEXT("StateTree Task1 should enter state"), Exec.Expect(Task1.GetName(), EnterStateStr));
		AITEST_TRUE(TEXT("StateTree exec status should be running"), Exec.GetLastTickStatus() == EStateTreeRunStatus::Running);
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Restart, "System.StateTree.Restart");

struct FStateTreeTest_SubTree_ActiveTasks : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		
		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		UStateTreeState& State1 = Root.AddChildState(FName(TEXT("State1")), EStateTreeStateType::Linked);
		UStateTreeState& State2 = Root.AddChildState(FName(TEXT("State2")));
		UStateTreeState& State3 = Root.AddChildState(FName(TEXT("State3")), EStateTreeStateType::Subtree);
		UStateTreeState& State3A = State3.AddChildState(FName(TEXT("State3A")));
		UStateTreeState& State3B = State3.AddChildState(FName(TEXT("State3B")));

		State1.SetLinkedState(State3.GetLinkToState());

		State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &State2);

		auto& Task2 = State2.AddTask<FTestTask_Stand>(FName(TEXT("Task2")));
		State2.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		auto& Task3A = State3A.AddTask<FTestTask_Stand>(FName(TEXT("Task3A")));
		State3A.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &State3B);

		auto& Task3B = State3B.AddTask<FTestTask_Stand>(FName(TEXT("Task3B")));
		State3B.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		const TCHAR* TickStr(TEXT("Tick"));
		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* ExitStateStr(TEXT("ExitState"));
		const TCHAR* StateCompletedStr(TEXT("StateCompleted"));

		// Start and enter state
		Status = Exec.Start();

		AITEST_TRUE(TEXT("StateTree Active States should be in Root/State1/State3/State3A"), Exec.ExpectInActiveStates(Root.Name, State1.Name, State3.Name, State3A.Name));
		AITEST_FALSE(TEXT("StateTree Task2 should not enter state"), Exec.Expect(Task2.GetName(), EnterStateStr));
		AITEST_TRUE(TEXT("StateTree Task3A should enter state"), Exec.Expect(Task3A.GetName(), EnterStateStr));
		AITEST_TRUE(TEXT("StateTree should be running"), Status == EStateTreeRunStatus::Running);
		Exec.LogClear();

		// Transition within subtree
		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Active States should be in Root/State1/State3/State3B"), Exec.ExpectInActiveStates(Root.Name, State1.Name, State3.Name, State3B.Name));
		AITEST_TRUE(TEXT("StateTree Task3B should enter state"), Exec.Expect(Task3B.GetName(), EnterStateStr));
		AITEST_TRUE(TEXT("StateTree should be running"), Status == EStateTreeRunStatus::Running);
		Exec.LogClear();

		// Complete subtree
		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Active States should be in Root/State2"), Exec.ExpectInActiveStates(Root.Name, State2.Name));
		AITEST_TRUE(TEXT("StateTree Task2 should enter state"), Exec.Expect(Task2.GetName(), EnterStateStr));
		AITEST_TRUE(TEXT("StateTree should be running"), Status == EStateTreeRunStatus::Running);
		Exec.LogClear();

		// Complete the whole tree
		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree should complete in succeeded"), Status == EStateTreeRunStatus::Succeeded);
		Exec.LogClear();

		Exec.Stop();
		
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_SubTree_ActiveTasks, "System.StateTree.SubTree.ActiveTasks");

struct FStateTreeTest_SubTree_NoActiveTasks : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		/*
		 - RootA
			- StateA : SubTree -> StateB
			- StateB
		 - RootB -> StateB
		 - SubTree[DisabledTask] -> StateB
			- StateC -> RootB
		 */
		UStateTreeState& RootA = EditorData.AddSubTree(FName(TEXT("RootA")));
		UStateTreeState& StateA = RootA.AddChildState(FName(TEXT("StateA")));
		UStateTreeState& StateB = RootA.AddChildState(FName(TEXT("StateB")));

		UStateTreeState& RootB = EditorData.AddSubTree(FName(TEXT("RootB")));

		UStateTreeState& SubTree = EditorData.AddSubTree(FName(TEXT("SubTree")));
		UStateTreeState& StateC = SubTree.AddChildState(FName(TEXT("StateC")));

		SubTree.Type = EStateTreeStateType::Subtree;
		StateA.Type = EStateTreeStateType::Linked;
		StateA.SetLinkedState(SubTree.GetLinkToState());

		StateA.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &StateB);
		RootB.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &StateB);
		SubTree.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &StateB);
		StateC.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &RootB);

		auto& TaskNode = SubTree.AddTask<FTestTask_Stand>(FName(TEXT("DisabledTask")));
		TaskNode.GetNode().bTaskEnabled = false;

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		// Start and enter state
		Status = Exec.Start();

		AITEST_TRUE(TEXT("StateTree Active States should be in RootA/StateA/SubTree/StateC"),
					Exec.ExpectInActiveStates(RootA.Name, StateA.Name, SubTree.Name, StateC.Name));
		AITEST_TRUE(TEXT("StateTree should be running"), Status == EStateTreeRunStatus::Running);
		Exec.LogClear();

		// Transition from the subtree frame. The parent frame and the disabled task should be ignored.
		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Active States should be in RootB"),
					Exec.ExpectInActiveStates(RootB.Name));
		AITEST_TRUE(TEXT("StateTree should be running"), Status == EStateTreeRunStatus::Running);
		Exec.LogClear();

		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Active States should be in RootA/StateB"),
					Exec.ExpectInActiveStates(RootA.Name, StateB.Name));
		AITEST_TRUE(TEXT("StateTree should be running"), Status == EStateTreeRunStatus::Running);
		Exec.LogClear();

		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Active States should be in RootA/StateA/SubTree/StateC"),
					Exec.ExpectInActiveStates(RootA.Name, StateA.Name, SubTree.Name, StateC.Name));
		AITEST_TRUE(TEXT("StateTree should be running"), Status == EStateTreeRunStatus::Running);
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_SubTree_NoActiveTasks, "System.StateTree.SubTree.NoActiveTasks");

struct FStateTreeTest_SubTree_Condition : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		/*
		- Root
			- Linked : Subtree -> Root
			- SubTree : Task1
				- ? State1 : Task2 -> Succeeded // condition linked to Task1
				- State2 : Task3
		*/
		
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		
		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		UStateTreeState& Linked = Root.AddChildState(FName(TEXT("Linked")), EStateTreeStateType::Linked);
		
		UStateTreeState& SubTree = Root.AddChildState(FName(TEXT("SubTree")), EStateTreeStateType::Subtree);
		UStateTreeState& State1 = SubTree.AddChildState(FName(TEXT("State1")));
		UStateTreeState& State2 = SubTree.AddChildState(FName(TEXT("State2")));

		Linked.SetLinkedState(SubTree.GetLinkToState());

		Linked.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &Linked);

		// SubTask should not complete during the test.
		TStateTreeEditorNode<FTestTask_Stand>& SubTask = SubTree.AddTask<FTestTask_Stand>(FName(TEXT("SubTask")));
		SubTask.GetNode().TicksToCompletion = 100;

		TStateTreeEditorNode<FTestTask_Stand>& Task1 = State1.AddTask<FTestTask_Stand>(FName(TEXT("Task1")));
		Task1.GetNode().TicksToCompletion = 1;

		TStateTreeEditorNode<FTestTask_Stand>& Task2 = State2.AddTask<FTestTask_Stand>(FName(TEXT("Task2")));
		Task2.GetNode().TicksToCompletion = 1;
		
		// Allow to enter State1 if Task1 instance data TicksToCompletion > 0.
		TStateTreeEditorNode<FStateTreeCompareIntCondition>& IntCond1 = State1.AddEnterCondition<FStateTreeCompareIntCondition>(UE::StateTree::EComparisonOperator::Greater);
		EditorData.AddPropertyBinding(SubTask, TEXT("CurrentTick"), IntCond1, TEXT("Left"));
		IntCond1.GetInstanceData().Right = 0;

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE("StateTree should get compiled", bResult);

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		const TCHAR* TickStr(TEXT("Tick"));
		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* ExitStateStr(TEXT("ExitState"));
		const TCHAR* StateCompletedStr(TEXT("StateCompleted"));

		GetTestRunner().AddExpectedMessage(TEXT("Evaluation forced to false"), ELogVerbosity::Warning, EAutomationExpectedErrorFlags::Contains, 1);

		// Start and enter state
		Status = Exec.Start();

		AITEST_TRUE(TEXT("StateTree Active States should be in Root/Linked/SubTree/State2"), Exec.ExpectInActiveStates(Root.Name, Linked.Name, SubTree.Name, State2.Name));
		AITEST_FALSE(TEXT("StateTree State1 should not be active"), Exec.ExpectInActiveStates(State1.Name)); // Enter condition should prevent to enter State1
		AITEST_TRUE(TEXT("StateTree SubTask should enter state"), Exec.Expect(SubTask.GetName(), EnterStateStr));
		AITEST_TRUE(TEXT("StateTree Task2 should enter state"), Exec.Expect(Task2.GetName(), EnterStateStr));
		AITEST_TRUE(TEXT("StateTree should be running"), Status == EStateTreeRunStatus::Running);
		Exec.LogClear();

		// Task1 completes, and we should enter State1 since the enter condition now passes.
		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Active States should be in Root/Linked/SubTree/State1"), Exec.ExpectInActiveStates(Root.Name, Linked.Name, SubTree.Name, State1.Name));
		AITEST_FALSE(TEXT("StateTree State2 should not be active"), Exec.ExpectInActiveStates(State2.Name));
		AITEST_TRUE(TEXT("StateTree Task1 should enter state"), Exec.Expect(Task1.GetName(), EnterStateStr));
		AITEST_TRUE(TEXT("StateTree should be running"), Status == EStateTreeRunStatus::Running);
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_SubTree_Condition, "System.StateTree.SubTree.Condition");

struct FStateTreeTest_SubTree_CascadedSucceeded : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		//	- Root [TaskA]
		//		- LinkedState>SubTreeState -> (F)Failed
		//		- SubTreeState [TaskB]
		//			- SubLinkedState>SubSubTreeState -> (S)Failed
		//		- SubSubTreeState
		//			- SubSubLeaf [TaskC] -> (S)Succeeded
		
		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		UStateTreeState& LinkedState = Root.AddChildState(FName(TEXT("Linked")), EStateTreeStateType::Linked);
		
		UStateTreeState& SubTreeState = Root.AddChildState(FName(TEXT("SubTreeState")), EStateTreeStateType::Subtree);
		UStateTreeState& SubLinkedState = SubTreeState.AddChildState(FName(TEXT("SubLinkedState")), EStateTreeStateType::Linked);
		
		UStateTreeState& SubSubTreeState = Root.AddChildState(FName(TEXT("SubSubTreeState")), EStateTreeStateType::Subtree);
		UStateTreeState& SubSubLeaf = SubSubTreeState.AddChildState(FName(TEXT("SubSubLeaf")));

		LinkedState.SetLinkedState(SubTreeState.GetLinkToState());
		SubLinkedState.SetLinkedState(SubSubTreeState.GetLinkToState());

		LinkedState.AddTransition(EStateTreeTransitionTrigger::OnStateFailed, EStateTreeTransitionType::Failed);
		SubLinkedState.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::Failed);
		SubSubLeaf.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::Succeeded);

		TStateTreeEditorNode<FTestTask_Stand>& TaskA = Root.AddTask<FTestTask_Stand>(FName(TEXT("TaskA")));
		TStateTreeEditorNode<FTestTask_Stand>& TaskB = SubTreeState.AddTask<FTestTask_Stand>(FName(TEXT("TaskB")));
		TStateTreeEditorNode<FTestTask_Stand>& TaskC = SubSubLeaf.AddTask<FTestTask_Stand>(FName(TEXT("TaskC")));

		TaskA.GetNode().TicksToCompletion = 2;
		TaskB.GetNode().TicksToCompletion = 2;
		TaskC.GetNode().TicksToCompletion = 1; // The deepest task completes first.
		
		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		const TCHAR* TickStr(TEXT("Tick"));
		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* ExitStateStr(TEXT("ExitState"));
		const TCHAR* StateCompletedStr(TEXT("StateCompleted"));

		// Start and enter state
		Status = Exec.Start();
		AITEST_TRUE(TEXT("StateTree Active States should be in Root/Linked/SubTreeState"), Exec.ExpectInActiveStates(Root.Name, LinkedState.Name, SubTreeState.Name, SubLinkedState.Name, SubSubTreeState.Name, SubSubLeaf.Name));
		AITEST_TRUE(TEXT("TaskA,B,C should enter state"), Exec.Expect(TaskA.GetName(), EnterStateStr).Then(TaskB.GetName(), EnterStateStr).Then(TaskC.GetName(), EnterStateStr));
		AITEST_TRUE(TEXT("StateTree should be running"), Status == EStateTreeRunStatus::Running);
		Exec.LogClear();

		// Subtrees completes, and it completes the whole tree too.
		// There's no good way to observe this externally. We switch the return along the way to make sure the transition does not happen directly from the leaf to failed.
		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree should be Failed"), Status == EStateTreeRunStatus::Failed);
		Exec.LogClear();

		Exec.Stop();
		
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_SubTree_CascadedSucceeded, "System.StateTree.SubTree.CascadedSucceeded");


struct FStateTreeTest_SharedInstanceData : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		
		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		auto& IntCond = Root.AddEnterCondition<FStateTreeTestCondition>();
		IntCond.GetInstanceData().Count = 1;

		auto& Task = Root.AddTask<FTestTask_Stand>(FName(TEXT("Task")));
		Task.GetNode().TicksToCompletion = 2;

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		// Init, nothing should access the shared data.
		constexpr int32 NumConcurrent = 100;
		UE_AUTORTFM_OPEN
		{
			FStateTreeTestConditionInstanceData::GlobalCounter = 0;
		};

		bool bInitSucceeded = true;
		TArray<FStateTreeInstanceData> InstanceDatas;

		InstanceDatas.SetNum(NumConcurrent);
		for (int32 Index = 0; Index < NumConcurrent; Index++)
		{
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceDatas[Index]);
			bInitSucceeded &= Exec.IsValid();
		}
		AITEST_TRUE(TEXT("All StateTree contexts should init"), bInitSucceeded);

		int32 GlobalCounterValue;
		UE_AUTORTFM_OPEN
		{
			GlobalCounterValue = FStateTreeTestConditionInstanceData::GlobalCounter;
		};
		AITEST_EQUAL(TEXT("Test condition global counter should be 0"), GlobalCounterValue, 0);
		
		// Start in parallel
		// This should create shared data per thread.
		// We expect that ParallelForWithTaskContext() creates a context per thread.
		TArray<FStateTreeTestRunContext> RunContexts;
		
		ParallelForWithTaskContext(
			RunContexts,
			InstanceDatas.Num(),
			[&InstanceDatas, &StateTree](FStateTreeTestRunContext& RunContext, int32 Index)
			{
				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceDatas[Index]);
				const EStateTreeRunStatus Status = Exec.Start();
				if (Status == EStateTreeRunStatus::Running)
				{
					RunContext.Count++;
				}
			}
		);

		int32 StartTotalRunning = 0;
		for (FStateTreeTestRunContext RunContext : RunContexts)
		{
			StartTotalRunning += RunContext.Count;
		}
		AITEST_EQUAL(TEXT("All StateTree contexts should be running after Start"), StartTotalRunning, NumConcurrent);

		UE_AUTORTFM_OPEN
		{
			GlobalCounterValue = FStateTreeTestConditionInstanceData::GlobalCounter;
		};
		AITEST_EQUAL(TEXT("Test condition global counter should equal context count after Start"), GlobalCounterValue, InstanceDatas.Num());
		
		// Tick in parallel
		// This should not recreate the data, so FStateTreeTestConditionInstanceData::GlobalCounter should stay as is.
		for (FStateTreeTestRunContext RunContext : RunContexts)
		{
			RunContext.Count = 0;
		}

		ParallelForWithTaskContext(
			RunContexts,
			InstanceDatas.Num(),
			[&InstanceDatas, &StateTree](FStateTreeTestRunContext& RunContext, int32 Index)
			{
				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceDatas[Index]);
				const EStateTreeRunStatus Status = Exec.Tick(0.1f);
				if (Status == EStateTreeRunStatus::Running)
				{
					RunContext.Count++;
				}
			}
		);

		int32 TickTotalRunning = 0;
		for (FStateTreeTestRunContext RunContext : RunContexts)
		{
			TickTotalRunning += RunContext.Count;
		}
		AITEST_EQUAL(TEXT("All StateTree contexts should be running after Tick"), TickTotalRunning, NumConcurrent);

		UE_AUTORTFM_OPEN
		{
			GlobalCounterValue = FStateTreeTestConditionInstanceData::GlobalCounter;
		};
		AITEST_EQUAL(TEXT("Test condition global counter should equal context count after Tick"), GlobalCounterValue, InstanceDatas.Num());

		ParallelForWithTaskContext(
			RunContexts,
			InstanceDatas.Num(),
			[&InstanceDatas, &StateTree](FStateTreeTestRunContext& RunContext, int32 Index)
			{
				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceDatas[Index]);
				Exec.Stop();
			}
		);

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_SharedInstanceData, "System.StateTree.SharedInstanceData");

struct FStateTreeTest_LastConditionWithIndent : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		
		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		UStateTreeState& State1 = Root.AddChildState(FName(TEXT("State1")));

		auto& Task1 = State1.AddTask<FTestTask_Stand>(FName(TEXT("Task1")));
		State1.AddEnterCondition<FStateTreeTestCondition>();
		auto& LastCondition = State1.AddEnterCondition<FStateTreeTestCondition>();

		// Last condition has Indent
		LastCondition.ExpressionIndent = 1;
		
		State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		const TCHAR* TickStr(TEXT("Tick"));
		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* ExitStateStr(TEXT("ExitState"));

		Status = Exec.Start();
		AITEST_TRUE(TEXT("StateTree Task1 should enter state"), Exec.Expect(Task1.GetName(), EnterStateStr));
		AITEST_FALSE(TEXT("StateTree Task1 should not tick"), Exec.Expect(Task1.GetName(), TickStr));
		Exec.LogClear();

		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Task1 should tick, and exit state"), Exec.Expect(Task1.GetName(), TickStr).Then(Task1.GetName(), ExitStateStr));
		AITEST_TRUE(TEXT("StateTree should be completed"), Status == EStateTreeRunStatus::Succeeded);
		Exec.LogClear();

		Status = Exec.Tick(0.1f);
		AITEST_FALSE(TEXT("StateTree Task1 should not tick"), Exec.Expect(Task1.GetName(), TickStr));
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_LastConditionWithIndent, "System.StateTree.LastConditionWithIndent");

struct FStateTreeTest_StateRequiringEvent : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		
		FGameplayTag ValidTag = GetTestTag1();
		FGameplayTag InvalidTag = GetTestTag2();

		using FValidPayload = FStateTreeTest_PropertyStructA;
		using FInvalidPayload = FStateTreeTest_PropertyStructB;

		// This state shouldn't be selected as it requires different tag.
		UStateTreeState& StateA = Root.AddChildState(FName(TEXT("A")));
		StateA.bHasRequiredEventToEnter  = true;
		StateA.RequiredEventToEnter.Tag = InvalidTag;
		auto& TaskA = StateA.AddTask<FTestTask_Stand>(FName(TEXT("TaskA")));

		// This state shouldn't be selected as it requires different payload.
		UStateTreeState& StateB = Root.AddChildState(FName(TEXT("B")));
		StateB.bHasRequiredEventToEnter  = true;
		StateB.RequiredEventToEnter.PayloadStruct = FInvalidPayload::StaticStruct();
		auto& TaskB = StateB.AddTask<FTestTask_Stand>(FName(TEXT("TaskB")));

		// This state shouldn't be selected as it requires the same tag, but different payload.
		UStateTreeState& StateC = Root.AddChildState(FName(TEXT("C")));
		StateC.bHasRequiredEventToEnter  = true;
		StateC.RequiredEventToEnter.Tag = ValidTag;
		StateC.RequiredEventToEnter.PayloadStruct = FInvalidPayload::StaticStruct();
		auto& TaskC = StateC.AddTask<FTestTask_Stand>(FName(TEXT("TaskC")));

		// This state shouldn't be selected as it requires the same payload, but different tag.
		UStateTreeState& StateD = Root.AddChildState(FName(TEXT("D")));
		StateD.bHasRequiredEventToEnter  = true;
		StateD.RequiredEventToEnter.Tag = InvalidTag;
		StateD.RequiredEventToEnter.PayloadStruct = FValidPayload::StaticStruct();
		auto& TaskD = StateD.AddTask<FTestTask_Stand>(FName(TEXT("TaskD")));

		// This state should be selected as the arrived event matches the requirement.
		UStateTreeState& StateE = Root.AddChildState(FName(TEXT("E")));
		StateE.bHasRequiredEventToEnter  = true;
		StateE.RequiredEventToEnter.Tag = ValidTag;
		StateE.RequiredEventToEnter.PayloadStruct = FValidPayload::StaticStruct();
		auto& TaskE = StateE.AddTask<FTestTask_Stand>(FName(TEXT("TaskE")));

		// This state should be selected only initially when there's not event in the queue.
		UStateTreeState& StateInitial = Root.AddChildState(FName(TEXT("Initial")));
		auto& TaskInitial = StateInitial.AddTask<FTestTask_Stand>(FName(TEXT("TaskInitial")));
		StateInitial.AddTransition(EStateTreeTransitionTrigger::OnEvent, ValidTag, EStateTreeTransitionType::GotoState, &Root);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);

		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;

		const TCHAR* EnterStateStr(TEXT("EnterState"));

		auto Test=[&](FTestStateTreeExecutionContext& Exec)
			{
				AITEST_FALSE(TEXT("StateTree TaskA should not enter state"), Exec.Expect(TaskA.GetName(), EnterStateStr));
				AITEST_FALSE(TEXT("StateTree TaskB should not enter state"), Exec.Expect(TaskB.GetName(), EnterStateStr));
				AITEST_FALSE(TEXT("StateTree TaskC should not enter state"), Exec.Expect(TaskC.GetName(), EnterStateStr));
				AITEST_FALSE(TEXT("StateTree TaskD should not enter state"), Exec.Expect(TaskD.GetName(), EnterStateStr));
				AITEST_TRUE(TEXT("StateTree TaskE should enter state"), Exec.Expect(TaskE.GetName(), EnterStateStr));
				return true;
			};

		{
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE("StateTree should init", bInitSucceeded);

			Status = Exec.Start();
			AITEST_TRUE(TEXT("StateTree TaskInitial should enter state"), Exec.Expect(TaskInitial.GetName(), EnterStateStr));
			Exec.LogClear();

			Exec.SendEvent(ValidTag, FConstStructView::Make(FValidPayload()));
			Status = Exec.Tick(0.1f);

			if (!Test(Exec))
			{
				return false;
			}
			Exec.LogClear();

			Exec.Stop();
		}
		// Same test but event sent with weak context while the FTestStateTreeExecutionContext still exist
		{
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

			Status = Exec.Start();
			AITEST_TRUE(TEXT("StateTree TaskInitial should enter state"), Exec.Expect(TaskInitial.GetName(), EnterStateStr));
			Exec.LogClear();

			FStateTreeWeakExecutionContext WeakExec = Exec.MakeWeakExecutionContext();
			WeakExec.SendEvent(ValidTag, FConstStructView::Make(FValidPayload()));
			Status = Exec.Tick(0.1f);

			if (!Test(Exec))
			{
				return false;
			}
			Exec.LogClear();

			Exec.Stop();
		}
		// Same test but event sent with weak context
		{
			FStateTreeWeakExecutionContext WeakExec;
			{
				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
				const bool bInitSucceeded = Exec.IsValid();
				AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

				Status = Exec.Start();
				AITEST_TRUE(TEXT("StateTree TaskInitial should enter state"), Exec.Expect(TaskInitial.GetName(), EnterStateStr));
				Exec.LogClear();

				WeakExec = Exec.MakeWeakExecutionContext();
			}
			{
				WeakExec.SendEvent(ValidTag, FConstStructView::Make(FValidPayload()));
			}
			{
				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
				Status = Exec.Tick(0.1f);

				if (!Test(Exec))
				{
					return false;
				}
				Exec.LogClear();

				Exec.Stop();
			}
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_StateRequiringEvent, "System.StateTree.StateRequiringEvent");

struct FStateTreeTest_Start : FStateTreeTestBase
{
	virtual UStateTree& SetupTree()
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		UStateTreeState& StateA = Root.AddChildState(FName(TEXT("A")));
		UStateTreeState& StateB = Root.AddChildState(FName(TEXT("B")));
		TStateTreeEditorNode<FTestTask_Stand>& TaskA = StateA.AddTask<FTestTask_Stand>(TaskAName);
		TStateTreeEditorNode<FTestTask_Stand>& TaskB = StateB.AddTask<FTestTask_Stand>(TaskBName);

		// Transition success 
		StateA.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::GotoState, &StateB);
		TaskA.GetNode().EnterStateResult = EStateTreeRunStatus::Succeeded;

		return StateTree;
	}

	virtual bool InstantTest() override
	{
		UStateTree& StateTree = SetupTree();

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);

		AITEST_TRUE("StateTree should get compiled", bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE("StateTree should init", bInitSucceeded);

		const TCHAR* TickStr(TEXT("Tick"));
		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* ExitStateStr(TEXT("ExitState"));
		const TCHAR* StateCompletedStr(TEXT("StateCompleted"));

		{
			EStateTreeRunStatus Status = Exec.Start();
			AITEST_EQUAL("Start should be running", Status, EStateTreeRunStatus::Running);
			AITEST_TRUE("StateTree Active States should be in states.", Exec.ExpectInActiveStates("Root", "A"));
			AITEST_TRUE("StateTree TaskA should enter state", Exec.Expect(TaskAName, EnterStateStr));
			AITEST_TRUE("StateTree TaskA should state complete", Exec.Expect(TaskAName, StateCompletedStr));
			AITEST_TRUE("StateTree execution should not sleep", !Exec.GetNextScheduledTick().ShouldSleep());
			Exec.LogClear();
		}
		{
			EStateTreeRunStatus Status = Exec.Tick(0.1f);
			AITEST_TRUE("StateTree Active States should be in states.", Exec.ExpectInActiveStates("Root", "B"));
			//@TODO Only one StateComplete
			//AITEST_FALSE("StateTree TaskA should state complete", Exec.Expect(TaskAName, StateCompletedStr));
			AITEST_TRUE("StateTree TaskA should get exit state expectedly", Exec.Expect(TaskAName, ExitStateStr));
			AITEST_TRUE("StateTree TaskB should get enter state expectedly", Exec.Expect(TaskBName, EnterStateStr));
			Exec.LogClear();
		}

		Exec.Stop();

		return true;
	}

protected:
	const FName TaskAName = FName("TaskA");
	const FName TaskBName = FName("TaskB");
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Start, "System.StateTree.Start.FirstStateSucceededImmediately");

struct FStateTreeTest_StartScheduledTick : FStateTreeTest_Start
{
	virtual UStateTree& SetupTree()
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		UStateTreeState& StateA = Root.AddChildState(FName(TEXT("A")));
		UStateTreeState& StateB = Root.AddChildState(FName(TEXT("B")));
		TStateTreeEditorNode<FTestTask_StandNoTick>& TaskA = StateA.AddTask<FTestTask_StandNoTick>(TaskAName);
		TStateTreeEditorNode<FTestTask_StandNoTick>& TaskB = StateB.AddTask<FTestTask_StandNoTick>(TaskBName);

		// Transition success 
		StateA.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::GotoState, &StateB);
		TaskA.GetNode().EnterStateResult = EStateTreeRunStatus::Succeeded;

		return StateTree;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_StartScheduledTick, "System.StateTree.Start.FirstStateSucceededImmediatelyWithScheduledTick");

//
// The stop tests test how the combinations of execution path to stop the tree are reported on ExitState() transition.  
//
struct FStateTreeTest_Stop : FStateTreeTestBase
{
	UStateTree& SetupTree()
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		UStateTreeState& StateA = Root.AddChildState(FName(TEXT("A")));
		TStateTreeEditorNode<FTestTask_Stand>& TaskA = StateA.AddTask<FTestTask_Stand>(TaskAName);
		TStateTreeEditorNode<FTestTask_Stand>& GlobalTask = EditorData.AddGlobalTask<FTestTask_Stand>(GlobalTaskName);

		// Transition success 
		StateA.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::Succeeded);
		StateA.AddTransition(EStateTreeTransitionTrigger::OnStateFailed, EStateTreeTransitionType::Failed);

		GlobalTask.GetNode().TicksToCompletion = GlobalTaskTicks;
		GlobalTask.GetNode().TickCompletionResult = GlobalTaskStatus;
		GlobalTask.GetNode().EnterStateResult = GlobalTaskEnterStatus;

		TaskA.GetNode().TicksToCompletion = NormalTaskTicks;
		TaskA.GetNode().TickCompletionResult = NormalTaskStatus;
		TaskA.GetNode().EnterStateResult = NormalTaskEnterStatus;

		return StateTree;
	}
	
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = SetupTree();
		
		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);

		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		const TCHAR* TickStr(TEXT("Tick"));
		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* ExitStateStr(TEXT("ExitState"));

		Status = Exec.Start();
		AITEST_EQUAL(TEXT("Start should be running"), Status, EStateTreeRunStatus::Running);
		AITEST_TRUE(TEXT("StateTree GlobalTask should enter state"), Exec.Expect(GlobalTaskName, EnterStateStr));
		AITEST_TRUE(TEXT("StateTree TaskA should enter state"), Exec.Expect(TaskAName, EnterStateStr));
		Exec.LogClear();

		Status = Exec.Tick(0.1f);
		AITEST_EQUAL(TEXT("Tree should end expectedly"), Status, ExpectedStatusAfterTick);
		AITEST_TRUE(TEXT("StateTree GlobalTask should get exit state expectedly"), Exec.Expect(GlobalTaskName, ExpectedExitStatusStr));
		AITEST_TRUE(TEXT("StateTree TaskA should get exit state expectedly"), Exec.Expect(TaskAName, ExpectedExitStatusStr));
		Exec.LogClear();
		
		Exec.Stop();

		return true;
	}

protected:

	const FName GlobalTaskName = FName(TEXT("GlobalTask"));
	const FName TaskAName = FName(TEXT("TaskA"));
	
	EStateTreeRunStatus NormalTaskStatus = EStateTreeRunStatus::Succeeded;
	EStateTreeRunStatus NormalTaskEnterStatus = EStateTreeRunStatus::Running;
	int32 NormalTaskTicks = 1;

	EStateTreeRunStatus GlobalTaskStatus = EStateTreeRunStatus::Succeeded;
	EStateTreeRunStatus GlobalTaskEnterStatus = EStateTreeRunStatus::Running;
	int32 GlobalTaskTicks = 1;

	EStateTreeRunStatus ExpectedStatusAfterTick = EStateTreeRunStatus::Succeeded;

	FString ExpectedExitStatusStr = TEXT("ExitSucceeded");
};

struct FStateTreeTest_Stop_NormalSucceeded : FStateTreeTest_Stop
{
	virtual bool SetUp() override
	{
		// Normal task completes as succeeded.
		NormalTaskStatus = EStateTreeRunStatus::Succeeded;
		NormalTaskTicks = 1;

		// Global task completes later
		GlobalTaskTicks = 2;

		// Tree should complete as succeeded.
		ExpectedStatusAfterTick = EStateTreeRunStatus::Succeeded;
		
		// Tasks should have Transition.CurrentRunStatus as succeeded 
		ExpectedExitStatusStr = TEXT("ExitSucceeded");

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Stop_NormalSucceeded, "System.StateTree.Stop.NormalSucceeded");

struct FStateTreeTest_Stop_NormalFailed : FStateTreeTest_Stop
{
	virtual bool SetUp() override
	{
		// Normal task completes as failed.
		NormalTaskStatus = EStateTreeRunStatus::Failed;
		NormalTaskTicks = 1;

		// Global task completes later.
		GlobalTaskTicks = 2;

		// Tree should complete as failed.
		ExpectedStatusAfterTick = EStateTreeRunStatus::Failed;
		
		// Tasks should have Transition.CurrentRunStatus as failed.
		ExpectedExitStatusStr = TEXT("ExitFailed");

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Stop_NormalFailed, "System.StateTree.Stop.NormalFailed");


struct FStateTreeTest_Stop_GlobalSucceeded : FStateTreeTest_Stop
{
	virtual bool SetUp() override
	{
		// Normal task completes later.
		NormalTaskTicks = 2;

		// Global task completes as succeeded.
		GlobalTaskStatus = EStateTreeRunStatus::Succeeded;
		GlobalTaskTicks = 1;

		// Tree should complete as succeeded.
		ExpectedStatusAfterTick = EStateTreeRunStatus::Succeeded;
		
		// Tasks should have Transition.CurrentRunStatus as succeeded.
		ExpectedExitStatusStr = TEXT("ExitSucceeded");

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Stop_GlobalSucceeded, "System.StateTree.Stop.GlobalSucceeded");

struct FStateTreeTest_Stop_GlobalFailed : FStateTreeTest_Stop
{
	virtual bool SetUp() override
	{
		// Normal task completes later
		NormalTaskTicks = 2;

		// Global task completes as failed.
		GlobalTaskStatus = EStateTreeRunStatus::Failed;
		GlobalTaskTicks = 1;

		// Tree should complete as failed.
		ExpectedStatusAfterTick = EStateTreeRunStatus::Failed;
		
		// Tasks should have Transition.CurrentRunStatus as failed.
		ExpectedExitStatusStr = TEXT("ExitFailed");

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Stop_GlobalFailed, "System.StateTree.Stop.GlobalFailed");


//
// Tests combinations of completing the tree on EnterState.
//
struct FStateTreeTest_StopEnterNormal : FStateTreeTest_Stop
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = SetupTree();
		
		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);

		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		const TCHAR* TickStr(TEXT("Tick"));
		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* ExitStateStr(TEXT("ExitState"));

		// If a normal task fails at start, the last tick status will be failed, but transition handling (and final execution status) will take place next tick. 
		Status = Exec.Start();
		AITEST_EQUAL(TEXT("Tree should be running after start"), Status, EStateTreeRunStatus::Running);
		AITEST_EQUAL(TEXT("Last execution status should be expected value"), Exec.GetLastTickStatus(), ExpectedStatusAfterStart);

		// Handles any transitions from failed transition
		Status = Exec.Tick(0.1f);
		AITEST_EQUAL(TEXT("Start should be expected value"), Status, ExpectedStatusAfterStart);
		AITEST_TRUE(TEXT("StateTree GlobalTask should get exit state expectedly"), Exec.Expect(GlobalTaskName, ExpectedExitStatusStr));

		AITEST_TRUE(TEXT("StateTree TaskA should enter state"), Exec.Expect(TaskAName, EnterStateStr));
		AITEST_TRUE(TEXT("StateTree TaskA should report exit status"), Exec.Expect(TaskAName, ExpectedExitStatusStr));

		Exec.Stop();
		
		return true;
	}

	EStateTreeRunStatus ExpectedStatusAfterStart = EStateTreeRunStatus::Succeeded;
	FString ExpectedExitStatusStr = TEXT("ExitSucceeded");
	bool bExpectNormalTaskToRun = true; 
};

struct FStateTreeTest_Stop_NormalEnterSucceeded : FStateTreeTest_StopEnterNormal
{
	virtual bool SetUp() override
	{
		// Tasks should complete later.
		NormalTaskTicks = 2;
		GlobalTaskTicks = 2;

		// Normal task EnterState as succeeded, completion is handled using completion transitions.
		NormalTaskEnterStatus = EStateTreeRunStatus::Succeeded;

		// Tree should complete as succeeded.
		ExpectedStatusAfterStart = EStateTreeRunStatus::Succeeded;
		
		// Tasks should have Transition.CurrentRunStatus as succeeded.
		ExpectedExitStatusStr = TEXT("ExitSucceeded");

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Stop_NormalEnterSucceeded, "System.StateTree.Stop.NormalEnterSucceeded");

struct FStateTreeTest_Stop_NormalEnterFailed : FStateTreeTest_StopEnterNormal
{
	virtual bool SetUp() override
	{
		// Tasks should complete later.
		NormalTaskTicks = 2;
		GlobalTaskTicks = 2;

		// Normal task EnterState as failed, completion is handled using completion transitions.
		NormalTaskEnterStatus = EStateTreeRunStatus::Failed;

		// Tree should complete as failed.
		ExpectedStatusAfterStart = EStateTreeRunStatus::Failed;
		
		// Tasks should have Transition.CurrentRunStatus as failed.
		ExpectedExitStatusStr = TEXT("ExitFailed");

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Stop_NormalEnterFailed, "System.StateTree.Stop.NormalEnterFailed");




struct FStateTreeTest_StopEnterGlobal : FStateTreeTest_Stop
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = SetupTree();
		
		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);

		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		const TCHAR* TickStr(TEXT("Tick"));
		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* ExitStateStr(TEXT("ExitState"));

		Status = Exec.Start();
		AITEST_EQUAL(TEXT("Start should be expected value"), Status, ExpectedStatusAfterStart);
		AITEST_TRUE(TEXT("StateTree GlobalTask should get exit state expectedly"), Exec.Expect(GlobalTaskName, ExpectedExitStatusStr));

		// Normal tasks should not run
		AITEST_FALSE(TEXT("StateTree TaskA should not enter state"), Exec.Expect(TaskAName, EnterStateStr));
		AITEST_FALSE(TEXT("StateTree TaskA should not report exit status"), Exec.Expect(TaskAName, ExpectedExitStatusStr));
		Exec.LogClear();

		Exec.Stop();
		
		return true;
	}

	EStateTreeRunStatus ExpectedStatusAfterStart = EStateTreeRunStatus::Succeeded;
	FString ExpectedExitStatusStr = TEXT("ExitSucceeded");
};

struct FStateTreeTest_Stop_GlobalEnterSucceeded : FStateTreeTest_StopEnterGlobal
{
	virtual bool SetUp() override
	{
		// Tasks should complete later.
		NormalTaskTicks = 2;
		GlobalTaskTicks = 2;

		// Global task EnterState as succeeded, completion is handled directly based on the global task status.
		GlobalTaskEnterStatus = EStateTreeRunStatus::Succeeded;

		// Tree should complete as succeeded.
		ExpectedStatusAfterStart = EStateTreeRunStatus::Succeeded;
		
		// Tasks should have Transition.CurrentRunStatus as Succeeded.
		ExpectedExitStatusStr = TEXT("ExitSucceeded");

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Stop_GlobalEnterSucceeded, "System.StateTree.Stop.GlobalEnterSucceeded");

struct FStateTreeTest_Stop_GlobalEnterFailed : FStateTreeTest_StopEnterGlobal
{
	virtual bool SetUp() override
	{
		// Tasks should complete later.
		NormalTaskTicks = 2;
		GlobalTaskTicks = 2;

		// Global task EnterState as failed, completion is handled directly based on the global task status.
		GlobalTaskEnterStatus = EStateTreeRunStatus::Failed;

		// Tree should complete as failed.
		ExpectedStatusAfterStart = EStateTreeRunStatus::Failed;
		
		// Tasks should have Transition.CurrentRunStatus as failed.
		ExpectedExitStatusStr = TEXT("ExitFailed");

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Stop_GlobalEnterFailed, "System.StateTree.Stop.GlobalEnterFailed");

struct FStateTreeTest_Stop_ExternalStop : FStateTreeTest_Stop
{
	virtual bool SetUp() override
	{
		// Tasks should complete later.
		NormalTaskTicks = 2;
		GlobalTaskTicks = 2;

		// Tree should tick and keep on running.
		ExpectedStatusAfterTick = EStateTreeRunStatus::Running;

		// Tree should stop as stopped.
		ExpectedStatusAfterStop = EStateTreeRunStatus::Stopped;
		
		// Tasks should have Transition.CurrentRunStatus as stopped. 
		ExpectedExitStatusStr = TEXT("ExitStopped");

		return true;
	}
	
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = SetupTree();
		
		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);

		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		const TCHAR* TickStr(TEXT("Tick"));
		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* ExitStateStr(TEXT("ExitState"));

		Status = Exec.Start();
		AITEST_EQUAL(TEXT("Start should be running"), Status, EStateTreeRunStatus::Running);
		AITEST_TRUE(TEXT("StateTree GlobalTask should enter state"), Exec.Expect(GlobalTaskName, EnterStateStr));
		AITEST_TRUE(TEXT("StateTree TaskA should enter state"), Exec.Expect(TaskAName, EnterStateStr));
		Exec.LogClear();

		Status = Exec.Tick(0.1f);
		AITEST_EQUAL(TEXT("Tree should end expectedly"), Status, ExpectedStatusAfterTick);
		Exec.LogClear();

		Status = Exec.Stop(EStateTreeRunStatus::Stopped);
		AITEST_EQUAL(TEXT("Start should be running"), Status, ExpectedStatusAfterStop);
		if (!ExpectedExitStatusStr.IsEmpty())
		{
			AITEST_TRUE(TEXT("StateTree GlobalTask should get exit state expectedly"), Exec.Expect(GlobalTaskName, ExpectedExitStatusStr));
			AITEST_TRUE(TEXT("StateTree TaskA should get exit state expectedly"), Exec.Expect(TaskAName, ExpectedExitStatusStr));
		}
		
		return true;
	}

	EStateTreeRunStatus ExpectedStatusAfterStop = EStateTreeRunStatus::Stopped;
	
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Stop_ExternalStop, "System.StateTree.Stop.ExternalStop");

struct FStateTreeTest_Stop_AlreadyStopped : FStateTreeTest_Stop_ExternalStop
{
	virtual bool SetUp() override
	{
		// Normal task completes before stop.
		NormalTaskTicks = 1;
		NormalTaskStatus = EStateTreeRunStatus::Succeeded;

		// Global task completes later
		GlobalTaskTicks = 2;

		// Tree should tick stop as succeeded.
		ExpectedStatusAfterTick = EStateTreeRunStatus::Succeeded;

		// Tree is already stopped, should keep the status (not Stopped).
		ExpectedStatusAfterStop = EStateTreeRunStatus::Succeeded;
		
		// Skip exit status check.
		ExpectedExitStatusStr = TEXT("");

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Stop_AlreadyStopped, "System.StateTree.Stop.AlreadyStopped");

//
// The deferred stop tests validates that the tree can be properly stopped if requested in the main entry points (Start, Tick, Stop).  
//
struct FStateTreeTest_DeferredStop : FStateTreeTestBase
{
	UStateTree& SetupTree() const
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		UStateTreeState& StateA = Root.AddChildState(FName(TEXT("A")));
		TStateTreeEditorNode<FTestTask_StopTree>& TaskA = StateA.AddTask<FTestTask_StopTree>(TEXT("Task"));
		TStateTreeEditorNode<FTestTask_StopTree>& GlobalTask = EditorData.AddGlobalTask<FTestTask_StopTree>(TEXT("GlobalTask"));

		StateA.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::Succeeded);
		StateA.AddTransition(EStateTreeTransitionTrigger::OnStateFailed, EStateTreeTransitionType::Failed);

		GlobalTask.GetNode().Phase = GlobalTaskPhase;
		TaskA.GetNode().Phase = TaskPhase;

		return StateTree;
	}

	virtual bool RunDerivedTest(FTestStateTreeExecutionContext& Exec) = 0;

	virtual bool InstantTest() override
	{
		UStateTree& StateTree = SetupTree();

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);

		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		return RunDerivedTest(Exec);
	}

protected:

	EStateTreeUpdatePhase GlobalTaskPhase = EStateTreeUpdatePhase::Unset;
	EStateTreeUpdatePhase TaskPhase = EStateTreeUpdatePhase::Unset;
};

struct FStateTreeTest_DeferredStop_EnterGlobalTask : FStateTreeTest_DeferredStop
{
	FStateTreeTest_DeferredStop_EnterGlobalTask() { GlobalTaskPhase = EStateTreeUpdatePhase::EnterStates; }
	virtual bool RunDerivedTest(FTestStateTreeExecutionContext& Exec) override
	{
		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;

		Status = Exec.Start();
		AITEST_EQUAL(TEXT("Tree should be stopped"), Status, EStateTreeRunStatus::Stopped);

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_DeferredStop_EnterGlobalTask, "System.StateTree.DeferredStop.EnterGlobalTask");

struct FStateTreeTest_DeferredStop_TickGlobalTask : FStateTreeTest_DeferredStop
{
	FStateTreeTest_DeferredStop_TickGlobalTask() { GlobalTaskPhase = EStateTreeUpdatePhase::TickStateTree; }
	virtual bool RunDerivedTest(FTestStateTreeExecutionContext& Exec) override
	{
		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;

		Status = Exec.Start();
		AITEST_EQUAL(TEXT("Tree should be running"), Status, EStateTreeRunStatus::Running);

		Status = Exec.Tick(0.1f);
		AITEST_EQUAL(TEXT("Tree should be stopped"), Status, EStateTreeRunStatus::Stopped);

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_DeferredStop_TickGlobalTask, "System.StateTree.DeferredStop.TickGlobalTask");

struct FStateTreeTest_DeferredStop_ExitGlobalTask : FStateTreeTest_DeferredStop
{
	FStateTreeTest_DeferredStop_ExitGlobalTask() { GlobalTaskPhase = EStateTreeUpdatePhase::ExitStates; }
	virtual bool RunDerivedTest(FTestStateTreeExecutionContext& Exec) override
	{
		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;

		Status = Exec.Start();
		AITEST_EQUAL(TEXT("Tree should be running"), Status, EStateTreeRunStatus::Running);

		Status = Exec.Tick(0.1f);
		AITEST_EQUAL(TEXT("Tree should be running"), Status, EStateTreeRunStatus::Running);

		Status = Exec.Stop();
		AITEST_EQUAL(TEXT("Tree should be stopped"), Status, EStateTreeRunStatus::Stopped);

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_DeferredStop_ExitGlobalTask, "System.StateTree.DeferredStop.ExitGlobalTask");

struct FStateTreeTest_DeferredStop_EnterTask : FStateTreeTest_DeferredStop
{
	FStateTreeTest_DeferredStop_EnterTask() { TaskPhase = EStateTreeUpdatePhase::EnterStates; }
	virtual bool RunDerivedTest(FTestStateTreeExecutionContext& Exec) override
	{
		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;

		Status = Exec.Start();
		AITEST_EQUAL(TEXT("Tree should be running"), Status, EStateTreeRunStatus::Stopped);

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_DeferredStop_EnterTask, "System.StateTree.DeferredStop.EnterTask");

struct FStateTreeTest_DeferredStop_TickTask : FStateTreeTest_DeferredStop
{
	FStateTreeTest_DeferredStop_TickTask() { TaskPhase = EStateTreeUpdatePhase::TickStateTree; }
	virtual bool RunDerivedTest(FTestStateTreeExecutionContext& Exec) override
	{
		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;

		Status = Exec.Start();
		AITEST_EQUAL(TEXT("Tree should be running"), Status, EStateTreeRunStatus::Running);

		Status = Exec.Tick(0.1f);
		AITEST_EQUAL(TEXT("Tree should be stopped"), Status, EStateTreeRunStatus::Stopped);

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_DeferredStop_TickTask, "System.StateTree.DeferredStop.TickTask");

struct FStateTreeTest_DeferredStop_ExitTask : FStateTreeTest_DeferredStop
{
	FStateTreeTest_DeferredStop_ExitTask() { TaskPhase = EStateTreeUpdatePhase::ExitStates; }
	virtual bool RunDerivedTest(FTestStateTreeExecutionContext& Exec) override
	{
		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;

		Status = Exec.Start();
		AITEST_EQUAL(TEXT("Tree should be running"), Status, EStateTreeRunStatus::Running);

		Status = Exec.Tick(0.1f);
		AITEST_EQUAL(TEXT("Tree should be running"), Status, EStateTreeRunStatus::Running);

		Status = Exec.Stop();
		AITEST_EQUAL(TEXT("Tree should be stopped"), Status, EStateTreeRunStatus::Stopped);
		
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_DeferredStop_ExitTask, "System.StateTree.DeferredStop.ExitTask");

struct FStateTreeTest_FinishTasks : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		/*
		 - RootA
			- StateA -> StateB
			- StateB -> StateA
		 */
		UStateTreeState& RootA = EditorData.AddSubTree(FName(TEXT("RootA")));
		UStateTreeState& StateA = RootA.AddChildState(FName(TEXT("StateA")));
		UStateTreeState& StateB = RootA.AddChildState(FName(TEXT("StateB")));

		StateA.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &StateB);
		StateB.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &StateA);
		
		TStateTreeEditorNode<FStateTreeTestCondition>& BoolCondB = StateB.AddEnterCondition<FStateTreeTestCondition>();
		BoolCondB.GetNode().bTestConditionResult = false;

		TStateTreeEditorNode<FTestTask_PrintValue>& StateATask = StateA.AddTask<FTestTask_PrintValue>("StateATaskA");
		StateATask.GetInstanceData().Value = 101;
		StateATask.GetNode().CustomTickFunc = [](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
			{
				Context.FinishTask(*Task, EStateTreeFinishTaskType::Succeeded);
				FTestTask_PrintValue::FInstanceDataType& InstanceData = Context.GetInstanceData(*Task);
				++InstanceData.Value;
			};

		// Test one finish
		{
			{
				FStateTreeCompilerLog Log;
				FStateTreeCompiler Compiler(Log);
				const bool bResult = Compiler.Compile(StateTree);
				AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);
			}
			{
				EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
				FStateTreeInstanceData InstanceData;
				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
				const bool bInitSucceeded = Exec.IsValid();
				AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

				// Start and enter state
				Status = Exec.Start();

				AITEST_TRUE(TEXT("StateTree Active States should be in RootA/StateA"),
					Exec.ExpectInActiveStates(RootA.Name, StateA.Name));
				AITEST_TRUE(TEXT("StateTree should be running"), Status == EStateTreeRunStatus::Running);
				Exec.LogClear();

				// On FinishTask, go to StateB but the condition will fail. Reselect a new StateA.
				Status = Exec.Tick(0.1f);
				AITEST_TRUE(TEXT("StateTree Active States should be in RootA/StateA"),
					Exec.ExpectInActiveStates(RootA.Name, StateA.Name));
				AITEST_TRUE(TEXT("StateTree should be running"), Status == EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("Expect the output tasks"), Exec.Expect("StateATaskA", TEXT("Tick101"))
					.Then("StateATaskA", TEXT("Exitstate102"))
					.Then("StateATaskA", TEXT("EnterState101"))
					);
				Exec.LogClear();

				Exec.Stop();
			}
		}
		// Test two finish
		{
			{
				StateATask.GetNode().CustomTickFunc = [](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						Context.FinishTask(*Task, EStateTreeFinishTaskType::Succeeded);
						Context.FinishTask(*Task, EStateTreeFinishTaskType::Succeeded);
						FTestTask_PrintValue::FInstanceDataType& InstanceData = Context.GetInstanceData(*Task);
						++InstanceData.Value;
					};

				FStateTreeCompilerLog Log;
				FStateTreeCompiler Compiler(Log);
				const bool bResult = Compiler.Compile(StateTree);
				AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);
			}
			{
				EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
				FStateTreeInstanceData InstanceData;
				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
				const bool bInitSucceeded = Exec.IsValid();
				AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

				// Start and enter state
				Status = Exec.Start();

				AITEST_TRUE(TEXT("StateTree Active States should be in RootA/StateA"),
					Exec.ExpectInActiveStates(RootA.Name, StateA.Name));
				AITEST_TRUE(TEXT("StateTree should be running"), Status == EStateTreeRunStatus::Running);
				Exec.LogClear();

				// On FinishTask, go to StateB but the condition will fail. Reselect a new StateA.
				Status = Exec.Tick(0.1f);
				AITEST_TRUE(TEXT("StateTree Active States should be in RootA/StateA"),
					Exec.ExpectInActiveStates(RootA.Name, StateA.Name));
				AITEST_TRUE(TEXT("StateTree should be running"), Status == EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("Expect the output tasks"), Exec.Expect("StateATaskA", TEXT("Tick101"))
					.Then("StateATaskA", TEXT("Exitstate102"))
					.Then("StateATaskA", TEXT("EnterState101"))
				);
				Exec.LogClear();

				Exec.Stop();
			}
		}
		// Test finish in exit state
		{
			{
				StateATask.GetNode().CustomTickFunc = [](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						Context.FinishTask(*Task, EStateTreeFinishTaskType::Succeeded);
						FTestTask_PrintValue::FInstanceDataType& InstanceData = Context.GetInstanceData(*Task);
						++InstanceData.Value;
					};
				StateATask.GetNode().CustomExitStateFunc = [](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						Context.FinishTask(*Task, EStateTreeFinishTaskType::Succeeded);
						FTestTask_PrintValue::FInstanceDataType& InstanceData = Context.GetInstanceData(*Task);
						++InstanceData.Value;
					};

				FStateTreeCompilerLog Log;
				FStateTreeCompiler Compiler(Log);
				const bool bResult = Compiler.Compile(StateTree);
				AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);
			}
			{
				EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
				FStateTreeInstanceData InstanceData;
				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
				const bool bInitSucceeded = Exec.IsValid();
				AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

				// Start and enter state
				Status = Exec.Start();

				AITEST_TRUE(TEXT("StateTree Active States should be in RootA/StateA"),
					Exec.ExpectInActiveStates(RootA.Name, StateA.Name));
				AITEST_TRUE(TEXT("StateTree should be running"), Status == EStateTreeRunStatus::Running);
				Exec.LogClear();

				// One FinishTask in exit but should not close StateA again. It should loop to StateA
				Status = Exec.Tick(0.1f);
				AITEST_TRUE(TEXT("StateTree Active States should be in RootA/StateA"),
					Exec.ExpectInActiveStates(RootA.Name, StateA.Name));
				AITEST_TRUE(TEXT("StateTree should be running"), Status == EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("Expect the output tasks"), Exec.Expect("StateATaskA", TEXT("Tick101"))
					.Then("StateATaskA", TEXT("ExitState102"))
					.Then("StateATaskA", TEXT("EnterState101"))
				);
				AITEST_FALSE(TEXT("Expect the output tasks"), Exec.Expect("StateATaskA", TEXT("ExitState103")));
				Exec.LogClear();

				Exec.Stop();
			}
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_FinishTasks, "System.StateTree.FinishTask");

// Test nested tree overrides
struct FStateTreeTest_NestedOverride : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		FStateTreeCompilerLog Log;
		
		const FGameplayTag Tag = GetTestTag1();
		const FGameplayTag Tag2 = GetTestTag2();

		// Asset 2
		UStateTree& StateTree2 = NewStateTree();
		UStateTreeEditorData& EditorData2 = *Cast<UStateTreeEditorData>(StateTree2.EditorData);
		FInstancedPropertyBag& RootPropertyBag2 = GetRootPropertyBag(EditorData2);
		RootPropertyBag2.AddProperty(FName(TEXT("Int")), EPropertyBagPropertyType::Int32);
		UStateTreeState& Root2 = EditorData2.AddSubTree(FName(TEXT("Root2")));
		TStateTreeEditorNode<FTestTask_Stand>& TaskRoot2 = Root2.AddTask<FTestTask_Stand>(FName(TEXT("TaskRoot2")));
		{
			FStateTreeCompiler Compiler2(Log);
			const bool bResult2 = Compiler2.Compile(StateTree2);
			AITEST_TRUE("StateTree2 should get compiled", bResult2);
		}
		
		// Asset 3
		UStateTree& StateTree3 = NewStateTree();
		UStateTreeEditorData& EditorData3 = *Cast<UStateTreeEditorData>(StateTree3.EditorData);
		FInstancedPropertyBag& RootPropertyBag3 = GetRootPropertyBag(EditorData3);
		RootPropertyBag3.AddProperty(FName(TEXT("Float")), EPropertyBagPropertyType::Float); // Different parameters
		UStateTreeState& Root3 = EditorData3.AddSubTree(FName(TEXT("Root3")));
		TStateTreeEditorNode<FTestTask_Stand>& TaskRoot3 = Root3.AddTask<FTestTask_Stand>(FName(TEXT("TaskRoot3")));
		{
			FStateTreeCompiler Compiler3(Log);
			const bool bResult3 = Compiler3.Compile(StateTree3);
			AITEST_TRUE(TEXT("StateTree3 should get compiled"), bResult3);
		}
		// Wrong Asset 4
		UStateTree* StateTree4 = NewObject<UStateTree>(&GetWorld());;
		{
			UStateTreeEditorData* EditorData = NewObject<UStateTreeEditorData>(StateTree4);
			check(EditorData);
			StateTree4->EditorData = EditorData;
			EditorData->Schema = NewObject<UStateTreeTestSchema2>();

			FInstancedPropertyBag& RootPropertyBag = GetRootPropertyBag(*EditorData);
			RootPropertyBag.AddProperty(FName(TEXT("Float")), EPropertyBagPropertyType::Float); // Different parameters
			UStateTreeState& Root4 = EditorData->AddSubTree(FName(TEXT("Root4")));
			TStateTreeEditorNode<FTestTask_Stand>& TaskRoot4 = Root3.AddTask<FTestTask_Stand>(FName(TEXT("TaskRoot4")));

			FStateTreeCompiler Compiler4(Log);
			const bool bResult4 = Compiler4.Compile(*StateTree4);
			AITEST_TRUE(TEXT("StateTree4 should get compiled"), bResult4);
		}

		// Main asset
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		FInstancedPropertyBag& RootPropertyBag = GetRootPropertyBag(EditorData);

		RootPropertyBag.AddProperty(FName(TEXT("Int")), EPropertyBagPropertyType::Int32);
		
		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root1")));
		UStateTreeState& StateA = Root.AddChildState(FName(TEXT("A1")), EStateTreeStateType::LinkedAsset);
		StateA.Tag = Tag;
		StateA.SetLinkedStateAsset(&StateTree2);

		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		const TCHAR* TickStr(TEXT("Tick"));
		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* ExitStateStr(TEXT("ExitState"));

		// Without overrides
		{
			EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

			Status = Exec.Start();
			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("StateTree should enter TaskRoot2"), Exec.Expect(TaskRoot2.GetName(), EnterStateStr));

			Exec.Stop();
		}

		// With overrides
		{
			EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
			FStateTreeInstanceData InstanceData;

			FStateTreeReferenceOverrides Overrides;
			FStateTreeReference OverrideRef;
			OverrideRef.SetStateTree(&StateTree3);
			Overrides.AddOverride(Tag, OverrideRef);
			
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
			Exec.SetLinkedStateTreeOverrides(Overrides);
			
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE("StateTree should init", bInitSucceeded);
			
			Status = Exec.Start();
			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("StateTree should enter TaskRoot3"), Exec.Expect(TaskRoot3.GetName(), EnterStateStr));
			AITEST_FALSE(TEXT("StateTree should not enter TaskRoot2"), Exec.Expect(TaskRoot2.GetName(), EnterStateStr));

			Exec.Stop();
		}

		// With wrong overrides
		{
			EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
			FStateTreeInstanceData InstanceData;

			FStateTreeReferenceOverrides Overrides;
			FStateTreeReference OverrideRef3;
			OverrideRef3.SetStateTree(&StateTree3);
			Overrides.AddOverride(Tag, OverrideRef3);
			FStateTreeReference OverrideRef4;
			OverrideRef4.SetStateTree(StateTree4);
			Overrides.AddOverride(Tag2, OverrideRef4);

			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);

			GetTestRunner().AddExpectedMessage(TEXT("their schemas don't match"), ELogVerbosity::Error, EAutomationExpectedMessageFlags::Contains, 1, false);
			Exec.SetLinkedStateTreeOverrides(Overrides);
			AITEST_TRUE(TEXT("Start should complete with Running"), GetTestRunner().HasMetExpectedErrors());

			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

			Status = Exec.Start();
			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("StateTree should enter TaskRoot2"), Exec.Expect(TaskRoot2.GetName(), EnterStateStr));
			AITEST_FALSE(TEXT("StateTree should not enter TaskRoot3"), Exec.Expect(TaskRoot3.GetName(), EnterStateStr));

			Exec.Stop();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_NestedOverride, "System.StateTree.NestedOverride");

struct FStateTreeTest_SubTreeTransition : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		/*
		- Root
			- PreLastStand [Task1] -> Reinforcements
				- BusinessAsUsual [Task2]
			- LastStand [Task3]
				- Reinforcements>TimeoutChecker
			- (f)TimeoutChecker
				- RemainingCount [Task4]
		*/
		
		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));

		UStateTreeState& PreLastStand = Root.AddChildState(FName(TEXT("PreLastStand")));
		UStateTreeState& BusinessAsUsual = PreLastStand.AddChildState(FName(TEXT("BusinessAsUsual")));

		UStateTreeState& LastStand = Root.AddChildState(FName(TEXT("LastStand")));
		UStateTreeState& Reinforcements = LastStand.AddChildState(FName(TEXT("Reinforcements")), EStateTreeStateType::Linked);
		
		UStateTreeState& TimeoutChecker = LastStand.AddChildState(FName(TEXT("TimeoutChecker")), EStateTreeStateType::Subtree);
		UStateTreeState& RemainingCount = TimeoutChecker.AddChildState(FName(TEXT("RemainingCount")));

		Reinforcements.SetLinkedState(TimeoutChecker.GetLinkToState());


		TStateTreeEditorNode<FTestTask_Stand>& Task1 = PreLastStand.AddTask<FTestTask_Stand>(FName(TEXT("Task1")));
		PreLastStand.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &Reinforcements);
		Task1.GetInstanceData().Value = 1; // This should finish before the child state

		TStateTreeEditorNode<FTestTask_Stand>& Task2 = BusinessAsUsual.AddTask<FTestTask_Stand>(FName(TEXT("Task2")));
		Task2.GetInstanceData().Value = 2;

		TStateTreeEditorNode<FTestTask_Stand>& Task3 = LastStand.AddTask<FTestTask_Stand>(FName(TEXT("Task3")));
		Task3.GetInstanceData().Value = 2;

		TStateTreeEditorNode<FTestTask_Stand>& Task4 = LastStand.AddTask<FTestTask_Stand>(FName(TEXT("Task4")));
		Task4.GetInstanceData().Value = 2;

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		const TCHAR* TickStr(TEXT("Tick"));
		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* ExitStateStr(TEXT("ExitState"));
		const TCHAR* StateCompletedStr(TEXT("StateCompleted"));

		// Start and enter state
		Status = Exec.Start();

		AITEST_TRUE(TEXT("StateTree Active States should be in Root/PreLastStand/BusinessAsUsual"), Exec.ExpectInActiveStates(Root.Name, PreLastStand.Name, BusinessAsUsual.Name));
		AITEST_TRUE(TEXT("StateTree Task1 should enter state"), Exec.Expect(Task1.GetName(), EnterStateStr));
		AITEST_TRUE(TEXT("StateTree Task2 should enter state"), Exec.Expect(Task2.GetName(), EnterStateStr));
		AITEST_TRUE(TEXT("StateTree should be running"), Status == EStateTreeRunStatus::Running);
		Exec.LogClear();

		// Transition to Reinforcements
		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Active States should be in Root/LastStand/Reinforcements/TimeoutChecker/RemainingCount"), Exec.ExpectInActiveStates(Root.Name, LastStand.Name, Reinforcements.Name, TimeoutChecker.Name, RemainingCount.Name));
		AITEST_TRUE(TEXT("StateTree Task3 should enter state"), Exec.Expect(Task3.GetName(), EnterStateStr));
		AITEST_TRUE(TEXT("StateTree Task4 should enter state"), Exec.Expect(Task4.GetName(), EnterStateStr));
		AITEST_TRUE(TEXT("StateTree should be running"), Status == EStateTreeRunStatus::Running);
		Exec.LogClear();

		Exec.Stop();
		
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_SubTreeTransition, "System.StateTree.SubTreeTransition");

struct FStateTreeTest_Reentrant : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		//Tree1
		//	-Root
		//		- State1
		//			- State2: OnComplete -> State1

		UStateTree& StateTree1 = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree1.EditorData);
			UStateTreeState& Root1 = EditorData.AddSubTree(FName(TEXT("Tree1Root")));
			UStateTreeState& State1 = Root1.AddChildState(FName(TEXT("Tree1State1")));
			UStateTreeState& State2 = State1.AddChildState(FName(TEXT("Tree1State2")));

			State2.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &State1);
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& State1Task1 = State1.AddTask<FTestTask_PrintValue>("Tree1State1Task1");
				State1Task1.GetInstanceData().Value = 101;
				State1Task1.GetInstanceData().TickRunStatus = EStateTreeRunStatus::Running;

			}
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& State2Task1 = State2.AddTask<FTestTask_PrintValue>("Tree1State2Task1");
				State2Task1.GetInstanceData().Value = 201;
				State2Task1.GetInstanceData().TickRunStatus = EStateTreeRunStatus::Succeeded;
			}
			{
				FStateTreeCompilerLog Log;
				FStateTreeCompiler Compiler(Log);
				const bool bResult = Compiler.Compile(StateTree1);
				AITEST_TRUE("StateTree should get compiled", bResult);
			}
		}
		{
			FStateTreeInstanceData InstanceData;
			{
				FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
				const bool bInitSucceeded = Exec.IsValid();
				AITEST_TRUE("StateTree should init", bInitSucceeded);

				// Start and enter state
				EStateTreeRunStatus Status = Exec.Start();
				AITEST_TRUE("StateTree Active States should be in Tree1Root/Tree1State1/Tree1State2",
					Exec.ExpectInActiveStates("Tree1Root", "Tree1State1", "Tree1State2"));
				AITEST_TRUE("StateTree should be running", Status == EStateTreeRunStatus::Running);
				Exec.LogClear();
			}
			{
				// On go to State1, reselect State1 and State2. It's a new State2 same State1.
				FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);

				UE::StateTree::FActiveStatePath BeforeStatePath = InstanceData.GetExecutionState()->GetActiveStatePath();

				EStateTreeRunStatus Status = Exec.Tick(0.1f);
				AITEST_TRUE("StateTree Active States should be in Tree1Root/Tree1State1/Tree1State2",
					Exec.ExpectInActiveStates("Tree1Root", "Tree1State1", "Tree1State2"));
				AITEST_TRUE("StateTree should be running", Status == EStateTreeRunStatus::Running);
				AITEST_TRUE("Expect the output tasks", Exec.Expect("Tree1State1Task1", TEXT("Tick101"))
					.Then("Tree1State2Task1", TEXT("Tick201"))
					.Then("Tree1State2Task1", TEXT("ExitState201"))
					.Then("Tree1State2Task1", TEXT("ExitState=Changed"))
					.Then("Tree1State1Task1", TEXT("ExitState101"))
					.Then("Tree1State1Task1", TEXT("ExitState=Sustained"))
					.Then("Tree1State1Task1", TEXT("EnterState101"))
					.Then("Tree1State1Task1", TEXT("EnterState=Sustained"))
					.Then("Tree1State2Task1", TEXT("EnterState201"))
					.Then("Tree1State2Task1", TEXT("EnterState=Changed"))
				);


				UE::StateTree::FActiveStatePath AfterStatePath = InstanceData.GetExecutionState()->GetActiveStatePath();

				AITEST_TRUE(TEXT("Same length"), AfterStatePath.Num() == BeforeStatePath.Num());
				AITEST_TRUE(TEXT("Length is 3"), AfterStatePath.Num() == 3);
				AITEST_TRUE(TEXT("Element 1 is the same"), AfterStatePath.GetView()[0] == BeforeStatePath.GetView()[0]);
				AITEST_TRUE(TEXT("Element 2 is the same"), AfterStatePath.GetView()[1] == BeforeStatePath.GetView()[1]);
				AITEST_TRUE(TEXT("Element 3 is different"), AfterStatePath.GetView()[2] != BeforeStatePath.GetView()[2]);

				Exec.LogClear();
			}
			{
				FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
				Exec.Stop();
			}
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Reentrant, "System.StateTree.Reentrant");

struct FStateTreeTest_OverrideStartState : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		//Tree
		//	Root1
		//    Child1
		//    Child2
		//      Child3
		//    Subtree1
		//  Root2
		//	  Child4
		//  Root3 SelectionBehavior: Utility
		//    Child5 score 0.5
		//    Child6 score 1
		UStateTree& StateTree = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
			UStateTreeState& Root1 = EditorData.AddSubTree(TEXT("Root1"));

			UStateTreeState& Child1 = Root1.AddChildState(TEXT("Child1"));

			UStateTreeState& Child2 = Root1.AddChildState(TEXT("Child2"));
			Child2.Tag = GetTestTag1();
			
			UStateTreeState& Child3 = Child2.AddChildState(TEXT("Child3"));
			Child3.Tag = GetTestTag2();

			UStateTreeState& Subtree1 = Root1.AddChildState(TEXT("Subtree1"));
			Subtree1.Type = EStateTreeStateType::Subtree;
			Subtree1.Tag = GetTestTag3();

			UStateTreeState& Root2 = EditorData.AddSubTree(TEXT("Root2"));
			Root2.Tag = GetTestTag4();
			UStateTreeState& Child4 = Root2.AddChildState(TEXT("Child4"));

			UStateTreeState& Root3 = EditorData.AddSubTree(TEXT("Root4"));
			Root3.SelectionBehavior = EStateTreeStateSelectionBehavior::TrySelectChildrenWithHighestUtility;
			{
				UStateTreeState& Child5 = Root3.AddChildState(TEXT("Child6"));
				Child5.Tag = GetTestTag5();
				TStateTreeEditorNode<FStateTreeConstantConsideration>& Child5ConsiderationEditorNode = Child5.AddConsideration<FStateTreeConstantConsideration>();
				Child5ConsiderationEditorNode.GetInstanceData().Constant = 0.5f;
			}
			{
				UStateTreeState& Child6 = Root3.AddChildState(TEXT("Child7"));
				TStateTreeEditorNode<FStateTreeConstantConsideration>& Child6ConsiderationEditorNode = Child6.AddConsideration<FStateTreeConstantConsideration>();
				Child6ConsiderationEditorNode.GetInstanceData().Constant = 1.f;
			}
		}

		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE("StateTree should get compiled", bResult);
		}

		{
			FStateTreeInstanceData InstanceData;
			
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE("StateTree should init", bInitSucceeded);
			
			// No start state override
			{
				EStateTreeRunStatus Status = Exec.Start();
				AITEST_TRUE("StateTree Active States should be in Root1/Child1",
					Exec.ExpectInActiveStates("Root1", "Child1"));
				AITEST_TRUE("StateTree should be running", Status == EStateTreeRunStatus::Running);
				
				Exec.LogClear();
			}

			// Start state Child2
			{
				FStateTreeExecutionContext::FStartParameters::FStateToSelectOverrideArgs SelectStateOverrideArgs =
				{ GetTestTag1(), UStateTree::EStateGameplayTagQueryMethod::MatchesExact };

				EStateTreeRunStatus Status = Exec.Start(FStateTreeExecutionContext::FStartParameters{ .SelectStateOverrideArgs = SelectStateOverrideArgs });

				AITEST_TRUE("StateTree Active States should be in Root1/Child2/Child3 with Child2 override",
					Exec.ExpectInActiveStates("Root1", "Child2", "Child3"));
				AITEST_TRUE("StateTree should be running", Status == EStateTreeRunStatus::Running);
				Exec.LogClear();
			}

			// Start state Child3
			{
				FStateTreeExecutionContext::FStartParameters::FStateToSelectOverrideArgs SelectStateOverrideArgs =
				{ GetTestTag2(), UStateTree::EStateGameplayTagQueryMethod::MatchesExact };

				EStateTreeRunStatus Status = Exec.Start(FStateTreeExecutionContext::FStartParameters{ .SelectStateOverrideArgs = SelectStateOverrideArgs });

				AITEST_TRUE("StateTree Active States should be in Root1/Child2/Child3 with Child3 override",
					Exec.ExpectInActiveStates("Root1", "Child2", "Child3"));
				AITEST_TRUE("StateTree should be running", Status == EStateTreeRunStatus::Running);
				Exec.LogClear();
			}

			// Start state Subtree1
			{
				FStateTreeExecutionContext::FStartParameters::FStateToSelectOverrideArgs SelectStateOverrideArgs =
				{ GetTestTag3(), UStateTree::EStateGameplayTagQueryMethod::MatchesExact };
				
				EStateTreeRunStatus Status = Exec.Start(FStateTreeExecutionContext::FStartParameters{ .SelectStateOverrideArgs = SelectStateOverrideArgs });

				AITEST_TRUE("StateTree Active States should be in Subtree1 with Subtree1 override",
					Exec.ExpectInActiveStates("Subtree1"));
				AITEST_TRUE("StateTree should be running", Status == EStateTreeRunStatus::Running);
				Exec.LogClear();
			}

			// Start state Root2
			{
				FStateTreeExecutionContext::FStartParameters::FStateToSelectOverrideArgs SelectStateOverrideArgs =
				{ GetTestTag4(), UStateTree::EStateGameplayTagQueryMethod::MatchesExact };
				
				EStateTreeRunStatus Status = Exec.Start(FStateTreeExecutionContext::FStartParameters{ .SelectStateOverrideArgs = SelectStateOverrideArgs });

				AITEST_TRUE("StateTree Active States should be in Root2/Child4 with Root2 override",
					Exec.ExpectInActiveStates("Root2", "Child4"));
				AITEST_TRUE("StateTree should be running", Status == EStateTreeRunStatus::Running);
				Exec.LogClear();
			}

			// Start state Child6
			{
				FStateTreeExecutionContext::FStartParameters::FStateToSelectOverrideArgs SelectStateOverrideArgs =
				{ GetTestTag5(), UStateTree::EStateGameplayTagQueryMethod::MatchesExact };
				
				EStateTreeRunStatus Status = Exec.Start(FStateTreeExecutionContext::FStartParameters{ .SelectStateOverrideArgs = SelectStateOverrideArgs });

				AITEST_TRUE("StateTree Active States should be in Root4/Child6 with Child6 override",
					Exec.ExpectInActiveStates("Root4", "Child6"));
				AITEST_TRUE("StateTree should be running", Status == EStateTreeRunStatus::Running);
				Exec.LogClear();
			}

			Exec.Stop();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_OverrideStartState, "System.StateTree.OverrideStartState");

} // namespace UE::StateTree::Tests

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE

