// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeSchema.h"
#include "StateTreeTypes.h"
#include "StateTreeTest.generated.h"

UCLASS(HideDropdown)
class UStateTreeTestSchema : public UStateTreeSchema
{
	GENERATED_BODY()

public:
	virtual bool IsStructAllowed(const UScriptStruct* InScriptStruct) const override
	{
		return true;
	}
	virtual bool IsExternalItemAllowed(const UStruct& InStruct) const override
	{
		return true;
	}
	virtual bool IsScheduledTickAllowed() const
	{
		return true;
	}
	virtual EStateTreeStateSelectionRules GetStateSelectionRules() const
	{
		return DefaultRules;
	}
	void SetStateSelectionRules(EStateTreeStateSelectionRules Rules)
	{
		DefaultRules = Rules;
	}

private:
	UPROPERTY()
	EStateTreeStateSelectionRules DefaultRules = EStateTreeStateSelectionRules::Default;
};

UCLASS(HideDropdown)
class UStateTreeTestSchema2 : public UStateTreeSchema
{
	GENERATED_BODY()
};

#define IMPLEMENT_STATE_TREE_INSTANT_TEST(TestClass, PrettyName) \
	IMPLEMENT_AI_INSTANT_TEST_WITH_FLAGS(TestClass, PrettyName, EAutomationTestFlags::ClientContext | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::SupportsAutoRTFM)
