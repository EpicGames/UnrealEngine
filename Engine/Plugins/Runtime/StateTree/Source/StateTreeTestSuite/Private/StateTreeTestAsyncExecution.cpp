// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeTest.h"
#include "StateTreeTestBase.h"
#include "StateTreeTestTypes.h"

#include "Misc/ScopedCVar.h"
#include "StateTreeCompilerLog.h"
#include "StateTreeEditorData.h"
#include "StateTreeCompiler.h"
#include "StateTreeAsyncExecutionContext.h"

#define LOCTEXT_NAMESPACE "AITestSuite_StateTreeAsyncExecution"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::StateTree::Tests
{

/**
 * Test: Creating a FStateTreeStrongExecutionContext from a valid WeakContext yields a valid strong context.
 *
 * Tree layout:
 *   Root
 *     State1 : Task (PrintValue)
 */
struct FStateTreeTest_AsyncExec_StrongContextFromWeakValid : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree("Root");
		UStateTreeState& State1 = Root.AddChildState("State1");

		FStateTreeWeakExecutionContext CapturedWeakContext;

		auto& Task1 = State1.AddTask<FTestTask_PrintValue>("Task1");
		Task1.GetNode().CustomEnterStateFunc = [&CapturedWeakContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
			{
				CapturedWeakContext = Context.MakeWeakExecutionContext();
			};
		Task1.GetInstanceData().Value = 99;
		State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		EStateTreeRunStatus Status = Exec.Start();
		AITEST_EQUAL(TEXT("Start should be running"), Status, EStateTreeRunStatus::Running);

		// The weak context should have been captured during EnterState
		FStateTreeStrongExecutionContext StrongContext = CapturedWeakContext.MakeStrongExecutionContext();
		AITEST_TRUE(TEXT("Strong context created from valid weak context should be valid"), StrongContext.IsValid());

		// Verify we can read instance data through the strong context
		const FTestTask_PrintValue::FInstanceDataType* DataPtr = StrongContext.GetInstanceDataPtr<const FTestTask_PrintValue::FInstanceDataType>();
		AITEST_NOT_NULL(TEXT("Instance data should be accessible through strong context"), DataPtr);
		AITEST_EQUAL(TEXT("Instance data should be the samel"), DataPtr->Value, 99);

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_AsyncExec_StrongContextFromWeakValid, "System.StateTree.AsyncExecution.StrongContextFromWeakValid");


/**
 * Test: A strong context created after the owning state has been exited should be invalid
 * (the state is no longer active).
 *
 * Tree layout:
 *   Root
 *     State1 : Task (Stand, completes after 1 tick) -> NextState
 *     State2 : Task (Stand)
 */
struct FStateTreeTest_AsyncExec_StrongContextInvalidAfterStateExit : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree("Root");
		UStateTreeState& State1 = Root.AddChildState("State1");
		UStateTreeState& State2 = Root.AddChildState("State2");

		FStateTreeWeakExecutionContext CapturedWeakContext;

		auto& Task1 = State1.AddTask<FTestTask_PrintValue>("Task1");
		Task1.GetNode().CustomEnterStateFunc = [&CapturedWeakContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
			{
				CapturedWeakContext = Context.MakeWeakExecutionContext();
			};
		Task1.GetInstanceData().TickRunStatus = EStateTreeRunStatus::Succeeded;
		State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::NextState);

		auto& Task2 = State2.AddTask<FTestTask_Stand>("Task2");
		Task2.GetNode().TicksToCompletion = 100;
		State2.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE(TEXT("StateTree should init"), Exec.IsValid());

		EStateTreeRunStatus Status = Exec.Start();
		AITEST_EQUAL(TEXT("Start should be running"), Status, EStateTreeRunStatus::Running);

		// After Start, we're in State1. Weak context was captured.
		{
			FStateTreeStrongExecutionContext StrongExecContext = CapturedWeakContext.MakeStrongExecutionContext();
			AITEST_TRUE(TEXT("Strong context should be valid while State1 is active"), StrongExecContext.IsValid());
		}

		// Tick causes State1 to complete -> transitions to State2
		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("Tick should be running (in State2)"), Exec.ExpectInActiveStates("Root", "State2"));

		// Now State1 is exited. The weak context from State1 should produce an invalid strong context.
		{
			FStateTreeStrongExecutionContext StrongExecContext = CapturedWeakContext.MakeStrongExecutionContext();
			AITEST_FALSE(TEXT("Strong context should be invalid after state exit"), StrongExecContext.IsValid());
		}

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_AsyncExec_StrongContextInvalidAfterStateExit, "System.StateTree.AsyncExecution.StrongContextInvalidAfterStateExit");


/**
 * Test: Read-only strong context can read data but the template is instantiated as read-only.
 *
 * Tree layout:
 *   Root
 *     State1 : Task (PrintValue)
 */
struct FStateTreeTest_AsyncExec_ReadOnlyStrongContext : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree("Root");
		UStateTreeState& State1 = Root.AddChildState("State1");

		FStateTreeWeakExecutionContext CapturedWeakContext;

		auto& Task1 = State1.AddTask<FTestTask_PrintValue>("Task1");
		Task1.GetInstanceData().Value = 99;
		Task1.GetNode().CustomEnterStateFunc = [&CapturedWeakContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
			{
				CapturedWeakContext = Context.MakeWeakExecutionContext();
			};

		State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE(TEXT("StateTree should init"), Exec.IsValid());

		Exec.Start();

		// Create read-only strong context
		FStateTreeStrongReadOnlyExecutionContext ReadOnlyCtx = CapturedWeakContext.MakeStrongReadOnlyExecutionContext();
		AITEST_TRUE(TEXT("Read-only strong context should be valid"), ReadOnlyCtx.IsValid());

		// Read instance data
		const FTestTask_PrintValue::FInstanceDataType* DataPtr = ReadOnlyCtx.GetInstanceDataPtr<const FTestTask_PrintValue::FInstanceDataType>();
		AITEST_NOT_NULL(TEXT("Should be able to read instance data via read-only context"), DataPtr);
		AITEST_EQUAL(TEXT("Instance data should be the samel"), DataPtr->Value, 99);

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_AsyncExec_ReadOnlyStrongContext, "System.StateTree.AsyncExecution.ReadOnlyStrongContext");


/**
 * Test: Writing instance data through a strong write-access context.
 *
 * Tree layout:
 *   Root
 *     State1 : Task (PrintValue) - we write to the instance data via strong context
 */
struct FStateTreeTest_AsyncExec_WriteInstanceData : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree("Root");
		UStateTreeState& State1 = Root.AddChildState("State1");

		FStateTreeWeakExecutionContext CapturedWeakContext;
		int32 TickValueObserved = INDEX_NONE;

		auto& Task1 = State1.AddTask<FTestTask_PrintValue>("Task1");
		Task1.GetInstanceData().Value = 99;
		Task1.GetNode().CustomEnterStateFunc = [&CapturedWeakContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
			{
				CapturedWeakContext = Context.MakeWeakExecutionContext();
			};
		Task1.GetNode().CustomTickFunc = [&TickValueObserved](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
			{
				const FTestTask_PrintValue::FInstanceDataType& InstanceData = Context.GetInstanceData(*Task);
				TickValueObserved = InstanceData.Value;
			};

		State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE(TEXT("StateTree should init"), Exec.IsValid());

		Exec.Start();

		// Write instance data through strong write context
		{
			FStateTreeStrongExecutionContext StrongExecContext = CapturedWeakContext.MakeStrongExecutionContext();
			AITEST_TRUE(TEXT("Strong write context should be valid"), StrongExecContext.IsValid());

			FTestTask_PrintValue::FInstanceDataType* DataPtr = StrongExecContext.GetInstanceDataPtr<FTestTask_PrintValue::FInstanceDataType>();
			AITEST_NOT_NULL(TEXT("Should be able to get mutable instance data"), DataPtr);
			AITEST_EQUAL(TEXT("Instance data should be the samel"), DataPtr->Value, 99);

			DataPtr->Value = 888;
		}

		// Tick - the task should observe the modified value
		Exec.Tick(0.1f);
		AITEST_EQUAL(TEXT("Task should see the value written via strong context"), TickValueObserved, 888);

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_AsyncExec_WriteInstanceData, "System.StateTree.AsyncExecution.WriteInstanceData");


/**
 * Test: SendEvent through the weak execution context.
 * We send an event from outside tick processing via WeakContext; it should be buffered
 * and handled on the next tick's transition processing.
 *
 * Tree layout:
 *   Root
 *     State1 : Task (PrintValue) -- event-triggered transition to State2
 *     State2 : Task (Stand)
 */
struct FStateTreeTest_AsyncExec_SendEventViaWeakContext : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree("Root");
		UStateTreeState& State1 = Root.AddChildState("State1");
		UStateTreeState& State2 = Root.AddChildState("State2");

		FStateTreeWeakExecutionContext CapturedWeakContext;

		auto& Task1 = State1.AddTask<FTestTask_PrintValue>("Task1");
		Task1.GetNode().CustomEnterStateFunc = [&CapturedWeakContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
			{
				CapturedWeakContext = Context.MakeWeakExecutionContext();
			};

		// Add event-based transition from State1 to State2
		const FGameplayTag TestEventTag = GetTestTag1();
		State1.AddTransition(EStateTreeTransitionTrigger::OnEvent, EStateTreeTransitionType::GotoState, &State2);
		State1.Transitions.Last().RequiredEvent.Tag = TestEventTag;

		auto& Task2 = State2.AddTask<FTestTask_Stand>("Task2");
		Task2.GetNode().TicksToCompletion = 100;
		State2.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE(TEXT("StateTree should init"), Exec.IsValid());

		EStateTreeRunStatus Status = Exec.Start();
		AITEST_EQUAL(TEXT("Start should be running"), Status, EStateTreeRunStatus::Running);
		AITEST_TRUE(TEXT("Should be in State1"), Exec.ExpectInActiveStates(TEXT("Root"), TEXT("State1")));

		// Send event via weak context (outside of tick)
		const bool bEventSent = CapturedWeakContext.SendEvent(TestEventTag);
		AITEST_TRUE(TEXT("SendEvent via weak context should succeed"), bEventSent);

		// Tick should process the buffered event and transition to State2
		Status = Exec.Tick(0.1f);
		AITEST_EQUAL(TEXT("Should still be running"), Status, EStateTreeRunStatus::Running);
		AITEST_TRUE(TEXT("Should have transitioned to State2"), Exec.ExpectInActiveStates(TEXT("Root"), TEXT("State2")));

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_AsyncExec_SendEventViaWeakContext, "System.StateTree.AsyncExecution.SendEventViaWeakContext");


/**
 * Test: FinishTask through a weak context from outside tick processing.
 * When FinishTask is called outside the tick, it should be buffered and result in
 * state completion on the next tick.
 *
 * Tree layout:
 *   Root
 *     State1 : Task (PrintValue, never completes on its own) -> Succeeded
 */
struct FStateTreeTest_AsyncExec_FinishTaskViaWeakContext : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree("Root");
		UStateTreeState& State1 = Root.AddChildState("State1");

		FStateTreeWeakExecutionContext CapturedWeakContext;

		auto& Task1 = State1.AddTask<FTestTask_PrintValue>("Task1");
		// Task never completes on its own (stays Running)
		Task1.GetInstanceData().TickRunStatus = EStateTreeRunStatus::Running;
		Task1.GetNode().CustomEnterStateFunc = [&CapturedWeakContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
			{
				CapturedWeakContext = Context.MakeWeakExecutionContext();
			};

		State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE(TEXT("StateTree should init"), Exec.IsValid());

		EStateTreeRunStatus Status = Exec.Start();
		AITEST_EQUAL(TEXT("Start should be running"), Status, EStateTreeRunStatus::Running);

		// Tick once - should stay running since task never completes
		Status = Exec.Tick(0.1f);
		AITEST_EQUAL(TEXT("Tick should stay running"), Status, EStateTreeRunStatus::Running);

		// Now finish the task from outside tick via weak context
		const bool bFinished = CapturedWeakContext.FinishTask(EStateTreeFinishTaskType::Succeeded);
		AITEST_TRUE(TEXT("FinishTask via weak context should succeed"), bFinished);

		// Next tick should process the completion
		Status = Exec.Tick(0.1f);
		AITEST_EQUAL(TEXT("Tree should have completed after FinishTask"), Status, EStateTreeRunStatus::Succeeded);

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_AsyncExec_FinishTaskViaWeakContext, "System.StateTree.AsyncExecution.FinishTaskViaWeakContext");


/**
 * Test: FinishTask with Failed status through a weak context.
 *
 * Tree layout:
 *   Root
 *     State1 : Task (PrintValue, never completes on its own) -> Failed on Failed
 */
struct FStateTreeTest_AsyncExec_FinishTaskFailed : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree("Root");
		UStateTreeState& State1 = Root.AddChildState("State1");

		FStateTreeWeakExecutionContext CapturedWeakContext;

		auto& Task1 = State1.AddTask<FTestTask_PrintValue>("Task1");
		Task1.GetInstanceData().TickRunStatus = EStateTreeRunStatus::Running;
		Task1.GetNode().CustomEnterStateFunc = [&CapturedWeakContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
			{
				CapturedWeakContext = Context.MakeWeakExecutionContext();
			};

		State1.AddTransition(EStateTreeTransitionTrigger::OnStateSucceeded, EStateTreeTransitionType::Succeeded);
		State1.AddTransition(EStateTreeTransitionTrigger::OnStateFailed, EStateTreeTransitionType::Failed);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE(TEXT("StateTree should init"), Exec.IsValid());

		EStateTreeRunStatus Status = Exec.Start();
		AITEST_EQUAL(TEXT("Start should be running"), Status, EStateTreeRunStatus::Running);

		// Finish with Failed from outside tick
		const bool bFinished = CapturedWeakContext.FinishTask(EStateTreeFinishTaskType::Failed);
		AITEST_TRUE(TEXT("FinishTask(Failed) via weak context should succeed"), bFinished);

		// Next tick should process the failure
		Status = Exec.Tick(0.1f);
		AITEST_EQUAL(TEXT("Tree should have failed after FinishTask(Failed)"), Status, EStateTreeRunStatus::Failed);

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_AsyncExec_FinishTaskFailed, "System.StateTree.AsyncExecution.FinishTaskFailed");


/**
 * Test: Delegate broadcast/bind through normal tick path to verify the pattern works
 * with the test infrastructure.
 *
 * Tree layout:
 *   Global: Task (BroadcastDelegate)
 *   Root
 *     State1 : Task (ListenDelegate)
 */
struct FStateTreeTest_AsyncExec_DelegateBroadcastViaTick : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		auto& GlobalTask = EditorData.AddGlobalTask<FTestTask_BroadcastDelegate>("GlobalBroadcast");

		UStateTreeState& Root = EditorData.AddSubTree("Root");
		UStateTreeState& State1 = Root.AddChildState("State1");

		auto& ListenTask = State1.AddTask<FTestTask_ListenDelegate>("Listener");
		State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		// Bind the listener's dispatcher to the global task's OnTickDelegate via property binding
		EditorData.AddPropertyBinding(GlobalTask, TEXT("OnTickDelegate"), ListenTask, TEXT("Listener"));

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE(TEXT("StateTree should init"), Exec.IsValid());

		EStateTreeRunStatus Status = Exec.Start();
		AITEST_EQUAL(TEXT("Start should be running"), Status, EStateTreeRunStatus::Running);

		// Tick to trigger the broadcast via normal tick path
		Exec.LogClear();
		Status = Exec.Tick(0.1f);
		AITEST_EQUAL(TEXT("Should still be running"), Status, EStateTreeRunStatus::Running);

		// The listener should have been triggered
		AITEST_TRUE(TEXT("Listener should have been triggered by broadcast"), Exec.Expect(TEXT("Listener"), TEXT("OnDelegate1")));

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_AsyncExec_DelegateBroadcastViaTick, "System.StateTree.AsyncExecution.DelegateBroadcastViaTick");


/**
 * Test: Strong context GetOwner and GetStateTree accessors return valid objects.
 *
 * Tree layout:
 *   Root
 *     State1 : Task (PrintValue)
 */
struct FStateTreeTest_AsyncExec_StrongContextAccessors : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree("Root");
		UStateTreeState& State1 = Root.AddChildState("State1");

		FStateTreeWeakExecutionContext CapturedWeakContext;

		auto& Task1 = State1.AddTask<FTestTask_PrintValue>("Task1");
		Task1.GetNode().CustomEnterStateFunc = [&CapturedWeakContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
			{
				CapturedWeakContext = Context.MakeWeakExecutionContext();
			};
		State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE(TEXT("StateTree should init"), Exec.IsValid());

		Exec.Start();

		// Verify strong context accessors
		{
			FStateTreeStrongExecutionContext StrongExecContext = CapturedWeakContext.MakeStrongExecutionContext();
			AITEST_TRUE(TEXT("Strong context should be valid"), StrongExecContext.IsValid());
			AITEST_NOT_NULL(TEXT("GetOwner should return valid owner"), StrongExecContext.GetOwner().Get());
			AITEST_NOT_NULL(TEXT("GetStateTree should return valid state tree"), StrongExecContext.GetStateTree().Get());
			AITEST_NOT_NULL(TEXT("GetStorage should return valid storage"), StrongExecContext.GetStorage().Get());
		}

		// Verify weak context accessors
		{
			AITEST_NOT_NULL(TEXT("WeakContext GetOwner should return valid owner"), CapturedWeakContext.GetOwner().Get());
			AITEST_NOT_NULL(TEXT("WeakContext GetStateTree should return valid state tree"), CapturedWeakContext.GetStateTree().Get());
		}

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_AsyncExec_StrongContextAccessors, "System.StateTree.AsyncExecution.StrongContextAccessors");


/**
 * Test: Default-constructed weak context produces an invalid strong context.
 */
struct FStateTreeTest_AsyncExec_DefaultWeakContextIsInvalid : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		FStateTreeWeakExecutionContext DefaultWeakContext;

		FStateTreeStrongExecutionContext StrongExecContext = DefaultWeakContext.MakeStrongExecutionContext();
		AITEST_FALSE(TEXT("Strong context from default weak context should be invalid"), StrongExecContext.IsValid());

		FStateTreeStrongReadOnlyExecutionContext ReadOnlyCtx = DefaultWeakContext.MakeStrongReadOnlyExecutionContext();
		AITEST_FALSE(TEXT("Read-only strong context from default weak context should be invalid"), ReadOnlyCtx.IsValid());

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_AsyncExec_DefaultWeakContextIsInvalid, "System.StateTree.AsyncExecution.DefaultWeakContextIsInvalid");


/**
 * Test: Strong context becomes invalid after the tree is stopped.
 *
 * Tree layout:
 *   Root
 *     State1 : Task (PrintValue)
 */
struct FStateTreeTest_AsyncExec_StrongContextInvalidAfterStop : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree("Root");
		UStateTreeState& State1 = Root.AddChildState("State1");

		FStateTreeWeakExecutionContext CapturedWeakContext;

		auto& Task1 = State1.AddTask<FTestTask_PrintValue>("Task1");
		Task1.GetNode().CustomEnterStateFunc = [&CapturedWeakContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
			{
				CapturedWeakContext = Context.MakeWeakExecutionContext();
			};

		State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE(TEXT("StateTree should init"), Exec.IsValid());

		Exec.Start();

		// Verify valid while running
		{
			FStateTreeStrongExecutionContext StrongExecContext = CapturedWeakContext.MakeStrongExecutionContext();
			AITEST_TRUE(TEXT("Strong context should be valid while tree is running"), StrongExecContext.IsValid());
		}

		// Stop the tree
		Exec.Stop();

		// After stop, the context should be invalid (states are no longer active)
		{
			FStateTreeStrongExecutionContext StrongExecContext = CapturedWeakContext.MakeStrongExecutionContext();
			AITEST_FALSE(TEXT("Strong context should be invalid after tree is stopped"), StrongExecContext.IsValid());
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_AsyncExec_StrongContextInvalidAfterStop, "System.StateTree.AsyncExecution.StrongContextInvalidAfterStop");


/**
 * Test: Strong context becomes invalid after the owner is GCed.
 *
 * Tree layout:
 *   Root
 *     State1 : Task (PrintValue)
 */
struct FStateTreeTest_AsyncExec_StrongContextInvalidAfterGC : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		TWeakObjectPtr<UStateTree> StateTreeWeak = &StateTree;

		UStateTreeState& Root = EditorData.AddSubTree("Root");
		UStateTreeState& State1 = Root.AddChildState("State1");

		FStateTreeWeakExecutionContext CapturedWeakContext;

		auto& Task1 = State1.AddTask<FTestTask_PrintValue>("Task1");
		Task1.GetNode().CustomEnterStateFunc = [&CapturedWeakContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
			{
				CapturedWeakContext = Context.MakeWeakExecutionContext();
			};

		State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		// We won't be able to call stop because of the GC.
		FScopedCVar StopValidation = FScopedCVar(TEXT("StateTree.RuntimeValidation.EnterExitState"), false);

		{
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
			AITEST_TRUE(TEXT("StateTree should init"), Exec.IsValid());

			Exec.Start();

			// Verify valid while running
			{
				FStateTreeStrongExecutionContext StrongExecContext = CapturedWeakContext.MakeStrongExecutionContext();
				AITEST_TRUE(TEXT("Strong context should be valid while tree is running"), StrongExecContext.IsValid());
			}

			// Trigger a GC to clear StateTree
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
			AITEST_FALSE(TEXT("StateTree asset is GCed."), StateTreeWeak.Get() != nullptr)

			// After stop, the context should be invalid (states are no longer active)
			{
				FStateTreeStrongExecutionContext StrongExecContext = CapturedWeakContext.MakeStrongExecutionContext();
				AITEST_FALSE(TEXT("Strong context should be invalid after tree is stopped"), StrongExecContext.IsValid());
			}
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_AsyncExec_StrongContextInvalidAfterGC, "System.StateTree.AsyncExecution.StrongContextInvalidAfterGC");

/**
 * Test: Global task weak context remains valid across state transitions within the same tree,
 * since global tasks are active for the entire tree lifetime.
 *
 * Tree layout:
 *   Global: Task (PrintValue)
 *   Root
 *     State1 : Task (Stand, completes after 1 tick) -> NextState
 *     State2 : Task (Stand)
 */
struct FStateTreeTest_AsyncExec_GlobalTaskWeakContextSurvivesTransitions : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		FStateTreeWeakExecutionContext CapturedGlobalWeakContext;

		auto& GlobalTask = EditorData.AddGlobalTask<FTestTask_PrintValue>("GlobalTask");
		GlobalTask.GetNode().CustomEnterStateFunc = [&CapturedGlobalWeakContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
			{
				CapturedGlobalWeakContext = Context.MakeWeakExecutionContext();
			};

		UStateTreeState& Root = EditorData.AddSubTree("Root");
		UStateTreeState& State1 = Root.AddChildState("State1");
		UStateTreeState& State2 = Root.AddChildState("State2");

		auto& Task1 = State1.AddTask<FTestTask_Stand>("Task1");
		Task1.GetNode().TicksToCompletion = 1;
		State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::NextState);

		auto& Task2 = State2.AddTask<FTestTask_Stand>("Task2");
		Task2.GetNode().TicksToCompletion = 100;
		State2.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE(TEXT("StateTree should init"), Exec.IsValid());

		EStateTreeRunStatus Status = Exec.Start();
		AITEST_EQUAL(TEXT("Start should be running"), Status, EStateTreeRunStatus::Running);

		// Global weak context should be valid
		{
			FStateTreeStrongExecutionContext StrongExecContext = CapturedGlobalWeakContext.MakeStrongExecutionContext();
			AITEST_TRUE(TEXT("Global task strong context should be valid during State1"), StrongExecContext.IsValid());
		}

		// Transition from State1 to State2
		Status = Exec.Tick(0.1f);
		AITEST_EQUAL(TEXT("Should still be running"), Status, EStateTreeRunStatus::Running);
		AITEST_TRUE(TEXT("Should be in State2"), Exec.ExpectInActiveStates(TEXT("Root"), TEXT("State2")));

		// Global weak context should STILL be valid after state transition
		{
			FStateTreeStrongExecutionContext StrongExecContext = CapturedGlobalWeakContext.MakeStrongExecutionContext();
			AITEST_TRUE(TEXT("Global task strong context should remain valid after state transition"), StrongExecContext.IsValid());
		}

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_AsyncExec_GlobalTaskWeakContextSurvivesTransitions, "System.StateTree.AsyncExecution.GlobalTaskWeakContextSurvivesTransitions");


/**
 * Test: RequestTransition through a weak context from outside tick processing.
 * The transition request should be buffered and processed on the next tick.
 *
 * Tree layout:
 *   Root
 *     State1 : Task (PrintValue)
 *     State2 : Task (Stand)
 */
struct FStateTreeTest_AsyncExec_RequestTransitionViaWeakContext : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree("Root");
		UStateTreeState& State1 = Root.AddChildState("State1");
		UStateTreeState& State2 = Root.AddChildState("State2");

		FStateTreeWeakExecutionContext CapturedWeakContext;


		auto& Task1 = State1.AddTask<FTestTask_PrintValue>("Task1");
		Task1.GetNode().CustomEnterStateFunc = [&CapturedWeakContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
			{
				CapturedWeakContext = Context.MakeWeakExecutionContext();
			};

		auto& Task2 = State2.AddTask<FTestTask_Stand>("Task2");
		Task2.GetNode().TicksToCompletion = 100;
		State2.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		// Find State2's compiled handle
		const FStateTreeStateHandle State2Handle = StateTree.GetStateHandleFromId(State2.ID);
		AITEST_TRUE(TEXT("State2 handle should be valid"), State2Handle.IsValid());

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE(TEXT("StateTree should init"), Exec.IsValid());

		EStateTreeRunStatus Status = Exec.Start();
		AITEST_EQUAL(TEXT("Start should be running"), Status, EStateTreeRunStatus::Running);
		AITEST_TRUE(TEXT("Should be in State1"), Exec.ExpectInActiveStates(TEXT("Root"), TEXT("State1")));

		// Request transition to State2 via weak context (outside tick)
		const bool bRequested = CapturedWeakContext.RequestTransition(State2Handle);
		AITEST_TRUE(TEXT("RequestTransition via weak context should succeed"), bRequested);

		// Next tick should process the buffered transition request
		Status = Exec.Tick(0.1f);
		AITEST_EQUAL(TEXT("Should still be running"), Status, EStateTreeRunStatus::Running);
		AITEST_TRUE(TEXT("Should have transitioned to State2"), Exec.ExpectInActiveStates(TEXT("Root"), TEXT("State2")));

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_AsyncExec_RequestTransitionViaWeakContext, "System.StateTree.AsyncExecution.RequestTransitionViaWeakContext");


/**
 * Test: ActivePathInfo from a strong context returns valid information for an active state.
 *
 * Tree layout:
 *   Root
 *     State1 : Task (PrintValue)
 */
struct FStateTreeTest_AsyncExec_ActivePathInfo : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree("Root");
		UStateTreeState& State1 = Root.AddChildState("State1");

		FStateTreeWeakExecutionContext CapturedWeakContext;

		auto& Task1 = State1.AddTask<FTestTask_PrintValue>("Task1");
		Task1.GetNode().CustomEnterStateFunc = [&CapturedWeakContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
			{
				CapturedWeakContext = Context.MakeWeakExecutionContext();
			};
		State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE(TEXT("StateTree should init"), Exec.IsValid());

		Exec.Start();

		{
			FStateTreeStrongExecutionContext StrongExecContext = CapturedWeakContext.MakeStrongExecutionContext();
			AITEST_TRUE(TEXT("Strong context should be valid"), StrongExecContext.IsValid());

			UE::StateTree::Async::FActivePathInfo PathInfo = StrongExecContext.GetActivePathInfo();
			AITEST_TRUE(TEXT("ActivePathInfo should be valid"), PathInfo.IsValid());
			AITEST_NOT_NULL(TEXT("ActivePathInfo Frame should not be null"), PathInfo.Frame);
			AITEST_TRUE(TEXT("ActivePathInfo NodeIndex should be valid"), PathInfo.NodeIndex.IsValid());
		}

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_AsyncExec_ActivePathInfo, "System.StateTree.AsyncExecution.ActivePathInfo");

/**
 * Test: CopyInputBindings propagates bound values into instance data asynchronously.
 * Verifies that an async callback can manually copy input bindings from global parameters into task instance data.
 */
struct FStateTreeTest_AsyncExec_CopyInputBindings_Success : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		// Set up global parameters
		FInstancedPropertyBag& GlobalPropertyBag = GetRootPropertyBag(EditorData);
		GlobalPropertyBag.AddProperty("IntParam", EPropertyBagPropertyType::Int32);
		GlobalPropertyBag.SetValueInt32("IntParam", 65);

		UStateTreeState& Root = EditorData.AddSubTree("Root");
		UStateTreeState& State1 = Root.AddChildState("State1");

		FStateTreeWeakExecutionContext CapturedWeakContext;

		TStateTreeEditorNode<FTestTask_PrintValue>& Task1 = State1.AddTask<FTestTask_PrintValue>("Task1");
		Task1.GetNode().CustomEnterStateFunc = [&CapturedWeakContext]
			(FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
			{
				CapturedWeakContext = Context.MakeWeakExecutionContext();
			};

		// Bind global param → task input
		EditorData.GetPropertyEditorBindings()->AddBinding(
			FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("IntParam")),
			FPropertyBindingPath(Task1.ID, TEXT("Value")));

		State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		AITEST_TRUE(TEXT("StateTree should compile"), Compiler.Compile(StateTree));

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE(TEXT("StateTree should init"), Exec.IsValid());

		Exec.Start();

		// Create strong context and copy bindings
		FStateTreeStrongExecutionContext StrongContext = CapturedWeakContext.MakeStrongExecutionContext();
		AITEST_TRUE(TEXT("Strong context should be valid"), StrongContext.IsValid());

		// Verify Value before the update
		const FTestTask_PrintValueInstanceData* Data1 = StrongContext.GetInstanceDataPtr<const FTestTask_PrintValueInstanceData>();
		AITEST_NOT_NULL(TEXT("Instance data should be accessible"), Data1);
		AITEST_EQUAL(TEXT("Value should reflect bound value"), Data1->Value, 65);

		// Mutate global param externally
		FStateTreeInstanceStorage& Storage = InstanceData.GetMutableStorage();
		FStructView GlobalParams = Storage.GetMutableGlobalParameters();
		FProperty* IntParamProperty = GlobalParams.GetScriptStruct()->FindPropertyByName("IntParam");
		int32 IntParamValue = 99;
		IntParamProperty->CopyCompleteValue_InContainer(GlobalParams.GetMemory(), &IntParamValue);

		const bool bCopied = StrongContext.CopyInputBindings();
		AITEST_TRUE(TEXT("CopyInputBindings should succeed"), bCopied);

		// Verify Value was updated
		const FTestTask_PrintValueInstanceData* Data2 = StrongContext.GetInstanceDataPtr<const FTestTask_PrintValueInstanceData>();
		AITEST_NOT_NULL(TEXT("Instance data should be accessible"), Data2);
		AITEST_EQUAL(TEXT("Value should reflect bound value"), Data2->Value, 99);

		Exec.Stop();
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_AsyncExec_CopyInputBindings_Success, "System.StateTree.AsyncExecution.CopyInputBindings.Success");

/**
 * Test: CopyInputBindings returns true when the node has no bindings.
 * Verifies early-return path when BindingsBatch is invalid.
 */
struct FStateTreeTest_AsyncExec_CopyInputBindings_NoBatch : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree("Root");
		UStateTreeState& State1 = Root.AddChildState("State1");

		FStateTreeWeakExecutionContext CapturedWeakContext;

		TStateTreeEditorNode<FTestTask_PrintValue>& Task1 = State1.AddTask<FTestTask_PrintValue>("Task1");
		Task1.GetNode().CustomEnterStateFunc = [&CapturedWeakContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
		{
			CapturedWeakContext = Context.MakeWeakExecutionContext();
		};

		// No bindings added

		State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		AITEST_TRUE(TEXT("StateTree should compile"), Compiler.Compile(StateTree));

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE(TEXT("StateTree should init"), Exec.IsValid());

		Exec.Start();

		FStateTreeStrongExecutionContext StrongContext = CapturedWeakContext.MakeStrongExecutionContext();
		AITEST_TRUE(TEXT("Strong context should be valid"), StrongContext.IsValid());

		const bool bCopied = StrongContext.CopyInputBindings();
		AITEST_TRUE(TEXT("CopyInputBindings with no batch should return true"), bCopied);

		Exec.Stop();
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_AsyncExec_CopyInputBindings_NoBatch, "System.StateTree.AsyncExecution.CopyInputBindings.NoBatch");

/**
 * Test: CopyInputBindings returns false when the captured frame is no longer active.
 * Verifies that stale contexts fail safely.
 */
struct FStateTreeTest_AsyncExec_CopyInputBindings_InvalidContext : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		// Set up global parameters
		FInstancedPropertyBag& GlobalPropertyBag = GetRootPropertyBag(EditorData);
		GlobalPropertyBag.AddProperty("IntParam", EPropertyBagPropertyType::Int32);
		GlobalPropertyBag.SetValueInt32("IntParam", 42);

		UStateTreeState& Root = EditorData.AddSubTree("Root");
		UStateTreeState& State1 = Root.AddChildState("State1");
		UStateTreeState& State2 = Root.AddChildState("State2");

		FStateTreeWeakExecutionContext CapturedWeakContext;

		TStateTreeEditorNode<FTestTask_PrintValue>& Task1 = State1.AddTask<FTestTask_PrintValue>("Task1");
		Task1.GetInstanceData().TickRunStatus = EStateTreeRunStatus::Succeeded;
		Task1.GetNode().CustomEnterStateFunc = [&CapturedWeakContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
		{
			CapturedWeakContext = Context.MakeWeakExecutionContext();
		};

		// Bind global param → task input
		EditorData.GetPropertyEditorBindings()->AddBinding(
			FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("IntParam")),
			FPropertyBindingPath(Task1.ID, TEXT("Value")));

		// Transition to State2 immediately
		State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &State2);
		State2.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bCompiled = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should compile"), bCompiled);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE(TEXT("StateTree should init"), Exec.IsValid());

		EStateTreeRunStatus Status = Exec.Start();
		AITEST_EQUAL(TEXT("Start should be running"), Status, EStateTreeRunStatus::Running);
		AITEST_TRUE(TEXT("Should be running (in State1)"), Exec.ExpectInActiveStates("Root", "State1"));

		// Tick to force transition out of State1
		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("Should be running (in State2)"), Exec.ExpectInActiveStates("Root", "State2"));

		// Strong context created from the captured weak context
		FStateTreeStrongExecutionContext StrongContext = CapturedWeakContext.MakeStrongExecutionContext();

		// Context should now be invalid (frame exited)
		AITEST_FALSE(TEXT("Strong context should be invalid after state exit"), StrongContext.IsValid());
		AITEST_FALSE(TEXT("CopyInputBindings should return false on invalid context"), StrongContext.CopyInputBindings());

		Exec.Stop();
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_AsyncExec_CopyInputBindings_InvalidContext, "System.StateTree.AsyncExecution.CopyInputBindings.InvalidContext");

/**
 * Test: CopyOutputBindings propagates task output back to global parameters asynchronously.
 * Verifies that output bindings can be manually invoked in async context.
 */
struct FStateTreeTest_AsyncExec_CopyOutputBindings_Success : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		// Set up global parameters
		FInstancedPropertyBag& GlobalPropertyBag = GetRootPropertyBag(EditorData);
		GlobalPropertyBag.AddProperty("IntParam", EPropertyBagPropertyType::Int32);
		GlobalPropertyBag.SetValueInt32("IntParam", 0);

		UStateTreeState& Root = EditorData.AddSubTree("Root");
		UStateTreeState& State1 = Root.AddChildState("State1");

		FStateTreeWeakExecutionContext CapturedWeakContext;

		TStateTreeEditorNode<FTestTask_PrintValue>& Task1 = State1.AddTask<FTestTask_PrintValue>("Task1");
		Task1.GetNode().CustomEnterStateFunc = [&CapturedWeakContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
		{
			CapturedWeakContext = Context.MakeWeakExecutionContext();
		};

		// Bind task output → global param (output binding)
		EditorData.GetPropertyEditorBindings()->AddOutputBinding(
			FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("IntParam")),
			FPropertyBindingPath(Task1.ID, TEXT("OutputValue")));

		State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		AITEST_TRUE(TEXT("StateTree should compile"), Compiler.Compile(StateTree));

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE(TEXT("StateTree should init"), Exec.IsValid());

		Exec.Start();

		FStateTreeStrongExecutionContext StrongContext = CapturedWeakContext.MakeStrongExecutionContext();
		AITEST_TRUE(TEXT("Strong context should be valid"), StrongContext.IsValid());

		// Mutate OutputValue in instance data
		FTestTask_PrintValueInstanceData* Data = StrongContext.GetInstanceDataPtr<FTestTask_PrintValueInstanceData>();
		AITEST_NOT_NULL(TEXT("Instance data should be accessible"), Data);
		Data->OutputValue = 77;

		// Copy output bindings (writes back to global param)
		const bool bCopied = StrongContext.CopyOutputBindings();
		AITEST_TRUE(TEXT("CopyOutputBindings should succeed"), bCopied);

		// Verify global param was updated
		FStateTreeInstanceStorage& Storage = InstanceData.GetMutableStorage();
		const FConstStructView GlobalParams = Storage.GetGlobalParameters();
		FProperty* IntParamProperty = GlobalParams.GetScriptStruct()->FindPropertyByName("IntParam");
		int32 IntParamValue = 0;
		IntParamProperty->CopyCompleteValue_InContainer(&IntParamValue, GlobalParams.GetMemory());
		AITEST_EQUAL(TEXT("Global param should reflect output value"), IntParamValue, 77);

		Exec.Stop();
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_AsyncExec_CopyOutputBindings_Success, "System.StateTree.AsyncExecution.CopyOutputBindings.Success");

/**
 * Test: CopyOutputBindings returns true when the node has no output bindings.
 * Verifies early-return path when OutputBindingsBatch is invalid.
 */
struct FStateTreeTest_AsyncExec_CopyOutputBindings_NoBatch : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree("Root");
		UStateTreeState& State1 = Root.AddChildState("State1");

		FStateTreeWeakExecutionContext CapturedWeakContext;

		TStateTreeEditorNode<FTestTask_PrintValue>& Task1 = State1.AddTask<FTestTask_PrintValue>("Task1");
		Task1.GetNode().CustomEnterStateFunc = [&CapturedWeakContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
		{
			CapturedWeakContext = Context.MakeWeakExecutionContext();
		};

		// No output bindings added

		State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		AITEST_TRUE(TEXT("StateTree should compile"), Compiler.Compile(StateTree));

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE(TEXT("StateTree should init"), Exec.IsValid());

		Exec.Start();

		FStateTreeStrongExecutionContext StrongContext = CapturedWeakContext.MakeStrongExecutionContext();
		AITEST_TRUE(TEXT("Strong context should be valid"), StrongContext.IsValid());

		const bool bCopied = StrongContext.CopyOutputBindings();
		AITEST_TRUE(TEXT("CopyOutputBindings with no batch should return true"), bCopied);

		Exec.Stop();
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_AsyncExec_CopyOutputBindings_NoBatch, "System.StateTree.AsyncExecution.CopyOutputBindings.NoBatch");

/**
 * Test: CopyInputBindings with property functions in binding batch.
 * Property functions (e.g., mathematical operations) cannot be evaluated in async context because they require
 * the full FStateTreeExecutionContext and evaluation scope instance data setup.
 * This test verifies that CopyInputBindings handles property functions gracefully:
 * - Logs a warning (via CopyBindingBatchForAsync)
 * - Skips property function evaluation
 * - Returns false indicating partial/unavailable evaluation
 */
struct FStateTreeTest_AsyncExec_CopyInputBindings_WithPropertyFunctions : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree("Root");
		UStateTreeState& State1 = Root.AddChildState("State1");

		FStateTreeWeakExecutionContext CapturedWeakContext;

		TStateTreeEditorNode<FTestTask_PrintValue>& Task1 = State1.AddTask<FTestTask_PrintValue>("Task1");
		Task1.GetInstanceData().Value = 99;
		Task1.GetNode().CustomEnterStateFunc = [&CapturedWeakContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
		{
			CapturedWeakContext = Context.MakeWeakExecutionContext();
		};

		// Create binding with property function transformation:
		// This simulates a binding chain: IntParamA -> [PropertyFunction] -> Task.Value
		// The property function cannot be evaluated in async context since it requires
		// the full execution context and evaluation scope setup.

		// Add binding through property function
		EditorData.AddPropertyBinding(
			CastChecked<UScriptStruct>(FTestPropertyFunction::StaticStruct()),
			{ FPropertyBindingPathSegment(TEXT("Result")) },
			FPropertyBindingPath(Task1.ID, TEXT("Value")));

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		AITEST_TRUE(TEXT("StateTree should compile"), Compiler.Compile(StateTree));

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		AITEST_TRUE(TEXT("StateTree should init"), Exec.IsValid());

		Exec.Start();

		FStateTreeStrongExecutionContext StrongContext = CapturedWeakContext.MakeStrongExecutionContext();
		AITEST_TRUE(TEXT("Strong context should be valid"), StrongContext.IsValid());

		FTestTask_PrintValueInstanceData* Data = StrongContext.GetInstanceDataPtr<FTestTask_PrintValueInstanceData>();
		AITEST_NOT_NULL(TEXT("Instance data should be accessible"), Data);

		// Value changed to 1 because of the property function eval ran with EnterState
		AITEST_EQUAL(TEXT("Value should be set to default"), Data->Value, 1);
		Data->Value = 88;

		// CopyInputBindings should handle property functions gracefully:
		// - Log a warning (property functions cannot be evaluated in async context)
		// - Return false (indicating unavailable sources/property functions)
		// - NOT crash despite property functions being in the binding batch
		const bool bCopied = StrongContext.CopyInputBindings();
		AITEST_FALSE(TEXT("CopyInputBindings should return false when property functions are in batch"), bCopied);

		// The instance data should be accessible even though copy failed
		const FTestTask_PrintValueInstanceData* DataAfter = StrongContext.GetInstanceDataPtr<const FTestTask_PrintValueInstanceData>();
		AITEST_EQUAL(TEXT("Instance data should still be accessible"), Data, DataAfter);
		// Value unchanged since property function evaluation was skipped
		AITEST_EQUAL(TEXT("Value should be unchanged (property function skipped)"), Data->Value, 88);

		Exec.Stop();
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_AsyncExec_CopyInputBindings_WithPropertyFunctions, "System.StateTree.AsyncExecution.CopyInputBindings.WithPropertyFunctions");


} // namespace UE::StateTree::Tests

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE