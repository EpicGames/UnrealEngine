// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeTestBase.h"
#include "StateTreeTest.h"
#include "StateTreeEditorData.h"
#include "Engine/World.h"
#include "GameplayTagsManager.h"

#define LOCTEXT_NAMESPACE "AITestSuite_StateTreeTest"

namespace UE::StateTree::Tests
{

UStateTree& FStateTreeTestBase::NewStateTree() const
{
	UStateTree* StateTree = NewObject<UStateTree>(&GetWorld());
	check(StateTree);
	UStateTreeEditorData* EditorData = NewObject<UStateTreeEditorData>(StateTree);
	check(EditorData);
	StateTree->EditorData = EditorData;
	EditorData->Schema = NewObject<UStateTreeTestSchema>();
	return *StateTree;
}

FStateTreePropertyPathBinding FStateTreeTestBase::MakeBinding(const FGuid& SourceID, const FStringView Source, const FGuid& TargetID, const FStringView Target, const bool bInIsOutputBinding /*false*/)
{
	FPropertyBindingPath SourcePath;
	SourcePath.FromString(Source);
	SourcePath.SetStructID(SourceID);

	FPropertyBindingPath TargetPath;
	TargetPath.FromString(Target);
	TargetPath.SetStructID(TargetID);

	return FStateTreePropertyPathBinding(SourcePath, TargetPath, bInIsOutputBinding);
}

// Helper struct to define some test tags
struct FNativeGameplayTags : public FGameplayTagNativeAdder
{
	virtual ~FNativeGameplayTags() {}

	FGameplayTag TestTag;
	FGameplayTag TestTag2;
	FGameplayTag TestTag3;
	FGameplayTag TestTag4;
	FGameplayTag TestTag5;
	FGameplayTag TestTag6;

	virtual void AddTags() override
	{
		UGameplayTagsManager& Manager = UGameplayTagsManager::Get();
		TestTag = Manager.AddNativeGameplayTag(TEXT("Test.StateTree.Tag"));
		TestTag2 = Manager.AddNativeGameplayTag(TEXT("Test.StateTree.Tag2"));
		TestTag3 = Manager.AddNativeGameplayTag(TEXT("Test.StateTree.Tag3"));
		TestTag4 = Manager.AddNativeGameplayTag(TEXT("Test.StateTree.Tag4"));
		TestTag5 = Manager.AddNativeGameplayTag(TEXT("Test.StateTree.Tag5"));
		TestTag6 = Manager.AddNativeGameplayTag(TEXT("Test.StateTree.Tag6"));
	}
};
static FNativeGameplayTags GameplayTagStaticInstance;

FGameplayTag FStateTreeTestBase::GetTestTag1()
{
	return GameplayTagStaticInstance.TestTag;
}

FGameplayTag FStateTreeTestBase::GetTestTag2()
{
	return GameplayTagStaticInstance.TestTag2;
}

FGameplayTag FStateTreeTestBase::GetTestTag3()
{
	return GameplayTagStaticInstance.TestTag3;
}

FGameplayTag FStateTreeTestBase::GetTestTag4()
{
	return GameplayTagStaticInstance.TestTag4;
}

FGameplayTag FStateTreeTestBase::GetTestTag5()
{
	return GameplayTagStaticInstance.TestTag5;
}

FGameplayTag FStateTreeTestBase::GetTestTag6()
{
	return GameplayTagStaticInstance.TestTag6;
}

FInstancedPropertyBag& FStateTreeTestBase::GetRootPropertyBag(UStateTreeEditorData& EditorData) const
{
	return const_cast<FInstancedPropertyBag&>(EditorData.GetRootParametersPropertyBag());
}
}//namespace UE::StateTree::Tests

#undef LOCTEXT_NAMESPACE

