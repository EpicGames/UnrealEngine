// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeTestSuiteModule.h"

#define LOCTEXT_NAMESPACE "StateTreeTestSuite"

class FStateTreeTestSuiteModule : public IStateTreeTestSuiteModule
{
};

IMPLEMENT_MODULE(FStateTreeTestSuiteModule, StateTreeTestSuite)

#undef LOCTEXT_NAMESPACE
