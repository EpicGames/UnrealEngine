// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeTaskBase.h"
#include "StateTreeEvaluatorBase.h"
#include "StateTreeConditionBase.h"
#include "StateTreeAsyncExecutionContext.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeLinker.h"
#include "StateTreePropertyRef.h"
#include "StateTreePropertyFunctionBase.h"
#include "StateTreeDelegate.h"
#include "AutoRTFM.h"
#include <atomic>

#include "StateTreeTestTypes.generated.h"

class UStateTree;
struct FStateTreeInstanceData;

// Test log that can be passed as external data.
USTRUCT()
struct FStateTreeTestLog
{
	GENERATED_BODY()

	struct FLogItem
	{
		FLogItem() = default;
		FLogItem(const FName& InName, const FString& InMessage) : Name(InName), Message(InMessage) {}
		FName Name;
		FString Message; 
	};
	TArray<FLogItem> Items;
	
	void Log(const FName& Name, const FString& Message)
	{
		Items.Emplace(Name, Message);
	}
};

struct FTestStateTreeExecutionContext : public FStateTreeExecutionContext
{
	FTestStateTreeExecutionContext(UObject& InOwner, const UStateTree& InStateTree, FStateTreeInstanceData& InInstanceData, const FOnCollectStateTreeExternalData& CollectExternalDataCallback = {}, const EStateTreeRecordTransitions RecordTransitions = EStateTreeRecordTransitions::No)
		: FTestStateTreeExecutionContext(&InOwner, &InStateTree, InInstanceData, CollectExternalDataCallback, RecordTransitions)
	{
	}

	FTestStateTreeExecutionContext(TNotNull<UObject*> InOwner, TNotNull<const UStateTree*> InStateTree, FStateTreeInstanceData& InInstanceData, const FOnCollectStateTreeExternalData& CollectExternalDataCallback = {}, const EStateTreeRecordTransitions RecordTransitions = EStateTreeRecordTransitions::No)
		: FStateTreeExecutionContext(InOwner, InStateTree, InInstanceData, CollectExternalDataCallback, RecordTransitions)
	{
		// Handle getting pointer to the test log.
		FStateTreeDataView TestLogView(TBaseStructure<FStateTreeTestLog>::Get(), (uint8*)&Log);
		
		CollectExternalDataDelegate = FOnCollectStateTreeExternalData::CreateLambda(
			[TestLogView](const FStateTreeExecutionContext& Context, const UStateTree* StateTree, TArrayView<const FStateTreeExternalDataDesc> ExternalDataDescs, TArrayView<FStateTreeDataView> OutDataViews)
			{
				for (int32 Index = 0; Index < ExternalDataDescs.Num(); Index++)
				{
					const FStateTreeExternalDataDesc& ItemDesc = ExternalDataDescs[Index];
					if (ItemDesc.Struct == TBaseStructure<FStateTreeTestLog>::Get())
					{
						OutDataViews[Index] = TestLogView;
						break;
					}
				}
				return true;
			});
	}

	FStateTreeTestLog Log;

	void LogClear()
	{
		Log.Items.Empty();
	}

	struct FLogOrder
	{
		FLogOrder(const FTestStateTreeExecutionContext& InContext, const int32 InIndex)
			: Context(&InContext), Index(InIndex)
		{}

		FLogOrder Then(const FName& Name) const
		{
			int32 NextIndex = Index;
			while (NextIndex < Context->Log.Items.Num())
			{
				const FStateTreeTestLog::FLogItem& Item = Context->Log.Items[NextIndex];
				if (Item.Name == Name)
				{
					break;
				}
				NextIndex++;
			}
			return FLogOrder(*Context, NextIndex);
		}

		FLogOrder Then(const FName& Name, const FStringView Message) const
		{
			int32 NextIndex = Index;
			while (NextIndex < Context->Log.Items.Num())
			{
				const FStateTreeTestLog::FLogItem& Item = Context->Log.Items[NextIndex];
				if (Item.Name == Name && Item.Message == Message)
				{
					break;
				}
				NextIndex++;
			}
			return FLogOrder(*Context, NextIndex);
		}

		operator bool() const
		{
			return Index < Context->Log.Items.Num();
		}

	private:
		const FTestStateTreeExecutionContext* Context;
		int32 Index = 0;
	};

	FLogOrder Expect(const FName& Name) const
	{
		return FLogOrder(*this, 0).Then(Name);
	}

	FLogOrder Expect(const FName& Name, const FStringView Message) const
	{
		return FLogOrder(*this, 0).Then(Name, Message);
	}

	template <class ...Args>
	bool ExpectInActiveStates(const Args&... States)
	{
		FName ExpectedStateNames[] = { States... };
		const int32 NumExpectedStateNames = sizeof...(States);

		TArray<FName> ActiveStateNames = GetActiveStateNames();

		if (ActiveStateNames.Num() != NumExpectedStateNames)
		{
			return false;
		}

		for (int32 Index = 0; Index != NumExpectedStateNames; Index++)
		{
			if (ExpectedStateNames[Index] != ActiveStateNames[Index])
			{
				return false;
			}
		}

		return true;
	}
	
};

namespace UE::StateTree::Test
{
	UENUM()
	enum class ECppClassEnumByte : uint8
	{
		A = 0,
		B = 0x11,
		C = 0x22,
	};

	UENUM()
	enum class ECppClassEnumInt : uint32
	{
		A = 0,
		B = 0xF333F333,
		C = 0xF444F444,
	};

	UENUM()
	namespace ENamespaceEnumByte
	{
		enum Type : uint8
		{
			A = 0,
			B = 0x55,
			C = 0x66,
		};
	}

	UENUM()
	enum EEnumByte : uint8
	{
		EEnumByte_A = 0,
		EEnumByte_B = 0x77,
		EEnumByte_C = 0x88,
	};

	USTRUCT()
	struct FStructWithEnum
	{
		GENERATED_BODY()

		UPROPERTY()
		ECppClassEnumByte CppClassEnumByte = ECppClassEnumByte::A;
		UPROPERTY()
		ECppClassEnumInt CppClassEnumInt = ECppClassEnumInt::A;
		UPROPERTY()
		TEnumAsByte<ENamespaceEnumByte::Type> NamespaceEnumByte = ENamespaceEnumByte::A;
		UPROPERTY()
		TEnumAsByte<EEnumByte> EnumByte = EEnumByte_A;
	};

	USTRUCT()
	struct FStructWithEnumRef
	{
		GENERATED_BODY()

		UPROPERTY()
		TStateTreePropertyRef<ECppClassEnumByte> CppClassEnumByte;
		UPROPERTY()
		TStateTreePropertyRef<ECppClassEnumInt> CppClassEnumInt;
		UPROPERTY()
		TStateTreePropertyRef<TEnumAsByte<ENamespaceEnumByte::Type>> NamespaceEnumByte;
		UPROPERTY()
		TStateTreePropertyRef<TEnumAsByte<EEnumByte>> EnumByte;
	};
}

USTRUCT()
struct FTestEval_AInstanceData
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = Parameter)
	float FloatA = 0.0f;

	UPROPERTY(EditAnywhere, Category = Parameter)
	int32 IntA = 0;

	UPROPERTY(EditAnywhere, Category = Parameter)
	bool bBoolA = false;
};

USTRUCT()
struct FTestEval_A : public FStateTreeEvaluatorBase
{
	GENERATED_BODY()

	using FInstanceDataType = FTestEval_AInstanceData;

	FTestEval_A() = default;
	FTestEval_A(const FName InName) { Name = InName; }
	virtual ~FTestEval_A() override = default;

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
};

USTRUCT()
struct FTestEval_Custom : public FStateTreeEvaluatorBase
{
	GENERATED_BODY()

	using FInstanceDataType = FTestEval_AInstanceData;

	FTestEval_Custom() = default;
	FTestEval_Custom(const FName InName) { Name = InName; }

	virtual const UStruct* GetInstanceDataType() const override
	{
		return FInstanceDataType::StaticStruct();
	}

	virtual void TreeStart(FStateTreeExecutionContext& Context) const override
	{
		if (CustomEnterStateFunc)
		{
			CustomEnterStateFunc(Context, this);
		}
	}

	virtual void TreeStop(FStateTreeExecutionContext& Context) const override 
	{
		if (CustomExitStateFunc)
		{
			CustomExitStateFunc(Context, this);
		}
	}

	virtual void Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override
	{
		if (CustomTickFunc)
		{
			CustomTickFunc(Context, this);
		}
	}

	virtual bool Link(FStateTreeLinker& Linker) override
	{
		Linker.LinkExternalData(LogHandle);
		return true;
	}

	TStateTreeExternalDataHandle<FStateTreeTestLog> LogHandle;

	TFunction<void(FStateTreeExecutionContext&, const FTestEval_Custom*)> CustomEnterStateFunc;
	TFunction<void(FStateTreeExecutionContext&, const FTestEval_Custom*)> CustomExitStateFunc;
	TFunction<void(FStateTreeExecutionContext&, const FTestEval_Custom*)> CustomTickFunc;
};

USTRUCT()
struct FTestTask_BInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Parameter)
	float FloatB = 0.0f;

	UPROPERTY(EditAnywhere, Category = Parameter)
	int32 IntB = 0;

	UPROPERTY(EditAnywhere, Category = Parameter)
	bool bBoolB = false;
};

USTRUCT()
struct FTestTask_B : public FStateTreeTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FTestTask_BInstanceData;

	FTestTask_B() = default;
	FTestTask_B(const FName InName) { Name = InName; }
	virtual ~FTestTask_B() override = default;
	
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override
	{
		FStateTreeTestLog& TestLog = Context.GetExternalData(LogHandle);
		TestLog.Log(Name,  TEXT("EnterState"));
		return EStateTreeRunStatus::Running;
	}

	virtual bool Link(FStateTreeLinker& Linker) override
	{
		Linker.LinkExternalData(LogHandle);
		return true;
	}
	
	TStateTreeExternalDataHandle<FStateTreeTestLog> LogHandle;
};

USTRUCT()
struct FTestTask_PrintValueInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Parameter")
	int32 Value = 0;

	UPROPERTY(EditAnywhere, Category = "Output")
	int32 OutputValue = 0;
	
	UPROPERTY(EditAnywhere, Category = "Parameter")
	TArray<int32> ArrayValue;
	
	UPROPERTY(EditAnywhere, Category = "Parameter")
	EStateTreeRunStatus EnterStateRunStatus = EStateTreeRunStatus::Running;
	
	UPROPERTY(EditAnywhere, Category = "Parameter")
	EStateTreeRunStatus TickRunStatus = EStateTreeRunStatus::Running;

	FString GetArrayDescription() const
	{
		if (ArrayValue.IsEmpty())
		{
			return TEXT("{}");
		}
		TStringBuilder<256> StringBuilder;
		StringBuilder << TEXT('{');
		bool bFirst = true;
		for (int32 It : ArrayValue)
		{
			if (!bFirst)
			{
				StringBuilder << TEXT(", ");
			}
			bFirst = false;
			StringBuilder << It;
		}
		StringBuilder << TEXT('}');
		return StringBuilder.ToString();
	}
};

USTRUCT()
struct FTestTask_PrintValueExecutionRuntimeData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Parameter")
	int32 Value = 0;
};

USTRUCT()
struct FTestTask_PrintValue : public FStateTreeTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FTestTask_PrintValueInstanceData;
	using FExecutionRuntimeDataType = FTestTask_PrintValueExecutionRuntimeData;

	FTestTask_PrintValue() = default;
	FTestTask_PrintValue(const FName InName) { Name = InName; }
	virtual ~FTestTask_PrintValue() override = default;
	
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	virtual const UStruct* GetExecutionRuntimeDataType() const override { return FExecutionRuntimeDataType::StaticStruct(); }

	virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override
	{
		FStateTreeTestLog& TestLog = Context.GetExternalData(LogHandle);
		const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
		TestLog.Log(Name, FString::Printf(TEXT("EnterState%d"), InstanceData.Value));
		TestLog.Log(Name, FString::Printf(TEXT("EnterState:%s"), *InstanceData.GetArrayDescription()));
		TestLog.Log(Name, FString::Printf(TEXT("EnterState=%s"), *StaticEnum<EStateTreeStateChangeType>()->GetNameStringByValue((int64)Transition.ChangeType)));

		const FExecutionRuntimeDataType& ExecData = Context.GetExecutionRuntimeData(*this);
		TestLog.Log(Name, FString::Printf(TEXT("EnterStateExec=%d"), ExecData.Value));
		if (CustomEnterStateFunc)
		{
			CustomEnterStateFunc(Context, this);
		}
		return InstanceData.EnterStateRunStatus;
	}

	virtual void ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override
	{
		FStateTreeTestLog& TestLog = Context.GetExternalData(LogHandle);
		const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
		TestLog.Log(Name, FString::Printf(TEXT("ExitState%d"), InstanceData.Value));
		TestLog.Log(Name, FString::Printf(TEXT("ExitState:%s"), *InstanceData.GetArrayDescription()));
		TestLog.Log(Name, FString::Printf(TEXT("ExitState=%s"), *StaticEnum<EStateTreeStateChangeType>()->GetNameStringByValue((int64)Transition.ChangeType)));
		if (CustomExitStateFunc)
		{
			CustomExitStateFunc(Context, this);
		}
	}

	virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override
	{
		FStateTreeTestLog& TestLog = Context.GetExternalData(LogHandle);
		const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
		TestLog.Log(Name, FString::Printf(TEXT("Tick%d"), InstanceData.Value));
		TestLog.Log(Name, FString::Printf(TEXT("Tick:%s"), *InstanceData.GetArrayDescription()));
		if (CustomTickFunc)
		{
			CustomTickFunc(Context, this);
		}
		return InstanceData.TickRunStatus;
	}

	virtual void TriggerTransitions(FStateTreeExecutionContext& Context) const override
	{
		FStateTreeTestLog& TestLog = Context.GetExternalData(LogHandle);
		const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
		TestLog.Log(Name, FString::Printf(TEXT("TriggerTransitions%d"), InstanceData.Value));
		TestLog.Log(Name, FString::Printf(TEXT("TriggerTransitions:%s"), *InstanceData.GetArrayDescription()));
	}

	virtual bool Link(FStateTreeLinker& Linker) override
	{
		Linker.LinkExternalData(LogHandle);
		return true;
	}
	
	TStateTreeExternalDataHandle<FStateTreeTestLog> LogHandle;
	TFunction<void(FStateTreeExecutionContext&, const FTestTask_PrintValue*)> CustomEnterStateFunc;
	TFunction<void(FStateTreeExecutionContext&, const FTestTask_PrintValue*)> CustomExitStateFunc;
	TFunction<void(FStateTreeExecutionContext&, const FTestTask_PrintValue*)> CustomTickFunc;
};

USTRUCT(meta = (Hidden))
struct FTestCondition_PrintValue : public FStateTreeConditionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FTestTask_PrintValueInstanceData;
	using FExecutionRuntimeDataType = FTestTask_PrintValueExecutionRuntimeData;

	FTestCondition_PrintValue() = default;
	FTestCondition_PrintValue(const FName InName)
	{
		Name = InName;
	}
	virtual ~FTestCondition_PrintValue() override {}

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	virtual const UStruct* GetExecutionRuntimeDataType() const override { return FExecutionRuntimeDataType::StaticStruct(); }

	virtual bool Link(FStateTreeLinker& Linker) override
	{
		Linker.LinkExternalData(LogHandle);
		return true;
	}

	virtual bool TestCondition(FStateTreeExecutionContext& Context) const override
	{
		FStateTreeTestLog& TestLog = Context.GetExternalData(LogHandle);
		const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
		TestLog.Log(Name, FString::Printf(TEXT("TestCondition%d"), InstanceData.Value));
		TestLog.Log(Name, FString::Printf(TEXT("TestCondition:%s"), *InstanceData.GetArrayDescription()));

		if (CustomTestConditionFunc)
		{
			return CustomTestConditionFunc(Context, this);
		}

		return true;
	}

	TFunction<bool(FStateTreeExecutionContext&, const FTestCondition_PrintValue*)> CustomTestConditionFunc;
	TStateTreeExternalDataHandle<FStateTreeTestLog> LogHandle;
};

USTRUCT()
struct FTestTask_PrintAndResetValue : public FTestTask_PrintValue
{
	GENERATED_BODY()

	using FTestTask_PrintValue::FTestTask_PrintValue;

	virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override
	{
		EStateTreeRunStatus Status = Super::EnterState(Context, Transition);
		FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
		InstanceData.Value = ResetValue;
		InstanceData.ArrayValue = ResetArrayValue;
		return Status;
	}

	virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override
	{
		EStateTreeRunStatus Status = Super::Tick(Context, DeltaTime);
		FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
		InstanceData.Value = ResetValue;
		InstanceData.ArrayValue = ResetArrayValue;
		return Status;
	}

	virtual void ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override
	{
		Super::ExitState(Context, Transition);
		FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
		InstanceData.Value = ResetValue;
		InstanceData.ArrayValue = ResetArrayValue;
	}

	UPROPERTY()
	int32 ResetValue = 0;

	UPROPERTY()
	TArray<int32> ResetArrayValue;
};

USTRUCT()
struct FTestTask_PrintValue_TransitionTick : public FTestTask_PrintValue
{
	GENERATED_BODY()

	FTestTask_PrintValue_TransitionTick()
	{
		bShouldCallTick = true;
		bShouldAffectTransitions = true;
	}
	FTestTask_PrintValue_TransitionTick(const FName InName)
		: FTestTask_PrintValue(InName)
	{
		bShouldCallTick = true;
		bShouldAffectTransitions = true;
	}
};

USTRUCT()
struct FTestTask_PrintValue_TransitionNoTick : public FTestTask_PrintValue
{
	GENERATED_BODY()

	FTestTask_PrintValue_TransitionNoTick()
	{
		bShouldCallTick = false;
		bShouldAffectTransitions = true;
	}
	FTestTask_PrintValue_TransitionNoTick(const FName InName)
		: FTestTask_PrintValue(InName)
	{
		bShouldCallTick = false;
		bShouldAffectTransitions = true;
	}
};

USTRUCT()
struct FTestTask_StopTreeInstanceData
{
	GENERATED_BODY()
};

USTRUCT()
struct FTestTask_StopTree : public FStateTreeTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FTestTask_PrintValueInstanceData;

	FTestTask_StopTree() = default;
	explicit FTestTask_StopTree(const FName InName) { Name = InName; }
	virtual ~FTestTask_StopTree() override = default;
	
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override
	{
		if (Phase == EStateTreeUpdatePhase::EnterStates)
		{
			return Context.Stop();
		}
		return EStateTreeRunStatus::Running;
	}

	virtual void ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override
	{
		if (Phase == EStateTreeUpdatePhase::ExitStates)
		{
			Context.Stop();
		}
	}

	virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override
	{
		if (Phase == EStateTreeUpdatePhase::TickStateTree)
		{
			return Context.Stop();
		}
		return EStateTreeRunStatus::Running;
	};

	virtual bool Link(FStateTreeLinker& Linker) override
	{
		Linker.LinkExternalData(LogHandle);
		return true;
	}
	
	TStateTreeExternalDataHandle<FStateTreeTestLog> LogHandle;
	
	/** Indicates in which phase the call to Stop should be performed. Possible values are EnterStates, ExitStats and TickStateTree */
	UPROPERTY()
	EStateTreeUpdatePhase Phase = EStateTreeUpdatePhase::Unset;
};

USTRUCT()
struct FTestTask_StandInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Parameter")
	int32 Value = 0;
	
	UPROPERTY()
	int32 CurrentTick = 0;
};

USTRUCT()
struct FTestTask_Stand : public FStateTreeTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FTestTask_StandInstanceData;
	
	FTestTask_Stand() = default;
	FTestTask_Stand(const FName InName) { Name = InName; }
	virtual ~FTestTask_Stand() = default;

	virtual const UStruct* GetInstanceDataType() const { return FInstanceDataType::StaticStruct(); }

	virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override
	{
		FStateTreeTestLog& TestLog = Context.GetExternalData(LogHandle);
		TestLog.Log(Name, TEXT("EnterState"));
		TestLog.Log(Name, FString::Printf(TEXT("EnterState=%s"), *StaticEnum<EStateTreeStateChangeType>()->GetNameStringByValue((int64)Transition.ChangeType)));

		FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
		
		if (Transition.ChangeType == EStateTreeStateChangeType::Changed)
		{
			InstanceData.CurrentTick = 0;
		}
		return EnterStateResult;
	}

	virtual void ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override
	{
		FStateTreeTestLog& TestLog = Context.GetExternalData(LogHandle);

		if (Transition.CurrentRunStatus == EStateTreeRunStatus::Succeeded)
		{
			TestLog.Log(Name, TEXT("ExitSucceeded"));
		}
		else if (Transition.CurrentRunStatus == EStateTreeRunStatus::Failed)
		{
			TestLog.Log(Name, TEXT("ExitFailed"));
		}
		else if (Transition.CurrentRunStatus == EStateTreeRunStatus::Stopped)
		{
			TestLog.Log(Name, TEXT("ExitStopped"));
		}

		TestLog.Log(Name, TEXT("ExitState"));
		TestLog.Log(Name, FString::Printf(TEXT("ExitState=%s"), *StaticEnum<EStateTreeStateChangeType>()->GetNameStringByValue((int64)Transition.ChangeType)));
	}

	virtual void StateCompleted(FStateTreeExecutionContext& Context, const EStateTreeRunStatus CompletionStatus, const FStateTreeActiveStates& CompletedActiveStates) const override
	{
		FStateTreeTestLog& TestLog = Context.GetExternalData(LogHandle);
		TestLog.Log(Name, TEXT("StateCompleted"));
	}
	
	virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override
	{
		FStateTreeTestLog& TestLog = Context.GetExternalData(LogHandle);
		TestLog.Log(Name, TEXT("Tick"));

		FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
		
		InstanceData.CurrentTick++;
		TestLog.Log(Name, FString::Printf(TEXT("CurrentTick=%u"), InstanceData.CurrentTick));
		
		return (InstanceData.CurrentTick >= TicksToCompletion) ? TickCompletionResult : EStateTreeRunStatus::Running;
	};

	virtual bool Link(FStateTreeLinker& Linker) override
	{
		Linker.LinkExternalData(LogHandle);
		return true;
	}
	
	TStateTreeExternalDataHandle<FStateTreeTestLog> LogHandle;
	
	UPROPERTY(EditAnywhere, Category = Parameter)
	int32 TicksToCompletion = 1;

	UPROPERTY(EditAnywhere, Category = Parameter)
	EStateTreeRunStatus TickCompletionResult = EStateTreeRunStatus::Succeeded;

	UPROPERTY(EditAnywhere, Category = Parameter)
	EStateTreeRunStatus EnterStateResult = EStateTreeRunStatus::Running;
};

USTRUCT()
struct FTestTask_StandNoTick : public FTestTask_Stand
{
	GENERATED_BODY()

	FTestTask_StandNoTick()
	{
		bShouldCallTick = false;
	}

	FTestTask_StandNoTick(const FName InName)
		: FTestTask_Stand(InName)
	{
		bShouldCallTick = false;
	}
};

USTRUCT()
struct FTestTask_StandNoStateChangeOnReselect : public FTestTask_Stand
{
	GENERATED_BODY()

	FTestTask_StandNoStateChangeOnReselect()
	{
		bShouldStateChangeOnReselect = false;
	}

	FTestTask_StandNoStateChangeOnReselect(const FName InName)
		: FTestTask_Stand(InName)
	{
		bShouldStateChangeOnReselect = false;
	}
};

USTRUCT()
struct FTestTask_IntegersOutput_InstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Output)
	int32 IntA = 0;

	UPROPERTY(EditAnywhere, Category=Output)
	int32 IntB = 0;

};

USTRUCT()
struct FTestTask_IntegersOutput : public FStateTreeTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FTestTask_IntegersOutput_InstanceData;

	FTestTask_IntegersOutput() = default;

	explicit FTestTask_IntegersOutput(FName InName)
	{
		Name = InName;
	}

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override
	{
		if (CustomEnterStateFunc)
		{
			CustomEnterStateFunc(Context, *this);
		}

		return EStateTreeRunStatus::Running;
	}

	TFunction<void(FStateTreeExecutionContext&, const FTestTask_IntegersOutput&)> CustomEnterStateFunc;
};

USTRUCT()
struct FTestTask_PropertyRefOnNodeAndInstance_InstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Parameter, meta = (RefType = "int32", Optional))
	FStateTreePropertyRef RefOnInstance;
};

USTRUCT()
struct FTestTask_PropertyRefOnNodeAndInstance : public FStateTreeTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FTestTask_PropertyRefOnNodeAndInstance_InstanceData;

	FTestTask_PropertyRefOnNodeAndInstance() = default;
	FTestTask_PropertyRefOnNodeAndInstance(FName InName)
	{
		Name = InName;
	}

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override
	{
		if (CustomEnterStateFunc)
		{
			CustomEnterStateFunc(Context, *this);
		}

		return EStateTreeRunStatus::Running;
	}

	UPROPERTY(EditAnywhere, Category = Parameter, meta = (RefType = "int32", Optional))
	FStateTreePropertyRef RefOnNode;

	TFunction<void(FStateTreeExecutionContext&, const FTestTask_PropertyRefOnNodeAndInstance&)> CustomEnterStateFunc;
};

USTRUCT()
struct FTestCondition_PropertyRefOnNodeAndInstance : public FStateTreeConditionBase
{
	GENERATED_BODY()

	using FInstanceDataType = FTestTask_PropertyRefOnNodeAndInstance_InstanceData;

	FTestCondition_PropertyRefOnNodeAndInstance() = default;
	FTestCondition_PropertyRefOnNodeAndInstance(FName InName)
	{
		Name = InName;
	}

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	virtual bool TestCondition(FStateTreeExecutionContext& Context) const override
	{
		if (CustomTestConditionFunc)
		{
			CustomTestConditionFunc(Context, *this);
		}

		return true;
	}

	UPROPERTY(EditAnywhere, Category = Parameter, meta = (RefType = "int32", Optional))
	FStateTreePropertyRef RefOnNode;

	TFunction<void(FStateTreeExecutionContext&, const FTestCondition_PropertyRefOnNodeAndInstance&)> CustomTestConditionFunc;
};

USTRUCT()
struct FStateTreeTestConditionInstanceData
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = Parameters)
	int32 Count = 1;

	static std::atomic<int32> GlobalCounter;
};

USTRUCT(meta = (Hidden))
struct FStateTreeTestCondition : public FStateTreeConditionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeTestConditionInstanceData;

	FStateTreeTestCondition() = default;

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	virtual bool TestCondition(FStateTreeExecutionContext& Context) const override
	{
		std::atomic<int32>& GlobalCounter = Context.GetInstanceData(*this).GlobalCounter;
		UE_AUTORTFM_OPEN
		{
			++GlobalCounter;
		};
		UE_AUTORTFM_ONABORT(&GlobalCounter)
		{
			--GlobalCounter;
		};
		return bTestConditionResult;
	}

	UPROPERTY()
	bool bTestConditionResult = true;
};

struct FStateTreeTestRunContext
{
	int32 Count = 0;
};


USTRUCT()
struct FStateTreeTest_PropertyStructA
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "")
	int32 A = 0;
};

USTRUCT()
struct FStateTreeTest_PropertyStructB
{
	GENERATED_BODY()

	FStateTreeTest_PropertyStructB()
	{
		NumConstructed++;
	}
	~FStateTreeTest_PropertyStructB()
	{
		NumConstructed--;
	}

	UPROPERTY(EditAnywhere, Category = "")
	int32 B = 0;

	static inline int32 NumConstructed = 0;
};

USTRUCT()
struct FStateTreeTest_PropertyStructObjectPtr
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "")
	TObjectPtr<UObject> A;

	UPROPERTY(EditAnywhere, Category = "")
	TWeakObjectPtr<UObject> B;

	UPROPERTY(EditAnywhere, Category = "")
	TSoftObjectPtr<UObject> C;

	UPROPERTY(EditAnywhere, Category = "")
	TArray<TObjectPtr<UObject>> ArrayA;

	UPROPERTY(EditAnywhere, Category = "")
	TArray<TWeakObjectPtr<UObject>> ArrayB;

	UPROPERTY(EditAnywhere, Category = "")
	TArray<TSoftObjectPtr<UObject>> ArrayC;
};

USTRUCT()
struct FStateTreeTest_PropertyStruct
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "")
	int32 A = 0;

	UPROPERTY(EditAnywhere, Category = "")
	int32 B = 0;

	UPROPERTY(EditAnywhere, Category = "")
	FStateTreeTest_PropertyStructB StructB;
};

UCLASS(HideDropdown)
class UStateTreeTest_PropertyObjectInstanced : public UObject
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category = "")
	int32 A = 0;

	UPROPERTY(EditAnywhere, Category = "")
	FInstancedStruct InstancedStruct;

	UPROPERTY(EditAnywhere, Category = "")
	TArray<FGameplayTag> ArrayOfTags;
};

UCLASS(HideDropdown)
class UStateTreeTest_PropertyObjectInstancedWithB : public UStateTreeTest_PropertyObjectInstanced
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category = "")
	int32 B = 0;
};

UCLASS(HideDropdown)
class UStateTreeTest_PropertyObject : public UObject
{
	GENERATED_BODY()
public:
	
	UPROPERTY(EditAnywhere, Instanced, Category = "")
	TObjectPtr<UStateTreeTest_PropertyObjectInstanced> InstancedObject;

	UPROPERTY(EditAnywhere, Instanced, Category = "")
	TArray<TObjectPtr<UStateTreeTest_PropertyObjectInstanced>> ArrayOfInstancedObjects;

	UPROPERTY(EditAnywhere, Category = "")
	TArray<int32> ArrayOfInts;

	UPROPERTY(EditAnywhere, Category = "")
	FInstancedStruct InstancedStruct;

	UPROPERTY(EditAnywhere, Category = "")
	TArray<FInstancedStruct> ArrayOfInstancedStructs;

	UPROPERTY(EditAnywhere, Category = "")
	FStateTreeTest_PropertyStruct Struct;

	UPROPERTY(EditAnywhere, Category = "")
	TArray<FStateTreeTest_PropertyStruct> ArrayOfStruct;
};

UCLASS(HideDropdown)
class UStateTreeTest_PropertyObject2 : public UObject
{
	GENERATED_BODY()
public:
};

USTRUCT()
struct FStateTreeTest_PropertyCopy
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "")
	FStateTreeTest_PropertyStruct Item;

	UPROPERTY(EditAnywhere, Category = "")
	TArray<FStateTreeTest_PropertyStruct> Array;

	UPROPERTY(EditAnywhere, Category = "", EditFixedSize)
	TArray<FStateTreeTest_PropertyStruct> FixedArray;

	UPROPERTY(EditAnywhere, Category = "")
	FStateTreeTest_PropertyStruct CArray[3];

	UPROPERTY(EditAnywhere, Category = "")
	TArray<float> ArrayOfFloat;

	UPROPERTY(EditAnywhere, Category = "")
	TArray<double> ArrayOfDouble;

};

USTRUCT()
struct FStateTreeTest_PropertyRefSourceStruct
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "")
	FStateTreeTest_PropertyStruct Item;

	UPROPERTY(EditAnywhere, Category = "Output")
	FStateTreeTest_PropertyStruct OutputItem;

	UPROPERTY(EditAnywhere, Category = "")
	TArray<FStateTreeTest_PropertyStruct> Array;
};

USTRUCT()
struct FStateTreeTest_PropertyRefTargetStruct
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "", meta = (RefType = "/Script/StateTreeTestSuite.StateTreeTest_PropertyStruct"))
	FStateTreePropertyRef RefToStruct;

	UPROPERTY(EditAnywhere, Category = "", meta = (RefType = "Int32"))
	FStateTreePropertyRef RefToInt;

	UPROPERTY(EditAnywhere, Category = "", meta = (RefType = "/Script/StateTreeTestSuite.StateTreeTest_PropertyStruct", IsRefToArray))
	FStateTreePropertyRef RefToStructArray;
};

USTRUCT()
struct FStateTreeTest_PropertyCopyObjects
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "")
	TObjectPtr<UObject> Object;

	UPROPERTY(EditAnywhere, Category = "")
	TSubclassOf<UObject> Class;

	UPROPERTY(EditAnywhere, Category = "")
	TSoftObjectPtr<UObject> SoftObject;

	UPROPERTY(EditAnywhere, Category = "")
	TSoftClassPtr<UObject> SoftClass;
};

USTRUCT()
struct FTestPropertyFunction_InstanceData
{
	GENERATED_BODY()

	UPROPERTY()
	int32 Input = 0;

	UPROPERTY()
	int32 Result = 0;
};
UE_STATETREE_ZEROED_TRIVIALLY_COPIED_NO_DESTRUCTOR_INSTANCEDATA(FTestPropertyFunction_InstanceData);

USTRUCT()
struct FTestPropertyFunction : public FStateTreePropertyFunctionBase
{
	GENERATED_BODY()

	using FInstanceDataType = FTestPropertyFunction_InstanceData;

	FTestPropertyFunction() = default;
	FTestPropertyFunction(const FName InName) { Name = InName; }
	
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	virtual void Execute(FStateTreeExecutionContext& Context) const override
	{
		FTestPropertyFunction_InstanceData& InstanceData = Context.GetInstanceData<FTestPropertyFunction_InstanceData>(*this);
		InstanceData.Result = InstanceData.Input + 1;
	}
};

USTRUCT()
struct FTestTask_PrintValue_StructRef_NoBindingUpdateInstanceData : public FTestTask_PrintValueInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Parameter")
	FStateTreeTest_PropertyStruct PropertyStruct;

	UPROPERTY(EditAnywhere, Category = "Parameter", meta = (BaseStruct = "/Script/StateTreeTestSuite.StateTreeTest_PropertyStruct"))
	FStateTreeStructRef StructRef;
};

USTRUCT()
struct FTestTask_PrintValue_StructRef_NoBindingUpdate : public FTestTask_PrintValue
{
	GENERATED_BODY()

	using FInstanceDataType = FTestTask_PrintValue_StructRef_NoBindingUpdateInstanceData;

	virtual const UStruct* GetInstanceDataType() const override
	{
		return FInstanceDataType::StaticStruct();
	}

	FTestTask_PrintValue_StructRef_NoBindingUpdate()
	{
		bShouldCopyBoundPropertiesOnTick = false;
	}
	FTestTask_PrintValue_StructRef_NoBindingUpdate(const FName InName)
		: FTestTask_PrintValue(InName)
	{
		bShouldCopyBoundPropertiesOnTick = false;
	}
};

USTRUCT()
struct FStateTreeTestBooleanConditionInstanceData
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = Parameters)
	bool bSuccess = true;
};
UE_STATETREE_CONSTRUCTED_TRIVIALLY_COPIED_NO_DESTRUCTOR_INSTANCEDATA(FStateTreeTestBooleanConditionInstanceData);

USTRUCT(meta = (Hidden))
struct FStateTreeTestBooleanCondition : public FStateTreeConditionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeTestBooleanConditionInstanceData;

	FStateTreeTestBooleanCondition() = default;

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	virtual bool TestCondition(FStateTreeExecutionContext& Context) const override
	{
		FStateTreeTestLog& TestLog = Context.GetExternalData(LogHandle);
		FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
		TestLog.Log(Name,  FString::Printf(TEXT("TestCondition=%d"), InstanceData.bSuccess));
		return InstanceData.bSuccess;
	}

	virtual bool Link(FStateTreeLinker& Linker) override
	{
		Linker.LinkExternalData(LogHandle);
		return true;
	}

	TStateTreeExternalDataHandle<FStateTreeTestLog> LogHandle;
};

USTRUCT()
struct FTestTask_BroadcastDelegate_InstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Parameter)
	FStateTreeDelegateDispatcher OnEnterDelegate;

	UPROPERTY(EditAnywhere, Category = Parameter)
	FStateTreeDelegateDispatcher OnTickDelegate;

	UPROPERTY(EditAnywhere, Category = Parameter)
	FStateTreeDelegateDispatcher OnExitDelegate;
};

USTRUCT()
struct FTestTask_BroadcastDelegate : public FStateTreeTaskBase
{
	GENERATED_BODY()

	FTestTask_BroadcastDelegate() = default;
	FTestTask_BroadcastDelegate(const FName InName) { Name = InName; }

	using FInstanceDataType = FTestTask_BroadcastDelegate_InstanceData;

	virtual const UStruct* GetInstanceDataType() const override
	{
		return FInstanceDataType::StaticStruct();
	}

	virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override
	{
		FTestTask_BroadcastDelegate_InstanceData& InstanceData = Context.GetInstanceData<FTestTask_BroadcastDelegate_InstanceData>(*this);
		Context.BroadcastDelegate(InstanceData.OnEnterDelegate);
		return EnterStateResult;
	}

	virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override
	{
		FTestTask_BroadcastDelegate_InstanceData& InstanceData = Context.GetInstanceData<FTestTask_BroadcastDelegate_InstanceData>(*this);
		Context.BroadcastDelegate(InstanceData.OnTickDelegate);
		return TickCompletionResult;
	}

	virtual void ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override
	{
		FTestTask_BroadcastDelegate_InstanceData& InstanceData = Context.GetInstanceData<FTestTask_BroadcastDelegate_InstanceData>(*this);
		Context.BroadcastDelegate(InstanceData.OnExitDelegate);
	}

	UPROPERTY(EditAnywhere, Category = Parameter)
	EStateTreeRunStatus EnterStateResult = EStateTreeRunStatus::Running;

	UPROPERTY(EditAnywhere, Category = Parameter)
	EStateTreeRunStatus TickCompletionResult = EStateTreeRunStatus::Running;
};

USTRUCT()
struct FTestTask_ListenDelegate_InstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Parameter)
	FStateTreeDelegateListener Listener;

	uint32 TriggersAmount = 0u;
};

USTRUCT()
struct FTestTask_ListenDelegate : public FStateTreeTaskBase
{
	GENERATED_BODY()

	FTestTask_ListenDelegate() = default;
	FTestTask_ListenDelegate(const FName InName) { Name = InName; }

	using FInstanceDataType = FTestTask_ListenDelegate_InstanceData;

	virtual const UStruct* GetInstanceDataType() const override
	{
		return FInstanceDataType::StaticStruct();
	}

	virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override
	{
		FTestTask_ListenDelegate_InstanceData& InstanceData = Context.GetInstanceData<FTestTask_ListenDelegate_InstanceData>(*this);
		FStateTreeTestLog& TestLog = Context.GetExternalData(LogHandle);
		Context.BindDelegate(InstanceData.Listener, FSimpleDelegate::CreateLambda([&TestLog, this, InstanceDataRef = Context.GetInstanceDataStructRef(*this)]()
			{
				if (FInstanceDataType* InstanceData = InstanceDataRef.GetPtr())
				{
					++InstanceData->TriggersAmount;
					TestLog.Log(Name, FString::Printf(TEXT("OnDelegate%d"), InstanceData->TriggersAmount));
				}
			}));

		return EStateTreeRunStatus::Running;
	}

	virtual bool Link(FStateTreeLinker& Linker) override
	{
		Linker.LinkExternalData(LogHandle);
		return true;
	}

	virtual void ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override
	{
		if (bRemoveOnExit)
		{
			FTestTask_ListenDelegate_InstanceData& InstanceData = Context.GetInstanceData<FTestTask_ListenDelegate_InstanceData>(*this);
			Context.UnbindDelegate(InstanceData.Listener);
		}
	}
	
	bool bRemoveOnExit = true;
	TStateTreeExternalDataHandle<FStateTreeTestLog> LogHandle;
};

USTRUCT()
struct FTestTask_RebroadcastDelegate_InstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Parameter)
	FStateTreeDelegateListener Listener;

	UPROPERTY(EditAnywhere, Category = Parameter)
	FStateTreeDelegateDispatcher Dispatcher;
};

USTRUCT()
struct FTestTask_RebroadcastDelegate : public FStateTreeTaskBase
{
	GENERATED_BODY()

	FTestTask_RebroadcastDelegate() = default;
	FTestTask_RebroadcastDelegate(const FName InName) { Name = InName; }

	using FInstanceDataType = FTestTask_RebroadcastDelegate_InstanceData;

	virtual const UStruct* GetInstanceDataType() const override
	{
		return FInstanceDataType::StaticStruct();
	}

	virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override
	{
		FTestTask_RebroadcastDelegate_InstanceData& InstanceData = Context.GetInstanceData<FTestTask_RebroadcastDelegate_InstanceData>(*this);

		Context.BindDelegate(InstanceData.Listener, FSimpleDelegate::CreateLambda([Dispatcher = InstanceData.Dispatcher, WeakContext = Context.MakeWeakExecutionContext()]()
			{
				WeakContext.BroadcastDelegate(Dispatcher);
			}));

		return EStateTreeRunStatus::Running;
	}

	virtual void ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override
	{
		FTestTask_RebroadcastDelegate_InstanceData& InstanceData = Context.GetInstanceData<FTestTask_RebroadcastDelegate_InstanceData>(*this);
		Context.UnbindDelegate(InstanceData.Listener);
	}
};

USTRUCT()
struct FTestTask_CustomFuncOnDelegate_InstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Parameter)
	FStateTreeDelegateListener Listener;
};

USTRUCT()
struct FTestTask_CustomFuncOnDelegate : public FStateTreeTaskBase
{
	GENERATED_BODY()

	FTestTask_CustomFuncOnDelegate() = default;
	FTestTask_CustomFuncOnDelegate(const FName InName) { Name = InName; }

	using FInstanceDataType = FTestTask_CustomFuncOnDelegate_InstanceData;

	virtual const UStruct* GetInstanceDataType() const override
	{
		return FInstanceDataType::StaticStruct();
	}

	virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override
	{
		FTestTask_CustomFuncOnDelegate_InstanceData& InstanceData = Context.GetInstanceData<FTestTask_CustomFuncOnDelegate_InstanceData>(*this);

		Context.BindDelegate(InstanceData.Listener, FSimpleDelegate::CreateLambda([Func = CustomFunc, Listener = InstanceData.Listener, WeakContext = Context.MakeWeakExecutionContext()]()
			{
				Func(WeakContext, Listener);
			}));
		return EStateTreeRunStatus::Running;
	}

	virtual void ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override
	{
		if (bRemoveOnExit)
		{
			FTestTask_CustomFuncOnDelegate_InstanceData& InstanceData = Context.GetInstanceData<FTestTask_CustomFuncOnDelegate_InstanceData>(*this);
			Context.UnbindDelegate(InstanceData.Listener);
		}
	}

	TFunction<void(const FStateTreeWeakExecutionContext&, FStateTreeDelegateListener)> CustomFunc;

	bool bRemoveOnExit = true;
};

USTRUCT()
struct FTestTask_DispatcherOnNodeAndInstance_InstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Parameter)
	FStateTreeDelegateDispatcher DispatcherOnInstance;
};

USTRUCT()
struct FTestTask_DispatcherOnNodeAndInstance : public FStateTreeTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FTestTask_DispatcherOnNodeAndInstance_InstanceData;

	FTestTask_DispatcherOnNodeAndInstance() = default;

	explicit FTestTask_DispatcherOnNodeAndInstance(FName InName)
	{
		Name = InName;
	}

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override
	{
		const FInstanceDataType& InstanceData = Context.GetInstanceData<FInstanceDataType>(*this);

		Context.BroadcastDelegate(DispatcherOnNode);
		Context.BroadcastDelegate(InstanceData.DispatcherOnInstance);

		return EStateTreeRunStatus::Running;
	}

	UPROPERTY(EditAnywhere, Category = Parameter)
	FStateTreeDelegateDispatcher DispatcherOnNode;
};

USTRUCT()
struct FTestTask_ListenerOnNode_InstanceData
{
	GENERATED_BODY()
};

USTRUCT()
struct FTestTask_ListenerOnNode : public FStateTreeTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FTestTask_ListenerOnNode_InstanceData;

	FTestTask_ListenerOnNode() = default;

	explicit FTestTask_ListenerOnNode(FName InName)
	{
		Name = InName;
	}

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct();  }

	virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override
	{
		Context.BindDelegate(ListenerOnNode, FSimpleDelegate::CreateLambda(
			[this, WeakExecContext = Context.MakeWeakExecutionContext()]()
			{
				CustomFunc(WeakExecContext);
			}));

		return EStateTreeRunStatus::Running;
	}

	UPROPERTY(EditAnywhere, Category = Parameter)
	FStateTreeDelegateListener ListenerOnNode;

	TFunction<void(const FStateTreeWeakExecutionContext&)> CustomFunc;
};

USTRUCT()
struct FTestTask_ListenerOnInstance_InstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Parameter)
	FStateTreeDelegateListener ListenerOnInstance;
};

USTRUCT()
struct FTestTask_ListenerOnInstance : public FStateTreeTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FTestTask_ListenerOnInstance_InstanceData;

	FTestTask_ListenerOnInstance() = default;

	explicit FTestTask_ListenerOnInstance(FName InName)
	{
		Name = InName;
	}

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override
	{
		const FInstanceDataType& InstanceData = Context.GetInstanceData<FInstanceDataType>(*this);
		Context.BindDelegate(InstanceData.ListenerOnInstance, FSimpleDelegate::CreateLambda(
			[this, WeakExecContext = Context.MakeWeakExecutionContext()]()
			{
				CustomFunc(WeakExecContext);
			}));

		return EStateTreeRunStatus::Running;
	}

	TFunction<void(const FStateTreeWeakExecutionContext&)> CustomFunc;
};

// @todo: add tests for CArray type
USTRUCT()
struct FIntWrapper
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Default)
	int32 Value = 0;


	bool operator==(const FIntWrapper& Other) const = default;
};

USTRUCT()
struct FTestTask_OutputBindingsTaskInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Parameter)
	FIntWrapper InputIntWrapper;

	UPROPERTY(EditAnywhere, Category = Parameter)
	TArray<FIntWrapper> InputIntWrapperArray;

	UPROPERTY(EditAnywhere, Category = Parameter)
	int32 InputIntA = 0;

	UPROPERTY(EditAnywhere, Category = Parameter)
	int32 InputIntB = 0;

	UPROPERTY(EditAnywhere, Category = Output)
	uint8 OutputBool : 1 = false;

	UPROPERTY(EditAnywhere, Category = Output)
	int32 OutputInt = 0;

	UPROPERTY(EditAnywhere, Category = Output)
	FIntWrapper OutputIntWrapper;

	UPROPERTY(EditAnywhere, Category = Output)
	TArray<FIntWrapper> OutputIntWrapperArray;
};

USTRUCT()
struct FTestTask_OutputBindingsTask : public FStateTreeTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FTestTask_OutputBindingsTaskInstanceData;

	FTestTask_OutputBindingsTask() = default;

	explicit FTestTask_OutputBindingsTask(FName InName)
	{
		Name = InName;
	}

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override
	{
		if (CustomEnterFunc.IsSet())
		{
			CustomEnterFunc(Context.GetInstanceData(*this));
		}

		return bShouldFailEnter ? EStateTreeRunStatus::Failed : EStateTreeRunStatus::Running;
	}

	virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override
	{
		if (CustomTickFunc.IsSet())
		{
			CustomTickFunc(Context.GetInstanceData(*this));
		}

		return bShouldFailTick ? EStateTreeRunStatus::Failed : EStateTreeRunStatus::Running;
	}

	virtual void ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override
	{
		if (CustomExitFunc.IsSet())
		{
			CustomExitFunc(Context.GetInstanceData(*this));
		}
	}

	TFunction<void(FTestTask_OutputBindingsTaskInstanceData&)> CustomEnterFunc;
	TFunction<void(FTestTask_OutputBindingsTaskInstanceData&)> CustomTickFunc;
	TFunction<void(FTestTask_OutputBindingsTaskInstanceData&)> CustomExitFunc;

	UPROPERTY()
	uint8 bShouldFailEnter : 1 = false;

	UPROPERTY()
	uint8 bShouldFailTick : 1 = false;
};
