// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeTest.h"
#include "StateTreeTestBase.h"
#include "StateTreeTestTypes.h"

#include "StateTreeCompilerLog.h"
#include "StateTreeEditorData.h"
#include "StateTreeCompiler.h"
#include "Conditions/StateTreeCommonConditions.h"

#define LOCTEXT_NAMESPACE "AITestSuite_StateTreeTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::StateTree::Tests
{

struct FStateTreeTest_TransitionPriority : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		/*
			- Root
				- State1 : Task1 -> Succeeded
					- State1A : Task1A -> Next
					- State1B : Task1B -> Next
					- State1C : Task1C
		
			Task1A completed first, transitioning to State1B.
			Task1, Task1B, and Task1C complete at the same time, we should take the transition on the first completed state (State1).
		*/
		
		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		UStateTreeState& State1 = Root.AddChildState(FName(TEXT("State1")));
		UStateTreeState& State1A = State1.AddChildState(FName(TEXT("State1A")));
		UStateTreeState& State1B = State1.AddChildState(FName(TEXT("State1B")));
		UStateTreeState& State1C = State1.AddChildState(FName(TEXT("State1C")));

		auto& Task1 = State1.AddTask<FTestTask_Stand>(FName(TEXT("Task1")));
		Task1.GetNode().TicksToCompletion = 2;
		State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);
		
		auto& Task1A = State1A.AddTask<FTestTask_Stand>(FName(TEXT("Task1A")));
		Task1A.GetNode().TicksToCompletion = 1;
		State1A.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::NextState);

		auto& Task1B = State1B.AddTask<FTestTask_Stand>(FName(TEXT("Task1B")));
		Task1B.GetNode().TicksToCompletion = 2;
		State1B.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::NextState);

		auto& Task1C = State1C.AddTask<FTestTask_Stand>(FName(TEXT("Task1C")));
		Task1C.GetNode().TicksToCompletion = 2;
		
		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE("StateTree should get compiled", bResult);

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE("StateTree should init", bInitSucceeded);

		const TCHAR* TickStr(TEXT("Tick"));
		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* ExitStateStr(TEXT("ExitState"));
		const TCHAR* StateCompletedStr(TEXT("StateCompleted"));

		// Start and enter state
		Status = Exec.Start();
		AITEST_TRUE(TEXT("StateTree Task1 should enter state"), Exec.Expect(Task1.GetName(), EnterStateStr));
		AITEST_TRUE(TEXT("StateTree Task1A should enter state"), Exec.Expect(Task1A.GetName(), EnterStateStr));
		Exec.LogClear();

		// Transition from Task1A to Task1B
		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Task1A should complete"), Exec.Expect(Task1A.GetName(), StateCompletedStr));
		AITEST_TRUE(TEXT("StateTree Task1B should enter state"), Exec.Expect(Task1B.GetName(), EnterStateStr));
		Exec.LogClear();

		// Task1 completes, and we should take State1 transition. 
		Status = Exec.Tick(0.1f);
		AITEST_TRUE("StateTree Task1 should complete", Exec.Expect(Task1.GetName(), StateCompletedStr));
		AITEST_EQUAL(TEXT("Tree execution should stop on success"), Status, EStateTreeRunStatus::Succeeded);
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_TransitionPriority, "System.StateTree.Transition.Priority");

struct FStateTreeTest_TransitionPriorityEnterState : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		
		UStateTreeState& Root =	EditorData.AddSubTree(FName(TEXT("Root")));
		UStateTreeState& State0 = Root.AddChildState(FName(TEXT("State0")));
		UStateTreeState& State1 = Root.AddChildState(FName(TEXT("State1")));
		UStateTreeState& State1A = State1.AddChildState(FName(TEXT("State1A")));
		UStateTreeState& State2 = Root.AddChildState(FName(TEXT("State2")));
		UStateTreeState& State3 = Root.AddChildState(FName(TEXT("State3")));

		auto& Task0 = State0.AddTask<FTestTask_Stand>(FName(TEXT("Task0")));
		State0.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &State1);

		auto& Task1 = State1.AddTask<FTestTask_Stand>(FName(TEXT("Task1")));
		Task1.GetNode().EnterStateResult = EStateTreeRunStatus::Failed;
		State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &State2);
		
		auto& Task1A = State1A.AddTask<FTestTask_Stand>(FName(TEXT("Task1A")));
		State1A.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &State3);

		auto& Task2 = State2.AddTask<FTestTask_Stand>(FName(TEXT("Task2")));
		State2.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		auto& Task3 = State3.AddTask<FTestTask_Stand>(FName(TEXT("Task3")));
		State3.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		
		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE("StateTree should get compiled", bResult);

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE("StateTree should init", bInitSucceeded);

		const TCHAR* TickStr(TEXT("Tick"));
		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* ExitStateStr(TEXT("ExitState"));
		const TCHAR* StateCompletedStr(TEXT("StateCompleted"));

		// Start and enter state
		Status = Exec.Start();
		AITEST_TRUE(TEXT("StateTree Task0 should enter state"), Exec.Expect(Task0.GetName(), EnterStateStr));
		Exec.LogClear();

		// Transition from State0 to State1, it should fail (Task1), and the transition on State1->State2 (and not State1A->State3)
		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Task0 should complete"), Exec.Expect(Task0.GetName(), StateCompletedStr));
		AITEST_TRUE(TEXT("StateTree Task2 should enter state"), Exec.Expect(Task2.GetName(), EnterStateStr));
		AITEST_FALSE(TEXT("StateTree Task3 should not enter state"), Exec.Expect(Task3.GetName(), EnterStateStr));
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_TransitionPriorityEnterState, "System.StateTree.Transition.PriorityEnterState");

struct FStateTreeTest_TransitionNextSelectableState : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		// Root
		//  State0: OnCompleted->Nextselectable
		//  State1: EnterCondition fails
		//  State2: EnterCondition fails
		//  State3: EnterCondition successed. OnCompleted->Succeeded
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root =	EditorData.AddSubTree(FName(TEXT("Root")));
		UStateTreeState& State0 = Root.AddChildState(FName(TEXT("State0")));
		UStateTreeState& State1 = Root.AddChildState(FName(TEXT("State1")));
		UStateTreeState& State2 = Root.AddChildState(FName(TEXT("State2")));
		UStateTreeState& State3 = Root.AddChildState(FName(TEXT("State3")));

		auto& EvalA = EditorData.AddEvaluator<FTestEval_A>();
		EvalA.GetInstanceData().bBoolA = true;

		auto& Task0 = State0.AddTask<FTestTask_Stand>(FName(TEXT("Task0")));
		State0.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::NextSelectableState);

		// Add Task 1 with Condition that will always fail
		auto& Task1 = State1.AddTask<FTestTask_Stand>(FName(TEXT("Task1")));
		auto& BoolCond1 = State1.AddEnterCondition<FStateTreeCompareBoolCondition>();

		EditorData.AddPropertyBinding(EvalA, TEXT("bBoolA"), BoolCond1, TEXT("bLeft"));
		BoolCond1.GetInstanceData().bRight = !EvalA.GetInstanceData().bBoolA;

		// Add Task 2 with Condition that will always fail
		auto& Task2 = State2.AddTask<FTestTask_Stand>(FName(TEXT("Task2")));
		auto& BoolCond2 = State2.AddEnterCondition<FStateTreeCompareBoolCondition>();

		EditorData.AddPropertyBinding(EvalA, TEXT("bBoolA"), BoolCond2, TEXT("bLeft"));
		BoolCond2.GetInstanceData().bRight = !EvalA.GetInstanceData().bBoolA;

		// Add Task 3 with Condition that will always succeed
		auto& Task3 = State3.AddTask<FTestTask_Stand>(FName(TEXT("Task3")));
		auto& BoolCond3 = State3.AddEnterCondition<FStateTreeCompareBoolCondition>();
		State3.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

		EditorData.AddPropertyBinding(EvalA, TEXT("bBoolA"), BoolCond3, TEXT("bLeft"));
		BoolCond3.GetInstanceData().bRight = EvalA.GetInstanceData().bBoolA;

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		const TCHAR* TickStr = TEXT("Tick");
		const TCHAR* EnterStateStr = TEXT("EnterState");
		const TCHAR* ExitStateStr = TEXT("ExitState");
		const TCHAR* StateCompletedStr = TEXT("StateCompleted");

		// Start and enter state
		Exec.Start();
		AITEST_TRUE(TEXT("Should be in active state"), Exec.ExpectInActiveStates("Root", "State0"));
		AITEST_TRUE(TEXT("StateTree Task0 should enter state"), Exec.Expect(Task0.GetName(), EnterStateStr));
		Exec.LogClear();

		// Transition from State0 and tries to select State1. It should fail (Task1 and Tasks2) and because transition is set to "Next Selectable", it should now select Task 3 and Enter State
		Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("Should be in active state"), Exec.ExpectInActiveStates("Root", "State3"));
		AITEST_TRUE(TEXT("StateTree Task0 should complete"), Exec.Expect(Task0.GetName(), StateCompletedStr));
		AITEST_FALSE(TEXT("StateTree Task1 should not enter state"), Exec.Expect(Task1.GetName(), EnterStateStr));
		AITEST_FALSE(TEXT("StateTree Task2 should not enter state"), Exec.Expect(Task2.GetName(), EnterStateStr));
		AITEST_TRUE(TEXT("StateTree Task3 should enter state"), Exec.Expect(Task3.GetName(), EnterStateStr));
		Exec.LogClear();

		// Complete Task3
		EStateTreeRunStatus Status = Exec.Tick(0.1f);
		AITEST_EQUAL(TEXT("Tick should succeed"), Status, EStateTreeRunStatus::Succeeded);
		AITEST_TRUE(TEXT("Should be in active state"), Exec.GetActiveStateNames().Num() == 0);
		AITEST_TRUE(TEXT("StateTree Task3 should complete"), Exec.Expect(Task3.GetName(), StateCompletedStr));
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_TransitionNextSelectableState, "System.StateTree.Transition.NextSelectableState");

struct FStateTreeTest_TransitionNextWithParentData : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root =	EditorData.AddSubTree(FName(TEXT("Root")));
		UStateTreeState& State0 = Root.AddChildState(FName(TEXT("State0")));
		UStateTreeState& State1 = Root.AddChildState(FName(TEXT("State1")));
		UStateTreeState& State1A = State1.AddChildState(FName(TEXT("State1A")));

		auto& RootTask = Root.AddTask<FTestTask_B>(FName(TEXT("RootTask")));
		RootTask.GetInstanceData().bBoolB = true;

		auto& Task0 = State0.AddTask<FTestTask_Stand>(FName(TEXT("Task0")));
		State0.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::NextState);

		auto& Task1A = State1A.AddTask<FTestTask_Stand>(FName(TEXT("Task1A")));
		auto& BoolCond1 = State1A.AddEnterCondition<FStateTreeCompareBoolCondition>();

		EditorData.AddPropertyBinding(RootTask, TEXT("bBoolB"), BoolCond1, TEXT("bLeft"));
		BoolCond1.GetInstanceData().bRight = true;

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE("StateTree should get compiled", bResult);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE("StateTree should init", bInitSucceeded);

		const TCHAR* TickStr(TEXT("Tick"));
		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* ExitStateStr(TEXT("ExitState"));
		const TCHAR* StateCompletedStr(TEXT("StateCompleted"));

		// Start and enter state
		Exec.Start();
		AITEST_TRUE(TEXT("StateTree Task0 should enter state"), Exec.Expect(Task0.GetName(), EnterStateStr));
		Exec.LogClear();

		// Transition from State0 and tries to select State1.
		// This tests that data from current shared active states (Root) is available during state selection.
		Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Task0 should complete"), Exec.Expect(Task0.GetName(), StateCompletedStr));
		AITEST_TRUE(TEXT("StateTree Task1A should enter state"), Exec.Expect(Task1A.GetName(), EnterStateStr));
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_TransitionNextWithParentData, "System.StateTree.Transition.NextWithParentData");

struct FStateTreeTest_TransitionGlobalDataView : FStateTreeTestBase
{
	// Tests that the global eval and task dataviews are kept up to date when transitioning from  
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		UStateTreeState& StateA = Root.AddChildState(FName(TEXT("A")));
		UStateTreeState& StateB = Root.AddChildState(FName(TEXT("B")));

		auto& EvalA = EditorData.AddEvaluator<FTestEval_A>(FName(TEXT("Eval")));
		EvalA.GetInstanceData().IntA = 42;
		auto& GlobalTask = EditorData.AddGlobalTask<FTestTask_PrintValue>(FName(TEXT("Global")));
		GlobalTask.GetInstanceData().Value = 123;
		
		// State A
		auto& Task0 = StateA.AddTask<FTestTask_Stand>(FName(TEXT("Task0")));
		StateA.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &StateB);

		// State B
		auto& Task1 = StateB.AddTask<FTestTask_PrintValue>(FName(TEXT("Task1")));
		EditorData.AddPropertyBinding(EvalA, TEXT("IntA"), Task1, TEXT("Value"));
		auto& Task2 = StateB.AddTask<FTestTask_PrintValue>(FName(TEXT("Task2")));
		EditorData.AddPropertyBinding(GlobalTask, TEXT("Value"), Task2, TEXT("Value"));

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);

		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* EnterState42Str(TEXT("EnterState42"));
		const TCHAR* EnterState123Str(TEXT("EnterState123"));

		// Start and enter state
		Status = Exec.Start();
		AITEST_TRUE(TEXT("StateTree Task0 should enter state"), Exec.Expect(Task0.GetName(), EnterStateStr));
		Exec.LogClear();

		// Transition from StateA to StateB, Task0 should enter state with evaluator value copied.
		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Task0 should enter state with value 42"), Exec.Expect(Task1.GetName(), EnterState42Str));
		AITEST_TRUE(TEXT("StateTree Task1 should enter state with value 123"), Exec.Expect(Task2.GetName(), EnterState123Str));
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_TransitionGlobalDataView, "System.StateTree.Transition.GlobalDataView");

struct FStateTreeTest_TransitionDelay : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		const FGameplayTag Tag = GetTestTag1();

		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		UStateTreeState& StateA = Root.AddChildState(FName(TEXT("A")));
		UStateTreeState& StateB = Root.AddChildState(FName(TEXT("B")));

		// State A
		auto& Task0 = StateA.AddTask<FTestTask_Stand>(FName(TEXT("Task0")));
		Task0.GetNode().TicksToCompletion = 100;
		
		FStateTreeTransition& Transition = StateA.AddTransition(EStateTreeTransitionTrigger::OnEvent, EStateTreeTransitionType::GotoState, &StateB);
		Transition.bDelayTransition = true;
		Transition.DelayDuration = 0.15f;
		Transition.DelayRandomVariance = 0.0f;
		Transition.RequiredEvent.Tag = Tag;

		// State B
		auto& Task1 = StateB.AddTask<FTestTask_Stand>(FName(TEXT("Task1")));
		Task1.GetNode().TicksToCompletion = 100;

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);

		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		const TCHAR* TickStr(TEXT("Tick"));
		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* ExitStateStr(TEXT("ExitState"));
		const TCHAR* StateCompletedStr(TEXT("StateCompleted"));

		// Start and enter state
		Status = Exec.Start();
		AITEST_TRUE(TEXT("StateTree Task0 should enter state"), Exec.Expect(Task0.GetName(), EnterStateStr));
		Exec.LogClear();

		// This should cause delayed transition.
		Exec.SendEvent(Tag);
		
		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Task0 should tick"), Exec.Expect(Task0.GetName(), TickStr));
		Exec.LogClear();

		// Should have execution frames
		AITEST_TRUE(TEXT("Should have active frames"), InstanceData.GetExecutionState()->ActiveFrames.Num() > 0);

		// Should have delayed transitions
		const int32 NumDelayedTransitions0 = InstanceData.GetExecutionState()->DelayedTransitions.Num();
		AITEST_EQUAL(TEXT("Should have a delayed transition"), NumDelayedTransitions0, 1);

		// Tick and expect a delayed transition. 
		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Task0 should tick"), Exec.Expect(Task0.GetName(), TickStr));
		Exec.LogClear();

		const int32 NumDelayedTransitions1 = InstanceData.GetExecutionState()->DelayedTransitions.Num();
		AITEST_EQUAL(TEXT("Should have a delayed transition"), NumDelayedTransitions1, 1);

		// Should complete delayed transition.
		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Task0 should exit state"), Exec.Expect(Task0.GetName(), ExitStateStr));
		AITEST_TRUE(TEXT("StateTree Task1 should enter state"), Exec.Expect(Task1.GetName(), EnterStateStr));
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_TransitionDelay, "System.StateTree.Transition.Delay");

struct FStateTreeTest_TransitionReactivateTargetState : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		//Root
		// State0

		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		FInstancedPropertyBag& RootPropertyBag = GetRootPropertyBag(EditorData);
		RootPropertyBag.AddProperty(FName(TEXT("Int")), EPropertyBagPropertyType::Int32);
		RootPropertyBag.SetValueInt32(FName(TEXT("Int")), 1);

		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		UStateTreeState& State0 = Root.AddChildState(FName(TEXT("State0")));

		// Note: Task does not reselect by default, so will not trigger entry state on reselect
		auto& Task0 = State0.AddTask<FTestTask_StandNoStateChangeOnReselect>(FName(TEXT("Task0")));
		Task0.GetNode().TicksToCompletion = 100;

		// Reactivate transtion back to root, by default will not fire due to condition. But should reactivate / re-trigger enter state when activated.
		FStateTreeTransition& ReactivateRootTransition = State0.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::GotoState, &Root);
		TStateTreeEditorNode<FStateTreeCompareIntCondition>& TransIntCond = ReactivateRootTransition.AddConditionWithOuter<FStateTreeCompareIntCondition>(&State0, UE::StateTree::EComparisonOperator::Equal);
		TransIntCond.GetInstanceData().Right = 0;
		EditorData.AddPropertyBinding(
			FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("Int")),
			FPropertyBindingPath(TransIntCond.ID, TEXT("Left")));
		ReactivateRootTransition.ReactivateTargetState = EStateTreeTransitionChangeTypeRules::ForceChanged;

		// Transition back to root, will always fire, but will not reactivate / re-trigger enter state
		FStateTreeTransition& DefaultRootTransition = State0.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::GotoState, &Root);

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE("StateTree should get compiled", bResult);

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE("StateTree should init", bInitSucceeded);

		const TCHAR* TickStr(TEXT("Tick"));
		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* ExitStateStr(TEXT("ExitState"));
		const TCHAR* StateCompletedStr(TEXT("StateCompleted"));

		FInstancedPropertyBag GlobalParameters = StateTree.GetDefaultParameters();

		// Start and enter state
		Status = Exec.Start();
		AITEST_TRUE(TEXT("StateTree Task0 should enter state"), Exec.Expect(Task0.GetName(), EnterStateStr));
		Exec.LogClear();

		// Transition from State1A -> State1A non-reactivate
		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Task0 1st tick count 1"), Exec.Expect(Task0.GetName(), TEXT("CurrentTick=1")));
		AITEST_FALSE(TEXT("StateTree Task0 should not re-enter state 1st"), Exec.Expect(Task0.GetName(), EnterStateStr));
		Exec.LogClear();

		// Transition from State1A -> State1A non-reactivate - should still hold after second tick 
		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Task0 2nd tick count 2"), Exec.Expect(Task0.GetName(), TEXT("CurrentTick=2")));
		AITEST_FALSE(TEXT("StateTree Task0 should not re-enter state 2nd"), Exec.Expect(Task0.GetName(), EnterStateStr));
		Exec.LogClear();

		GlobalParameters.SetValueInt32(FName(TEXT("Int")), 0);
		InstanceData.GetMutableStorage().SetGlobalParameters(GlobalParameters.GetValue());

		// Transition from State1A -> State1A reactivate
		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Task0 tick before 1st re-enter tick count 3"), Exec.Expect(Task0.GetName(), TEXT("CurrentTick=3")));
		AITEST_TRUE(TEXT("StateTree Task0 should re-enter state 1st"), Exec.Expect(Task0.GetName(), EnterStateStr));
		Exec.LogClear();

		// Transition from State1A -> State1A reactivate  should still hold after second tick 
		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Task0 tick after 1st re-enter tick count 1"), Exec.Expect(Task0.GetName(), TEXT("CurrentTick=1")));
		AITEST_TRUE(TEXT("StateTree Task0 should re-enter state 2nd"), Exec.Expect(Task0.GetName(), EnterStateStr));
		Exec.LogClear();

		GlobalParameters.SetValueInt32(FName(TEXT("Int")), 1);
		InstanceData.GetMutableStorage().SetGlobalParameters(GlobalParameters.GetValue());

		// Transition from State1A -> State1A final non-reactivate after condition reset
		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Task0 tick after 2nd re-enter tick count 1"), Exec.Expect(Task0.GetName(), TEXT("CurrentTick=1")));
		AITEST_FALSE(TEXT("StateTree Task0 should not re-enter state final"), Exec.Expect(Task0.GetName(), EnterStateStr));
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_TransitionReactivateTargetState, "System.StateTree.Transition.ReactivateTargetState");

struct FStateTreeTest_TransitionDelayZero : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		const FGameplayTag Tag = GetTestTag1();

		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		UStateTreeState& StateA = Root.AddChildState(FName(TEXT("A")));
		UStateTreeState& StateB = Root.AddChildState(FName(TEXT("B")));

		// State A
		auto& Task0 = StateA.AddTask<FTestTask_Stand>(FName(TEXT("Task0")));
		Task0.GetNode().TicksToCompletion = 100;
		
		FStateTreeTransition& Transition = StateA.AddTransition(EStateTreeTransitionTrigger::OnEvent, EStateTreeTransitionType::GotoState, &StateB);
		Transition.bDelayTransition = true;
		Transition.DelayDuration = 0.0f;
		Transition.DelayRandomVariance = 0.0f;
		Transition.RequiredEvent.Tag = Tag;

		// State B
		auto& Task1 = StateB.AddTask<FTestTask_Stand>(FName(TEXT("Task1")));
		Task1.GetNode().TicksToCompletion = 100;

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);

		AITEST_TRUE("StateTree should get compiled", bResult);

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		const TCHAR* TickStr(TEXT("Tick"));
		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* ExitStateStr(TEXT("ExitState"));
		const TCHAR* StateCompletedStr(TEXT("StateCompleted"));

		// Start and enter state
		Status = Exec.Start();
		AITEST_TRUE(TEXT("StateTree Task0 should enter state"), Exec.Expect(Task0.GetName(), EnterStateStr));
		Exec.LogClear();

		// This should cause delayed transition. Because the time is 0, it should happen immediately.
		Exec.SendEvent(Tag);
		
		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("StateTree Task0 should exit state"), Exec.Expect(Task0.GetName(), ExitStateStr));
		AITEST_TRUE(TEXT("StateTree Task1 should enter state"), Exec.Expect(Task1.GetName(), EnterStateStr));
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_TransitionDelayZero, "System.StateTree.Transition.DelayZero");

struct FStateTreeTest_PassingTransitionEventToStateSelection : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));

		FPropertyBindingPath PathToPayloadMember;
		{
			const bool bParseResult = PathToPayloadMember.FromString(TEXT("Payload.A"));

			AITEST_TRUE(TEXT("Parsing path should succeeed"), bParseResult);

			FStateTreeEvent EventWithPayload;
			EventWithPayload.Payload = FInstancedStruct::Make<FStateTreeTest_PropertyStructA>();
			const bool bUpdateSegments = PathToPayloadMember.UpdateSegmentsFromValue(FStateTreeDataView(FStructView::Make(EventWithPayload)));
			AITEST_TRUE(TEXT("Updating segments should succeeed"), bUpdateSegments);
		}

		// This state shouldn't be selected, because transition's condition and state's enter condition exlude each other.
		UStateTreeState& StateA = Root.AddChildState(FName(TEXT("A")));
		StateA.bHasRequiredEventToEnter  = true;
		StateA.RequiredEventToEnter.PayloadStruct = FStateTreeTest_PropertyStructA::StaticStruct();
		auto& TaskA = StateA.AddTask<FTestTask_Stand>(FName(TEXT("TaskA")));
		TStateTreeEditorNode<FStateTreeCompareIntCondition>& AIntCond = StateA.AddEnterCondition<FStateTreeCompareIntCondition>(UE::StateTree::EComparisonOperator::Equal);
		AIntCond.GetInstanceData().Right = 0;
		EditorData.AddPropertyBinding(
			FPropertyBindingPath(StateA.GetEventID(), PathToPayloadMember.GetSegments()),
			FPropertyBindingPath(AIntCond.ID, TEXT("Left")));

		// This state should be selected as the sent event fullfils both transition's condition and state's enter condition.
		UStateTreeState& StateB = Root.AddChildState(FName(TEXT("B")));
		StateB.bHasRequiredEventToEnter  = true;
		StateB.RequiredEventToEnter.PayloadStruct = FStateTreeTest_PropertyStructA::StaticStruct();
		auto& TaskB = StateB.AddTask<FTestTask_PrintValue>(FName(TEXT("TaskB")));
		// Test copying data from the state event. The condition properties are copied from temp instance data during selection, this gets copied from active instance data.
		TaskB.GetInstanceData().Value = -1; // Initially -1, expected to be overridden by property binding below. 
		EditorData.AddPropertyBinding(
			FPropertyBindingPath(StateB.GetEventID(), PathToPayloadMember.GetSegments()),
			FPropertyBindingPath(TaskB.ID, TEXT("Value")));
		
		TStateTreeEditorNode<FStateTreeCompareIntCondition>& BIntCond = StateB.AddEnterCondition<FStateTreeCompareIntCondition>(UE::StateTree::EComparisonOperator::Equal);
		BIntCond.GetInstanceData().Right = 1;
		EditorData.AddPropertyBinding(
			FPropertyBindingPath(StateB.GetEventID(), PathToPayloadMember.GetSegments()),
			FPropertyBindingPath(BIntCond.ID, TEXT("Left")));

		// This state should be selected only initially when there's not event in the queue.
		UStateTreeState& StateInitial = Root.AddChildState(FName(TEXT("Initial")));
		auto& TaskInitial = StateInitial.AddTask<FTestTask_Stand>(FName(TEXT("TaskInitial")));
		// Transition from Initial -> StateA
		FStateTreeTransition& TransA = StateInitial.AddTransition(EStateTreeTransitionTrigger::OnEvent, FGameplayTag(), EStateTreeTransitionType::GotoState, &StateA);
		TransA.RequiredEvent.PayloadStruct = FStateTreeTest_PropertyStructA::StaticStruct();
		TStateTreeEditorNode<FStateTreeCompareIntCondition>& TransAIntCond = TransA.AddConditionWithOuter<FStateTreeCompareIntCondition>(&StateInitial, UE::StateTree::EComparisonOperator::Equal);
		TransAIntCond.GetInstanceData().Right = 1;
		EditorData.AddPropertyBinding(
			FPropertyBindingPath(TransA.GetEventID(), PathToPayloadMember.GetSegments()),
			FPropertyBindingPath(TransAIntCond.ID, TEXT("Left")));
		// Transition from Initial -> StateB
		FStateTreeTransition& TransB = StateInitial.AddTransition(EStateTreeTransitionTrigger::OnEvent, FGameplayTag(), EStateTreeTransitionType::GotoState, &StateB);
		TransB.RequiredEvent.PayloadStruct = FStateTreeTest_PropertyStructA::StaticStruct();
		TStateTreeEditorNode<FStateTreeCompareIntCondition>& TransBIntCond = TransB.AddConditionWithOuter<FStateTreeCompareIntCondition>(&StateInitial, UE::StateTree::EComparisonOperator::Equal);
		TransBIntCond.GetInstanceData().Right = 1;
		EditorData.AddPropertyBinding(
			FPropertyBindingPath(TransB.GetEventID(), PathToPayloadMember.GetSegments()),
			FPropertyBindingPath(TransBIntCond.ID, TEXT("Left")));

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);

		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		const TCHAR* EnterStateStr(TEXT("EnterState"));

		Status = Exec.Start();
		AITEST_TRUE(TEXT("StateTree TaskInitial should enter state"), Exec.Expect(TaskInitial.GetName(), EnterStateStr));
		Exec.LogClear();

		// The conditions test for payload Value=1, the first event should not trigger transition. 
		Exec.SendEvent(GetTestTag1(), FConstStructView::Make(FStateTreeTest_PropertyStructA{0}));
		Exec.SendEvent(GetTestTag1(), FConstStructView::Make(FStateTreeTest_PropertyStructA{1}));
		Status = Exec.Tick(0.1f);

		AITEST_FALSE(TEXT("StateTree TaskA should not enter state"), Exec.Expect(TaskA.GetName(), EnterStateStr));
		AITEST_TRUE(TEXT("StateTree TaskB should enter state"), Exec.Expect(TaskB.GetName(), TEXT("EnterState1"))); // TaskB decorates "EnterState" with value from the payload.
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_PassingTransitionEventToStateSelection, "System.StateTree.Transition.PassingTransitionEventToStateSelection");

struct FStateTreeTest_FollowTransitions : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		//Root
		// Trans
		// A
		// B
		// C
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		FInstancedPropertyBag& RootPropertyBag = GetRootPropertyBag(EditorData);
		RootPropertyBag.AddProperty(FName(TEXT("Int")), EPropertyBagPropertyType::Int32);
		RootPropertyBag.SetValueInt32(FName(TEXT("Int")), 1);

		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		UStateTreeState& StateTrans = Root.AddChildState(FName(TEXT("Trans")));
		UStateTreeState& StateA = Root.AddChildState(FName(TEXT("A")));
		UStateTreeState& StateB = Root.AddChildState(FName(TEXT("B")));
		UStateTreeState& StateC = Root.AddChildState(FName(TEXT("C")));

		// Root

		// Trans
		{
			StateTrans.SelectionBehavior = EStateTreeStateSelectionBehavior::TryFollowTransitions;

			{
				// This transition should be skipped due to the condition
				FStateTreeTransition& TransA = StateTrans.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::GotoState, &StateA);
				TStateTreeEditorNode<FStateTreeCompareIntCondition>& TransIntCond = TransA.AddConditionWithOuter<FStateTreeCompareIntCondition>(&StateTrans, UE::StateTree::EComparisonOperator::Equal);
				TransIntCond.GetInstanceData().Right = 0;
				EditorData.AddPropertyBinding(
					FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("Int")),
					FPropertyBindingPath(TransIntCond.ID, TEXT("Left")));
			}

			{
				// This transition leads to selection, but will be overridden.
				FStateTreeTransition& TransB = StateTrans.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::GotoState, &StateB);
				TransB.Priority = EStateTreeTransitionPriority::Normal;
				TStateTreeEditorNode<FStateTreeCompareIntCondition>& TransIntCond = TransB.AddConditionWithOuter<FStateTreeCompareIntCondition>(&StateTrans, UE::StateTree::EComparisonOperator::Equal);
				TransIntCond.GetInstanceData().Right = 1;
				EditorData.AddPropertyBinding(
					FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("Int")),
					FPropertyBindingPath(TransIntCond.ID, TEXT("Left")));
			}

			{
				// This transition is selected, should override previous one due to priority.
				FStateTreeTransition& TransC = StateTrans.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::GotoState, &StateC);
				TransC.Priority = EStateTreeTransitionPriority::High;
				TStateTreeEditorNode<FStateTreeCompareIntCondition>& TransIntCond = TransC.AddConditionWithOuter<FStateTreeCompareIntCondition>(&StateTrans, UE::StateTree::EComparisonOperator::Equal);
				TransIntCond.GetInstanceData().Right = 1;
				EditorData.AddPropertyBinding(
					FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("Int")),
					FPropertyBindingPath(TransIntCond.ID, TEXT("Left")));
			}
		}

		auto& TaskA = StateA.AddTask<FTestTask_Stand>(FName(TEXT("TaskA")));
		auto& TaskB = StateB.AddTask<FTestTask_Stand>(FName(TEXT("TaskB")));
		auto& TaskC = StateC.AddTask<FTestTask_Stand>(FName(TEXT("TaskC")));

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);

		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		const TCHAR* TickStr = TEXT("Tick");
		const TCHAR* EnterStateStr = TEXT("EnterState");
		const TCHAR* ExitStateStr = TEXT("ExitState");

		Status = Exec.Start();
		AITEST_FALSE(TEXT("StateTree TaskA should not enter state"), Exec.Expect(TaskA.GetName(), EnterStateStr));
		AITEST_FALSE(TEXT("StateTree TaskB should not enter state"), Exec.Expect(TaskB.GetName(), EnterStateStr));
		AITEST_TRUE(TEXT("StateTree TaskC should enter state"), Exec.Expect(TaskC.GetName(), EnterStateStr));
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_FollowTransitions, "System.StateTree.Transition.FollowTransitions");

struct FStateTreeTest_InfiniteLoop : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
		FInstancedPropertyBag& RootPropertyBag = GetRootPropertyBag(EditorData);
		RootPropertyBag.AddProperty(FName(TEXT("Int")), EPropertyBagPropertyType::Int32);
		RootPropertyBag.SetValueInt32(FName(TEXT("Int")), 1);
		
		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		UStateTreeState& StateA = Root.AddChildState(FName(TEXT("A")));
		UStateTreeState& StateB = StateA.AddChildState(FName(TEXT("B")));

		// Root

		// State A
		{
			StateA.SelectionBehavior = EStateTreeStateSelectionBehavior::TryFollowTransitions;
			{
				// A -> B
				FStateTreeTransition& Trans = StateA.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::GotoState, &StateB);
				TStateTreeEditorNode<FStateTreeCompareIntCondition>& TransIntCond = Trans.AddConditionWithOuter<FStateTreeCompareIntCondition>(&StateA, UE::StateTree::EComparisonOperator::Equal);
				TransIntCond.GetInstanceData().Right = 1;
				EditorData.AddPropertyBinding(
					FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("Int")),
					FPropertyBindingPath(TransIntCond.ID, TEXT("Left")));
			}
		}

		// State B
		{
			StateB.SelectionBehavior = EStateTreeStateSelectionBehavior::TryFollowTransitions;
			{
				// B -> A
				FStateTreeTransition& Trans = StateB.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::GotoState, &StateA);
				TStateTreeEditorNode<FStateTreeCompareIntCondition>& TransIntCond = Trans.AddConditionWithOuter<FStateTreeCompareIntCondition>(&StateB, UE::StateTree::EComparisonOperator::Equal);
				TransIntCond.GetInstanceData().Right = 1;
				EditorData.AddPropertyBinding(
					FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("Int")),
					FPropertyBindingPath(TransIntCond.ID, TEXT("Left")));
			}
		}
		
		auto& TaskA = StateA.AddTask<FTestTask_Stand>(FName(TEXT("TaskA")));
		auto& TaskB = StateB.AddTask<FTestTask_Stand>(FName(TEXT("TaskB")));

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);

		AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

		const TCHAR* TickStr = TEXT("Tick");
		const TCHAR* EnterStateStr = TEXT("EnterState");
		const TCHAR* ExitStateStr = TEXT("ExitState");

		GetTestRunner().AddExpectedMessage(TEXT("Loop detected when trying to select state"), ELogVerbosity::Warning, EAutomationExpectedErrorFlags::Contains, 1);
		GetTestRunner().AddExpectedError(TEXT("Failed to select initial state"), EAutomationExpectedErrorFlags::Contains, 1);
		
		Status = Exec.Start();
		AITEST_EQUAL(TEXT("Start should fail"), Status, EStateTreeRunStatus::Failed);
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FStateTreeTest_InfiniteLoop, "System.StateTree.Transition.InfiniteLoop");

struct FStateTreeTest_RegularTransitions : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		//Tree 1
		//  Global task and parameter
		//	RootA
		//		StateB -> Next
		//		StateC -> Next
		//		StateD -> Next
		//		StateE -> Next
		//		StateF -> Next
		//		StateG -> Succeeded

		FStateTreeCompilerLog Log;

		// Main asset
		UStateTree& StateTree = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
			FGuid RootParameter_ValueID;
			{
				// Parameters
				FInstancedPropertyBag& RootPropertyBag = GetRootPropertyBag(EditorData);
				RootPropertyBag.AddProperty("Value", EPropertyBagPropertyType::Int32);
				RootPropertyBag.SetValueInt32("Value", -111);
				RootParameter_ValueID = RootPropertyBag.FindPropertyDescByName("Value")->ID;

				TStateTreeEditorNode<FTestTask_PrintValue>& GlobalTask = EditorData.AddGlobalTask<FTestTask_PrintValue>("GlobalTask");
				GlobalTask.GetInstanceData().Value = -1;
				EditorData.AddPropertyBinding(FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("Value")), FPropertyBindingPath(GlobalTask.ID, TEXT("Value")));
			}

			UStateTreeState& Root = EditorData.AddSubTree("RootA");
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& Task = Root.AddTask<FTestTask_PrintValue>("TaskA");
				Task.GetInstanceData().Value = -1;
				EditorData.AddPropertyBinding(FPropertyBindingPath(EditorData.GetRootParametersGuid(), TEXT("Value")), FPropertyBindingPath(Task.ID, TEXT("Value")));
			}
			{
				UStateTreeState& StateB = Root.AddChildState("StateB", EStateTreeStateType::State);
				TStateTreeEditorNode<FTestTask_PrintValue>& Task = StateB.AddTask<FTestTask_PrintValue>("TaskB");
				Task.GetInstanceData().Value = 1;
				FStateTreeTransition& Transition = StateB.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::NextState);
				Transition.bDelayTransition = true;
				Transition.DelayDuration = 1.0;
			}
			{
				UStateTreeState& StateB = Root.AddChildState("StateC", EStateTreeStateType::State);
				TStateTreeEditorNode<FTestTask_PrintValue>& Task = StateB.AddTask<FTestTask_PrintValue>("TaskC");
				Task.GetInstanceData().Value = 2;
				FStateTreeTransition& Transition = StateB.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::NextState);
				Transition.bDelayTransition = true;
				Transition.DelayDuration = 1.0;
			}
			{
				UStateTreeState& StateD = Root.AddChildState("StateD", EStateTreeStateType::State);
				TStateTreeEditorNode<FTestTask_PrintValue>& Task = StateD.AddTask<FTestTask_PrintValue>("TaskD");
				Task.GetInstanceData().Value = 3;
				FStateTreeTransition& Transition = StateD.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::GotoState, &Root);
				Transition.bDelayTransition = true;
				Transition.DelayDuration = 1.0;
			}

			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE(TEXT("StateTree should get compiled"), bResult);
		}

		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* ExitStateStr(TEXT("ExitState"));

		{
			EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);

			FInstancedPropertyBag Parameters;
			Parameters.MigrateToNewBagInstance(StateTree.GetDefaultParameters());
			Parameters.SetValueInt32("Value", 111);

			Status = Exec.Start(Parameters.GetValue());
			AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Start should enter Global tasks"), Exec.Expect("GlobalTask", TEXT("EnterState111")));
			AITEST_TRUE(TEXT("Start should enter StateA"), Exec.Expect("TaskA", TEXT("EnterState111")));
			AITEST_TRUE(TEXT("Start should enter StateB"), Exec.Expect("TaskB", TEXT("EnterState1")));
			Exec.LogClear();

			Status = Exec.Tick(1.5f); // over tick, should trigger
			AITEST_EQUAL(TEXT("1st Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("1st Tick should tick Global tasks"), Exec.Expect("GlobalTask", TEXT("Tick111")));
			AITEST_TRUE(TEXT("1st Tick should tick StateA"), Exec.Expect("TaskA", TEXT("Tick111")));
			AITEST_TRUE(TEXT("1st Tick should tick StateB"), Exec.Expect("TaskB", TEXT("Tick1")));
			Exec.LogClear();

			// B go to C
			Status = Exec.Tick(1.0f);
			AITEST_EQUAL(TEXT("2nd Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("2nd Tick should tick Global tasks"), Exec.Expect("GlobalTask", TEXT("Tick111")));
			AITEST_TRUE(TEXT("2nd Tick should tick StateA"), Exec.Expect("TaskA", TEXT("Tick111")));
			AITEST_TRUE(TEXT("2nd Tick should tick the StateB"), Exec.Expect("TaskB", TEXT("Tick1")));
			AITEST_TRUE(TEXT("2nd Tick should exit the StateB"), Exec.Expect("TaskB", TEXT("ExitState1")));
			AITEST_TRUE(TEXT("2nd Tick should enter the StateC"), Exec.Expect("TaskC", TEXT("EnterState2")));
			AITEST_FALSE(TEXT("2nd Tick should not exit the StateA"), Exec.Expect("TaskA", TEXT("ExitState111")));
			AITEST_FALSE(TEXT("2nd Tick should not exit the Global tasks"), Exec.Expect("GlobalTask", TEXT("ExitState111")));
			Exec.LogClear();

			Status = Exec.Tick(1.0f);
			AITEST_EQUAL(TEXT("3rd Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("3rd Tick should tick Global tasks"), Exec.Expect("GlobalTask", TEXT("Tick111")));
			AITEST_TRUE(TEXT("3rd Tick should tick StateA"), Exec.Expect("TaskA", TEXT("Tick111")));
			AITEST_TRUE(TEXT("3rd Tick should tick StateC"), Exec.Expect("TaskC", TEXT("Tick2")));
			AITEST_FALSE(TEXT("3th Tick should not exit StateC"), Exec.Expect("TaskD", TEXT("ExitState2")));
			Exec.LogClear();

			// C go to D
			Status = Exec.Tick(1.0f);
			AITEST_EQUAL(TEXT("4th Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("4th Tick should tick Global tasks"), Exec.Expect("GlobalTask", TEXT("Tick111")));
			AITEST_TRUE(TEXT("4th Tick should tick StateA"), Exec.Expect("TaskA", TEXT("Tick111")));
			AITEST_TRUE(TEXT("4th Tick should tick the StateC"), Exec.Expect("TaskC", TEXT("Tick2")));
			AITEST_TRUE(TEXT("4th Tick should exit the StateC"), Exec.Expect("TaskC", TEXT("ExitState2")));
			AITEST_TRUE(TEXT("4th Tick should enter the StateD"), Exec.Expect("TaskD", TEXT("EnterState3")));
			AITEST_FALSE(TEXT("4th Tick should not exit the StateA"), Exec.Expect("TaskA", TEXT("ExitState111")));
			AITEST_FALSE(TEXT("4th Tick should not exit the Global tasks"), Exec.Expect("GlobalTask", TEXT("ExitState111")));
			Exec.LogClear();

			Status = Exec.Tick(0.001f);
			AITEST_EQUAL(TEXT("5th Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("5th Tick should tick Global tasks"), Exec.Expect("GlobalTask", TEXT("Tick111")));
			AITEST_TRUE(TEXT("5th Tick should tick StateA"), Exec.Expect("TaskA", TEXT("Tick111")));
			AITEST_TRUE(TEXT("5th Tick should tick StateD"), Exec.Expect("TaskD", TEXT("Tick3")));
			AITEST_FALSE(TEXT("5th Tick should not exit StateD"), Exec.Expect("TaskD", TEXT("ExitState3")));
			Exec.LogClear();

			// D go to root
			Status = Exec.Tick(1.0f);
			AITEST_EQUAL(TEXT("6th Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("6th Tick should tick Global tasks"), Exec.Expect("GlobalTask", TEXT("Tick111")));
			AITEST_TRUE(TEXT("6th Tick should tick StateA"), Exec.Expect("TaskA", TEXT("Tick111")));
			AITEST_TRUE(TEXT("6th Tick should tick StateD"), Exec.Expect("TaskD", TEXT("Tick3")));
			AITEST_TRUE(TEXT("6th Tick should exit the StateD"), Exec.Expect("TaskD", TEXT("ExitState3")));
			AITEST_FALSE(TEXT("6th Tick should not exit the Global tasks"), Exec.Expect("GlobalTask", TEXT("ExitState111")));
			AITEST_FALSE(TEXT("6th Tick should not enter the Global tasks"), Exec.Expect("GlobalTask", TEXT("EnterState111")));
			AITEST_TRUE(TEXT("6th Tick should exit the StateA"), Exec.Expect("TaskA", TEXT("ExitState=Sustained")));
			AITEST_TRUE(TEXT("6th Tick should enter the StateA"), Exec.Expect("TaskA", TEXT("EnterState=Sustained")));
			AITEST_TRUE(TEXT("6th Tick should enter the StateB"), Exec.Expect("TaskB", TEXT("EnterState1")));
			AITEST_TRUE(TEXT("6th Tick should enter the StateB"), Exec.Expect("TaskB", TEXT("EnterState=Changed")));
			Exec.LogClear();

			Status = Exec.Tick(1.0f);
			AITEST_EQUAL(TEXT("7th Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("7th Tick should tick Global tasks"), Exec.Expect("GlobalTask", TEXT("Tick111")));
			AITEST_TRUE(TEXT("7th Tick should tick StateA"), Exec.Expect("TaskA", TEXT("Tick111")));
			AITEST_TRUE(TEXT("7th Tick should tick StateB"), Exec.Expect("TaskB", TEXT("Tick1")));
			Exec.LogClear();

			Exec.Stop();
			AITEST_TRUE(TEXT("Stop Tick should exit the StateB"), Exec.Expect("TaskB", TEXT("ExitState1")));
			AITEST_TRUE(TEXT("Stop Tick should exit the StateA"), Exec.Expect("TaskA", TEXT("ExitState111")));
			AITEST_TRUE(TEXT("Stop should tick Global tasks"), Exec.Expect("GlobalTask", TEXT("ExitState111")));
			Exec.LogClear();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_RegularTransitions, "System.StateTree.Transition.RegularTransitions");

struct FStateTreeTest_RequestTransition : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		//Tree 1
		//RootA
		// StateB
		//  StateC
		//   StateD
		//    StateE
		//     StateF
		//  StateI
		//   StateJ
		// StateX
		//  StateY
		//RootO
		// StateP

		FStateTreeCompilerLog Log;

		enum class ECustomFunctionToRun
		{
			None,
			TransitionD_To_E,
			TransitionB_To_I,
			TransitionB_To_C,
			TransitionC_To_B,
			TransitionB_To_X,
			TransitionB_To_B,
			TransitionB_To_P,
		} TransitionToExecute = ECustomFunctionToRun::None;
		struct FStateHandle
		{
			FStateTreeStateHandle B;
			FStateTreeStateHandle C;
			FStateTreeStateHandle E;
			FStateTreeStateHandle I;
			FStateTreeStateHandle X;
			FStateTreeStateHandle P;
			FGuid B_ID;
			FGuid C_ID;
			FGuid E_ID;
			FGuid I_ID;
			FGuid X_ID;
			FGuid P_ID;
			bool bFinishTasksB = false;
			bool bFinishTasksC = false;
			bool bFinishTasksD = false;
		} AllStateHandle;

		// Main asset
		UStateTree& StateTree = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);
			UStateTreeState& StateA = EditorData.AddSubTree("StateA");
			StateA.AddTask<FTestTask_PrintValue>("StateATask");
			
			UStateTreeState& StateB = StateA.AddChildState("StateB", EStateTreeStateType::State);
			FTestTask_PrintValue& TaskB = StateB.AddTask<FTestTask_PrintValue>("StateBTask").GetNode();
			AllStateHandle.B_ID = StateB.ID;

			UStateTreeState& StateC = StateB.AddChildState("StateC", EStateTreeStateType::State);
			FTestTask_PrintValue& TaskC = StateC.AddTask<FTestTask_PrintValue>("StateCTask").GetNode();
			AllStateHandle.C_ID = StateC.ID;

			UStateTreeState& StateD = StateC.AddChildState("StateD", EStateTreeStateType::State);
			FTestTask_PrintValue& TaskD = StateD.AddTask<FTestTask_PrintValue>("StateDTask").GetNode();
			StateD.SelectionBehavior = EStateTreeStateSelectionBehavior::TryEnterState;
			
			UStateTreeState& StateE = StateD.AddChildState("StateE", EStateTreeStateType::State);
			StateE.AddTask<FTestTask_PrintValue>("StateETask");
			AllStateHandle.E_ID = StateE.ID;
			
			UStateTreeState& StateF = StateE.AddChildState("StateF", EStateTreeStateType::State);
			StateF.AddTask<FTestTask_PrintValue>("StateFTask");

			UStateTreeState& StateI = StateB.AddChildState("StateI", EStateTreeStateType::State);
			StateI.AddTask<FTestTask_PrintValue>("StateITask");
			AllStateHandle.I_ID = StateI.ID;
			
			UStateTreeState& StateJ = StateI.AddChildState("StateJ", EStateTreeStateType::State);
			StateJ.AddTask<FTestTask_PrintValue>("StateJTask");

			UStateTreeState& StateX = StateA.AddChildState("StateX", EStateTreeStateType::State);
			StateX.AddTask<FTestTask_PrintValue>("StateXTask");
			AllStateHandle.X_ID = StateX.ID;

			UStateTreeState& StateY = StateX.AddChildState("StateY", EStateTreeStateType::State);
			StateY.AddTask<FTestTask_PrintValue>("StateYTask");

			UStateTreeState& StateO = EditorData.AddSubTree("StateO");
			StateO.AddTask<FTestTask_PrintValue>("StateOTask");

			UStateTreeState& StateP = StateO.AddChildState("StateP", EStateTreeStateType::State);
			StateP.AddTask<FTestTask_PrintValue>("StatePTask");
			AllStateHandle.P_ID = StateP.ID;

			TaskB.CustomTickFunc = [&TransitionToExecute, &AllStateHandle]
				(FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
				{
					if (TransitionToExecute == ECustomFunctionToRun::TransitionB_To_I)
					{
						Context.RequestTransition(AllStateHandle.I);
					}
					if (TransitionToExecute == ECustomFunctionToRun::TransitionB_To_C)
					{
						Context.RequestTransition(AllStateHandle.C);
					}
					if (TransitionToExecute == ECustomFunctionToRun::TransitionB_To_X)
					{
						Context.RequestTransition(AllStateHandle.X);
					}
					if (TransitionToExecute == ECustomFunctionToRun::TransitionB_To_B)
					{
						Context.RequestTransition(AllStateHandle.B);
					}
					if (TransitionToExecute == ECustomFunctionToRun::TransitionB_To_P)
					{
						Context.RequestTransition(AllStateHandle.P);
					}
					if (AllStateHandle.bFinishTasksB)
					{
						Context.FinishTask(*Task, EStateTreeFinishTaskType::Succeeded);
					}
				};

			TaskC.CustomTickFunc = [&TransitionToExecute, &AllStateHandle]
				(FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
				{
					if (TransitionToExecute == ECustomFunctionToRun::TransitionC_To_B)
					{
						Context.RequestTransition(AllStateHandle.B);
					}
					if (AllStateHandle.bFinishTasksC)
					{
						Context.FinishTask(*Task, EStateTreeFinishTaskType::Succeeded);
					}
				};

			TaskD.CustomTickFunc = [&TransitionToExecute, &AllStateHandle]
				(FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
				{
					if (TransitionToExecute == ECustomFunctionToRun::TransitionD_To_E)
					{
						Context.RequestTransition(AllStateHandle.E);
					}

					if (AllStateHandle.bFinishTasksD)
					{
						Context.FinishTask(*Task, EStateTreeFinishTaskType::Succeeded);
					}
				};

			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE("StateTree should get compiled", bResult);

			AllStateHandle.B = StateTree.GetStateHandleFromId(AllStateHandle.B_ID);
			AllStateHandle.C = StateTree.GetStateHandleFromId(AllStateHandle.C_ID);
			AllStateHandle.E = StateTree.GetStateHandleFromId(AllStateHandle.E_ID);
			AllStateHandle.I = StateTree.GetStateHandleFromId(AllStateHandle.I_ID);
			AllStateHandle.X = StateTree.GetStateHandleFromId(AllStateHandle.X_ID);
			AllStateHandle.P = StateTree.GetStateHandleFromId(AllStateHandle.P_ID);
		}

		FStateTreeInstanceData InstanceData;
		{
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE("StateTree should init", bInitSucceeded);
		}

		constexpr int32 MaxRules = 4;
		auto MakeStateSelectionRule = [](int32 Index)
			{
				EStateTreeStateSelectionRules Rule = EStateTreeStateSelectionRules::None;
				if ((Index % 2) == 1)
				{
					Rule |= EStateTreeStateSelectionRules::CompletedTransitionStatesCreateNewStates;
					Rule |= EStateTreeStateSelectionRules::CompletedStateBeforeTransitionSourceFailsTransition;
				}
				if (Index >= 2)
				{
					Rule |= EStateTreeStateSelectionRules::ReselectedStateCreatesNewStates;
				}
				return Rule;
			};

		auto ResetInstanceData = [this, &StateTree, &InstanceData]()
			{
				{
					FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
					EStateTreeRunStatus Status = Exec.Stop();
					AITEST_EQUAL(TEXT("Should stop"), Status, EStateTreeRunStatus::Stopped);
				}
				{
					FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
					const EStateTreeRunStatus Status = Exec.Start();
					AITEST_EQUAL(TEXT("State should complete with Running"), Status, EStateTreeRunStatus::Running);
					AITEST_TRUE(TEXT("Start should"), Exec.ExpectInActiveStates("StateA", "StateB", "StateC", "StateD"));
				}
				return true;
			};

		const TCHAR* EnterStateChangedStr = TEXT("EnterState=Changed");
		const TCHAR* EnterStateSustainedStr = TEXT("EnterState=Sustained");
		const TCHAR* ExitStateChangedStr = TEXT("ExitState=Changed");;
		const TCHAR* ExitStateSustainedStr = TEXT("ExitState=Sustained");

		for (int32 Index = 0; Index < MaxRules; ++Index)
		{
			const EStateTreeStateSelectionRules StateSelectionRules = MakeStateSelectionRule(Index);
			InstanceData = FStateTreeInstanceData();
			if (StateTree.GetStateSelectionRules() != StateSelectionRules)
			{
				StateTree.ResetCompiled();
				UStateTreeEditorData* EditorData = CastChecked<UStateTreeEditorData>(StateTree.EditorData);
				UStateTreeTestSchema* Schema = CastChecked<UStateTreeTestSchema>(EditorData->Schema);
				Schema->SetStateSelectionRules(StateSelectionRules);

				FStateTreeCompiler Compiler(Log);
				const bool bResult = Compiler.Compile(StateTree);
				AITEST_TRUE("StateTree should get compiled", bResult);
			}

			// Normal Start and tick. Make sure we are in ABCD before testing the transitions
			{
				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
				const EStateTreeRunStatus Status = Exec.Start();
				AITEST_EQUAL("Start should complete with Running", Status, EStateTreeRunStatus::Running);
				AITEST_TRUE("Start should enter Global tasks", Exec.ExpectInActiveStates("StateA", "StateB", "StateC", "StateD"));
				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("StateATask", EnterStateChangedStr);
				AITEST_TRUE(TEXT("Start StateA"), LogOrder);
				LogOrder = LogOrder.Then("StateBTask", EnterStateChangedStr);
				AITEST_TRUE(TEXT("Start StateB"), LogOrder);
				LogOrder = LogOrder.Then("StateCTask", EnterStateChangedStr);
				AITEST_TRUE(TEXT("Start StateC"), LogOrder);
				LogOrder = LogOrder.Then("StateDTask", EnterStateChangedStr);
				AITEST_TRUE(TEXT("Start StateD"), LogOrder);
				AITEST_FALSE(TEXT("Tick StateA"), Exec.Expect("StateATask", TEXT("Tick0")));
				AITEST_FALSE(TEXT("Tick StateB"), Exec.Expect("StateBTask", TEXT("Tick0")));
				AITEST_FALSE(TEXT("Tick StateC"), Exec.Expect("StateCTask", TEXT("Tick0")));
				AITEST_FALSE(TEXT("Tick StateD"), Exec.Expect("StateDTask", TEXT("Tick0")));
				AITEST_FALSE(TEXT("Start StateA"), Exec.Expect("StateATask", TEXT("ExitState0")));
				AITEST_FALSE(TEXT("Start StateB"), Exec.Expect("StateBTask", TEXT("ExitState0")));
				AITEST_FALSE(TEXT("Start StateC"), Exec.Expect("StateCTask", TEXT("ExitState0")));
				AITEST_FALSE(TEXT("Start StateD"), Exec.Expect("StateDTask", TEXT("ExitState0")));
				Exec.LogClear();
			}
			{
				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
				const EStateTreeRunStatus Status = Exec.Tick(0.1f);
				AITEST_EQUAL("1st Tick should complete with Running", Status, EStateTreeRunStatus::Running);
				AITEST_TRUE("1st Tick no transition", Exec.ExpectInActiveStates("StateA", "StateB", "StateC", "StateD"));
				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("StateATask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateA"), LogOrder);
				LogOrder = LogOrder.Then("StateBTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateB"), LogOrder);
				LogOrder = LogOrder.Then("StateCTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateC"), LogOrder);
				LogOrder = LogOrder.Then("StateDTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateD"), LogOrder);
				AITEST_FALSE(TEXT("Enter StateA"), Exec.Expect("StateATask", TEXT("EnterState0")));
				AITEST_FALSE(TEXT("Enter StateB"), Exec.Expect("StateBTask", TEXT("EnterState0")));
				AITEST_FALSE(TEXT("Enter StateC"), Exec.Expect("StateCTask", TEXT("EnterState0")));
				AITEST_FALSE(TEXT("Enter StateD"), Exec.Expect("StateDTask", TEXT("EnterState0")));
				AITEST_FALSE(TEXT("Exit StateA"), Exec.Expect("StateATask", TEXT("ExitState0")));
				AITEST_FALSE(TEXT("Exit StateB"), Exec.Expect("StateBTask", TEXT("ExitState0")));
				AITEST_FALSE(TEXT("Exit StateC"), Exec.Expect("StateCTask", TEXT("ExitState0")));
				AITEST_FALSE(TEXT("Exit StateD"), Exec.Expect("StateDTask", TEXT("ExitState0")));
				Exec.LogClear();
			}
			// Select new child
			{
				TransitionToExecute = ECustomFunctionToRun::TransitionD_To_E;

				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
				const EStateTreeRunStatus Status = Exec.Tick(0.1f);
				AITEST_EQUAL(TEXT("2nd Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("2nd tick should be in new state"), Exec.ExpectInActiveStates("StateA", "StateB", "StateC", "StateD", "StateE", "StateF"));
				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("StateATask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateA"), LogOrder);
				LogOrder = LogOrder.Then("StateBTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateB"), LogOrder);
				LogOrder = LogOrder.Then("StateCTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateC"), LogOrder);
				LogOrder = LogOrder.Then("StateDTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateD"), LogOrder);
				LogOrder = LogOrder.Then("StateETask", EnterStateChangedStr);
				AITEST_TRUE(TEXT("Enter StateE"), LogOrder);
				LogOrder = LogOrder.Then("StateFTask", EnterStateChangedStr);
				AITEST_TRUE(TEXT("Enter StateF"), LogOrder);
				AITEST_FALSE(TEXT("Enter StateA"), Exec.Expect("StateATask", TEXT("EnterState0")));
				AITEST_FALSE(TEXT("Enter StateB"), Exec.Expect("StateBTask", TEXT("EnterState0")));
				AITEST_FALSE(TEXT("Enter StateC"), Exec.Expect("StateCTask", TEXT("EnterState0")));
				AITEST_FALSE(TEXT("Enter StateD"), Exec.Expect("StateDTask", TEXT("EnterState0")));
				AITEST_FALSE(TEXT("Exit StateA"), Exec.Expect("StateATask", TEXT("ExitState0")));
				AITEST_FALSE(TEXT("Exit StateB"), Exec.Expect("StateBTask", TEXT("ExitState0")));
				AITEST_FALSE(TEXT("Exit StateC"), Exec.Expect("StateCTask", TEXT("ExitState0")));
				AITEST_FALSE(TEXT("Exit StateD"), Exec.Expect("StateDTask", TEXT("ExitState0")));
				Exec.LogClear();
				TransitionToExecute = ECustomFunctionToRun::None;
			}

			// Select a new child but with a completed D
			{
				TransitionToExecute = ECustomFunctionToRun::TransitionD_To_E;
				AllStateHandle.bFinishTasksD = true;
				if (!ResetInstanceData())
				{
					return false;
				}

				const bool bUseCompletedRule = EnumHasAllFlags(StateSelectionRules, EStateTreeStateSelectionRules::CompletedTransitionStatesCreateNewStates);

				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
				const EStateTreeRunStatus Status = Exec.Tick(0.1f);
				AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("Start should"), Exec.ExpectInActiveStates("StateA", "StateB", "StateC", "StateD", "StateE", "StateF"));
				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("StateATask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateA"), LogOrder);
				LogOrder = LogOrder.Then("StateBTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateB"), LogOrder);
				LogOrder = LogOrder.Then("StateCTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateC"), LogOrder);
				LogOrder = LogOrder.Then("StateDTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateD"), LogOrder);
				if (bUseCompletedRule)
				{
					LogOrder = LogOrder.Then("StateDTask", ExitStateChangedStr);
					AITEST_TRUE(TEXT("Exit StateD"), LogOrder);
					LogOrder = LogOrder.Then("StateDTask", EnterStateChangedStr);
					AITEST_TRUE(TEXT("Enter StateD"), LogOrder);
				}
				LogOrder = LogOrder.Then("StateETask", EnterStateChangedStr);
				AITEST_TRUE(TEXT("Enter StateE"), LogOrder);
				LogOrder = LogOrder.Then("StateFTask", EnterStateChangedStr);
				AITEST_TRUE(TEXT("Enter StateF"), LogOrder);
				AITEST_FALSE(TEXT("Enter StateA"), Exec.Expect("StateATask", TEXT("EnterState0")));
				AITEST_FALSE(TEXT("Enter StateB"), Exec.Expect("StateBTask", TEXT("EnterState0")));
				AITEST_FALSE(TEXT("Enter StateC"), Exec.Expect("StateCTask", TEXT("EnterState0")));
				AITEST_FALSE(TEXT("Exit StateA"), Exec.Expect("StateATask", TEXT("ExitState0")));
				AITEST_FALSE(TEXT("Exit StateB"), Exec.Expect("StateBTask", TEXT("ExitState0")));
				AITEST_FALSE(TEXT("Exit StateC"), Exec.Expect("StateCTask", TEXT("ExitState0")));
				Exec.LogClear();
				TransitionToExecute = ECustomFunctionToRun::None;
				AllStateHandle.bFinishTasksD = false;
			}
			// Select a new child but B is completed (before the source/target)
			{
				TransitionToExecute = ECustomFunctionToRun::TransitionD_To_E;
				AllStateHandle.bFinishTasksB = true;
				if (!ResetInstanceData())
				{
					return false;
				}

				const bool bUseCompletedRule = EnumHasAllFlags(StateSelectionRules, EStateTreeStateSelectionRules::CompletedStateBeforeTransitionSourceFailsTransition);
				const bool bUseReselectedRule = EnumHasAllFlags(StateSelectionRules, EStateTreeStateSelectionRules::ReselectedStateCreatesNewStates);

				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
				const EStateTreeRunStatus Status = Exec.Tick(0.1f);
				AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				if (bUseCompletedRule)
				{
					AITEST_TRUE(TEXT("Start should"), Exec.ExpectInActiveStates("StateA", "StateB", "StateC", "StateD"));
				}
				else
				{
					AITEST_TRUE(TEXT("Start should"), Exec.ExpectInActiveStates("StateA", "StateB", "StateC", "StateD", "StateE", "StateF"));
				}
				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("StateATask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateA"), LogOrder);
				LogOrder = LogOrder.Then("StateBTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateB"), LogOrder);
				LogOrder = LogOrder.Then("StateCTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateC"), LogOrder);
				LogOrder = LogOrder.Then("StateDTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateD"), LogOrder);
				if (bUseCompletedRule)
				{
					const TCHAR* SchemaEnterStateStr = bUseReselectedRule ? EnterStateChangedStr : EnterStateSustainedStr;
					const TCHAR* SchemaExitStateStr = bUseReselectedRule ? ExitStateChangedStr : ExitStateSustainedStr;

					LogOrder = LogOrder.Then("StateDTask", ExitStateChangedStr);
					AITEST_TRUE(TEXT("Enter StateD"), LogOrder);
					LogOrder = LogOrder.Then("StateCTask", ExitStateChangedStr);
					AITEST_TRUE(TEXT("Enter StateC"), LogOrder);
					LogOrder = LogOrder.Then("StateBTask", ExitStateChangedStr);
					AITEST_TRUE(TEXT("Enter StateB"), LogOrder);
					LogOrder = LogOrder.Then("StateATask", SchemaExitStateStr);
					AITEST_TRUE(TEXT("Enter StateA"), LogOrder);
					LogOrder = LogOrder.Then("StateATask", SchemaEnterStateStr);
					AITEST_TRUE(TEXT("Enter StateA"), LogOrder);
					LogOrder = LogOrder.Then("StateBTask", EnterStateChangedStr);
					AITEST_TRUE(TEXT("Enter StateB"), LogOrder);
					LogOrder = LogOrder.Then("StateCTask", EnterStateChangedStr);
					AITEST_TRUE(TEXT("Enter StateC"), LogOrder);
					LogOrder = LogOrder.Then("StateDTask", EnterStateChangedStr);
					AITEST_TRUE(TEXT("Enter StateD"), LogOrder);

					// Only one transition
					LogOrder = LogOrder.Then("StateBTask", TEXT("EnterState0"));
					AITEST_FALSE(TEXT("Enter StateD"), LogOrder);

					AITEST_FALSE(TEXT("Enter StateE"), Exec.Expect("StateETask", TEXT("EnterState0")));
					AITEST_FALSE(TEXT("Enter StateF"), Exec.Expect("StateFTask", TEXT("EnterState0")));
					AITEST_FALSE(TEXT("Exit StateE"), Exec.Expect("StateETask", TEXT("ExitState0")));
					AITEST_FALSE(TEXT("Exit StateF"), Exec.Expect("StateFTask", TEXT("ExitState0")));
				}
				else
				{
					LogOrder = LogOrder.Then("StateETask", EnterStateChangedStr);
					AITEST_TRUE(TEXT("Enter StateE"), LogOrder);
					LogOrder = LogOrder.Then("StateFTask", EnterStateChangedStr);
					AITEST_TRUE(TEXT("Enter StateF"), LogOrder);
					AITEST_FALSE(TEXT("Enter StateA"), Exec.Expect("StateATask", TEXT("EnterState0")));
					AITEST_FALSE(TEXT("Enter StateB"), Exec.Expect("StateBTask", TEXT("EnterState0")));
					AITEST_FALSE(TEXT("Enter StateC"), Exec.Expect("StateCTask", TEXT("EnterState0")));
					AITEST_FALSE(TEXT("Exit StateA"), Exec.Expect("StateATask", TEXT("ExitState0")));
					AITEST_FALSE(TEXT("Exit StateB"), Exec.Expect("StateBTask", TEXT("ExitState0")));
					AITEST_FALSE(TEXT("Exit StateC"), Exec.Expect("StateCTask", TEXT("ExitState0")));
				}
				Exec.LogClear();
				TransitionToExecute = ECustomFunctionToRun::None;
				AllStateHandle.bFinishTasksB = false;
			}
			// Select a sibling
			{
				TransitionToExecute = ECustomFunctionToRun::TransitionB_To_I;
				if (!ResetInstanceData())
				{
					return false;
				}

				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
				const EStateTreeRunStatus Status = Exec.Tick(0.1f);
				AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("Start should"), Exec.ExpectInActiveStates("StateA", "StateB", "StateI", "StateJ"));
				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("StateATask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateA"), LogOrder);
				LogOrder = LogOrder.Then("StateBTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateB"), LogOrder);
				LogOrder = LogOrder.Then("StateCTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateC"), LogOrder);
				LogOrder = LogOrder.Then("StateDTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateD"), LogOrder);
				LogOrder = LogOrder.Then("StateDTask", ExitStateChangedStr);
				AITEST_TRUE(TEXT("Exit StateD"), LogOrder);
				LogOrder = LogOrder.Then("StateCTask", ExitStateChangedStr);
				AITEST_TRUE(TEXT("Exit StateC"), LogOrder);
				LogOrder = LogOrder.Then("StateITask", EnterStateChangedStr);
				AITEST_TRUE(TEXT("Enter StateI"), LogOrder);
				LogOrder = LogOrder.Then("StateJTask", EnterStateChangedStr);
				AITEST_TRUE(TEXT("Enter StateJ"), LogOrder);
				AITEST_FALSE(TEXT("Enter StateA"), Exec.Expect("StateATask", TEXT("EnterState0")));
				AITEST_FALSE(TEXT("Enter StateB"), Exec.Expect("StateBTask", TEXT("EnterState0")));
				AITEST_FALSE(TEXT("Exit StateA"), Exec.Expect("StateATask", TEXT("ExitState0")));
				AITEST_FALSE(TEXT("Exit StateB"), Exec.Expect("StateBTask", TEXT("ExitState0")));
				Exec.LogClear();
				TransitionToExecute = ECustomFunctionToRun::None;
			}
			// No state completed. Reselect child.
			{
				TransitionToExecute = ECustomFunctionToRun::TransitionB_To_C;
				if (!ResetInstanceData())
				{
					return false;
				}

				const bool bUseReselectedRule = EnumHasAllFlags(StateSelectionRules, EStateTreeStateSelectionRules::ReselectedStateCreatesNewStates);
				const TCHAR* SchemaEnterStateStr = bUseReselectedRule ? EnterStateChangedStr : EnterStateSustainedStr;
				const TCHAR* SchemaExitStateStr = bUseReselectedRule ? ExitStateChangedStr : ExitStateSustainedStr;
				 
				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
				const EStateTreeRunStatus Status = Exec.Tick(0.1f);
				AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("Start should"), Exec.ExpectInActiveStates("StateA", "StateB", "StateC", "StateD"));
				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("StateATask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateA"), LogOrder);
				LogOrder = LogOrder.Then("StateBTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateB"), LogOrder);
				LogOrder = LogOrder.Then("StateCTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateC"), LogOrder);
				LogOrder = LogOrder.Then("StateDTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateD"), LogOrder);
	
				LogOrder = LogOrder.Then("StateDTask", SchemaExitStateStr);
				AITEST_TRUE(TEXT("Exit StateD"), LogOrder);
				LogOrder = LogOrder.Then("StateCTask", SchemaExitStateStr);
				AITEST_TRUE(TEXT("Exit StateC"), LogOrder);
				LogOrder = LogOrder.Then("StateCTask", SchemaEnterStateStr);
				AITEST_TRUE(TEXT("Enter StateC"), LogOrder);
				LogOrder = LogOrder.Then("StateDTask", SchemaEnterStateStr);
				AITEST_TRUE(TEXT("Enter StateD"), LogOrder);

				AITEST_FALSE(TEXT("Enter StateA"), Exec.Expect("StateATask", TEXT("EnterState0")));
				AITEST_FALSE(TEXT("Enter StateB"), Exec.Expect("StateBTask", TEXT("EnterState0")));
				AITEST_FALSE(TEXT("Exit StateA"), Exec.Expect("StateATask", TEXT("ExitState0")));
				AITEST_FALSE(TEXT("Exit StateB"), Exec.Expect("StateBTask", TEXT("ExitState0")));
				Exec.LogClear();
				TransitionToExecute = ECustomFunctionToRun::None;
			}
			// State completed. Reselect child.
			{
				TransitionToExecute = ECustomFunctionToRun::TransitionB_To_C;
				AllStateHandle.bFinishTasksC = true;
				if (!ResetInstanceData())
				{
					return false;
				}

				const bool bUseCompletedRule = EnumHasAllFlags(StateSelectionRules, EStateTreeStateSelectionRules::CompletedTransitionStatesCreateNewStates);
				const bool bUseReselectedRule = EnumHasAllFlags(StateSelectionRules, EStateTreeStateSelectionRules::ReselectedStateCreatesNewStates);
				const TCHAR* SchemaEnterStateStr = bUseCompletedRule || bUseReselectedRule ? EnterStateChangedStr : EnterStateSustainedStr;
				const TCHAR* SchemaExitStateStr = bUseCompletedRule || bUseReselectedRule ? ExitStateChangedStr : ExitStateSustainedStr;

				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
				const EStateTreeRunStatus Status = Exec.Tick(0.1f);
				AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("Start should"), Exec.ExpectInActiveStates("StateA", "StateB", "StateC", "StateD"));
				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("StateATask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateA"), LogOrder);
				LogOrder = LogOrder.Then("StateBTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateB"), LogOrder);
				LogOrder = LogOrder.Then("StateCTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateC"), LogOrder);
				LogOrder = LogOrder.Then("StateDTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateD"), LogOrder);
				LogOrder = LogOrder.Then("StateDTask", SchemaExitStateStr);
				AITEST_TRUE(TEXT("Exit StateD"), LogOrder);
				LogOrder = LogOrder.Then("StateCTask", SchemaExitStateStr);
				AITEST_TRUE(TEXT("Exit StateC"), LogOrder);
				LogOrder = LogOrder.Then("StateCTask", SchemaEnterStateStr);
				AITEST_TRUE(TEXT("Enter StateC"), LogOrder);
				LogOrder = LogOrder.Then("StateDTask", SchemaEnterStateStr);
				AITEST_TRUE(TEXT("Enter StateD"), LogOrder);
				AITEST_FALSE(TEXT("Enter StateA"), Exec.Expect("StateATask", TEXT("EnterState0")));
				AITEST_FALSE(TEXT("Enter StateB"), Exec.Expect("StateBTask", TEXT("EnterState0")));
				AITEST_FALSE(TEXT("Exit StateA"), Exec.Expect("StateATask", TEXT("ExitState0")));
				AITEST_FALSE(TEXT("Exit StateB"), Exec.Expect("StateBTask", TEXT("ExitState0")));
				Exec.LogClear();
				TransitionToExecute = ECustomFunctionToRun::None;
				AllStateHandle.bFinishTasksC = false;
			}
			// No state completed. Reselect parent.
			{
				TransitionToExecute = ECustomFunctionToRun::TransitionC_To_B;
				if (!ResetInstanceData())
				{
					return false;
				}

				const bool bUseReselectedRule = EnumHasAllFlags(StateSelectionRules, EStateTreeStateSelectionRules::ReselectedStateCreatesNewStates);
				const TCHAR* SchemaEnterStateStr = bUseReselectedRule ? EnterStateChangedStr : EnterStateSustainedStr;
				const TCHAR* SchemaExitStateStr = bUseReselectedRule ? ExitStateChangedStr : ExitStateSustainedStr;

				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
				const EStateTreeRunStatus Status = Exec.Tick(0.1f);
				AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("Start should"), Exec.ExpectInActiveStates("StateA", "StateB", "StateC", "StateD"));
				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("StateATask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateA"), LogOrder);
				LogOrder = LogOrder.Then("StateBTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateB"), LogOrder);
				LogOrder = LogOrder.Then("StateCTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateC"), LogOrder);
				LogOrder = LogOrder.Then("StateDTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateD"), LogOrder);
				LogOrder = LogOrder.Then("StateDTask", SchemaExitStateStr);
				AITEST_TRUE(TEXT("Exit StateD"), LogOrder);
				LogOrder = LogOrder.Then("StateCTask", SchemaExitStateStr);
				AITEST_TRUE(TEXT("Exit StateC"), LogOrder);
				LogOrder = LogOrder.Then("StateBTask", SchemaExitStateStr);
				AITEST_TRUE(TEXT("Exit StateB"), LogOrder);
				LogOrder = LogOrder.Then("StateBTask", SchemaEnterStateStr);
				AITEST_TRUE(TEXT("Enter StateB"), LogOrder);
				LogOrder = LogOrder.Then("StateCTask", SchemaEnterStateStr);
				AITEST_TRUE(TEXT("Enter StateC"), LogOrder);
				LogOrder = LogOrder.Then("StateDTask", SchemaEnterStateStr);
				AITEST_TRUE(TEXT("Enter StateD"), LogOrder);
				AITEST_FALSE(TEXT("Enter StateA"), Exec.Expect("StateATask", TEXT("EnterState0")));
				AITEST_FALSE(TEXT("Exit StateA"), Exec.Expect("StateATask", TEXT("ExitState0")));

				Exec.LogClear();
				TransitionToExecute = ECustomFunctionToRun::None;
			}
			// State completed. Reselect parent.
			{
				TransitionToExecute = ECustomFunctionToRun::TransitionC_To_B;
				AllStateHandle.bFinishTasksC = true;
				if (!ResetInstanceData())
				{
					return false;
				}

				const bool bUseCompletedRule = EnumHasAllFlags(StateSelectionRules, EStateTreeStateSelectionRules::CompletedTransitionStatesCreateNewStates);
				const bool bUseReselectedRule = EnumHasAllFlags(StateSelectionRules, EStateTreeStateSelectionRules::ReselectedStateCreatesNewStates);
				const TCHAR* SchemaEnterStateStr = bUseCompletedRule || bUseReselectedRule ? EnterStateChangedStr : EnterStateSustainedStr;
				const TCHAR* SchemaExitStateStr = bUseCompletedRule || bUseReselectedRule ? ExitStateChangedStr : ExitStateSustainedStr;

				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
				const EStateTreeRunStatus Status = Exec.Tick(0.1f);
				AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("Start should"), Exec.ExpectInActiveStates("StateA", "StateB", "StateC", "StateD"));
				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("StateATask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateA"), LogOrder);
				LogOrder = LogOrder.Then("StateBTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateB"), LogOrder);
				LogOrder = LogOrder.Then("StateCTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateC"), LogOrder);
				LogOrder = LogOrder.Then("StateDTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateD"), LogOrder);
				LogOrder = LogOrder.Then("StateDTask", SchemaExitStateStr);
				AITEST_TRUE(TEXT("Exit StateD"), LogOrder);
				LogOrder = LogOrder.Then("StateCTask", SchemaExitStateStr);
				if (bUseReselectedRule)
				{
					AITEST_TRUE(TEXT("Exit StateC"), LogOrder);
					LogOrder = LogOrder.Then("StateBTask", ExitStateChangedStr);
					AITEST_TRUE(TEXT("Exit StateB"), LogOrder);
					LogOrder = LogOrder.Then("StateBTask", EnterStateChangedStr);
				}
				else
				{
					AITEST_TRUE(TEXT("Exit StateC"), LogOrder);
					LogOrder = LogOrder.Then("StateBTask", ExitStateSustainedStr);
					AITEST_TRUE(TEXT("Exit StateB"), LogOrder);
					LogOrder = LogOrder.Then("StateBTask", EnterStateSustainedStr);
				}
				AITEST_TRUE(TEXT("Enter StateB"), LogOrder);
				LogOrder = LogOrder.Then("StateCTask", SchemaEnterStateStr);
				AITEST_TRUE(TEXT("Enter StateC"), LogOrder);
				LogOrder = LogOrder.Then("StateDTask", SchemaEnterStateStr);
				AITEST_TRUE(TEXT("Enter StateD"), LogOrder);
				AITEST_FALSE(TEXT("Enter StateA"), Exec.Expect("StateATask", TEXT("EnterState0")));
				AITEST_FALSE(TEXT("Exit StateA"), Exec.Expect("StateATask", TEXT("ExitState0")));

				Exec.LogClear();
				AllStateHandle.bFinishTasksC = false;
				TransitionToExecute = ECustomFunctionToRun::None;
			}
			// linked state
			{
				TransitionToExecute = ECustomFunctionToRun::TransitionB_To_X;
				if (!ResetInstanceData())
				{
					return false;
				}

				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
				const EStateTreeRunStatus Status = Exec.Tick(0.1f);
				AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("Start should"), Exec.ExpectInActiveStates("StateA", "StateX", "StateY"));
				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("StateATask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateA"), LogOrder);
				LogOrder = LogOrder.Then("StateBTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateB"), LogOrder);
				LogOrder = LogOrder.Then("StateCTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateC"), LogOrder);
				LogOrder = LogOrder.Then("StateDTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateD"), LogOrder);
				LogOrder = LogOrder.Then("StateDTask", ExitStateChangedStr);
				AITEST_TRUE(TEXT("Exit StateD"), LogOrder);
				LogOrder = LogOrder.Then("StateCTask", ExitStateChangedStr);
				AITEST_TRUE(TEXT("Exit StateC"), LogOrder);
				LogOrder = LogOrder.Then("StateBTask", ExitStateChangedStr);
				AITEST_TRUE(TEXT("Exit StateB"), LogOrder);
				LogOrder = LogOrder.Then("StateXTask", EnterStateChangedStr);
				AITEST_TRUE(TEXT("Enter StateX"), LogOrder);
				LogOrder = LogOrder.Then("StateYTask", EnterStateChangedStr);
				AITEST_TRUE(TEXT("Enter StateY"), LogOrder);
				AITEST_FALSE(TEXT("Enter StateA"), Exec.Expect("StateATask", TEXT("EnterState0")));
				AITEST_FALSE(TEXT("Exit StateA"), Exec.Expect("StateATask", TEXT("ExitState0")));
				Exec.LogClear();
				TransitionToExecute = ECustomFunctionToRun::None;
			}
			// Reselection same state
			{
				TransitionToExecute = ECustomFunctionToRun::TransitionB_To_B;
				if (!ResetInstanceData())
				{
					return false;
				}
				const bool bUseReselectedRule = EnumHasAllFlags(StateSelectionRules, EStateTreeStateSelectionRules::ReselectedStateCreatesNewStates);
				const TCHAR* SchemaEnterStateStr = bUseReselectedRule ? EnterStateChangedStr : EnterStateSustainedStr;
				const TCHAR* SchemaExitStateStr = bUseReselectedRule ? ExitStateChangedStr : ExitStateSustainedStr;

				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
				const EStateTreeRunStatus Status = Exec.Tick(0.1f);
				AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("Start should"), Exec.ExpectInActiveStates("StateA", "StateB", "StateC", "StateD"));
				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("StateATask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateA"), LogOrder);
				LogOrder = LogOrder.Then("StateBTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateB"), LogOrder);
				LogOrder = LogOrder.Then("StateCTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateC"), LogOrder);
				LogOrder = LogOrder.Then("StateDTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateD"), LogOrder);
				LogOrder = LogOrder.Then("StateDTask", SchemaExitStateStr);
				AITEST_TRUE(TEXT("Exit StateD"), LogOrder);
				LogOrder = LogOrder.Then("StateCTask", SchemaExitStateStr);
				AITEST_TRUE(TEXT("Exit StateC"), LogOrder);
				LogOrder = LogOrder.Then("StateBTask", SchemaExitStateStr);
				AITEST_TRUE(TEXT("Exit StateB"), LogOrder);
				LogOrder = LogOrder.Then("StateBTask", SchemaEnterStateStr);
				AITEST_TRUE(TEXT("Enter StateB"), LogOrder);
				LogOrder = LogOrder.Then("StateCTask", SchemaEnterStateStr);
				AITEST_TRUE(TEXT("Enter StateC"), LogOrder);
				LogOrder = LogOrder.Then("StateDTask", SchemaEnterStateStr);
				AITEST_TRUE(TEXT("Enter StateD"), LogOrder);
				AITEST_FALSE(TEXT("Enter StateA"), Exec.Expect("StateATask", TEXT("EnterState0")));
				AITEST_FALSE(TEXT("Exit StateA"), Exec.Expect("StateATask", TEXT("ExitState0")));
				Exec.LogClear();
				TransitionToExecute = ECustomFunctionToRun::None;
			}
			// Select another root.
			{
				TransitionToExecute = ECustomFunctionToRun::TransitionB_To_P;
				if (!ResetInstanceData())
				{
					return false;
				}

				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
				const EStateTreeRunStatus Status = Exec.Tick(0.1f);
				AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("Start should"), Exec.ExpectInActiveStates("StateO", "StateP"));
				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("StateATask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateA"), LogOrder);
				LogOrder = LogOrder.Then("StateBTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateB"), LogOrder);
				LogOrder = LogOrder.Then("StateCTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateC"), LogOrder);
				LogOrder = LogOrder.Then("StateDTask", TEXT("Tick0"));
				AITEST_TRUE(TEXT("Tick StateD"), LogOrder);
				LogOrder = LogOrder.Then("StateDTask", ExitStateChangedStr);
				AITEST_TRUE(TEXT("Exit StateD"), LogOrder);
				LogOrder = LogOrder.Then("StateCTask", ExitStateChangedStr);
				AITEST_TRUE(TEXT("Exit StateC"), LogOrder);
				LogOrder = LogOrder.Then("StateBTask", ExitStateChangedStr);
				AITEST_TRUE(TEXT("Exit StateB"), LogOrder);
				LogOrder = LogOrder.Then("StateATask", ExitStateChangedStr);
				AITEST_TRUE(TEXT("Exit StateA"), LogOrder);
				LogOrder = LogOrder.Then("StateOTask", EnterStateChangedStr);
				AITEST_TRUE(TEXT("Enter StateO"), LogOrder);
				LogOrder = LogOrder.Then("StatePTask", EnterStateChangedStr);
				AITEST_TRUE(TEXT("Enter StateP"), LogOrder);
				Exec.LogClear();
				TransitionToExecute = ECustomFunctionToRun::None;
			}
			{
				FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
				EStateTreeRunStatus Status = Exec.Stop();
				AITEST_EQUAL(TEXT("Should stop"), Status, EStateTreeRunStatus::Stopped);
			}
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FStateTreeTest_RequestTransition, "System.StateTree.Transition.RequestTransition");

struct FStateTreeTest_TransitionToNone : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		const FGameplayTag Tag = GetTestTag1();

		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
		UStateTreeState& State1 = Root.AddChildState(FName(TEXT("State1")));
		UStateTreeState& State2 = Root.AddChildState(FName(TEXT("State2")));

		FStateTreeTransition& TransitionRoot = Root.AddTransition(EStateTreeTransitionTrigger::OnEvent, EStateTreeTransitionType::GotoState);
		TransitionRoot.State = State1.GetLinkToState();
		TransitionRoot.RequiredEvent.Tag = Tag;

		TStateTreeEditorNode<FTestTask_StandNoTick>& Task1 = State1.AddTask<FTestTask_StandNoTick>(FName(TEXT("Task1")));
		FStateTreeTransition& TransitionState1 = State1.AddTransition(EStateTreeTransitionTrigger::OnEvent, EStateTreeTransitionType::None);
		TransitionState1.RequiredEvent.Tag = Tag;

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bResult = Compiler.Compile(StateTree);
		AITEST_TRUE("StateTree should get compiled", bResult);

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE("StateTree should init", bInitSucceeded);

		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* ExitStateStr(TEXT("ExitState"));

		// Start and enter state
		Status = Exec.Start();
		AITEST_TRUE(TEXT("StateTree Task1 should enter state"), Exec.Expect(Task1.GetName(), EnterStateStr));
		Exec.LogClear();

		// Send event with Tag
		Exec.SendEvent(Tag);

		// Transition from Root to State2 should not be triggered
		Status = Exec.Tick(0.1f);
		AITEST_TRUE(TEXT("Active states should stay the same"), Exec.ExpectInActiveStates("Root", "State1"));
		AITEST_FALSE(TEXT("StateTree Task1 should not exit state"), Exec.Expect(Task1.GetName(), ExitStateStr));
		Exec.LogClear();

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_TransitionToNone, "System.StateTree.Transition.ToNone");

struct FStateTreeTest_TransitionTwoEvents : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		UStateTree& StateTree = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

			UStateTreeState& Root = EditorData.AddSubTree(FName(TEXT("Root")));
			UStateTreeState& State1 = Root.AddChildState(FName(TEXT("State1")));
			UStateTreeState& State2 = Root.AddChildState(FName(TEXT("State2")));
			UStateTreeState& State3 = State2.AddChildState(FName(TEXT("State3")));
			UStateTreeState& State4 = State2.AddChildState(FName(TEXT("State4")));

			State2.bHasRequiredEventToEnter = true;
			State2.RequiredEventToEnter.Tag = GetTestTag1();

			State3.bHasRequiredEventToEnter = true;
			State3.RequiredEventToEnter.Tag = GetTestTag2();

			FStateTreeTransition& TransitionA = State1.AddTransition(EStateTreeTransitionTrigger::OnEvent, EStateTreeTransitionType::GotoState);
			TransitionA.State = State2.GetLinkToState();
			TransitionA.RequiredEvent.Tag = GetTestTag1();

			TStateTreeEditorNode<FTestTask_StandNoTick>& State3Task1 = State3.AddTask<FTestTask_StandNoTick>(FName(TEXT("State3Task1")));
		}
		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE("StateTree should get compiled", bResult);
		}

		const TCHAR* EnterStateStr(TEXT("EnterState"));
		const TCHAR* ExitStateStr(TEXT("ExitState"));

		FStateTreeInstanceData InstanceData;
		TArray<FRecordedStateTreeTransitionResult> RecordedTransitions;
		{
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData, {}, EStateTreeRecordTransitions::Yes);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE("StateTree should init", bInitSucceeded);

			// Start and enter state
			EStateTreeRunStatus Status = Exec.Start();
			AITEST_TRUE(TEXT("Active states should stay the same"), Exec.ExpectInActiveStates("Root", "State1"));
			Exec.LogClear();

			// Transition from State1 to State2. State3 fails (missing event), enter State4.
			Exec.SendEvent(GetTestTag1());
			Exec.Tick(0.1f);
			AITEST_TRUE(TEXT("Active states should stay the same"), Exec.ExpectInActiveStates("Root", "State2", "State4"));
			Exec.LogClear();

			Exec.Stop();
			Exec.Start();
			AITEST_TRUE(TEXT("Active states should stay the same"), Exec.ExpectInActiveStates("Root", "State1"));

			// Transition from State1 to State2 failed.
			Exec.SendEvent(GetTestTag2());
			Status = Exec.Tick(0.1f);
			AITEST_TRUE(TEXT("Active states should stay the same"), Exec.ExpectInActiveStates("Root", "State1"));
			Exec.LogClear();

			// Transition from State1 to State2. State3 succeed.
			Exec.SendEvent(GetTestTag2());
			Exec.SendEvent(GetTestTag1());
			Status = Exec.Tick(0.1f);
			AITEST_TRUE(TEXT("Active states should stay the same"), Exec.ExpectInActiveStates("Root", "State2", "State3"));
			Exec.LogClear();

			RecordedTransitions = Exec.GetRecordedTransitions();
			Exec.Stop();
		}

		{
			FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE("StateTree should init", bInitSucceeded);

			// Start and enter state
			EStateTreeRunStatus Status = Exec.Start();
			AITEST_TRUE(TEXT("Active states should stay the same"), Exec.ExpectInActiveStates("Root", "State1"));
			Exec.LogClear();

			AITEST_TRUE(TEXT("Wrong number of transitions"), RecordedTransitions.Num() == 5);

			Exec.ForceTransition(RecordedTransitions[0]); // start
			AITEST_TRUE(TEXT("Start."), Exec.ExpectInActiveStates("Root", "State1"));

			Exec.ForceTransition(RecordedTransitions[1]); // state 3 doesn't have the event
			AITEST_TRUE(TEXT("State3 failed."), Exec.ExpectInActiveStates("Root", "State2", "State4"));
			
			Exec.ForceTransition(RecordedTransitions[2]); // start
			AITEST_TRUE(TEXT("Start and failed transition."), Exec.ExpectInActiveStates("Root", "State1"));

			Exec.ForceTransition(RecordedTransitions[3]); // failed
			AITEST_TRUE(TEXT("Start and failed transition."), Exec.ExpectInActiveStates("Root", "State1"));

			Exec.ForceTransition(RecordedTransitions[4]); // succeed
			AITEST_TRUE(TEXT("State3 succeed."), Exec.ExpectInActiveStates("Root", "State2", "State3"));

			Exec.Stop();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_TransitionTwoEvents, "System.StateTree.Transition.TwoEvents");

struct FStateTreeTest_PayloadSelector : FStateTreeTestBase
{
	UStateTree& SetupTree()
	{
		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		// Root
		//  StateA (default)
		//  StateB
		//   StateB1
		//   StateB2
		UStateTreeState& Root = EditorData.AddSubTree(FName("Root"));

		// StateA - default state
		UStateTreeState& StateA = Root.AddChildState(FName("StateA"));
		TStateTreeEditorNode<FTestTask_PrintValue>& TaskA = StateA.AddTask<FTestTask_PrintValue>(FName("TaskA"));
		TaskA.GetInstanceData().Value = 1;

		// StateB - selector state that chooses between B1 and B2 based on payload
		UStateTreeState& StateB = Root.AddChildState(FName("StateB"), EStateTreeStateType::State);
		{
			StateB.bHasRequiredEventToEnter = true;
			StateB.RequiredEventToEnter.Tag = GetTestTag1();
			StateB.RequiredEventToEnter.PayloadStruct = FStateTreeTest_PropertyStructA::StaticStruct();
			StateB.SelectionBehavior = EStateTreeStateSelectionBehavior::TrySelectChildrenInOrder;
			TStateTreeEditorNode<FTestTask_PrintValue>& TaskB = StateB.AddTask<FTestTask_PrintValue>(FName("TaskB"));
			TaskB.GetInstanceData().Value = 2;
			EditorData.AddPropertyBinding(FPropertyBindingPath(StateB.GetEventID(), { FPropertyBindingPathSegment("Payload"), FPropertyBindingPathSegment("A") }), FPropertyBindingPath(TaskB.ID, TEXT("Value")));

		}

		// StateB1 - child of StateB, only enters if event tag matches and payload.A < 10
		UStateTreeState& StateB1 = StateB.AddChildState(FName("StateB1"));
		{
			StateB1.bHasRequiredEventToEnter = true;
			StateB1.RequiredEventToEnter.Tag = GetTestTag1();
			StateB1.RequiredEventToEnter.PayloadStruct = FStateTreeTest_PropertyStructA::StaticStruct();
			TStateTreeEditorNode<FStateTreeCompareIntCondition>& CondB1 = StateB1.AddEnterCondition<FStateTreeCompareIntCondition>(UE::StateTree::EComparisonOperator::Less);
			CondB1.GetInstanceData().Right = 10;
			EditorData.AddPropertyBinding(FPropertyBindingPath(StateB1.GetEventID(), {FPropertyBindingPathSegment("Payload"), FPropertyBindingPathSegment("A")}), FPropertyBindingPath(CondB1.ID, TEXT("Left")));
			TStateTreeEditorNode<FTestTask_PrintValue>& TaskB1 = StateB1.AddTask<FTestTask_PrintValue>(FName("TaskB1"));
			TaskB1.GetInstanceData().Value = 21;
			EditorData.AddPropertyBinding(FPropertyBindingPath(StateB.GetEventID(), {FPropertyBindingPathSegment("Payload"), FPropertyBindingPathSegment("A")}), FPropertyBindingPath(TaskB1.ID, TEXT("Value")));
		}

		// StateB2 - child of StateB, only enters if event tag matches and payload.A >= 10
		UStateTreeState& StateB2 = StateB.AddChildState(FName("StateB2"));
		{
			StateB2.bHasRequiredEventToEnter = true;
			StateB2.RequiredEventToEnter.Tag = GetTestTag1();
			StateB2.RequiredEventToEnter.PayloadStruct = FStateTreeTest_PropertyStructA::StaticStruct();
			TStateTreeEditorNode<FStateTreeCompareIntCondition>& CondB2 = StateB2.AddEnterCondition<FStateTreeCompareIntCondition>(UE::StateTree::EComparisonOperator::GreaterOrEqual);
			CondB2.GetInstanceData().Right = 10;
			EditorData.AddPropertyBinding(FPropertyBindingPath(StateB2.GetEventID(), { FPropertyBindingPathSegment("Payload"), FPropertyBindingPathSegment("A") }), FPropertyBindingPath(CondB2.ID, TEXT("Left")));
			TStateTreeEditorNode<FTestTask_PrintValue>& TaskB2 = StateB2.AddTask<FTestTask_PrintValue>(FName("TaskB2"));
			TaskB2.GetInstanceData().Value = 22;
			EditorData.AddPropertyBinding(FPropertyBindingPath(StateB.GetEventID(), { FPropertyBindingPathSegment("Payload"), FPropertyBindingPathSegment("A") }), FPropertyBindingPath(TaskB2.ID, TEXT("Value")));
		}

		// Transitions
		// StateA -> StateB on event
		StateA.AddTransition(EStateTreeTransitionTrigger::OnEvent, EStateTreeTransitionType::GotoState, &StateB);
		StateA.Transitions.Last().RequiredEvent.Tag = GetTestTag1();
		StateA.Transitions.Last().RequiredEvent.PayloadStruct = FStateTreeTest_PropertyStructA::StaticStruct();

		// StateB1 -> StateB on event with payload.A < 10
		StateB1.AddTransition(EStateTreeTransitionTrigger::OnEvent, EStateTreeTransitionType::GotoState, &StateB);
		StateB1.Transitions.Last().RequiredEvent.Tag = GetTestTag1();
		StateB1.Transitions.Last().RequiredEvent.PayloadStruct = FStateTreeTest_PropertyStructA::StaticStruct();

		// StateB2 -> StateB on event with payload.A >= 10
		StateB2.AddTransition(EStateTreeTransitionTrigger::OnEvent, EStateTreeTransitionType::GotoState, &StateB);
		StateB2.Transitions.Last().RequiredEvent.Tag = GetTestTag1();
		StateB2.Transitions.Last().RequiredEvent.PayloadStruct = FStateTreeTest_PropertyStructA::StaticStruct();

		return StateTree;
	}

	virtual bool InstantTest() override
	{
		UStateTree& StateTree = SetupTree();

		FStateTreeCompilerLog Log;
		FStateTreeCompiler Compiler(Log);
		const bool bCompiled = Compiler.Compile(StateTree);
		AITEST_TRUE(TEXT("StateTree should compile"), bCompiled);

		FStateTreeInstanceData InstanceData;
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE(TEXT("StateTree should initialize"), bInitSucceeded);

		// Test 1: Start in StateA
		{
			EStateTreeRunStatus Status = Exec.Start();
			AITEST_EQUAL(TEXT("Should start successfully"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Should be in StateA"), Exec.ExpectInActiveStates("Root", "StateA"));
			AITEST_TRUE(TEXT("TaskA should enter"), Exec.Expect(FName("TaskA"), TEXT("EnterState1")));
			Exec.LogClear();
		}

		// Test 2: Send event to transition StateA -> StateB with payload A < 10
		{
			FStateTreeTest_PropertyStructA Payload;
			Payload.A = -99;
			Exec.SendEvent(GetTestTag1(), FConstStructView::Make(Payload));

			EStateTreeRunStatus Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Should continue running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Should be in StateB and StateB1"), Exec.ExpectInActiveStates("Root", "StateB", "StateB1"));
			AITEST_TRUE(TEXT("Should transition to StateB"), Exec.Expect(FName("TaskA"), TEXT("ExitState1")));
			AITEST_TRUE(TEXT("TaskB should enter"), Exec.Expect(FName("TaskB"), TEXT("EnterState-99")));
			AITEST_TRUE(TEXT("TaskB1 should enter"), Exec.Expect(FName("TaskB1"), TEXT("EnterState-99")));
			Exec.LogClear();
		}

		// Test 2.1: Value are valid still
		{
			EStateTreeRunStatus Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Should continue running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Should be in StateB and StateB1"), Exec.ExpectInActiveStates("Root", "StateB", "StateB1"));
			AITEST_TRUE(TEXT("TaskB1 should tick"), Exec.Expect(FName("TaskB"), TEXT("Tick-99")));
			AITEST_TRUE(TEXT("TaskB1 should tick"), Exec.Expect(FName("TaskB1"), TEXT("Tick-99")));
			Exec.LogClear();
		}

		// Test 3: StateB selects StateB2 when payload. A >= 10
		{
			FStateTreeTest_PropertyStructA Payload;
			Payload.A = 15;
			Exec.SendEvent(GetTestTag1(), FConstStructView::Make(Payload));

			EStateTreeRunStatus Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Should continue running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Should be in StateB and StateB2"), Exec.ExpectInActiveStates("Root", "StateB", "StateB2"));
			AITEST_TRUE(TEXT("TaskB1 should tick"), Exec.Expect(FName("TaskB1"), TEXT("ExitState-99")));
			AITEST_TRUE(TEXT("TaskB2 should enter"), Exec.Expect(FName("TaskB2"), TEXT("EnterState15")));
			Exec.LogClear();
		}

		// Test 4: Send event from StateB2 back to StateB with payload. A < 10 (select StateB1)
		{
			FStateTreeTest_PropertyStructA Payload;
			Payload.A = 5;
			Exec.SendEvent(GetTestTag1(), FConstStructView::Make(Payload));

			EStateTreeRunStatus Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Should continue running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Should be in StateB and StateB1"), Exec.ExpectInActiveStates("Root", "StateB", "StateB1"));
			AITEST_TRUE(TEXT("StateB2 should exit"), Exec.Expect(FName("TaskB2"), TEXT("ExitState15")));
			AITEST_TRUE(TEXT("TaskB should enter again"), Exec.Expect(FName("TaskB"), TEXT("EnterState5")));
			AITEST_TRUE(TEXT("TaskB should enter again"), Exec.Expect(FName("TaskB1"), TEXT("EnterState5")));
			Exec.LogClear();
		}

		// Test 5: StateB selects StateB1 when payload. A < 10
		{
			FStateTreeTest_PropertyStructA Payload;
			Payload.A = 8;
			Exec.SendEvent(GetTestTag1(), FConstStructView::Make(Payload));

			EStateTreeRunStatus Status = Exec.Tick(0.1f);
			AITEST_EQUAL(TEXT("Should continue running"), Status, EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Should be in StateB and StateB1"), Exec.ExpectInActiveStates("Root", "StateB", "StateB1"));
			AITEST_TRUE(TEXT("TaskB1 should enter"), Exec.Expect(FName("TaskB"), TEXT("EnterState8")));
			AITEST_TRUE(TEXT("TaskB1 should enter"), Exec.Expect(FName("TaskB1"), TEXT("EnterState8")));
			Exec.LogClear();
		}

		Exec.Stop();
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_PayloadSelector, "System.StateTree.Transition.CapturePayloadForSustainedState");

struct FStateTreeTest_TransitionLinkedAssetFromCompletedParent : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		/*
		- Tree1
			- Root1
				- State1: TryEnter, Task w/ 1 frame to complete
					- State2 : LinkedAsset (On Complete -> State 3)
				- State3
		- Tree2
			- Root1
				- State1: Task w/ 2 frames to complete
		*/

		UStateTree& StateTree1 = NewStateTree();
		UStateTree& StateTree2 = NewStateTree();

		// Setup linked asset
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree2.EditorData);
			UStateTreeState& Root1 = EditorData.AddSubTree(FName("Tree2_Root1"));
			UStateTreeState& State1 = Root1.AddChildState(FName("Tree2_State1"));

			State1.AddTask<FTestTask_PrintValue>("Tree2_State1_TaskPrint")
				.GetInstanceData().Value = 21;
			State1.AddTask<FTestTask_Stand>(FName("Tree2_State1_TaskStand"))
				.GetNode().TicksToCompletion = 2;
			State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &Root1);

			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree2);
			AITEST_TRUE("StateTree should get compiled", bResult);
		}

		// Setup main asset
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree1.EditorData);
			UStateTreeState& Root1 = EditorData.AddSubTree(FName("Tree1_Root1"));
			UStateTreeState& State1 = Root1.AddChildState(FName("Tree1_State1"));

			UStateTreeState& State2 = State1.AddChildState(FName("Tree1_State2"), EStateTreeStateType::LinkedAsset);
			UStateTreeState& State3 = Root1.AddChildState(FName("Tree1_State3"));

			{
				TStateTreeEditorNode<FTestTask_PrintValue>& PrintTask = State1.AddTask<FTestTask_PrintValue>("Tree1_State1_TaskPrint");
				PrintTask.GetInstanceData().Value = 11;
				TStateTreeEditorNode<FTestTask_Stand>& StandTask = State1.AddTask<FTestTask_Stand>(FName("Tree1_State1_TaskStand"));
				StandTask.GetNode().TicksToCompletion = 1;
				StandTask.GetInstanceData().Value = 11;
				State1.SelectionBehavior = EStateTreeStateSelectionBehavior::TryEnterState;
				State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &State2);
			}

			{
				State2.SetLinkedStateAsset(&StateTree2);
				TStateTreeEditorNode<FTestTask_PrintValue>& PrintTask = State2.AddTask<FTestTask_PrintValue>("Tree1_State2_TaskPrint");
				PrintTask.GetInstanceData().Value = 12;
			}

			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree1);
			AITEST_TRUE("StateTree should get compiled", bResult);
		}

		FStateTreeInstanceData InstanceData;
		{
			FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE("StateTree should init", bInitSucceeded);
		}

		constexpr int32 MaxRules = 4;
		auto MakeStateSelectionRule = [](int32 Index)
			{
				EStateTreeStateSelectionRules Rule = EStateTreeStateSelectionRules::None;
				if ((Index % 2) == 1)
				{
					Rule |= EStateTreeStateSelectionRules::CompletedTransitionStatesCreateNewStates;
					Rule |= EStateTreeStateSelectionRules::CompletedStateBeforeTransitionSourceFailsTransition;
				}
				if (Index >= 2)
				{
					Rule |= EStateTreeStateSelectionRules::ReselectedStateCreatesNewStates;
				}
				return Rule;
			};

		const TCHAR* EnterStateChangedStr = TEXT("EnterState=Changed");
		const TCHAR* EnterStateSustainedStr = TEXT("EnterState=Sustained");
		const TCHAR* ExitStateChangedStr = TEXT("ExitState=Changed");
		const TCHAR* ExitStateSustainedStr = TEXT("ExitState=Sustained");

		for (int32 Index = 0; Index < MaxRules; ++Index)
		{
			const EStateTreeStateSelectionRules StateSelectionRules = MakeStateSelectionRule(Index);
			InstanceData = FStateTreeInstanceData();
			if (StateTree1.GetStateSelectionRules() != StateSelectionRules)
			{
				StateTree1.ResetCompiled();
				UStateTreeEditorData* EditorData = CastChecked<UStateTreeEditorData>(StateTree1.EditorData);
				UStateTreeTestSchema* Schema = CastChecked<UStateTreeTestSchema>(EditorData->Schema);
				Schema->SetStateSelectionRules(StateSelectionRules);

				FStateTreeCompilerLog Log;
				FStateTreeCompiler Compiler(Log);
				const bool bResult = Compiler.Compile(StateTree1);
				AITEST_TRUE("StateTree should get compiled", bResult);
			}

			{
				FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
				const bool bInitSucceeded = Exec.IsValid();
				AITEST_TRUE("StateTree should init", bInitSucceeded);

				// Start and enter state
				EStateTreeRunStatus Status = Exec.Start();
				AITEST_TRUE(TEXT("Valid states"), Exec.ExpectInActiveStates("Tree1_Root1", "Tree1_State1"));
				AITEST_TRUE(TEXT("Expected amount of frames."), InstanceData.GetExecutionState()->ActiveFrames.Num() == 1);

				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("Tree1_State1_TaskPrint", TEXT("EnterState11"));
				AITEST_TRUE(TEXT("Start enters in the correct order Tree1_State1_TaskPrint"), LogOrder);
				LogOrder = LogOrder.Then("Tree1_State1_TaskPrint", EnterStateChangedStr);
				AITEST_TRUE(TEXT("Start enters in the correct order Tree1_State1_TaskPrint"), LogOrder);

				LogOrder = LogOrder.Then("Tree1_State1_TaskStand", TEXT("EnterState"));
				AITEST_TRUE(TEXT("Start enters in the correct order Tree1_State1_TaskStand"), LogOrder);
				LogOrder = LogOrder.Then("Tree1_State1_TaskStand", EnterStateChangedStr);
				AITEST_TRUE(TEXT("Start enters in the correct order Tree1_State1_TaskStand"), LogOrder);
				Exec.LogClear();
			}

			{
				FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
				EStateTreeRunStatus Status = Exec.Tick(0.f);
				AITEST_TRUE(TEXT("Active states should now include linked asset states"), Exec.ExpectInActiveStates("Tree1_Root1", "Tree1_State1", "Tree1_State2", "Tree2_Root1", "Tree2_State1"));
				AITEST_TRUE(TEXT("Expected amount of frames."), InstanceData.GetExecutionState()->ActiveFrames.Num() == 2);

				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("Tree1_State1_TaskPrint", TEXT("Tick11"));
				AITEST_TRUE(TEXT("Tree1_State1_TaskPrint ticked"), LogOrder);
				LogOrder = LogOrder.Then("Tree1_State1_TaskStand", TEXT("Tick"));
				AITEST_TRUE(TEXT("Tree1_State1_TaskStand ticked"), LogOrder);
				LogOrder = LogOrder.Then("Tree1_State1_TaskStand", TEXT("StateCompleted"));
				AITEST_TRUE(TEXT("Completed State1 in Tree1"), LogOrder);
				LogOrder = LogOrder.Then("Tree1_State2_TaskPrint", TEXT("EnterState12"));
				AITEST_TRUE(TEXT("Entered State2 in Tree1"), LogOrder);
				LogOrder = LogOrder.Then("Tree1_State2_TaskPrint", EnterStateChangedStr);
				AITEST_TRUE(TEXT("Entered State2 in Tree1"), LogOrder);
				LogOrder = LogOrder.Then("Tree2_State1_TaskPrint", TEXT("EnterState21"));
				AITEST_TRUE(TEXT("Entered State1 in Tree2"), LogOrder);
				LogOrder = LogOrder.Then("Tree2_State1_TaskStand", TEXT("EnterState"));
				AITEST_TRUE(TEXT("Entered State1 in Tree2"), LogOrder);
				LogOrder = LogOrder.Then("Tree2_State1_TaskStand", EnterStateChangedStr);
				AITEST_TRUE(TEXT("Entered State1 in Tree2"), LogOrder);
				Exec.LogClear();
			}

			{
				FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
				EStateTreeRunStatus Status = Exec.Tick(0.f);
				AITEST_TRUE(TEXT("Active states should stay the same after Tree1_State1 completion"), Exec.ExpectInActiveStates("Tree1_Root1", "Tree1_State1", "Tree1_State2", "Tree2_Root1", "Tree2_State1"));
				AITEST_TRUE(TEXT("Expected amount of frames."), InstanceData.GetExecutionState()->ActiveFrames.Num() == 2);

				const bool bCompletedStatedCreateNewStates = EnumHasAnyFlags(StateSelectionRules, EStateTreeStateSelectionRules::CompletedTransitionStatesCreateNewStates);
				const bool bReselectCreateNewStates = EnumHasAnyFlags(StateSelectionRules, EStateTreeStateSelectionRules::ReselectedStateCreatesNewStates);
				const TCHAR* EnterStateStr = (bCompletedStatedCreateNewStates || bReselectCreateNewStates) ? EnterStateChangedStr: EnterStateSustainedStr;
				const TCHAR* ExitStateStr = (bCompletedStatedCreateNewStates || bReselectCreateNewStates) ? ExitStateChangedStr: ExitStateSustainedStr;

				FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("Tree1_State1_TaskPrint", TEXT("Tick11"));
				AITEST_TRUE(TEXT("Tree1_State1_TaskPrint ticked"), LogOrder);

				if (bCompletedStatedCreateNewStates)
				{
					LogOrder = LogOrder.Then("Tree1_State1_TaskStand", TEXT("Tick"));
					AITEST_TRUE(TEXT("Tree1_State1_TaskStand ticked"), LogOrder);
				}

				LogOrder = LogOrder.Then("Tree1_State2_TaskPrint", TEXT("Tick12"));
				AITEST_TRUE(TEXT("Tree1_State2_TaskPrint ticked"), LogOrder);
				LogOrder = LogOrder.Then("Tree2_State1_TaskPrint", TEXT("Tick21"));
				AITEST_TRUE(TEXT("Tree2_State1_TaskPrint ticked"), LogOrder);
				LogOrder = LogOrder.Then("Tree2_State1_TaskStand", TEXT("Tick"));
				AITEST_TRUE(TEXT("Tree2_State1_TaskStand ticked"), LogOrder);
				LogOrder = LogOrder.Then("Tree2_State1_TaskStand", TEXT("StateCompleted"));
				AITEST_TRUE(TEXT("Completed State1 in Tree2"), LogOrder);
				LogOrder = LogOrder.Then("Tree1_State1_TaskStand", TEXT("StateCompleted"));
				AITEST_TRUE(TEXT("Completed State1 in Tree1"), LogOrder);
				LogOrder = LogOrder.Then("Tree2_State1_TaskStand", ExitStateStr);
				AITEST_TRUE(TEXT("Exited State1 in Tree2"), LogOrder);
				LogOrder = Exec.Expect("Tree2_State1_TaskPrint", TEXT("ExitState21"));
				AITEST_TRUE(TEXT("Exited State1 in Tree2"), LogOrder);
				LogOrder = Exec.Expect("Tree2_State1_TaskPrint", ExitStateStr);
				AITEST_TRUE(TEXT("Exited State1 in Tree2"), LogOrder);
				LogOrder = Exec.Expect("Tree1_State2_TaskPrint", TEXT("ExitState12"));
				AITEST_TRUE(TEXT("Exited State2 in Tree1"), LogOrder);
				LogOrder = Exec.Expect("Tree1_State2_TaskPrint", ExitStateStr);
				AITEST_TRUE(TEXT("Exited State2 in Tree1"), LogOrder);

				if (bCompletedStatedCreateNewStates)
				{
					LogOrder = LogOrder.Then("Tree1_State1_TaskStand", ExitStateStr);
					AITEST_TRUE(TEXT("Exited State1 in Tree1"), LogOrder);
					LogOrder = Exec.Expect("Tree1_State1_TaskPrint", TEXT("ExitState11"));
					AITEST_TRUE(TEXT("Exited State1 in Tree1"), LogOrder);
					LogOrder = Exec.Expect("Tree1_State1_TaskPrint", ExitStateStr);
					AITEST_TRUE(TEXT("Exited State1 in Tree1"), LogOrder);
					LogOrder = Exec.Expect("Tree1_State1_TaskPrint", TEXT("EnterState11"));
					AITEST_TRUE(TEXT("Entered State1 in Tree1"), LogOrder);
					LogOrder = LogOrder.Then("Tree1_State1_TaskStand", EnterStateStr);
					AITEST_TRUE(TEXT("Entered State1 in Tree1"), LogOrder);
				}

				LogOrder = Exec.Expect("Tree1_State2_TaskPrint", TEXT("EnterState12"));
				AITEST_TRUE(TEXT("Entered State2 in Tree1"), LogOrder);
				LogOrder = LogOrder.Then("Tree1_State2_TaskPrint", EnterStateStr);
				AITEST_TRUE(TEXT("Entered State2 in Tree1"), LogOrder);
				LogOrder = Exec.Expect("Tree2_State1_TaskPrint", TEXT("EnterState21"));
				AITEST_TRUE(TEXT("Entered State1 in Tree2"), LogOrder);
				LogOrder = LogOrder.Then("Tree2_State1_TaskStand", EnterStateStr);
				AITEST_TRUE(TEXT("Entered State1 in Tree2"), LogOrder);
				Exec.LogClear();
			}

			{
				FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
				Exec.Stop();
			}
		}
		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_TransitionLinkedAssetFromCompletedParent, "System.StateTree.Transition.LinkedAssetFromCompletedParent");

struct FStateTreeTest_TransitionLinkedAssetWith2Root : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		/*
		- Tree1
			- Root1
				- State1 : LinkedAsset
		- Tree2
			- Root1
				- State1: Transition To Root3
			- Root2
				- State2
		*/

		UStateTree& StateTree1 = NewStateTree();
		UStateTree& StateTree2 = NewStateTree();
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree2.EditorData);
			UStateTreeState& Root1 = EditorData.AddSubTree(FName("Root1"));
			UStateTreeState& State1 = Root1.AddChildState(FName("State1"));
			UStateTreeState& Root2 = EditorData.AddSubTree(FName("Root2"));
			UStateTreeState& State2 = Root2.AddChildState(FName("State2"));

			auto& Task1 = State1.AddTask<FTestTask_Stand>(FName("Tree2State1Task1"));
			Task1.GetNode().TicksToCompletion = 1;
			State1.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &State2);

			auto& Task2 = State2.AddTask<FTestTask_Stand>(FName(TEXT("Task1A")));
			Task2.GetNode().TicksToCompletion = 1;
			State2.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::Succeeded);

			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree2);
			AITEST_TRUE("StateTree should get compiled", bResult);
		}
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree1.EditorData);
			UStateTreeState& Root1 = EditorData.AddSubTree(FName("Root"));
			UStateTreeState& State1 = Root1.AddChildState(FName("State1"), EStateTreeStateType::LinkedAsset);

			State1.SetLinkedStateAsset(&StateTree2);

			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree1);
			AITEST_TRUE("StateTree should get compiled", bResult);
		}

		FStateTreeInstanceData InstanceData;
		{
			FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE("StateTree should init", bInitSucceeded);

			// Start and enter state
			EStateTreeRunStatus Status = Exec.Start();
			AITEST_TRUE(TEXT("Valid states"), Exec.ExpectInActiveStates("Root", "State1", "Root1", "State1"));
			AITEST_TRUE(TEXT("Expected amount of frames."), InstanceData.GetExecutionState()->ActiveFrames.Num() == 2);
			Exec.LogClear();
		}
			{
			FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
			EStateTreeRunStatus Status = Exec.Tick(0.f);
			AITEST_TRUE(TEXT("Active states should stay the same"), Exec.ExpectInActiveStates("Root", "State1", "Root2", "State2"));
			AITEST_TRUE(TEXT("Expected amount of frames."), InstanceData.GetExecutionState()->ActiveFrames.Num() == 2);
			Exec.LogClear();
		}
		{
			FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
			Exec.Stop();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_TransitionLinkedAssetWith2Root, "System.StateTree.Transition.LinkedAssetWith2Root");

struct FStateTreeTest_MultipleTransition_TemporaryFrame_GlobalTask : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		/*
		- Tree1
			- Root
				- State1 Transition1 -> Goto State2; Transition2 -> Goto State3
				- State2 : Tree2
				- State3

		- Tree2 : Global Tasks
			- Root : Tree3

		- Tree3 : Global Tasks
		*/

		UStateTree& StateTree1 = NewStateTree();
		UStateTree& StateTree2 = NewStateTree();
		UStateTree& StateTree3 = NewStateTree();

		{
			UStateTreeEditorData& Tree1EditorData = *Cast<UStateTreeEditorData>(StateTree1.EditorData);
			UStateTreeState& Root = Tree1EditorData.AddSubTree(FName("Tree1Root"));
			UStateTreeState& State1 = Root.AddChildState(FName("Tree1State1"));
			UStateTreeState& State2 = Root.AddChildState(FName("Tree1State2"));
			UStateTreeState& State3 = Root.AddChildState(FName("Tree1State3"));

			FStateTreeTransition& LowPriorityTrans = State1.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::GotoState, &State2);
			LowPriorityTrans.Priority = EStateTreeTransitionPriority::Low;
			FStateTreeTransition& HighPriorityTrans = State1.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::GotoState, &State3);
			HighPriorityTrans.Priority = EStateTreeTransitionPriority::High;

			State2.Type = EStateTreeStateType::LinkedAsset;
			State2.SetLinkedStateAsset(&StateTree2);
		}

		{
			UStateTreeEditorData& Tree2EditorData = *Cast<UStateTreeEditorData>(StateTree2.EditorData);
			TStateTreeEditorNode<FTestTask_StandNoTick>& TaskEditorNode = Tree2EditorData.AddGlobalTask<FTestTask_StandNoTick>(FName("Tree2Stand"));
			UStateTreeState& Root = Tree2EditorData.AddSubTree(FName("Tree2Root"));
			Root.Type = EStateTreeStateType::LinkedAsset;
			Root.SetLinkedStateAsset(&StateTree3);
		}

		{
			UStateTreeEditorData& Tree3EditorData = *Cast<UStateTreeEditorData>(StateTree3.EditorData);
			Tree3EditorData.AddGlobalTask<FTestTask_StandNoTick>(FName("Tree3Stand"));

			UStateTreeState& Root = Tree3EditorData.AddSubTree(FName("Tree3Root"));
		}

		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree3);
			AITEST_TRUE("StateTree should get compiled", bResult);
		}
		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree2);
			AITEST_TRUE("StateTree should get compiled", bResult);
		}
		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree1);
			AITEST_TRUE("StateTree should get compiled", bResult)
		}

		{
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE("StateTree should init", bInitSucceeded);

			EStateTreeRunStatus Status = Exec.Start();
			AITEST_TRUE(TEXT("Valid states"), Exec.ExpectInActiveStates("Tree1Root", "Tree1State1"));
			Exec.LogClear();

			Exec.Tick(0.1f);

			FTestStateTreeExecutionContext::FLogOrder LogOrder = FTestStateTreeExecutionContext::FLogOrder(Exec, 0);
			LogOrder = LogOrder.Then("Tree2Stand", TEXT("EnterState=Changed")).Then("Tree3Stand", TEXT("EnterState=Changed"));
			AITEST_TRUE(TEXT("Enter Global tasks on temporary frames correctly"), LogOrder);

			LogOrder = LogOrder.Then("Tree3Stand", TEXT("ExitStopped"))
				.Then("Tree3Stand", TEXT("ExitState=Changed"))
				.Then("Tree2Stand", TEXT("ExitStopped"))
				.Then("Tree2Stand", TEXT("ExitState=Changed"));

			AITEST_TRUE(TEXT("Exit Global tasks on temporary frames correctly"), LogOrder);

			AITEST_TRUE(TEXT("Valid states"), Exec.ExpectInActiveStates("Tree1Root", "Tree1State3"));

			Exec.Stop();
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_MultipleTransition_TemporaryFrame_GlobalTask, "System.StateTree.Transition.MultipleTransition.TemporaryFrame.GlobalTask");

struct FStateTreeTest_Transition_DifferentRoot_GlobalTask : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		/*
		- Tree1: PrintTask
			- Root1
				- State1 Transition1 -> Goto Root2
			- Root2
				- State2
		*/

		UStateTree& StateTree1 = NewStateTree();

		{
			UStateTreeEditorData& Tree1EditorData = *Cast<UStateTreeEditorData>(StateTree1.EditorData);
			UStateTreeState& Root1 = Tree1EditorData.AddSubTree(FName("Tree1Root1"));
			UStateTreeState& State1 = Root1.AddChildState(FName("Tree1State1"));
			UStateTreeState& Root2 = Tree1EditorData.AddSubTree(FName("Tree1Root2"));
			UStateTreeState& State2 = Root2.AddChildState(FName("Tree1State2"));

			FStateTreeTransition& LowPriorityTrans = State1.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::GotoState, &Root2);

			Tree1EditorData.AddGlobalTask<FTestTask_StandNoTick>(FName("Tree1GlobalTask1"));
			Tree1EditorData.AddGlobalTask<FTestTask_StandNoTick>(FName("Tree1GlobalTask2"));
			Root1.AddTask<FTestTask_StandNoTick>(FName("Tree1Root1Task1"));
			State1.AddTask<FTestTask_StandNoTick>(FName("Tree1State1Task1"));
			Root2.AddTask<FTestTask_StandNoTick>(FName("Tree1Root2Task1"));
			State2.AddTask<FTestTask_StandNoTick>(FName("Tree1State2Task1"));
		}
		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree1);
			AITEST_TRUE("StateTree should get compiled", bResult)
		}

		const TCHAR* EnterStateChangedStr = TEXT("EnterState=Changed");
		const TCHAR* ExitStateChangedStr = TEXT("ExitState=Changed");
		const TCHAR* ExitStateStr = TEXT("ExitState");

		{
			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
			const bool bInitSucceeded = Exec.IsValid();
			AITEST_TRUE("StateTree should init", bInitSucceeded);

			EStateTreeRunStatus Status = Exec.Start();
			AITEST_TRUE(TEXT("Valid states"), Exec.ExpectInActiveStates("Tree1Root1", "Tree1State1"));
			FTestStateTreeExecutionContext::FLogOrder LogOrder = Exec.Expect("Tree1GlobalTask1", EnterStateChangedStr);
			AITEST_TRUE(TEXT("Tree1GlobalTask1 enter"), LogOrder);
			LogOrder = LogOrder.Then("Tree1GlobalTask2", EnterStateChangedStr);
			AITEST_TRUE(TEXT("Tree1GlobalTask2 enter"), LogOrder);
			LogOrder = LogOrder.Then("Tree1Root1Task1", EnterStateChangedStr);
			AITEST_TRUE(TEXT("Tree1Root1Task1 enter"), LogOrder);
			LogOrder = LogOrder.Then("Tree1State1Task1", EnterStateChangedStr);
			AITEST_TRUE(TEXT("Tree1State1Task1 enter"), LogOrder);
			Exec.LogClear();

			Exec.Tick(0.1f);
			AITEST_TRUE(TEXT("Valid states"), Exec.ExpectInActiveStates("Tree1Root2", "Tree1State2"));
			LogOrder = Exec.Expect("Tree1State1Task1", ExitStateChangedStr);
			AITEST_TRUE(TEXT("Tree1State1Task1 exit"), LogOrder);
			LogOrder = LogOrder.Then("Tree1Root1Task1", ExitStateChangedStr);
			AITEST_TRUE(TEXT("Tree1Root1Task1 exit"), LogOrder);
			LogOrder = LogOrder.Then("Tree1Root2Task1", EnterStateChangedStr);
			AITEST_TRUE(TEXT("Tree1Root2Task1 enter"), LogOrder);
			LogOrder = LogOrder.Then("Tree1State2Task1", EnterStateChangedStr);
			AITEST_TRUE(TEXT("Tree1State2Task1 enter"), LogOrder);

			AITEST_FALSE(TEXT("Tree1GlobalTask1 changed"), Exec.Expect("Tree1GlobalTask1", ExitStateStr));
			AITEST_FALSE(TEXT("Tree1GlobalTask2 changed"), Exec.Expect("Tree1GlobalTask2", ExitStateStr));
			Exec.LogClear();

			Exec.Stop();
			LogOrder = Exec.Expect("Tree1GlobalTask2", ExitStateChangedStr);
			AITEST_TRUE(TEXT("Tree1GlobalTask2 exit"), LogOrder);
			LogOrder = LogOrder.Then("Tree1GlobalTask1", ExitStateChangedStr);
			AITEST_TRUE(TEXT("Tree1GlobalTask1 exit"), LogOrder);
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_Transition_DifferentRoot_GlobalTask, "System.StateTree.Transition.BetweenDifferentRoot.GlobalTasksShouldStay");

struct FStateTreeTest_TransitionTypes : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		//Root
		// State1
		//  State1C1
		//  State1C2
		//  State1C3
		// State2
		// State3
		struct FTree
		{
			UStateTree* StateTree = nullptr;
			UStateTreeState* Root = nullptr;
			UStateTreeState* State1 = nullptr;
			UStateTreeState* State1C1 = nullptr;
			UStateTreeState* State1C2 = nullptr;
			UStateTreeState* State1C3 = nullptr;
			UStateTreeState* State2 = nullptr;
			UStateTreeState* State3 = nullptr;
			FGuid EvaluatorA_ID;
		};

		auto Compile = [this](TNotNull<UStateTree*> StateTree, bool bExpectedToCompile)
			{
				FStateTreeCompilerLog Log;
				FStateTreeCompiler Compiler(Log);
				const bool bResult = Compiler.Compile(StateTree);
				if (bExpectedToCompile)
				{
					AITEST_TRUE(TEXT("StateTree should compiled"), bResult);
					return bResult;
				}
				else
				{
					AITEST_FALSE(TEXT("StateTree should not compiled"), bResult);
					return !bResult;
				}
			};


		auto Start = [this](FTestStateTreeExecutionContext& Exec)
			{
				const bool bInitSucceeded = Exec.IsValid();
				AITEST_TRUE("StateTree should init", bInitSucceeded);

				EStateTreeRunStatus Status = Exec.Start();
				AITEST_TRUE(TEXT("State1C1 should enter"), Exec.Expect("State1C1Task2", TEXT("EnterState")));
				Exec.LogClear();
				return true;
			};

		auto MakeBasicTree = [this]()
			{
				FTree Result;
				Result.StateTree = &NewStateTree();
				UStateTreeEditorData* EditorData = Cast<UStateTreeEditorData>(Result.StateTree->EditorData);

				Result.Root = &EditorData->AddSubTree(FName("Root"));
				Result.State1 = &Result.Root->AddChildState(FName("State1"));
				Result.State1C1 = &Result.State1->AddChildState(FName("State1C1"));
				Result.State1C2 = &Result.State1->AddChildState(FName("State1C2"));
				Result.State1C3 = &Result.State1->AddChildState(FName("State1C3"));
				Result.State2 = &Result.Root->AddChildState(FName("State2"));
				Result.State3 = &Result.Root->AddChildState(FName("State3"));

				// Tasks
				Result.State1C1->AddTask<FTestTask_PrintValue>(FName(TEXT("State1C1Task1"))).GetInstanceData().Value = 1;
				Result.State1C1->AddTask<FTestTask_Stand>(FName(TEXT("State1C1Task2"))).GetNode().TicksToCompletion = 1;
				Result.State1C2->AddTask<FTestTask_PrintValue>(FName(TEXT("State1C2Task1"))).GetInstanceData().Value = 2;
				Result.State1C2->AddTask<FTestTask_Stand>(FName(TEXT("State1C2Task2"))).GetNode().TicksToCompletion = 1;
				Result.State1C3->AddTask<FTestTask_PrintValue>(FName(TEXT("State1C3Task1"))).GetInstanceData().Value = 3;
				Result.State1C3->AddTask<FTestTask_Stand>(FName(TEXT("State1C3Task2"))).GetNode().TicksToCompletion = 1;
				Result.State2->AddTask<FTestTask_PrintValue>(FName(TEXT("State2Task1"))).GetInstanceData().Value = 4;
				Result.State3->AddTask<FTestTask_PrintValue>(FName(TEXT("State3Task1"))).GetInstanceData().Value = 5;

				// Evaluators
				TStateTreeEditorNode<FTestEval_A>& Evaluator = EditorData->AddEvaluator<FTestEval_A>(FName("Evaluator1"));
				Evaluator.GetInstanceData().bBoolA = true;
				Result.EvaluatorA_ID = Evaluator.ID;

				return Result;
			};

		auto AddPropertyBinding = [this](TNotNull<UStateTree*> StateTree, const FGuid& SourceID, FName SourcePath, const FGuid& TargetID, FName TargetPath)
			{
				UStateTreeEditorData* EditorData = Cast<UStateTreeEditorData>(StateTree->EditorData);
				EditorData->AddPropertyBinding(FPropertyBindingPath(SourceID, SourcePath), FPropertyBindingPath(TargetID, TargetPath));
				return true;
			};

		// None -> state should remain the same state after transition trigger
		{
			FTree Tree = MakeBasicTree();

			Tree.State1C1->AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::None);

			if (!Compile(Tree.StateTree, /*bExpectedToCompile*/false))
			{
				return false;
			}
		}

		// NextState & NextParent -> should select next sibling State1C2, State1C3
		{
			FTree Tree = MakeBasicTree();

			Tree.State1C1->AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::NextState);
			Tree.State1C2->AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::NextState);
			// State1C3 NextState doesn't compile.
			Tree.State1C3->AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::NextState);

			if (!Compile(Tree.StateTree, /*bExpectedToCompile*/false))
			{
				return false;
			}

			Tree.State1C3->Transitions.Reset();
			Tree.State1C3->AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::NextParent);

			if (!Compile(Tree.StateTree, /*bExpectedToCompile*/true))
			{
				return false;
			}

			FStateTreeInstanceData InstanceData;
			FTestStateTreeExecutionContext Exec(Tree.StateTree, Tree.StateTree, InstanceData);
			if (!Start(Exec))
			{
				return false;
			}

			// Go to State1C2
			EStateTreeRunStatus Status = Exec.Tick(0.1f);
			AITEST_TRUE(TEXT("State1C1Task1 should complete"), Exec.Expect("State1C1Task2", TEXT("StateCompleted")));
			AITEST_TRUE(TEXT("Should transition to in State1C2"), Exec.ExpectInActiveStates("Root", "State1", "State1C2"));
			Exec.LogClear();

			// Go to State1C3
			Status = Exec.Tick(0.1f);
			AITEST_TRUE(TEXT("State1C1Task1 should complete"), Exec.Expect("State1C2Task2", TEXT("StateCompleted")));
			AITEST_TRUE(TEXT("Should transition to in State1C2"), Exec.ExpectInActiveStates("Root", "State1", "State1C3"));
			Exec.LogClear();

			// Go to State2
			Status = Exec.Tick(0.1f);
			AITEST_TRUE(TEXT("State1C1Task1 should complete"), Exec.Expect("State1C3Task2", TEXT("StateCompleted")));
			AITEST_TRUE(TEXT("Should transition to in State1C2"), Exec.ExpectInActiveStates("Root", "State2"));
			Exec.LogClear();

			Exec.Stop();
		}

		// NextState -> select next sibling (State1C2) but fail because can't enter. Use NextSelectableState.
		{
			FTree Tree = MakeBasicTree();

			Tree.State1C1->AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::NextState);
			TStateTreeEditorNode<FStateTreeCompareBoolCondition>& State1C2Condition = Tree.State1C2->AddEnterCondition<FStateTreeCompareBoolCondition>();
			State1C2Condition.GetNode().EvaluationMode = EStateTreeConditionEvaluationMode::ForcedFalse;
			State1C2Condition.GetInstanceData().bLeft = false;

			if (!AddPropertyBinding(Tree.StateTree, Tree.EvaluatorA_ID, FName("bBoolA"), State1C2Condition.ID, FName("bLeft")))
			{
				return false;
			}

			if (!Compile(Tree.StateTree, /*bExpectedToCompile*/true))
			{
				return false;
			}

			{
				FStateTreeInstanceData InstanceData;
				FTestStateTreeExecutionContext Exec(Tree.StateTree, Tree.StateTree, InstanceData);
				if (!Start(Exec))
				{
					return false;
				}

				const EStateTreeRunStatus Status = Exec.Tick(1.1f);
				AITEST_TRUE(TEXT("State1C1Task2 should complete"), Exec.Expect("State1C1Task2", TEXT("StateCompleted")));
				AITEST_TRUE(TEXT("State1C1Task2 should exit"), Exec.Expect("State1C1Task2", TEXT("ExitState")));
				AITEST_TRUE(TEXT("State1C1Task2 should enter"), Exec.Expect("State1C1Task2", TEXT("EnterState")));
				AITEST_FALSE(TEXT("State1C2Task2 should enter"), Exec.Expect("State1C2Task2", TEXT("EnterState")));
				AITEST_TRUE(TEXT("Should transition to root and back"), Exec.ExpectInActiveStates("Root", "State1", "State1C1"));
				Exec.LogClear();

				Exec.Stop();
			}

			// Use NextSelectableState (go to C3)
			Tree.State1C1->Transitions.Reset();
			Tree.State1C1->AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::NextSelectableState);
			if (!Compile(Tree.StateTree, /*bExpectedToCompile*/true))
			{
				return false;
			}

			{
				FStateTreeInstanceData InstanceData;
				FTestStateTreeExecutionContext Exec(Tree.StateTree, Tree.StateTree, InstanceData);
				if (!Start(Exec))
				{
					return false;
				}

				const EStateTreeRunStatus Status = Exec.Tick(1.1f);
				AITEST_TRUE(TEXT("State1C1Task2 should complete"), Exec.Expect("State1C1Task2", TEXT("StateCompleted")));
				AITEST_TRUE(TEXT("State1C1Task2 should exit"), Exec.Expect("State1C1Task2", TEXT("ExitState")));
				AITEST_FALSE(TEXT("State1C2Task2 should enter"), Exec.Expect("State1C2Task2", TEXT("EnterState")));
				AITEST_TRUE(TEXT("State1C3Task2 should enter"), Exec.Expect("State1C3Task2", TEXT("EnterState")));
				AITEST_TRUE(TEXT("Should transition to State1C3"), Exec.ExpectInActiveStates("Root", "State1", "State1C3"));
				Exec.LogClear();

				Exec.Stop();
			}
		}

		// NextParent -> select next parent (State2) but fail because can't enter. Use NextSelectableParent.
		{
			FTree Tree = MakeBasicTree();

			Tree.State1C1->AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::NextParent);
			TStateTreeEditorNode<FStateTreeCompareBoolCondition>& State2Condition = Tree.State2->AddEnterCondition<FStateTreeCompareBoolCondition>();
			State2Condition.GetNode().EvaluationMode = EStateTreeConditionEvaluationMode::ForcedFalse;
			State2Condition.GetInstanceData().bLeft = false;

			if (!AddPropertyBinding(Tree.StateTree, Tree.EvaluatorA_ID, FName("bBoolA"), State2Condition.ID, FName("bLeft")))
			{
				return false;
			}

			if (!Compile(Tree.StateTree, /*bExpectedToCompile*/true))
			{
				return false;
			}

			{
				FStateTreeInstanceData InstanceData;
				FTestStateTreeExecutionContext Exec(Tree.StateTree, Tree.StateTree, InstanceData);
				if (!Start(Exec))
				{
					return false;
				}

				const EStateTreeRunStatus Status = Exec.Tick(1.1f);
				AITEST_TRUE(TEXT("State1C1Task2 should complete"), Exec.Expect("State1C1Task2", TEXT("StateCompleted")));
				AITEST_TRUE(TEXT("State1C1Task2 should exit"), Exec.Expect("State1C1Task2", TEXT("ExitState")));
				AITEST_TRUE(TEXT("State1C1Task2 should enter"), Exec.Expect("State1C1Task2", TEXT("EnterState")));
				AITEST_FALSE(TEXT("State2Task1 should not enter"), Exec.Expect("State2Task1", TEXT("EnterState4")));
				AITEST_TRUE(TEXT("Should transition to root and back"), Exec.ExpectInActiveStates("Root", "State1", "State1C1"));
				Exec.LogClear();

				Exec.Stop();
			}

			// Use NextSelectableParent
			Tree.State1C1->Transitions.Reset();
			Tree.State1C1->AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::NextSelectableParent);
			if (!Compile(Tree.StateTree, /*bExpectedToCompile*/true))
			{
				return false;
			}

			{
				FStateTreeInstanceData InstanceData;
				FTestStateTreeExecutionContext Exec(Tree.StateTree, Tree.StateTree, InstanceData);
				if (!Start(Exec))
				{
					return false;
				}

				const EStateTreeRunStatus Status = Exec.Tick(1.1f);
				AITEST_TRUE(TEXT("State1C1Task2 should complete"), Exec.Expect("State1C1Task2", TEXT("StateCompleted")));
				AITEST_TRUE(TEXT("State1C1Task2 should exit"), Exec.Expect("State1C1Task2", TEXT("ExitState")));
				AITEST_FALSE(TEXT("State1C2Task2 should not enter"), Exec.Expect("State1C2Task2", TEXT("EnterState")));
				AITEST_FALSE(TEXT("State2Task1 should not enter"), Exec.Expect("State2Task1", TEXT("EnterState4")));
				AITEST_TRUE(TEXT("State3Task1 should enter"), Exec.Expect("State3Task1", TEXT("EnterState5")));
				AITEST_TRUE(TEXT("Should transition to State3"), Exec.ExpectInActiveStates("Root", "State3"));
				Exec.LogClear();

				Exec.Stop();
			}
		}

		// NextParent fail to compile when there is no next parent
		{
			FTree Tree = MakeBasicTree();

			Tree.State2->AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::NextParent);

			if (!Compile(Tree.StateTree, /*bExpectedToCompile*/false))
			{
				return false;
			}
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_TransitionTypes, "System.StateTree.Transition.Types");

/*
 * Test state selection triggered by trantiions of different types can always access events.
 */
struct FStateTreeTest_TransitionToRequiredEventState : FStateTreeTestBase
{
	virtual bool InstantTest() override
	{
		// Root
		//   A1 -> OnTick Transition to A3
		//   A2 -> OnComplete Transition to A4
		//   (Required Event To Enter) A3 -> On Complete Transition to A2
		//   (Required Event To Enter) A4 

		UStateTree& StateTree = NewStateTree();
		UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree.EditorData);

		const FGameplayTag TagA3 = GetTestTag1();
		const FGameplayTag TagA4 = GetTestTag2();

		UStateTreeState& Root = EditorData.AddSubTree(FName("Root"));

		UStateTreeState& StateA1 = Root.AddChildState(FName("A1"));
		{
			TStateTreeEditorNode<FTestTask_Stand>& TaskA1 = StateA1.AddTask<FTestTask_Stand>(FName("TaskA1"));
			TaskA1.GetNode().TicksToCompletion = 100;
		}

		UStateTreeState& StateA2 = Root.AddChildState(FName("A2"));

		UStateTreeState& StateA3 = Root.AddChildState(FName("A3"));
		{
			StateA3.bHasRequiredEventToEnter = true;
			StateA3.RequiredEventToEnter.Tag = TagA3;
		}

		UStateTreeState& StateA4 = Root.AddChildState(FName("A4"));
		{
			StateA4.bHasRequiredEventToEnter = true;
			StateA4.RequiredEventToEnter.Tag = TagA4;
		}

		StateA1.AddTransition(EStateTreeTransitionTrigger::OnTick, EStateTreeTransitionType::GotoState, &StateA3);
		StateA2.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &StateA4);
		StateA3.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::GotoState, &StateA2);

		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree);
			AITEST_TRUE("StateTree should get compiled", bResult);
		}

		FStateTreeInstanceData InstanceData;
		const FStateTreeEventQueue& EventQueue = InstanceData.GetStorage().GetEventQueue();
		FTestStateTreeExecutionContext Exec(StateTree, StateTree, InstanceData);
		const bool bInitSucceeded = Exec.IsValid();
		AITEST_TRUE("StateTree should be initialized", bInitSucceeded);

		EStateTreeRunStatus Status = EStateTreeRunStatus::Unset;
		{
			Status = Exec.Start();
			AITEST_TRUE("State Tree should be running", Status == EStateTreeRunStatus::Running);
			AITEST_TRUE("Active states should be [Root, A1]", Exec.ExpectInActiveStates("Root", "A1"));
			AITEST_TRUE(TEXT("Event Queue should be empty"), EventQueue.GetEventsView().IsEmpty());

			Exec.LogClear();
		}

		{
			Exec.SendEvent(TagA3);

			Status = Exec.Tick(0.1f);
			AITEST_TRUE("State Tree should be running", Status == EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Active states should be [Root, A3]"), Exec.ExpectInActiveStates("Root", "A3"));
			AITEST_TRUE(TEXT("Event Queue should be empty"), EventQueue.GetEventsView().IsEmpty());
			
			Exec.LogClear();
		}

		{
			Status = Exec.Tick(0.1f);
			AITEST_TRUE("State Tree should be running", Status == EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Active states should be [Root, A2]"), Exec.ExpectInActiveStates("Root", "A2"));
			AITEST_TRUE(TEXT("Event Queue should be empty"), EventQueue.GetEventsView().IsEmpty());

			Exec.LogClear();
		}

		{
			Exec.SendEvent(TagA4);
			
			Status = Exec.Tick(0.1f);
			AITEST_TRUE("State Tree should be running", Status == EStateTreeRunStatus::Running);
			AITEST_TRUE(TEXT("Active states should be [Root, A4]"), Exec.ExpectInActiveStates("Root", "A4"));
			AITEST_TRUE(TEXT("Event Queue should be empty"), EventQueue.GetEventsView().IsEmpty());

			Exec.LogClear();
		}

		Exec.Stop();

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_TransitionToRequiredEventState, "System.StateTree.Transition.RequiredEvent");

} // namespace UE::StateTree::Tests

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
