// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeTest.h"
#include "StateTreeTestBase.h"
#include "StateTreeTestTypes.h"

#include "StateTreeCompilerLog.h"
#include "StateTreeEditorData.h"
#include "StateTreeCompiler.h"

#define LOCTEXT_NAMESPACE "AITestSuite_StateTreeTestWeakContext"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::StateTree::Tests
{
struct FStateTreeTest_WeakContext_FinishTask : FStateTreeTestBase
{
	//Tree 1 : Global Task
		//	Root : Task 
		//		State1 : Task -> Root
		//			State2 : Task -> Root
	virtual bool InstantTest() override
	{
		struct FWeakContext
		{
			FStateTreeWeakExecutionContext ContextTree1GlobalTask;
			FStateTreeWeakExecutionContext ContextTree1RootTask;
			FStateTreeWeakExecutionContext ContextTree1State1Task;
			FStateTreeWeakExecutionContext ContextTree1State2Task;

			bool bGlobalFinishTaskSuccessOnTick = false;
			bool bState1lFinishTaskFailOnTick = false;
		};

		FWeakContext WeakContext;

		// Building up the State Tree
		UStateTree& StateTree1 = NewStateTree();
		{
			UStateTreeEditorData& EditorData1 = *Cast<UStateTreeEditorData>(StateTree1.EditorData);
			{
				// Global Task
				TStateTreeEditorNode<FTestTask_PrintValue>& Tree1GlobalTask = EditorData1.AddGlobalTask<FTestTask_PrintValue>(FName(TEXT("Tree1GlobalTask")));
				{
					Tree1GlobalTask.GetNode().CustomEnterStateFunc = [&WeakContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
						{
							WeakContext.ContextTree1GlobalTask = Context.MakeWeakExecutionContext();
						};
					Tree1GlobalTask.GetNode().CustomTickFunc = [&WeakContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
						{
							if (WeakContext.bGlobalFinishTaskSuccessOnTick)
							{
								WeakContext.ContextTree1GlobalTask.FinishTask(EStateTreeFinishTaskType::Succeeded);
							}
						};
				}

				// Root State
				UStateTreeState& Root = EditorData1.AddSubTree("Tree1StateRoot");
				{
					TStateTreeEditorNode<FTestTask_PrintValue>& Tree1RootTask = Root.AddTask<FTestTask_PrintValue>(FName(TEXT("Tree1RootTask")));
					Tree1RootTask.GetNode().CustomEnterStateFunc = [&WeakContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
						{
							WeakContext.ContextTree1RootTask = Context.MakeWeakExecutionContext();
						};

				}

				// State 1
				UStateTreeState& State1 = Root.AddChildState("Tree1State1");
				{
					TStateTreeEditorNode<FTestTask_PrintValue>& Tree1State1Task = State1.AddTask<FTestTask_PrintValue>(FName(TEXT("Tree1State1Task")));
					Tree1State1Task.GetNode().CustomEnterStateFunc = [&WeakContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
						{
							WeakContext.ContextTree1State1Task = Context.MakeWeakExecutionContext();
						};
					Tree1State1Task.GetNode().CustomTickFunc = [&WeakContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
						{
							if (WeakContext.bState1lFinishTaskFailOnTick)
							{
								WeakContext.ContextTree1State1Task.FinishTask(EStateTreeFinishTaskType::Failed);
							}
						};
				}

				// State 2
				UStateTreeState& State2 = State1.AddChildState("Tree1State2");
				{
					TStateTreeEditorNode<FTestTask_PrintValue>& Tree1State2Task = State2.AddTask<FTestTask_PrintValue>(FName(TEXT("Tree1State2Task")));
					Tree1State2Task.GetNode().CustomEnterStateFunc = [&WeakContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
						{
							WeakContext.ContextTree1State2Task = Context.MakeWeakExecutionContext();
						};
				}
			};
		}

		// Compile tree
		{
			FStateTreeCompilerLog Log;
			FStateTreeCompiler Compiler(Log);
			const bool bResult = Compiler.Compile(StateTree1);
			AITEST_TRUE(TEXT("StateTree1 should get compiled"), bResult);
		}

		// Create context
		FStateTreeInstanceData InstanceData;
		{
			{
				FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
				const bool bInitSucceeded = Exec.IsValid();
				AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);
			}
			{
				FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
				EStateTreeRunStatus Status = Exec.Start();
				AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1State1", "Tree1State2"));

				AITEST_TRUE(TEXT("Start should EnterState"), Exec.Expect("Tree1GlobalTask", TEXT("EnterState0"))
							.Then("Tree1RootTask", TEXT("EnterState0"))
							.Then("Tree1State1Task", TEXT("EnterState0"))
							.Then("Tree1State2Task", TEXT("EnterState0")));
				Exec.LogClear();
			}

			// Test that everything tick and there are no transitions.
			{
				FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
				EStateTreeRunStatus Status = Exec.Tick(0.1f);
				AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1State1", "Tree1State2"));

				AITEST_TRUE(TEXT("Tick should Tick"), Exec.Expect("Tree1GlobalTask", TEXT("Tick0"))
							.Then("Tree1RootTask", TEXT("Tick0"))
							.Then("Tree1State1Task", TEXT("Tick0"))
							.Then("Tree1State2Task", TEXT("Tick0")));
				Exec.LogClear();
			}

			// Test Finish GlobalTask inside the tick
			{
				WeakContext.bGlobalFinishTaskSuccessOnTick = true;
				FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
				EStateTreeRunStatus Status = Exec.Tick(0.1f);
				AITEST_EQUAL(TEXT("Tick should complete with Succeeded"), Status, EStateTreeRunStatus::Succeeded);
				AITEST_TRUE(TEXT("Tick should Tick"), Exec.Expect("Tree1GlobalTask", TEXT("Tick0"))
							.Then("Tree1State2Task", TEXT("ExitState0"))
							.Then("Tree1State1Task", TEXT("ExitState0"))
							.Then("Tree1RootTask", TEXT("ExitState0")));
				WeakContext.bGlobalFinishTaskSuccessOnTick = false;
			}

			// Finished global task stop the execution. Reset the execution.
			{
				FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
				Exec.Start();
				Exec.LogClear();
			}

			// Test Finish GlobalTask outside the tick
			{
				FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
				WeakContext.ContextTree1GlobalTask.FinishTask(EStateTreeFinishTaskType::Failed);
				EStateTreeRunStatus Status = Exec.Tick(0.1f);
				AITEST_EQUAL(TEXT("Tick should complete with Failed"), Status, EStateTreeRunStatus::Failed);
				AITEST_FALSE(TEXT("Tick should not Tick"), Exec.Expect("Tree1GlobalTask", TEXT("Tick0")));
				AITEST_FALSE(TEXT("Tick should not Tick"), Exec.Expect("Tree1RootTask", TEXT("Tick0")));
				AITEST_TRUE(TEXT("Tick should Tick"), Exec.Expect("Tree1State2Task", TEXT("ExitState0"))
							.Then("Tree1State1Task", TEXT("ExitState0"))
							.Then("Tree1RootTask", TEXT("ExitState0")));
			}

			// Finished global task stop the execution. Reset the execution.
			{
				FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
				Exec.Start();
				Exec.LogClear();
			}

			// Test Finish StateTask inside the tick
			{
				WeakContext.bState1lFinishTaskFailOnTick = true;
				FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
				EStateTreeRunStatus Status = Exec.Tick(0.1f);
				AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("Tick should Tick"), Exec.Expect("Tree1GlobalTask", TEXT("Tick0"))
							.Then("Tree1RootTask", TEXT("Tick0"))
							.Then("Tree1State1Task", TEXT("Tick0"))
							.Then("Tree1State2Task", TEXT("ExitState0"))
							.Then("Tree1State1Task", TEXT("ExitState0"))
							.Then("Tree1RootTask", TEXT("ExitState0")));

				WeakContext.bGlobalFinishTaskSuccessOnTick = false;
			}

			// Test Finish StateTask outside the tick
			{
				FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
				WeakContext.ContextTree1State1Task.FinishTask(EStateTreeFinishTaskType::Succeeded);
				EStateTreeRunStatus Status = Exec.Tick(0.1f);
				AITEST_EQUAL(TEXT("Tick should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("Tick should Tick"), Exec.Expect("Tree1GlobalTask", TEXT("Tick0"))
							.Then("Tree1RootTask", TEXT("Tick0"))
							.Then("Tree1State2Task", TEXT("Tick0"))
							.Then("Tree1State2Task", TEXT("ExitState0"))
							.Then("Tree1State1Task", TEXT("ExitState0"))
							.Then("Tree1RootTask", TEXT("ExitState0")));
			}

			// Stop the Exec
			{
				FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
				Exec.Stop();
			}
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_WeakContext_FinishTask, "System.StateTree.WeakContext.FinishTask");

struct FStateTreeTest_WeakContext_InstanceData : FStateTreeTestBase
{
	//Tree 1 : 2x Global Tasks, 2x global evaluators
	//	Root
	//		State1 : 2x tasks -> Next
	//			State2
	//				State3Linked -> Tree 2 (fails) -> Next
	//				State4Linked -> Tree 2 (complete) -> Next
	//				State5 -> Next (dummy)
	//Tree 2 : 2x Global tasks, 2x global evaluators
	//	Root
	//		State1 : 2x tasks -> fail or wait


	//(1)Tree1GlobalEvaluator1 Context.Weak can access data
	//(2)Tree1GlobalEvaluator2 Context.Weak can access data and access data of Tree1GlobalEvaluator1
	//(3)Tree1GlobalTask1 Context.Weak can access data and access data of Tree1GlobalEvaluator2
	//(4)Tree1GlobalTask2 Context.Weak can access data and access data of Tree1GlobalTask1
	//(1)Tree2GlobalEvaluator1 Context.Weak can access data and access data of Tree1GlobalTask2
	//(2)Tree2GlobalEvaluator2 Context.Weak can access data and access data of Tree2GlobalEvaluator1
	//(3)Tree2GlobalTask1 Context.Weak can access data and access data of Tree2GlobalEvaluator
	//(4)Tree2GlobalTask2 Context.Weak can access data and access data of Tree2GlobalTask1
	//Tree2GlobalTask2 fails
	//(1)Tree2GlobalEvaluator1 Context.Weak can access data and access data of Tree1GlobalTask2 but not of previous Tree2GlobalEvaluator1
	//(2)Tree2GlobalEvaluator2 Context.Weak can access data and access data of Tree2GlobalEvaluator1
	//(3)Tree2GlobalTask1 Context.Weak can access data and access data of Tree2GlobalEvaluator
	//(4)Tree2GlobalTask2 Context.Weak can access data and access data of Tree2GlobalTask1
	//Tree2GlobalTask2 success
	//can access all data
	//trigger another state selection of State1 to test the same during state selection

	virtual bool InstantTest() override
	{
		struct FTestContext
		{
			FStateTreeWeakExecutionContext ContextTree1GlobalEvaluator1;
			FStateTreeWeakExecutionContext ContextTree1GlobalEvaluator2;
			FStateTreeWeakExecutionContext ContextTree1GlobalTask1;
			FStateTreeWeakExecutionContext ContextTree1GlobalTask2;
			FStateTreeWeakExecutionContext ContextTree1State1Task1;
			FStateTreeWeakExecutionContext ContextTree1State1Task2;
			FStateTreeWeakExecutionContext ContextTree2GlobalEvaluator1_1;
			FStateTreeWeakExecutionContext ContextTree2GlobalEvaluator2_1;
			FStateTreeWeakExecutionContext ContextTree2GlobalTask1_1;
			FStateTreeWeakExecutionContext ContextTree2GlobalTask2_1;
			FStateTreeWeakExecutionContext ContextTree2State1Task1_1;
			FStateTreeWeakExecutionContext ContextTree2State1Task2_1;
			FStateTreeWeakExecutionContext ContextTree2GlobalEvaluator1_2;
			FStateTreeWeakExecutionContext ContextTree2GlobalEvaluator2_2;
			FStateTreeWeakExecutionContext ContextTree2GlobalTask1_2;
			FStateTreeWeakExecutionContext ContextTree2GlobalTask2_2;
			FStateTreeWeakExecutionContext ContextTree2State1Task1_2;
			FStateTreeWeakExecutionContext ContextTree2State1Task2_2;

			bool bSecondTree2Trigger = false;

			TArray<int32> ReadAllData() const
			{
				TArray<int32> Result;
				Result.Add(ReadEvalData(ContextTree1GlobalEvaluator1));
				Result.Add(ReadEvalData(ContextTree1GlobalEvaluator2));
				Result.Add(ReadTaskData(ContextTree1GlobalTask1));
				Result.Add(ReadTaskData(ContextTree1GlobalTask2));
				Result.Add(ReadTaskData(ContextTree1State1Task1));
				Result.Add(ReadTaskData(ContextTree1State1Task2));
				Result.Add(ReadEvalData(ContextTree2GlobalEvaluator1_1));
				Result.Add(ReadEvalData(ContextTree2GlobalEvaluator2_1));
				Result.Add(ReadTaskData(ContextTree2GlobalTask1_1));
				Result.Add(ReadTaskData(ContextTree2GlobalTask2_1));
				Result.Add(ReadTaskData(ContextTree2State1Task1_1));
				Result.Add(ReadTaskData(ContextTree2State1Task2_1));
				Result.Add(ReadEvalData(ContextTree2GlobalEvaluator1_2));
				Result.Add(ReadEvalData(ContextTree2GlobalEvaluator2_2));
				Result.Add(ReadTaskData(ContextTree2GlobalTask1_2));
				Result.Add(ReadTaskData(ContextTree2GlobalTask2_2));
				Result.Add(ReadTaskData(ContextTree2State1Task1_2));
				Result.Add(ReadTaskData(ContextTree2State1Task2_2));
				return Result;
			}

			FString ReadAllData_ToString() const
			{
				TStringBuilder<256> StringBuilder;
				StringBuilder << TEXT('{');
				bool bFirst = true;
				for (int32 It : ReadAllData())
				{
					if (!bFirst)
					{
						StringBuilder << TEXT(", ");
					}
					bFirst = false;
					StringBuilder << It;
				}
				StringBuilder << TEXT('}');
				return StringBuilder.ToString();
			}

			FString ExepectData(TArrayView<const int32> AllElements, TArrayView<const int32> ValidElements)
			{
				TStringBuilder<256> StringBuilder;
				StringBuilder << TEXT("AccessData:{");
				bool bFirst = true;
				for (int32 Index = 0; Index < AllElements.Num(); ++Index)
				{
					if (!bFirst)
					{
						StringBuilder << TEXT(", ");
					}
					bFirst = false;
					if (ValidElements.Contains(Index))
					{
						StringBuilder << AllElements[Index];
					}
					else
					{
						constexpr int32 InvalidNumber = - 1;
						StringBuilder << InvalidNumber;
					}
				}
				StringBuilder << TEXT('}');
				return StringBuilder.ToString();
			}

			int32 ReadTaskData(const FStateTreeWeakExecutionContext& WeakContext) const
			{
				FStateTreeStrongReadOnlyExecutionContext StrongContext = WeakContext.MakeStrongReadOnlyExecutionContext();
				const FTestTask_PrintValue::FInstanceDataType* Ptr = StrongContext.GetInstanceDataPtr<const FTestTask_PrintValue::FInstanceDataType>();
				return Ptr ? Ptr->Value : INDEX_NONE;
			}
			bool WriteTaskData(FStateTreeWeakExecutionContext& WeakContext, int32 NewValue) const
			{
				FStateTreeStrongExecutionContext StrongContext = WeakContext.MakeStrongExecutionContext();
				FTestTask_PrintValue::FInstanceDataType* Ptr = StrongContext.GetInstanceDataPtr<FTestTask_PrintValue::FInstanceDataType>();
				if (Ptr)
				{
					Ptr->Value = NewValue;
					return true;
				}
				return false;
			}
			int32 ReadEvalData(const FStateTreeWeakExecutionContext& WeakContext) const
			{
				FStateTreeStrongReadOnlyExecutionContext StrongContext = WeakContext.MakeStrongReadOnlyExecutionContext();
				const FTestEval_Custom::FInstanceDataType* Ptr = StrongContext.GetInstanceDataPtr<const FTestEval_Custom::FInstanceDataType>();
				return Ptr ? Ptr->IntA : INDEX_NONE;
			}
			bool WriteEvalData(FStateTreeWeakExecutionContext& WeakContext, int32 NewValue) const
			{
				FStateTreeStrongExecutionContext StrongContext = WeakContext.MakeStrongExecutionContext();
				FTestEval_Custom::FInstanceDataType* Ptr = StrongContext.GetInstanceDataPtr<FTestEval_Custom::FInstanceDataType>();
				if (Ptr)
				{
					Ptr->IntA = NewValue;
					return true;
				}
				return false;
			}
		} TestContext;

		// Building up the State Tree
		UStateTree& StateTree2 = NewStateTree();
		UStateTree& StateTree1 = NewStateTree();

		#define UE_STATETREE_TEST_CUSTOMENTERSTATEFUNC(T1) \
			TestContext.T1 = Context.MakeWeakExecutionContext();\
			FStateTreeTestLog& TestLog = Context.GetExternalData(Task->LogHandle);\
			TArray<int32> AllDatas = TestContext.ReadAllData();\
			TestLog.Log(Task->Name, FString::Printf(TEXT("AccessData:%s"), *TestContext.ReadAllData_ToString()));

		#define UE_STATETREE_TEST_CUSTOMENTERSTATEFUNC_2(T1, T2) \
			if (!TestContext.bSecondTree2Trigger)\
			{\
				TestContext.T1 = Context.MakeWeakExecutionContext();\
			}\
			else\
			{\
				TestContext.T2 = Context.MakeWeakExecutionContext();\
			}\
			FStateTreeTestLog& TestLog = Context.GetExternalData(Task->LogHandle);\
			TArray<int32> AllDatas = TestContext.ReadAllData();\
			TestLog.Log(Task->Name, FString::Printf(TEXT("AccessData:%s"), *TestContext.ReadAllData_ToString()));

		#define UE_STATETREE_TEST_CUSTOMTICKFUNC() \
			FStateTreeTestLog& TestLog = Context.GetExternalData(Task->LogHandle);\
			TArray<int32> AllDatas = TestContext.ReadAllData();\
			TestLog.Log(Task->Name, FString::Printf(TEXT("AccessData:%s"), *TestContext.ReadAllData_ToString()));
		
		// Tree2
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree2.EditorData);
			{
				TStateTreeEditorNode<FTestEval_Custom>& Tree2GlobalEval1 = EditorData.AddEvaluator<FTestEval_Custom>(FName("Tree2GlobalEval1"));
				Tree2GlobalEval1.GetInstanceData().IntA = 201;
				Tree2GlobalEval1.GetNode().CustomEnterStateFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestEval_Custom* Task)
					{
						UE_STATETREE_TEST_CUSTOMENTERSTATEFUNC_2(ContextTree2GlobalEvaluator1_1, ContextTree2GlobalEvaluator1_2);
					};
				Tree2GlobalEval1.GetNode().CustomTickFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestEval_Custom* Task)
					{
						UE_STATETREE_TEST_CUSTOMTICKFUNC();
					};
				Tree2GlobalEval1.GetNode().CustomExitStateFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestEval_Custom* Task)
					{
						UE_STATETREE_TEST_CUSTOMTICKFUNC();
					};
				TStateTreeEditorNode<FTestEval_Custom>& Tree2GlobalEval2 = EditorData.AddEvaluator<FTestEval_Custom>(FName("Tree2GlobalEval2"));
				Tree2GlobalEval2.GetInstanceData().IntA = 202;
				Tree2GlobalEval2.GetNode().CustomEnterStateFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestEval_Custom* Task)
					{
						UE_STATETREE_TEST_CUSTOMENTERSTATEFUNC_2(ContextTree2GlobalEvaluator2_1, ContextTree2GlobalEvaluator2_2);
					};
				Tree2GlobalEval2.GetNode().CustomTickFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestEval_Custom* Task)
					{
						UE_STATETREE_TEST_CUSTOMTICKFUNC();
					};
				Tree2GlobalEval2.GetNode().CustomExitStateFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestEval_Custom* Task)
					{
						UE_STATETREE_TEST_CUSTOMTICKFUNC();
					};
				// Global Task
				TStateTreeEditorNode<FTestTask_PrintValue>& Tree2GlobalTask1 = EditorData.AddGlobalTask<FTestTask_PrintValue>(FName("Tree2GlobalTask1"));
				Tree2GlobalTask1.GetInstanceData().Value = 203;
				Tree2GlobalTask1.GetNode().CustomEnterStateFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						UE_STATETREE_TEST_CUSTOMENTERSTATEFUNC_2(ContextTree2GlobalTask1_1, ContextTree2GlobalTask1_2);
					};
				Tree2GlobalTask1.GetNode().CustomTickFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						UE_STATETREE_TEST_CUSTOMTICKFUNC();
					};
				Tree2GlobalTask1.GetNode().CustomExitStateFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						UE_STATETREE_TEST_CUSTOMTICKFUNC();
					};

				TStateTreeEditorNode<FTestTask_PrintValue>& Tree2GlobalTask2 = EditorData.AddGlobalTask<FTestTask_PrintValue>(FName("Tree2GlobalTask2"));
				Tree2GlobalTask2.GetInstanceData().Value = 204;
				Tree2GlobalTask2.GetNode().CustomEnterStateFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						UE_STATETREE_TEST_CUSTOMENTERSTATEFUNC_2(ContextTree2GlobalTask2_1, ContextTree2GlobalTask2_2);
					};
				Tree2GlobalTask2.GetNode().CustomTickFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						UE_STATETREE_TEST_CUSTOMTICKFUNC();
					};
				Tree2GlobalTask2.GetNode().CustomExitStateFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						UE_STATETREE_TEST_CUSTOMTICKFUNC();
					};
			}
			// Root State
			UStateTreeState& Root = EditorData.AddSubTree("Tree2StateRoot");
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& Tree2RootTask1 = Root.AddTask<FTestTask_PrintValue>(FName("Tree2RootTask1"));
				Tree2RootTask1.GetInstanceData().Value = 211;
				TStateTreeEditorNode<FTestTask_PrintValue>& Tree2RootTask2 = Root.AddTask<FTestTask_PrintValue>(FName("Tree2RootTask2"));
				Tree2RootTask2.GetInstanceData().Value = 212;
			}
			// State 1
			UStateTreeState& State1 = Root.AddChildState("Tree2State1");
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& Tree2State1Task1 = State1.AddTask<FTestTask_PrintValue>(FName("Tree2State1Task1"));
				Tree2State1Task1.GetInstanceData().Value = 221;
				Tree2State1Task1.GetNode().CustomEnterStateFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						UE_STATETREE_TEST_CUSTOMENTERSTATEFUNC_2(ContextTree2State1Task1_1, ContextTree2State1Task1_2);

						if (!TestContext.bSecondTree2Trigger)
						{
							FTestTask_PrintValue::FInstanceDataType& InstanceData = Context.GetInstanceData(*Task);
							InstanceData.EnterStateRunStatus = EStateTreeRunStatus::Failed;
							TestContext.bSecondTree2Trigger = true;
						}
					};
				Tree2State1Task1.GetNode().CustomTickFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						UE_STATETREE_TEST_CUSTOMTICKFUNC();
					};
				Tree2State1Task1.GetNode().CustomExitStateFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						UE_STATETREE_TEST_CUSTOMTICKFUNC();
					};

				TStateTreeEditorNode<FTestTask_PrintValue>& Tree2State1Task2 = State1.AddTask<FTestTask_PrintValue>(FName("Tree2State1Task2"));
				Tree2State1Task2.GetInstanceData().Value = 222;
				Tree2State1Task2.GetNode().CustomEnterStateFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						UE_STATETREE_TEST_CUSTOMENTERSTATEFUNC_2(ContextTree2State1Task2_1, ContextTree2State1Task2_2);
					};
				Tree2State1Task2.GetNode().CustomTickFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						UE_STATETREE_TEST_CUSTOMTICKFUNC();
					};
				Tree2State1Task2.GetNode().CustomExitStateFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						UE_STATETREE_TEST_CUSTOMTICKFUNC();
					};
			}

			// Compile tree
			{
				FStateTreeCompilerLog Log;
				FStateTreeCompiler Compiler(Log);
				const bool bResult = Compiler.Compile(StateTree2);
				AITEST_TRUE(TEXT("StateTree2 should get compiled"), bResult);
			}
		}
		// Tree1
		{
			UStateTreeEditorData& EditorData = *Cast<UStateTreeEditorData>(StateTree1.EditorData);
			{
				// Global Evaluator
				TStateTreeEditorNode<FTestEval_Custom>& Tree1GlobalEval1 = EditorData.AddEvaluator<FTestEval_Custom>(FName("Tree1GlobalEval1"));
				Tree1GlobalEval1.GetInstanceData().IntA = 101;
				Tree1GlobalEval1.GetNode().CustomEnterStateFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestEval_Custom* Task)
					{
						UE_STATETREE_TEST_CUSTOMENTERSTATEFUNC(ContextTree1GlobalEvaluator1);
					};
				Tree1GlobalEval1.GetNode().CustomTickFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestEval_Custom* Task)
					{
						UE_STATETREE_TEST_CUSTOMTICKFUNC();
					};
				Tree1GlobalEval1.GetNode().CustomExitStateFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestEval_Custom* Task)
					{
						UE_STATETREE_TEST_CUSTOMTICKFUNC();
					};
				TStateTreeEditorNode<FTestEval_Custom>& Tree1GlobalEval2 = EditorData.AddEvaluator<FTestEval_Custom>(FName("Tree1GlobalEval2"));
				Tree1GlobalEval2.GetInstanceData().IntA = 102;
				Tree1GlobalEval2.GetNode().CustomEnterStateFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestEval_Custom* Task)
					{
						UE_STATETREE_TEST_CUSTOMENTERSTATEFUNC(ContextTree1GlobalEvaluator2);
					};
				Tree1GlobalEval2.GetNode().CustomTickFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestEval_Custom* Task)
					{
						UE_STATETREE_TEST_CUSTOMTICKFUNC();
					};
				Tree1GlobalEval2.GetNode().CustomExitStateFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestEval_Custom* Task)
					{
						UE_STATETREE_TEST_CUSTOMTICKFUNC();
					};

				// Global Task
				TStateTreeEditorNode<FTestTask_PrintValue>& Tree1GlobalTask1 = EditorData.AddGlobalTask<FTestTask_PrintValue>(FName("Tree1GlobalTask1"));
				Tree1GlobalTask1.GetInstanceData().Value = 103;
				Tree1GlobalTask1.GetNode().CustomEnterStateFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						UE_STATETREE_TEST_CUSTOMENTERSTATEFUNC(ContextTree1GlobalTask1);
					};
				Tree1GlobalTask1.GetNode().CustomTickFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						UE_STATETREE_TEST_CUSTOMTICKFUNC();
					};
				Tree1GlobalTask1.GetNode().CustomExitStateFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						UE_STATETREE_TEST_CUSTOMTICKFUNC();
					};

				TStateTreeEditorNode<FTestTask_PrintValue>& Tree1GlobalTask2 = EditorData.AddGlobalTask<FTestTask_PrintValue>(FName("Tree1GlobalTask2"));
				Tree1GlobalTask2.GetInstanceData().Value = 104;
				Tree1GlobalTask2.GetNode().CustomEnterStateFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						UE_STATETREE_TEST_CUSTOMENTERSTATEFUNC(ContextTree1GlobalTask2);
					};
				Tree1GlobalTask2.GetNode().CustomTickFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						UE_STATETREE_TEST_CUSTOMTICKFUNC();
					};
				Tree1GlobalTask2.GetNode().CustomExitStateFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						UE_STATETREE_TEST_CUSTOMTICKFUNC();
					};
			}
			// Root State
			UStateTreeState& Root = EditorData.AddSubTree("Tree1StateRoot");
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& Tree1RootTask1 = Root.AddTask<FTestTask_PrintValue>(FName("Tree1RootTask1"));
				Tree1RootTask1.GetInstanceData().Value = 111;
				TStateTreeEditorNode<FTestTask_PrintValue>& Tree1RootTask2 = Root.AddTask<FTestTask_PrintValue>(FName("Tree1RootTask2"));
				Tree1RootTask2.GetInstanceData().Value = 112;
			}
			// State 1
			UStateTreeState& State1 = Root.AddChildState("Tree1State1");
			State1.SelectionBehavior = EStateTreeStateSelectionBehavior::TrySelectChildrenInOrder;
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& Tree1State1Task1 = State1.AddTask<FTestTask_PrintValue>(FName("Tree1State1Task1"));
				Tree1State1Task1.GetInstanceData().Value = 121;
				Tree1State1Task1.GetNode().CustomEnterStateFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						UE_STATETREE_TEST_CUSTOMENTERSTATEFUNC(ContextTree1State1Task1);
					};
				Tree1State1Task1.GetNode().CustomTickFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						UE_STATETREE_TEST_CUSTOMTICKFUNC();
					};
				Tree1State1Task1.GetNode().CustomExitStateFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						UE_STATETREE_TEST_CUSTOMTICKFUNC();
					};

				TStateTreeEditorNode<FTestTask_PrintValue>& Tree1State1Task2 = State1.AddTask<FTestTask_PrintValue>(FName("Tree1State1Task2"));
				Tree1State1Task2.GetInstanceData().Value = 122;
				Tree1State1Task2.GetNode().CustomEnterStateFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						UE_STATETREE_TEST_CUSTOMENTERSTATEFUNC(ContextTree1State1Task2);
					};
				Tree1State1Task2.GetNode().CustomTickFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						UE_STATETREE_TEST_CUSTOMTICKFUNC();
					};
				Tree1State1Task2.GetNode().CustomExitStateFunc = [&TestContext](FStateTreeExecutionContext& Context, const FTestTask_PrintValue* Task)
					{
						UE_STATETREE_TEST_CUSTOMTICKFUNC();
					};
			}
			UStateTreeState& State2 = State1.AddChildState("Tree1State2");
			State1.SelectionBehavior = EStateTreeStateSelectionBehavior::TrySelectChildrenInOrder;
			UStateTreeState& State3 = State2.AddChildState("Tree1State3", EStateTreeStateType::LinkedAsset);
			State3.SetLinkedStateAsset(&StateTree2);
			State3.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, EStateTreeTransitionType::NextState);
			UStateTreeState& State4 = State2.AddChildState("Tree1State4", EStateTreeStateType::LinkedAsset);
			State4.SetLinkedStateAsset(&StateTree2);
			UStateTreeState& State5 = State2.AddChildState("Tree1State5");
			{
				TStateTreeEditorNode<FTestTask_PrintValue>& Tree1State5Task1 = State4.AddTask<FTestTask_PrintValue>(FName("Tree1State5Task1"));
				Tree1State5Task1.GetInstanceData().Value = 155;
			}

			// Compile tree
			{
				FStateTreeCompilerLog Log;
				FStateTreeCompiler Compiler(Log);
				const bool bResult = Compiler.Compile(StateTree1);
				AITEST_TRUE(TEXT("StateTree2 should get compiled"), bResult);
			}
		}

		#undef UE_STATETREE_TEST_CUSTOMENTERSTATEFUNC
		#undef UE_STATETREE_TEST_CUSTOMENTERSTATEFUNC_2
		#undef UE_STATETREE_TEST_CUSTOMTICKFUNC

		//                                 0    1    2    3    4    5    6    7    8    9    10   11   12   13   14   15   16   17
		const int32 AllValidReadData[] = { 101, 102, 103, 104, 121, 122, 201, 202, 203, 204, 221, 222, 201, 202, 203, 204, 221, 222 };

		// Create context
		FStateTreeInstanceData InstanceData;
		{
			{
				FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
				const bool bInitSucceeded = Exec.IsValid();
				AITEST_TRUE(TEXT("StateTree should init"), bInitSucceeded);
			}
			{
				FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
				EStateTreeRunStatus Status = Exec.Start();
				AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1State1", "Tree1State2", "Tree1State3", "Tree2StateRoot", "Tree2State1"));

				FTestStateTreeExecutionContext::FLogOrder ExpectedLog = Exec.Expect("Tree1GlobalEval1", TestContext.ExepectData(AllValidReadData, { 0 }));
				AITEST_TRUE(TEXT("Start should EnterState"), ExpectedLog);
				ExpectedLog = ExpectedLog.Then("Tree1GlobalEval2", TestContext.ExepectData(AllValidReadData, { 0, 1 }));
				AITEST_TRUE(TEXT("Start should EnterState"), ExpectedLog);
				ExpectedLog = ExpectedLog.Then("Tree1GlobalTask1", TestContext.ExepectData(AllValidReadData, { 0, 1, 2 }));
				AITEST_TRUE(TEXT("Start should EnterState"), ExpectedLog);
				ExpectedLog = ExpectedLog.Then("Tree1GlobalTask2", TestContext.ExepectData(AllValidReadData, { 0, 1, 2, 3 }));
				AITEST_TRUE(TEXT("Start should EnterState"), ExpectedLog);
				ExpectedLog = ExpectedLog.Then("Tree1State1Task1", TestContext.ExepectData(AllValidReadData, { 0, 1, 2, 3, 4, 6, 7, 8, 9 }));
				AITEST_TRUE(TEXT("Start should EnterState"), ExpectedLog);
				ExpectedLog = ExpectedLog.Then("Tree1State1Task2", TestContext.ExepectData(AllValidReadData, { 0, 1, 2, 3, 4, 5, 6, 7, 8, 9 }));
				AITEST_TRUE(TEXT("Start should EnterState"), ExpectedLog);

				ExpectedLog = Exec.Expect("Tree2GlobalEval1", TestContext.ExepectData(AllValidReadData, { 0, 1, 2, 3, 6 }));
				AITEST_TRUE(TEXT("Start should EnterState"), ExpectedLog);
				ExpectedLog = ExpectedLog.Then("Tree2GlobalEval2", TestContext.ExepectData(AllValidReadData, { 0, 1, 2, 3, 6, 7 }));
				AITEST_TRUE(TEXT("Start should EnterState"), ExpectedLog);
				ExpectedLog = ExpectedLog.Then("Tree2GlobalTask1", TestContext.ExepectData(AllValidReadData, { 0, 1, 2, 3, 6, 7, 8 }));
				AITEST_TRUE(TEXT("Start should EnterState"), ExpectedLog);
				ExpectedLog = ExpectedLog.Then("Tree2GlobalTask2", TestContext.ExepectData(AllValidReadData, { 0, 1, 2, 3, 6, 7, 8, 9 }));
				AITEST_TRUE(TEXT("Start should EnterState"), ExpectedLog);
				ExpectedLog = ExpectedLog.Then("Tree2State1Task1", TestContext.ExepectData(AllValidReadData, { 0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10 }));
				AITEST_TRUE(TEXT("Start should EnterState"), ExpectedLog);
				ExpectedLog = ExpectedLog.Then("Tree2State1Task2", TestContext.ExepectData(AllValidReadData, { 0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11 })); // 11 is not valid
				AITEST_FALSE(TEXT("Start should EnterState"), ExpectedLog);
			}
			{
				TArray<int32> AllDatas = TestContext.ReadAllData();
				AITEST_TRUE(TEXT("After enter. Same num"), AllDatas.Num() == UE_ARRAY_COUNT(AllValidReadData));
				AITEST_TRUE(TEXT("After enter. Same Value"), AllDatas == MakeArrayView({ 101, 102, 103, 104, 121, 122, 201, 202, 203, 204, 221, -1, -1, -1, -1, -1, -1, -1 }));
			}
			{
				FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
				EStateTreeRunStatus Status = Exec.Tick(0.1f);
				AITEST_EQUAL(TEXT("Start should complete with Running"), Status, EStateTreeRunStatus::Running);
				AITEST_TRUE(TEXT("In correct states"), Exec.ExpectInActiveStates("Tree1StateRoot", "Tree1State1", "Tree1State2", "Tree1State4", "Tree2StateRoot", "Tree2State1"));

				// Tick
				FTestStateTreeExecutionContext::FLogOrder ExpectedLog = Exec.Expect("Tree2GlobalTask2", TEXT("Tick204"));
				AITEST_TRUE(TEXT("Tick"), ExpectedLog);
				ExpectedLog = ExpectedLog.Then("Tree2GlobalTask2", TestContext.ExepectData(AllValidReadData, { 0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10 }));
				AITEST_TRUE(TEXT("Tick"), ExpectedLog);

				// Enter second Tree 2
				{
					ExpectedLog = ExpectedLog.Then("Tree2GlobalEval1", TestContext.ExepectData(AllValidReadData, { 0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 12 }));
					AITEST_TRUE(TEXT("Tick Enter new tree"), ExpectedLog);
					ExpectedLog = ExpectedLog.Then("Tree2GlobalEval2", TestContext.ExepectData(AllValidReadData, { 0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 12, 13 }));
					AITEST_TRUE(TEXT("Start should EnterState"), ExpectedLog);
					ExpectedLog = ExpectedLog.Then("Tree2GlobalTask1", TestContext.ExepectData(AllValidReadData, { 0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 12, 13, 14 }));
					AITEST_TRUE(TEXT("Start should EnterState"), ExpectedLog);
					ExpectedLog = ExpectedLog.Then("Tree2GlobalTask2", TestContext.ExepectData(AllValidReadData, { 0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 12, 13, 14, 15 }));
					AITEST_TRUE(TEXT("Start should EnterState"), ExpectedLog);
				}
				// Exit first Tree 2
				{
					ExpectedLog = ExpectedLog.Then("Tree2State1Task1", TEXT("ExitState221"));
					AITEST_TRUE(TEXT("Exit first tree2"), ExpectedLog);
					ExpectedLog = ExpectedLog.Then("Tree2State1Task1", TestContext.ExepectData(AllValidReadData, { 0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 12, 13, 14, 15 }));
					AITEST_TRUE(TEXT("Exit first tree2"), ExpectedLog);
					ExpectedLog = ExpectedLog.Then("Tree2RootTask1", TEXT("ExitState211"));
					AITEST_TRUE(TEXT("Exit first tree2"), ExpectedLog);
					ExpectedLog = ExpectedLog.Then("Tree2GlobalTask2", TestContext.ExepectData(AllValidReadData, { 0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 12, 13, 14, 15 })); // -10
					AITEST_TRUE(TEXT("Exit first tree2"), ExpectedLog);
					ExpectedLog = ExpectedLog.Then("Tree2GlobalTask1", TestContext.ExepectData(AllValidReadData, { 0, 1, 2, 3, 4, 5, 6, 7, 8, 12, 13, 14, 15 })); //-9
					AITEST_TRUE(TEXT("Exit first tree2"), ExpectedLog);
					ExpectedLog = ExpectedLog.Then("Tree2GlobalEval2", TestContext.ExepectData(AllValidReadData, { 0, 1, 2, 3, 4, 5, 6, 7, 12, 13, 14, 15 })); //-8
					AITEST_TRUE(TEXT("Exit first tree2"), ExpectedLog);
					ExpectedLog = ExpectedLog.Then("Tree2GlobalEval1", TestContext.ExepectData(AllValidReadData, { 0, 1, 2, 3, 4, 5, 6, 12, 13, 14, 15 })); //-7
					AITEST_TRUE(TEXT("Exit first tree2"), ExpectedLog);
				}
				// Enter second Tree 2
				{
					ExpectedLog = ExpectedLog.Then("Tree2RootTask1", TEXT("EnterState211"));
					AITEST_TRUE(TEXT("Exit first tree2"), ExpectedLog);
					ExpectedLog = ExpectedLog.Then("Tree2State1Task1", TestContext.ExepectData(AllValidReadData, { 0, 1, 2, 3, 4, 5, 12, 13, 14, 15, 16 })); //-6+16
					AITEST_TRUE(TEXT("Exit first tree2"), ExpectedLog);
					ExpectedLog = ExpectedLog.Then("Tree2State1Task2", TestContext.ExepectData(AllValidReadData, { 0, 1, 2, 3, 4, 5, 12, 13, 14, 15, 16, 17 })); //+17
					AITEST_TRUE(TEXT("Exit first tree2"), ExpectedLog);
				}
			}
			{
				TArray<int32> AllDatas = TestContext.ReadAllData();
				AITEST_TRUE(TEXT("After enter. Same num"), AllDatas.Num() == UE_ARRAY_COUNT(AllValidReadData));
				AITEST_TRUE(TEXT("After enter. Same Value"), AllDatas == MakeArrayView({ 101, 102, 103, 104, 121, 122, -1, -1, -1, -1, -1, -1, 201, 202, 203, 204, 221, 222 }));
			}
			// Stop the Exec
			{
				FTestStateTreeExecutionContext Exec(StateTree1, StateTree1, InstanceData);
				Exec.Stop();
			}
		}

		return true;
	}
};
IMPLEMENT_STATE_TREE_INSTANT_TEST(FStateTreeTest_WeakContext_InstanceData, "System.StateTree.WeakContext.InstanceData");
} // namespace UE::StateTree::Tests

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
