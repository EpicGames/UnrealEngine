// Copyright Epic Games, Inc. All Rights Reserved.

#pragma  once

#include "AITestsCommon.h"

#include "StateTree.h"
#include "StateTreePropertyBindings.h"

class UStateTreeEditorData;
namespace UE::StateTree::Tests
{
	
/**
 * Base class for StateTree test
 */
struct FStateTreeTestBase : public FAITestBase
{
protected:
	UStateTree& NewStateTree() const;
	static FStateTreePropertyPathBinding MakeBinding(const FGuid& SourceID, const FStringView Source, const FGuid& TargetID, const FStringView Target, const bool bInIsOutputBinding = false);
	static FGameplayTag GetTestTag1();
	static FGameplayTag GetTestTag2();
	static FGameplayTag GetTestTag3();
	static FGameplayTag GetTestTag4();
	static FGameplayTag GetTestTag5();
	static FGameplayTag GetTestTag6();
	FInstancedPropertyBag& GetRootPropertyBag(UStateTreeEditorData& EditorData) const;
};

} // namespace UE::StateTree::Tests