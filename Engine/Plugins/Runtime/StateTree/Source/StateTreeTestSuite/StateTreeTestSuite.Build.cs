// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

namespace UnrealBuildTool.Rules
{
	public class StateTreeTestSuite : ModuleRules
	{
		public StateTreeTestSuite(ReadOnlyTargetRules Target) : base(Target)
		{
			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

			bAllowUETypesInNamespaces = true;

			PublicIncludePaths.AddRange(
				new string[] {
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"AIModule",
					"StateTreeModule",
					"StateTreeEditorModule",
					"AITestSuite",
					"GameplayTags",
					"PropertyBindingUtils",
					"PropertyBindingUtilsEditor"
				}
			);

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.Add("EditorFramework");
				PrivateDependencyModuleNames.Add("UnrealEd");
			}
		}
	}
}