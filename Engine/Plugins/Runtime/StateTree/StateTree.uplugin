{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "StateTree",
	"Description" : "General purpose hierarchical state machine",
	"Category" : "Gameplay",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : true,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules": [
		{
			"Name": "StateTreeModule",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "StateTreeTestSuite",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "StateTreeDeveloper",
			"Type": "DeveloperTool",
			"LoadingPhase": "Default"
		},
		{
			"Name": "StateTreeEditorModule",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "GameplayInsights",
			"Enabled": true,
			"TargetAllowList": [
				"Editor"
			]
		},
		{
			"Name": "PropertyAccessEditor",
			"Enabled": true
		},
		{
			"Name": "PropertyBindingUtils",
			"Enabled": true
		}
	]
}